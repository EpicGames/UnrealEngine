[CoreRedirects]
+PropertyRedirects=(OldName="RemoteControlField.Label",NewName="RemoteControlEntity.Label")
+PropertyRedirects=(OldName="RemoteControlField.Id",NewName="RemoteControlEntity.Id")
+PropertyRedirects=(OldName="/Script/RemoteControlLogic.RemoteControlLogicConfig.SupportedControllerObjectTypes",NewName="/Script/RemoteControlLogic.RemoteControlLogicConfig.SupportedControllerObjectClassPaths")
+EnumRedirects=(OldName="ERCMask",ValueChanges=(("None","NoMask")))

[/Script/RemoteControlLogic.RemoteControlLogicConfig]
+SupportedControllerTypes=EPropertyBagPropertyType::String
+SupportedControllerTypes=EPropertyBagPropertyType::Int32
+SupportedControllerTypes=EPropertyBagPropertyType::Float
+SupportedControllerTypes=EPropertyBagPropertyType::Bool
+SupportedControllerTypes=EPropertyBagPropertyType::Struct
+SupportedControllerTypes=EPropertyBagPropertyType::Object
+SupportedControllerTypes=EPropertyBagPropertyType::Enum
+SupportedControllerStructTypes=Vector
+SupportedControllerStructTypes=Vector2D
+SupportedControllerStructTypes=Color
+SupportedControllerStructTypes=Rotator
+SupportedControllerObjectClassPaths=/Script/Engine.StaticMesh
+SupportedControllerObjectClassPaths=/Script/Engine.MaterialInterface
+SupportedControllerCustomTypes=Texture
