{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control API",
	"Description": "A suite of tools for controlling the Unreal Engine, both in Editor or at Runtime via a webserver. This allows users to control Unreal Engine remotely through HTTP or WebSockets requests. This functionality allows developers to control Unreal through 3rd party applications and web services.",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Plugins": [
		{ 
			"Name": "WebSocketNetworking",
			"Enabled": true
		},
		{
			"Name": "RemoteControlInterception",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		}
	],
	"Modules": [
		{
			"Name": "RemoteControl",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlLogic",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "WebRemoteControl",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Mac",
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "RemoteControlCommon",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlUI",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlProtocol",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlProtocolWidgets",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "RemoteControlMultiUser",
			"Type": "UncookedOnly",
			"LoadingPhase": "PostEngineInit"
		}
	]
}