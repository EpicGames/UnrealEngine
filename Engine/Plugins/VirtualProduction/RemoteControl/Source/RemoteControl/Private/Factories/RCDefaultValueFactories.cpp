// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RCDefaultValueFactories.h"
#include "Components/LightComponent.h"
#include "Components/MeshComponent.h"
#include "Components/SkinnedMeshComponent.h"
#include "Components/StaticMeshComponent.h"

TSharedRef<IRCDefaultValueFactory> FLightIntensityDefaultValueFactory::MakeInstance()
{
	return MakeShared<FLightIntensityDefaultValueFactory>();
}

bool FLightIntensityDefaultValueFactory::CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const
{
	if (const ULightComponent* LightComponent = Cast<ULightComponent>(InObject))
	{
		if (const ULightComponent* ArchetypeComponent = Cast<ULightComponent>(LightComponent->GetArchetype()))
		{
			return !FMath::IsNearlyEqual(LightComponent->ComputeLightBrightness(), ArchetypeComponent->ComputeLightBrightness());
		}
	}

	return false;
}

void FLightIntensityDefaultValueFactory::ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs)
{
#if WITH_EDITOR
	
	bool bReset = false;

	InObject->PreEditChange(InArgs.Property);
	InObject->Modify();

	if (ULightComponent* LightComponent = Cast<ULightComponent>(InObject))
	{
		if (const ULightComponent* ArchetypeComponent = Cast<ULightComponent>(LightComponent->GetArchetype()))
		{
			LightComponent->SetLightBrightness(ArchetypeComponent->ComputeLightBrightness());
			bReset = true;
		}
	}

	if (bReset)
	{
		FPropertyChangedEvent ChangeEvent(InArgs.Property, EPropertyChangeType::ValueSet);
		InObject->PostEditChangeProperty(ChangeEvent);
	}

#endif // WITH_EDITOR
}

bool FLightIntensityDefaultValueFactory::SupportsClass(const UClass* InObjectClass) const
{
	if (!InObjectClass)
	{
		return false;
	}

	return InObjectClass->IsChildOf<ULightComponentBase>();
}

bool FLightIntensityDefaultValueFactory::SupportsProperty(const FProperty* InProperty) const
{
	if (!InProperty)
	{
		return false;
	}

	return InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(ULightComponentBase, Intensity);
}

TSharedRef<IRCDefaultValueFactory> FOverrideMaterialsDefaultValueFactory::MakeInstance()
{
	return MakeShared<FOverrideMaterialsDefaultValueFactory>();
}

bool FOverrideMaterialsDefaultValueFactory::CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const
{
	return true;
}

void FOverrideMaterialsDefaultValueFactory::ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs)
{
#if WITH_EDITOR

	bool bReset = false;
	
	InObject->PreEditChange(InArgs.Property);
	InObject->Modify();

	if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(InObject))
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			if (StaticMeshComponent->GetMaterial(InArgs.ArrayIndex))
			{
				StaticMeshComponent->SetMaterial(InArgs.ArrayIndex, NULL);
				MeshComponent->SetMaterial(InArgs.ArrayIndex, StaticMeshComponent->GetMaterial(InArgs.ArrayIndex));
				bReset = true;
			}
		}
		else if (USkinnedMeshComponent* SkinnedMeshComponent = Cast<USkinnedMeshComponent>(MeshComponent))
		{
			if (SkinnedMeshComponent->GetMaterial(InArgs.ArrayIndex))
			{
				SkinnedMeshComponent->SetMaterial(InArgs.ArrayIndex, NULL);
				MeshComponent->SetMaterial(InArgs.ArrayIndex, SkinnedMeshComponent->GetMaterial(InArgs.ArrayIndex));
				bReset = true;
			}
		}
	}

	if (bReset)
	{
		FPropertyChangedEvent ChangeEvent(InArgs.Property, EPropertyChangeType::ValueSet);
		
		TArray<TMap<FString, int32>, TInlineAllocator<1>> ArrayIndexPerObject;
		ArrayIndexPerObject.Add({ { InArgs.Property->GetName(), InArgs.ArrayIndex } });
		
		ChangeEvent.SetArrayIndexPerObject(ArrayIndexPerObject);
		InObject->PostEditChangeProperty(ChangeEvent);
	}

#endif // WITH_EDITOR
}

bool FOverrideMaterialsDefaultValueFactory::SupportsClass(const UClass* InObjectClass) const
{
	if (!InObjectClass)
	{
		return false;
	}

	return InObjectClass->IsChildOf<UMeshComponent>();
}

bool FOverrideMaterialsDefaultValueFactory::SupportsProperty(const FProperty* InProperty) const
{
	if (!InProperty)
	{
		return false;
	}

	return InProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials);
}