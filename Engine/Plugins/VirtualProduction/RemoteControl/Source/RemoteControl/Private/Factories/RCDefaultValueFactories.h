// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/IRCDefaultValueFactory.h"

/**
 * A specialized default value factory for handling light components.
 */
class FLightIntensityDefaultValueFactory : public IRCDefaultValueFactory
{
public:
	static TSharedRef<IRCDefaultValueFactory> MakeInstance();

	//~ Begin IRCDefaultValueFactory interface
	virtual bool CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const override;
	virtual void ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs) override;
	virtual bool SupportsClass(const UClass* InObjectClass) const override;
	virtual bool SupportsProperty(const FProperty* InProperty) const override;
	//~ End IRCDefaultValueFactory interface
};

class FOverrideMaterialsDefaultValueFactory : public IRCDefaultValueFactory
{
public:
	static TSharedRef<IRCDefaultValueFactory> MakeInstance();

	//~ Begin IRCDefaultValueFactory interface
	virtual bool CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const override;
	virtual void ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs) override;
	virtual bool SupportsClass(const UClass* InObjectClass) const override;
	virtual bool SupportsProperty(const FProperty* InProperty) const override;
	//~ End IRCDefaultValueFactory interface
};