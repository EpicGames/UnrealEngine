// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlMaskingFactories.h"

// Entirely deprecated 5.5
PRAGMA_DISABLE_DEPRECATION_WARNINGS

/**
 * FVectorMaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FVectorMaskingFactory::MakeInstance()
{
	return MakeShared<FVectorMaskingFactory>();
}

void FVectorMaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif

			if (const FVector* VectorProp = ToStructProp->ContainerPtrToValuePtr<FVector>(OwningObject))
			{
				FVector MaskedVector;

				MaskedVector.X = InMaskingOperation->HasMask(ERCMask::MaskA) ? VectorProp->X : InMaskingOperation->PreMaskingCache.X;
				MaskedVector.Y = InMaskingOperation->HasMask(ERCMask::MaskB) ? VectorProp->Y : InMaskingOperation->PreMaskingCache.Y;
				MaskedVector.Z = InMaskingOperation->HasMask(ERCMask::MaskC) ? VectorProp->Z : InMaskingOperation->PreMaskingCache.Z;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedVector);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif
		}
	}
}

void FVectorMaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FVector* VectorProp = FromStructProp->ContainerPtrToValuePtr<FVector>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = VectorProp->X;
				InMaskingOperation->PreMaskingCache.Y = VectorProp->Y;
				InMaskingOperation->PreMaskingCache.Z = VectorProp->Z;
			}
		}
	}
}

bool FVectorMaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FVector>::Get();
}

/**
 * FVector4MaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FVector4MaskingFactory::MakeInstance()
{
	return MakeShared<FVector4MaskingFactory>();
}

void FVector4MaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FVector4* Vector4Prop = ToStructProp->ContainerPtrToValuePtr<FVector4>(OwningObject))
			{
				FVector4 MaskedVector4;

				MaskedVector4.X = InMaskingOperation->HasMask(ERCMask::MaskA) ? Vector4Prop->X : InMaskingOperation->PreMaskingCache.X;
				MaskedVector4.Y = InMaskingOperation->HasMask(ERCMask::MaskB) ? Vector4Prop->Y : InMaskingOperation->PreMaskingCache.Y;
				MaskedVector4.Z = InMaskingOperation->HasMask(ERCMask::MaskC) ? Vector4Prop->Z : InMaskingOperation->PreMaskingCache.Z;
				MaskedVector4.W = InMaskingOperation->HasMask(ERCMask::MaskD) ? Vector4Prop->W : InMaskingOperation->PreMaskingCache.W;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedVector4);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FVector4MaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FVector4* Vector4Prop = FromStructProp->ContainerPtrToValuePtr<FVector4>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = Vector4Prop->X;
				InMaskingOperation->PreMaskingCache.Y = Vector4Prop->Y;
				InMaskingOperation->PreMaskingCache.Z = Vector4Prop->Z;
				InMaskingOperation->PreMaskingCache.W = Vector4Prop->W;
			}
		}
	}
}

bool FVector4MaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FVector4>::Get();
}

/**
 * FIntVectorMaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FIntVectorMaskingFactory::MakeInstance()
{
	return MakeShared<FIntVectorMaskingFactory>();
}

void FIntVectorMaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FIntVector* IntVectorProp = ToStructProp->ContainerPtrToValuePtr<FIntVector>(OwningObject))
			{
				FIntVector MaskedIntVector;

				MaskedIntVector.X = InMaskingOperation->HasMask(ERCMask::MaskA) ? IntVectorProp->X : InMaskingOperation->PreMaskingCache.X;
				MaskedIntVector.Y = InMaskingOperation->HasMask(ERCMask::MaskB) ? IntVectorProp->Y : InMaskingOperation->PreMaskingCache.Y;
				MaskedIntVector.Z = InMaskingOperation->HasMask(ERCMask::MaskC) ? IntVectorProp->Z : InMaskingOperation->PreMaskingCache.Z;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedIntVector);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FIntVectorMaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FIntVector* IntVectorProp = FromStructProp->ContainerPtrToValuePtr<FIntVector>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = IntVectorProp->X;
				InMaskingOperation->PreMaskingCache.Y = IntVectorProp->Y;
				InMaskingOperation->PreMaskingCache.Z = IntVectorProp->Z;
			}
		}
	}
}

bool FIntVectorMaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FIntVector>::Get();
}

/**
 * FIntVector4MaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FIntVector4MaskingFactory::MakeInstance()
{
	return MakeShared<FIntVector4MaskingFactory>();
}

void FIntVector4MaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FIntVector4* IntVector4Prop = ToStructProp->ContainerPtrToValuePtr<FIntVector4>(OwningObject))
			{
				FIntVector4 MaskedIntVector4;

				MaskedIntVector4.X = InMaskingOperation->HasMask(ERCMask::MaskA) ? IntVector4Prop->X : InMaskingOperation->PreMaskingCache.X;
				MaskedIntVector4.Y = InMaskingOperation->HasMask(ERCMask::MaskB) ? IntVector4Prop->Y : InMaskingOperation->PreMaskingCache.Y;
				MaskedIntVector4.Z = InMaskingOperation->HasMask(ERCMask::MaskC) ? IntVector4Prop->Z : InMaskingOperation->PreMaskingCache.Z;
				MaskedIntVector4.W = InMaskingOperation->HasMask(ERCMask::MaskD) ? IntVector4Prop->W : InMaskingOperation->PreMaskingCache.W;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedIntVector4);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FIntVector4MaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FIntVector4* IntVector4Prop = FromStructProp->ContainerPtrToValuePtr<FIntVector4>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = IntVector4Prop->X;
				InMaskingOperation->PreMaskingCache.Y = IntVector4Prop->Y;
				InMaskingOperation->PreMaskingCache.Z = IntVector4Prop->Z;
				InMaskingOperation->PreMaskingCache.W = IntVector4Prop->W;
			}
		}
	}
}

bool FIntVector4MaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FIntVector4>::Get();
}

/**
 * FRotatorMaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FRotatorMaskingFactory::MakeInstance()
{
	return MakeShared<FRotatorMaskingFactory>();
}

void FRotatorMaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FRotator* RotatorProp = ToStructProp->ContainerPtrToValuePtr<FRotator>(OwningObject))
			{
				FRotator MaskedRotator;

				MaskedRotator.Roll = InMaskingOperation->HasMask(ERCMask::MaskA) ? RotatorProp->Roll : InMaskingOperation->PreMaskingCache.X;
				MaskedRotator.Pitch = InMaskingOperation->HasMask(ERCMask::MaskB) ? RotatorProp->Pitch : InMaskingOperation->PreMaskingCache.Y;
				MaskedRotator.Yaw = InMaskingOperation->HasMask(ERCMask::MaskC) ? RotatorProp->Yaw : InMaskingOperation->PreMaskingCache.Z;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedRotator);
			}

#if WITH_EDITOR			
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FRotatorMaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FRotator* RotatorProp = FromStructProp->ContainerPtrToValuePtr<FRotator>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = RotatorProp->Roll;
				InMaskingOperation->PreMaskingCache.Y = RotatorProp->Pitch;
				InMaskingOperation->PreMaskingCache.Z = RotatorProp->Yaw;
			}
		}
	}
}

bool FRotatorMaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FRotator>::Get();
}

/**
 * FColorMaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FColorMaskingFactory::MakeInstance()
{
	return MakeShared<FColorMaskingFactory>();
}

void FColorMaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FColor* ColorProp = ToStructProp->ContainerPtrToValuePtr<FColor>(OwningObject))
			{
				FColor MaskedColor;

				MaskedColor.R = InMaskingOperation->HasMask(ERCMask::MaskA) ? ColorProp->R : InMaskingOperation->PreMaskingCache.X;
				MaskedColor.G = InMaskingOperation->HasMask(ERCMask::MaskB) ? ColorProp->G : InMaskingOperation->PreMaskingCache.Y;
				MaskedColor.B = InMaskingOperation->HasMask(ERCMask::MaskC) ? ColorProp->B : InMaskingOperation->PreMaskingCache.Z;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedColor);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FColorMaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FColor* ColorProp = FromStructProp->ContainerPtrToValuePtr<FColor>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = ColorProp->R;
				InMaskingOperation->PreMaskingCache.Y = ColorProp->G;
				InMaskingOperation->PreMaskingCache.Z = ColorProp->B;
			}
		}
	}
}

bool FColorMaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FColor>::Get();
}

/**
 * FLinearColorMaskingFactory
 */
TSharedRef<IRemoteControlMaskingFactory> FLinearColorMaskingFactory::MakeInstance()
{
	return MakeShared<FLinearColorMaskingFactory>();
}

void FLinearColorMaskingFactory::ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive)
{
	if (FStructProperty* ToStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
#if WITH_EDITOR
			OwningObject->PreEditChange(ToStructProp);
#endif // WITH_EDITOR

			if (const FLinearColor* LinearColorProp = ToStructProp->ContainerPtrToValuePtr<FLinearColor>(OwningObject))
			{
				FLinearColor MaskedLinearColor;

				MaskedLinearColor.R = InMaskingOperation->HasMask(ERCMask::MaskA) ? LinearColorProp->R : InMaskingOperation->PreMaskingCache.X;
				MaskedLinearColor.G = InMaskingOperation->HasMask(ERCMask::MaskB) ? LinearColorProp->G : InMaskingOperation->PreMaskingCache.Y;
				MaskedLinearColor.B = InMaskingOperation->HasMask(ERCMask::MaskC) ? LinearColorProp->B : InMaskingOperation->PreMaskingCache.Z;
				MaskedLinearColor.A = InMaskingOperation->HasMask(ERCMask::MaskD) ? LinearColorProp->A : InMaskingOperation->PreMaskingCache.W;

				ToStructProp->SetValue_InContainer(OwningObject, &MaskedLinearColor);
			}

#if WITH_EDITOR
			FPropertyChangedEvent ChangeEvent(ToStructProp, bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet);
			OwningObject->PostEditChangeProperty(ChangeEvent);
#endif // WITH_EDITOR
		}
	}
}

void FLinearColorMaskingFactory::CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	if (const FStructProperty* FromStructProp = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const UObject* OwningObject = InMaskingOperation->ObjectRef.Object.Get())
		{
			if (const FLinearColor* LinearColorProp = FromStructProp->ContainerPtrToValuePtr<FLinearColor>(OwningObject))
			{
				InMaskingOperation->PreMaskingCache.X = LinearColorProp->R;
				InMaskingOperation->PreMaskingCache.Y = LinearColorProp->G;
				InMaskingOperation->PreMaskingCache.Z = LinearColorProp->B;
				InMaskingOperation->PreMaskingCache.W = LinearColorProp->A;
			}
		}
	}
}

bool FLinearColorMaskingFactory::SupportsExposedEntity(UScriptStruct* ScriptStruct) const
{
	return ScriptStruct == TBaseStructure<FLinearColor>::Get();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS
