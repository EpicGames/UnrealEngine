// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/IRemoteControlMaskingFactory.h"

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FVectorMaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVectorMaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FVectorMaskingFactory() = default;
	FVectorMaskingFactory(const FVectorMaskingFactory&) = default;
	FVectorMaskingFactory(FVectorMaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FVector4MaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVector4MaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FVector4MaskingFactory() = default;
	FVector4MaskingFactory(const FVector4MaskingFactory&) = default;
	FVector4MaskingFactory(FVector4MaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FIntVectorMaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FIntVectorMaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FIntVectorMaskingFactory() = default;
	FIntVectorMaskingFactory(const FIntVectorMaskingFactory&) = default;
	FIntVectorMaskingFactory(FIntVectorMaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FIntVector4MaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FIntVector4MaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FIntVector4MaskingFactory() = default;
	FIntVector4MaskingFactory(const FIntVector4MaskingFactory&) = default;
	FIntVector4MaskingFactory(FIntVector4MaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FRotatorMaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FRotatorMaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FRotatorMaskingFactory() = default;
	FRotatorMaskingFactory(const FRotatorMaskingFactory&) = default;
	FRotatorMaskingFactory(FRotatorMaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FColorMaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FColorMaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FColorMaskingFactory() = default;
	FColorMaskingFactory(const FColorMaskingFactory&) = default;
	FColorMaskingFactory(FColorMaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class UE_DEPRECATED(5.5, "Deprecated in favor of RemoteControlMaskingUtil.") FLinearColorMaskingFactory;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FLinearColorMaskingFactory : public IRemoteControlMaskingFactory
{
public:
	static TSharedRef<IRemoteControlMaskingFactory> MakeInstance();

	//~ Begin IRemoteControlMaskingFactory interface
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) override;
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const override;
	//~ End IRemoteControlMaskingFactory interface

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	FLinearColorMaskingFactory() = default;
	FLinearColorMaskingFactory(const FLinearColorMaskingFactory&) = default;
	FLinearColorMaskingFactory(FLinearColorMaskingFactory&&) = default;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
