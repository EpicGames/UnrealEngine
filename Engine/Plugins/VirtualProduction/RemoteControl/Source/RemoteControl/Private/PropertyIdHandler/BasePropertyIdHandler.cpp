// Copyright Epic Games, Inc. All Rights Reserved.

#include "BasePropertyIdHandler.h"

bool FBasePropertyIdHandler::IsPropertySupported(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		return !(Property->GetClass()->IsChildOf(FStructProperty::StaticClass()) ||
				 Property->GetClass()->IsChildOf(FObjectProperty::StaticClass()) ||
				 Property->GetClass()->IsChildOf(FEnumProperty::StaticClass())	  );
	}
	return false;
}

EPropertyBagPropertyType FBasePropertyIdHandler::GetPropertyType(const FProperty* InProperty) const
{
	EPropertyBagPropertyType PropertyType = EPropertyBagPropertyType::None;
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		const FName PropertyName = Property->GetClass()->GetFName();
		if (PropertyName == NAME_BoolProperty)
		{
			PropertyType = EPropertyBagPropertyType::Bool;
		}
		else if (PropertyName == NAME_ByteProperty)
		{
			if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
			{
				if (ByteProperty->Enum)
				{
					PropertyType = EPropertyBagPropertyType::Enum;
				}
				else
				{
					PropertyType = EPropertyBagPropertyType::Byte;
				}
			}
		}
		else if (PropertyName == NAME_Int32Property)
		{
			PropertyType = EPropertyBagPropertyType::Int32;
		}
		else if (PropertyName == NAME_Int64Property)
		{
			PropertyType = EPropertyBagPropertyType::Int64;
		}
		// We treat Float as Double
		else if (PropertyName == NAME_FloatProperty)
		{
			PropertyType = EPropertyBagPropertyType::Double;
		}
		else if (PropertyName == NAME_DoubleProperty)
		{
			PropertyType = EPropertyBagPropertyType::Double;
		}
		else if (PropertyName == NAME_NameProperty)
		{
			PropertyType = EPropertyBagPropertyType::Name;
		}
		else if (PropertyName == NAME_StrProperty)
		{
			PropertyType = EPropertyBagPropertyType::String;
		}
		else if (PropertyName == NAME_TextProperty)
		{
			PropertyType = EPropertyBagPropertyType::Text;
		}
	}
	return PropertyType;
}

FName FBasePropertyIdHandler::GetPropertySuperTypeName(const FProperty* InProperty) const
{
	return GetPropertySubTypeName(InProperty);
}

FName FBasePropertyIdHandler::GetPropertySubTypeName(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (ByteProperty->Enum)
			{
				return ByteProperty->Enum->GetFName();
			}
		}
		const FName& ClassName = Property->GetClass()->GetFName();
		if (ClassName == NAME_FloatProperty)
		{
			return NAME_DoubleProperty;
		}

		return ClassName;
	}
	return NAME_None;
}

UObject* FBasePropertyIdHandler::GetPropertyTypeObject(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (ByteProperty->Enum)
			{
				return ByteProperty->Enum;
			}
		}
	}
	return nullptr;
}
