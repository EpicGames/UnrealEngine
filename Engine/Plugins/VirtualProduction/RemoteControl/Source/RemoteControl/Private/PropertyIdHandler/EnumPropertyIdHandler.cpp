// Copyright Epic Games, Inc. All Rights Reserved.

#include "EnumPropertyIdHandler.h"

bool FEnumPropertyIdHandler::IsPropertySupported(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		return Property->GetClass()->IsChildOf(FEnumProperty::StaticClass());
	}
	return false;
}

EPropertyBagPropertyType FEnumPropertyIdHandler::GetPropertyType(const FProperty* InProperty) const
{
	return EPropertyBagPropertyType::Enum;
}

FName FEnumPropertyIdHandler::GetPropertySuperTypeName(const FProperty* InProperty) const
{
	return NAME_EnumProperty;
}

FName FEnumPropertyIdHandler::GetPropertySubTypeName(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			return EnumProperty->GetEnum()->GetFName();
		}
	}
	return NAME_None;
}

UObject* FEnumPropertyIdHandler::GetPropertyTypeObject(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			return EnumProperty->GetEnum();
		}
	}
	return nullptr;
}
