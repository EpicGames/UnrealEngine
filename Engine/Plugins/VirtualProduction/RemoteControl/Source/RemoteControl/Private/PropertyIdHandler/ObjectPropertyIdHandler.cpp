// Copyright Epic Games, Inc. All Rights Reserved.

#include "ObjectPropertyIdHandler.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

bool FObjectPropertyIdHandler::IsPropertySupported(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		return Property->GetClass()->IsChildOf(FObjectProperty::StaticClass());
	}
	return false;
}

EPropertyBagPropertyType FObjectPropertyIdHandler::GetPropertyType(const FProperty* InProperty) const
{
	return EPropertyBagPropertyType::Object;
}

FName FObjectPropertyIdHandler::GetPropertySuperTypeName(const FProperty* InProperty) const
{
	return NAME_ObjectProperty;
}

FName FObjectPropertyIdHandler::GetPropertySubTypeName(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if (ObjectProperty->PropertyClass)
			{
				return ObjectProperty->PropertyClass->GetFName();
			}
		}
	}
	return NAME_None;
}

UObject* FObjectPropertyIdHandler::GetPropertyTypeObject(const FProperty* InProperty) const
{
	return nullptr;
}
