// Copyright Epic Games, Inc. All Rights Reserved.

#include "StructPropertyIdHandler.h"

bool FStructPropertyIdHandler::IsPropertySupported(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		return Property->GetClass()->IsChildOf(FStructProperty::StaticClass());
	}
	return false;
}

EPropertyBagPropertyType FStructPropertyIdHandler::GetPropertyType(const FProperty* InProperty) const
{
	return EPropertyBagPropertyType::Struct;
}

FName FStructPropertyIdHandler::GetPropertySuperTypeName(const FProperty* InProperty) const
{
	return NAME_StructProperty;
}

FName FStructPropertyIdHandler::GetPropertySubTypeName(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct)
			{
				const FName& StructName = StructProperty->Struct->GetFName();
				if (StructName == NAME_Color)
				{
					return NAME_LinearColor;
				}
				return StructName;
			}
		}
	}
	return NAME_None;
}

UObject* FStructPropertyIdHandler::GetPropertyTypeObject(const FProperty* InProperty) const
{
	if (InProperty)
	{
		const FProperty* Property = GetPropertyInsideContainer(InProperty);
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct->GetFName() == NAME_Color)
			{
				return TBaseStructure<FLinearColor>::Get();
			}
			return StructProperty->Struct;
		}
	}
	return nullptr;
}
