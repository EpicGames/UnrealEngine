// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyIdHandler.h"

class FStructPropertyIdHandler : public IPropertyIdHandler
{
public:
	//~ Begin IPropertyIdHandler
	virtual bool IsPropertySupported(const FProperty* InProperty) const override;
	virtual EPropertyBagPropertyType GetPropertyType(const FProperty* InProperty) const override;
	virtual FName GetPropertySuperTypeName(const FProperty* InProperty) const override;
	virtual FName GetPropertySubTypeName(const FProperty* InProperty) const override;
	virtual UObject* GetPropertyTypeObject(const FProperty* InProperty) const override;
	//~ End IPropertyIdHandler
};
