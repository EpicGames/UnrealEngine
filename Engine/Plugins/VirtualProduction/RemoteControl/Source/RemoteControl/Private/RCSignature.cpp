// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignature.h"
#include "GameFramework/Actor.h"
#include "RCSignatureAction.h"
#include "RCSignatureActionInstance.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"

namespace UE::RemoteControl::Private
{
	TArray<UObject*> ResolveObjects(TConstArrayView<TWeakObjectPtr<UObject>> InObjects)
	{
		TArray<UObject*> ResolvedObjects;
		ResolvedObjects.Reserve(InObjects.Num());
		for (const TWeakObjectPtr<UObject>& ObjectWeak : InObjects)
		{
			if (UObject* Object = ObjectWeak.Get())
			{
				ResolvedObjects.Add(Object);
			}
		}
		return ResolvedObjects;
	}

	UObject* FindContext(const FRCSignatureField& InField, UObject* InOuterObject, UClass* InSupportedClass)
	{
		UObject* Context;

		if (InField.ObjectRelativePath.IsEmpty())
		{
			Context = InOuterObject;
		}
		else
		{
			UClass* FindClass = InSupportedClass ? InSupportedClass : UObject::StaticClass();
			Context = StaticFindObject(FindClass, InOuterObject, *InField.ObjectRelativePath);

			// Slow Path: if Subobject Path did not find the object, try to find the first sub-object of the object matching the class.
			if (!Context && InSupportedClass)
			{
				ForEachObjectWithOuterBreakable(InOuterObject, [&Context, InSupportedClass](UObject* InSubobject)->bool
					{
						UClass* SubobjectClass = InSubobject->GetClass();
						if (SubobjectClass && SubobjectClass->IsChildOf(InSupportedClass))
						{
							Context = InSubobject;
							return false;
						}
						return true;
					}
					, EGetObjectsFlags::IncludeNestedObjects);
			}
		}

		return Context;
	}
}

FRCSignatureField FRCSignatureField::CreateField(const FRCFieldPathInfo& InFieldPathInfo, const UObject* InOwnerObject, const FProperty* InProperty)
{
	const UClass* SupportedClass = nullptr;
	if (InProperty)
	{
		SupportedClass = InProperty->GetOwnerClass();
	}

	if (!SupportedClass && InFieldPathInfo.GetSegmentCount() > 0 && InFieldPathInfo.GetFieldSegment(0).IsResolved())
	{
		SupportedClass = InFieldPathInfo.GetFieldSegment(0).ResolvedData.Field->GetOwnerClass();
	}

	FRCSignatureField Field;
	Field.bEnabled = true;
	Field.FieldPath = InFieldPathInfo;
	Field.SupportedClass = SupportedClass;

	if (InProperty)
	{
		Field.PropertyDesc = FPropertyBagPropertyDesc(InProperty->GetFName(), InProperty);
	}

	if (InOwnerObject)
	{
		if (AActor* ActorOwner = InOwnerObject->GetTypedOuter<AActor>())
		{
			Field.ObjectRelativePath = InOwnerObject->GetPathName(ActorOwner);
		}
	}

	return Field;
}

void FRCSignatureField::PostLoad()
{
	for (FRCSignatureActionInstance& Action : Actions)
	{
		Action.PostLoad(*this);
	}
}

void FRCSignature::PostLoad()
{
	for (FRCSignatureField& Field : Fields)
	{
		Field.PostLoad();
	}
}

int32 FRCSignature::AddFields(TConstArrayView<FRCSignatureField> InFields)
{
	const int32 PreviousNum = Fields.Num();
	Fields.Reserve(PreviousNum + InFields.Num());

	for (const FRCSignatureField& Field : InFields)
	{
		Fields.AddUnique(Field);
	}

	return Fields.Num() - PreviousNum;
}

int32 FRCSignature::ApplySignature(URemoteControlPreset* InPreset, TConstArrayView<TWeakObjectPtr<UObject>> InObjects) const
{
	if (!InPreset || InObjects.IsEmpty())
	{
		return 0;
	}

	TArray<UObject*> ResolvedObjects = UE::RemoteControl::Private::ResolveObjects(InObjects);
	if (ResolvedObjects.IsEmpty())
	{
		return false;
	}

	int32 AffectedCount = 0;

	FRemoteControlPresetExposeArgs ExposeArgs;

	for (const FRCSignatureField& Field : Fields)
	{
		if (!Field.bEnabled)
		{
			continue;
		}

		// Resolve, not load. It should be loaded already if relevant to the Objects to apply this signature to
		UClass* SupportedClass = Field.SupportedClass.ResolveClass();

		// Copy field path as it's going to be used to resolve (mutable)
		FRCFieldPathInfo Path = Field.FieldPath;

		for (UObject* Object : ResolvedObjects)
		{
			UObject* Context = UE::RemoteControl::Private::FindContext(Field, Object, SupportedClass);

			// Attempt to resolve the path from the given context
			if (!Context || !Path.Resolve(Context))
			{
				continue;
			}

			// If path resolved, expose the property
			FRCSignatureActionContext ActionContext;
			ActionContext.Preset = InPreset;
			ActionContext.Object = Object;
			ActionContext.Property = InPreset->FindExposedProperty(Context, Field.FieldPath);

			// Expose the existing property if not existing already.
			if (!ActionContext.Property.IsValid())
			{
				ActionContext.Property = InPreset->ExposeProperty(Context, Field.FieldPath, ExposeArgs).Pin();
			}

			if (!ActionContext.Property.IsValid())
			{
				continue;
			}

			// Property was exposed successfully, increase affected count
			++AffectedCount;

			// Execute the actions for the newly exposed property
			for (const FRCSignatureActionInstance& Action : Field.Actions)
			{
				Action.Execute(ActionContext);
			}
		}
	}

	return AffectedCount;
}
