// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureActionInstance.h"
#include "RCSignatureAction.h"
#include "UObject/StructOnScope.h"

FRCSignatureActionInstance::FRCSignatureActionInstance(const UScriptStruct* InScriptStruct, const FRCSignatureField& InFieldOwner)
{
	ActionInstance.InitializeAsScriptStruct(InScriptStruct, /*StructMemory*/nullptr);

	if (FRCSignatureAction* Action = ActionInstance.GetMutablePtr())
	{
		checkSlow(Action->IsSupported(InFieldOwner));
		Action->Initialize(InFieldOwner);
	}
}

const FRCSignatureAction* FRCSignatureActionInstance::GetAction() const
{
	return ActionInstance.GetPtr();
}

TSharedRef<FStructOnScope> FRCSignatureActionInstance::MakeStructOnScope()
{
	return MakeShared<FStructOnScope>(ActionInstance.GetScriptStruct(), ActionInstance.GetMutableMemory());
}

void FRCSignatureActionInstance::PostLoad(const FRCSignatureField& InFieldOwner)
{
	if (FRCSignatureAction* Action = ActionInstance.GetMutablePtr())
	{
		return Action->Initialize(InFieldOwner);
	}
}

bool FRCSignatureActionInstance::Execute(const FRCSignatureActionContext& InContext) const
{
	if (const FRCSignatureAction* Action = ActionInstance.GetPtr())
	{
		return Action->Execute(InContext);
	}
	return false;
}

#if WITH_EDITOR
void FRCSignatureActionInstance::PostEditChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (FRCSignatureAction* Action = ActionInstance.GetMutablePtr())
	{
		Action->PostEditChange(InPropertyChangedEvent, InPropertyThatChanged);
	}
}
#endif
