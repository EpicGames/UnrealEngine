// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureRegistry.h"

#define LOCTEXT_NAMESPACE "RCSignatureRegistry"

const FRCSignature* URCSignatureRegistry::FindSignature(const FGuid& InSignatureId) const
{
	return Signatures.FindByKey(InSignatureId);
}

FRCSignature* URCSignatureRegistry::FindSignatureMutable(const FGuid& InSignatureId)
{
	return Signatures.FindByKey(InSignatureId);
}

FRCSignature& URCSignatureRegistry::AddSignature()
{
	FRCSignature& Signature = Signatures.AddDefaulted_GetRef();
	Signature.DisplayName = LOCTEXT("NewSignatureDisplayName", "New Signature");
	Signature.Id = FGuid::NewGuid();
	return Signature;
}

int32 URCSignatureRegistry::RemoveSignature(const FGuid& InSignatureId)
{
	return Signatures.RemoveAll([&InSignatureId](const FRCSignature& InSignature)
		{
			return InSignature.Id == InSignatureId;
		});
}

void URCSignatureRegistry::EmptySignatures()
{
	Signatures.Empty();
}

void URCSignatureRegistry::PostLoad()
{
	Super::PostLoad();

	for (FRCSignature& Signature : Signatures)
	{
		Signature.PostLoad();
	}
}

#undef LOCTEXT_NAMESPACE
