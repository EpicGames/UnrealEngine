// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCVirtualProperty.h"

#include "Containers/Set.h"
#include "IStructDeserializerBackend.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "UObject/TextProperty.h"
#include "UObject/StructOnScope.h"

template<typename T>
concept CHasMethodGetPropertyValueInContainer = requires(T && InObject, uint8 * InPtr)
{
	{
		InObject.GetPropertyValue_InContainer(InPtr)
	};
};

const FProperty* URCVirtualPropertyBase::GetProperty() const
{
	if (const FPropertyBagPropertyDesc* PropertyBagPropertyDesc = GetBagPropertyDesc())
	{
		return PropertyBagPropertyDesc->CachedProperty;
	}

	return nullptr;
}

FProperty* URCVirtualPropertyBase::GetProperty()
{
	if (const FPropertyBagPropertyDesc* PropertyBagPropertyDesc = GetBagPropertyDesc())
	{
		return const_cast<FProperty*>(PropertyBagPropertyDesc->CachedProperty);
	}

	return nullptr;
}

const uint8* URCVirtualPropertyBase::GetContainerPtr() const
{
	if (const FInstancedPropertyBag* InstancedPropertyBag = GetPropertyBagInstance())
	{
		return InstancedPropertyBag->GetValue().GetMemory();
	}
	
	return GetContainerPtr();
}

uint8* URCVirtualPropertyBase::GetContainerPtr()
{
	if (const FInstancedPropertyBag* InstancedPropertyBag = GetPropertyBagInstance())
	{
		return const_cast<FInstancedPropertyBag*>(InstancedPropertyBag)->GetMutableValue().GetMemory();
	}
	
	return nullptr;
}

const uint8* URCVirtualPropertyBase::GetValuePtr() const
{
	if (const uint8* ContainerPtr = GetContainerPtr())
	{
		if (const FProperty* Property = GetProperty())
		{
			return Property->ContainerPtrToValuePtr<uint8>(ContainerPtr);
		}
	}

	return nullptr;
}

uint8* URCVirtualPropertyBase::GetValuePtr()
{
	if (uint8* ContainerPtr = GetContainerPtr())
	{
		if (FProperty* Property = GetProperty())
		{
			return Property->ContainerPtrToValuePtr<uint8>(ContainerPtr);
		}
	}

	return nullptr;
}

const FPropertyBagPropertyDesc* URCVirtualPropertyBase::GetBagPropertyDesc() const
{
	if (const FInstancedPropertyBag* InstancedPropertyBag = GetPropertyBagInstance())
	{
		return InstancedPropertyBag->FindPropertyDescByName(PropertyName);
	}
	
	return nullptr;
}

const FInstancedPropertyBag* URCVirtualPropertyBase::GetPropertyBagInstance() const
{
	// That is should be implemented in child classes
	check(false);
	return nullptr;
}

EPropertyBagPropertyType URCVirtualPropertyBase::GetValueType() const
{
	if (const FPropertyBagPropertyDesc* PropertyBagPropertyDesc = GetBagPropertyDesc())
	{
		return PropertyBagPropertyDesc->ValueType;
	}

	return EPropertyBagPropertyType::None;
}

FString URCVirtualPropertyBase::GetMetadataValue(FName Key) const
{
	const FString* MapString = Metadata.Find(Key);
	return MapString ? *MapString : "";
}

void URCVirtualPropertyBase::SetMetadataValue(FName Key, FString Data)
{
	Metadata.FindOrAdd(Key) = Data;
}

void URCVirtualPropertyBase::RemoveMetadataValue(FName Key)
{
	FString OutString;
	Metadata.RemoveAndCopyValue(Key, OutString);
}

const UObject* URCVirtualPropertyBase::GetValueTypeObjectWeakPtr() const
{
	if (const FPropertyBagPropertyDesc* PropertyBagPropertyDesc = GetBagPropertyDesc())
	{
		return PropertyBagPropertyDesc->ValueTypeObject;
	}

	return nullptr;
}

void URCVirtualPropertyBase::SerializeToBackend(IStructSerializerBackend& OutBackend)
{
	FStructSerializerPolicies Policies; 
	Policies.MapSerialization = EStructSerializerMapPolicies::Array;
	FStructSerializer::SerializeElement(GetContainerPtr(), GetProperty(), INDEX_NONE, OutBackend, Policies);
}

bool URCVirtualPropertyBase::DeserializeFromBackend(IStructDeserializerBackend& InBackend)
{
	const UPropertyBag* TypeInfo = GetPropertyBagInstance()->GetPropertyBagStruct();
	UPropertyBag* TypeInfoNonConst = const_cast<UPropertyBag*>(TypeInfo);

	bool bSuccess = false;
	FStructDeserializerPolicies Policies;
	const FProperty* Property = GetProperty();
	if (Property->GetClass() == FStructProperty::StaticClass())
	{
		Policies.PropertyFilter = [&Property](const FProperty* CurrentProp, const FProperty* ParentProp)
		{
			return CurrentProp == Property || ParentProp == Property;
		};
		bSuccess = FStructDeserializer::Deserialize(GetContainerPtr(), *TypeInfoNonConst, InBackend, Policies);
	}
	else
	{
		Policies.PropertyFilter = [&Property](const FProperty* CurrentProp, const FProperty* ParentProp)
		{
			return CurrentProp == Property;
		};
		bSuccess = FStructDeserializer::DeserializeElement(GetContainerPtr(), *TypeInfoNonConst, INDEX_NONE, InBackend, Policies);
	}

	if (bSuccess)
	{
		OnModifyPropertyValue();
	}

	return bSuccess;
}

bool URCVirtualPropertyBase::IsNumericType() const
{
	const FProperty* Property = GetProperty();

	return Property && Property->IsA(FNumericProperty::StaticClass());
}

bool URCVirtualPropertyBase::IsVectorType() const
{
	const FProperty* Property = GetProperty();
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct == TBaseStructure<FVector>::Get();
	}

	return false;
}

bool URCVirtualPropertyBase::IsVector2DType() const
{
	const FProperty* Property = GetProperty();
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct == TBaseStructure<FVector2D>::Get();
	}

	return false;
}

bool URCVirtualPropertyBase::IsColorType() const
{
	const FProperty* Property = GetProperty();
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct == TBaseStructure<FColor>::Get();
	}

	return false;
}

bool URCVirtualPropertyBase::IsLinearColorType() const
{
	const FProperty* Property = GetProperty();
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct == TBaseStructure<FLinearColor>::Get();
	}

	return false;
}

bool URCVirtualPropertyBase::IsRotatorType() const
{
	const FProperty* Property = GetProperty();
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		return StructProperty->Struct == TBaseStructure<FRotator>::Get();
	}

	return false;
}

bool URCVirtualPropertyBase::IsValueEqual(URCVirtualPropertyBase* InVirtualProperty) const
{
	const FProperty* ThisProperty = GetProperty();
	const uint8* ThisContainerPtr = GetContainerPtr();
	if (ThisProperty == nullptr || ThisContainerPtr == nullptr)
	{
		return false;
	}
	
	const FProperty* CompareProperty = InVirtualProperty->GetProperty();;
	const uint8* CompareContainerPtr = InVirtualProperty->GetContainerPtr();
	if (CompareProperty == nullptr || CompareContainerPtr == nullptr)
	{
		return false;
	}

	if (ThisProperty->GetClass() != CompareProperty->GetClass())
	{
		return false;
	}

	const uint8* ThisValuePtr = ThisProperty->ContainerPtrToValuePtr<uint8>(ThisContainerPtr);
	const uint8* CompareValuePtr = CompareProperty->ContainerPtrToValuePtr<uint8>(CompareContainerPtr);

	return ThisProperty->Identical(ThisValuePtr, CompareValuePtr);
}

bool URCVirtualPropertyBase::CopyCompleteValue(const FProperty* InTargetProperty, uint8* InTargetValuePtr)
{
	const FProperty* SourceProperty = GetProperty();

	if (SourceProperty == nullptr || InTargetProperty == nullptr || InTargetValuePtr == nullptr)
	{
		ensureMsgf(false, TEXT("Invalid input passed to CopyCompleteValue"));

		return false;
	}

	if (SourceProperty->GetClass() != InTargetProperty->GetClass())
	{
		ensureMsgf(false, TEXT("Invalid property type passed to CopyCompleteValue.\nExpected: %s, Found: %s"), *SourceProperty->GetClass()->GetName(), *InTargetProperty->GetClass()->GetName());

		return false;
	}

	SourceProperty->CopyCompleteValue(InTargetValuePtr /*Dest*/, GetValuePtr() /*Source*/);

	return true;

}

bool URCVirtualPropertyBase::CopyCompleteValue(const FProperty* InTargetProperty, uint8* InTargetValuePtr, bool bPassByteEnumPropertyComparison)
{
	const FProperty* SourceProperty = GetProperty();

	if (SourceProperty == nullptr || InTargetProperty == nullptr || InTargetValuePtr == nullptr)
	{
		ensureMsgf(false, TEXT("Invalid input passed to CopyCompleteValue"));

		return false;
	}

	bool bAreClassDifferent = SourceProperty->GetClass() != InTargetProperty->GetClass();

	//FByteProperties are saved inside RC as FEnumProperty this will cause the original property and the RC property to differ.
	//If it is the case than check if both has the same enum, and if they have ignore that they are different types.
	if (bAreClassDifferent)
	{
		bool bShouldIgnore = false;
		if (bPassByteEnumPropertyComparison)
		{
			if (const FEnumProperty* SourceEnumProperty = CastField<FEnumProperty>(SourceProperty))
			{
				if (const FByteProperty* ByteTargetProperty = CastField<FByteProperty>(InTargetProperty))
				{
					if (SourceEnumProperty->GetEnum() && ByteTargetProperty->Enum)
					{
						bShouldIgnore = SourceEnumProperty->GetEnum()->GetFName() == ByteTargetProperty->Enum->GetFName();
					}
				}
			}
		}
		if (bShouldIgnore == false)
		{
			ensureMsgf(false, TEXT("Invalid property type passed to CopyCompleteValue.\nExpected: %s, Found: %s"), *SourceProperty->GetClass()->GetName(), *InTargetProperty->GetClass()->GetName());

			return false;
		}
	}

	SourceProperty->CopyCompleteValue(InTargetValuePtr /*Dest*/, GetValuePtr() /*Source*/);

	return true;

}

bool URCVirtualPropertyBase::CopyCompleteValue(URCVirtualPropertyBase* InVirtualProperty)
{
	return CopyCompleteValue(InVirtualProperty->GetProperty(), InVirtualProperty->GetValuePtr());
}

struct FRCVirtualPropertyCastHelpers
{
	template<typename TPropertyType, typename TValueType>
	static bool GetPrimitiveValue(const URCVirtualPropertyBase* InVirtualProperty, TValueType& OutValue)
	{
		const uint8* ContainerPtr = InVirtualProperty->GetContainerPtr();
		if (!ContainerPtr)
		{
			return false;
		}
		
		const FProperty* Property = InVirtualProperty->GetProperty();
		
		if (const TPropertyType* CastProperty = CastField<TPropertyType>(Property))
		{
			if constexpr (CHasMethodGetPropertyValueInContainer<TPropertyType>)
			{
				OutValue = CastProperty->GetPropertyValue_InContainer(ContainerPtr);
			}
			else
			{
				FMemory::Memcpy(
					&OutValue,
					CastProperty->template ContainerPtrToValuePtr<void>(ContainerPtr),
					sizeof(OutValue)
				);
			}

			return true;
		}

		return false;
	}

	template<typename TPropertyType, typename TValueType>
	static bool SetPrimitiveValue(URCVirtualPropertyBase* InVirtualProperty, const TValueType& InValue)
	{
		uint8* ValuePtr = InVirtualProperty->GetValuePtr();
		if (!ValuePtr)
		{
			return false;
		}
		
		const FProperty* Property = InVirtualProperty->GetProperty();
		
		if (const TPropertyType* CastProperty = CastField<TPropertyType>(Property))
		{
			CastProperty->SetPropertyValue(ValuePtr, InValue);
			return true;
		}

		return false;
	}

	template<typename TValueType>
	static bool GetStructValue(const URCVirtualPropertyBase* InVirtualProperty, UScriptStruct* InScriptStruct, TValueType* OutValuePtr)
	{
		const uint8* ValuePtr = InVirtualProperty->GetValuePtr();
		if (!ValuePtr)
		{
			return false;
		}

		const FProperty* Property = InVirtualProperty->GetProperty();
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (InScriptStruct == StructProperty->Struct)
			{
				StructProperty->Struct->CopyScriptStruct(OutValuePtr, ValuePtr);
				return true;
			}
		}

		return false;
	}

	template<typename TValueType>
	static bool SetStructValue(URCVirtualPropertyBase* InVirtualProperty, UScriptStruct* InScriptStruct, const TValueType& InValue)
	{
		uint8* ValuePtr = InVirtualProperty->GetValuePtr();
		if (!ValuePtr)
		{
			return false;
		}

		const FProperty* Property = InVirtualProperty->GetProperty();
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (InScriptStruct == StructProperty->Struct)
			{
				StructProperty->Struct->CopyScriptStruct(ValuePtr, &InValue);
				return true;
			}
		}

		return false;
	}

	static UObject* GetObjectValue(const URCVirtualPropertyBase* InVirtualProperty)
	{
		if (InVirtualProperty)
		{
			const FProperty* Property = InVirtualProperty->GetProperty();
			if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				const void* SrcAddress = Property->ContainerPtrToValuePtr<void>(InVirtualProperty->GetContainerPtr());
				return ObjectProperty->GetObjectPropertyValue(SrcAddress);
			}
		}

		return nullptr;
	}

	static bool SetObjectValue(URCVirtualPropertyBase* InVirtualProperty, UObject* InValue)
	{
		const FProperty* Property = InVirtualProperty->GetProperty();
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{			
			ObjectProperty->SetObjectPropertyValue(InVirtualProperty->GetContainerPtr(), InValue);
			return true;
		}

		return false;
	}
};

bool URCVirtualPropertyBase::GetValueBool(bool& OutBoolValue) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FBoolProperty, bool>(this, OutBoolValue);
}

bool URCVirtualPropertyBase::GetValueInt8(int8& OutInt8) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FInt8Property, int8>(this, OutInt8);
}

bool URCVirtualPropertyBase::GetValueByte(uint8& OutByte) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FByteProperty, uint8&>(this, OutByte);
}

bool URCVirtualPropertyBase::GetValueEnum(uint8& OutByte) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FEnumProperty, uint8&>(this, OutByte);
}

bool URCVirtualPropertyBase::GetValueInt16(int16& OutInt16) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FInt16Property, int16&>(this, OutInt16);
}

bool URCVirtualPropertyBase::GetValueUint16(uint16& OutUInt16) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FUInt16Property, uint16&>(this, OutUInt16);
}

bool URCVirtualPropertyBase::GetValueInt32(int32& OutInt32) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FIntProperty, int32&>(this, OutInt32);
}

bool URCVirtualPropertyBase::GetValueUInt32(uint32& OutUInt32) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FUInt32Property, uint32&>(this, OutUInt32);
}

bool URCVirtualPropertyBase::GetValueInt64(int64& OuyInt64) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FInt64Property, int64>(this, OuyInt64);
}

bool URCVirtualPropertyBase::GetValueUint64(uint64& OuyUInt64) const
{
return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FUInt64Property, uint64>(this, OuyUInt64);
}

bool URCVirtualPropertyBase::GetValueFloat(float& OutFloat) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FFloatProperty, float>(this, OutFloat);
}

bool URCVirtualPropertyBase::GetValueDouble(double& OutDouble) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FDoubleProperty, double>(this, OutDouble);
}

bool URCVirtualPropertyBase::GetValueString(FString& OutStringValue) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FStrProperty, FString>(this, OutStringValue);
}

bool URCVirtualPropertyBase::GetValueName(FName& OutNameValue) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FNameProperty, FName>(this, OutNameValue);
}

bool URCVirtualPropertyBase::GetValueText(FText& OutTextValue) const
{
	return FRCVirtualPropertyCastHelpers::GetPrimitiveValue<FTextProperty, FText>(this, OutTextValue);
}

bool URCVirtualPropertyBase::GetValueNumericInteger(int64& OutInt64Value) const
{
	const uint8* ContainerPtr = GetContainerPtr();
	if (!ContainerPtr)
	{
		return false;
	}
		
	const FProperty* Property = GetProperty();
		
	if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
	{
		if (NumericProperty->IsInteger())
		{
			const void* DataResult = NumericProperty->ContainerPtrToValuePtr<void>(ContainerPtr);
			OutInt64Value = reinterpret_cast<int64>(DataResult);
			return true;
		}
	}

	return false;
}

bool URCVirtualPropertyBase::GetValueVector(FVector& OutVector) const
{
	return FRCVirtualPropertyCastHelpers::GetStructValue<FVector>(this, TBaseStructure<FVector>::Get(), &OutVector);
}

bool URCVirtualPropertyBase::GetValueVector2D(FVector2D& OutVector2D) const
{
	return FRCVirtualPropertyCastHelpers::GetStructValue<FVector2D>(this, TBaseStructure<FVector2D>::Get(), &OutVector2D);
}

bool URCVirtualPropertyBase::GetValueRotator(FRotator& OutRotator) const
{
	return FRCVirtualPropertyCastHelpers::GetStructValue<FRotator>(this, TBaseStructure<FRotator>::Get(), &OutRotator);
}

bool URCVirtualPropertyBase::GetValueColor(FColor& OutColor) const
{
	return FRCVirtualPropertyCastHelpers::GetStructValue<FColor>(this, TBaseStructure<FColor>::Get(), &OutColor);
}

bool URCVirtualPropertyBase::GetValueLinearColor(FLinearColor& OutLinearColor) const
{
	return FRCVirtualPropertyCastHelpers::GetStructValue<FLinearColor>(this, TBaseStructure<FLinearColor>::Get(), &OutLinearColor);
}

UObject* URCVirtualPropertyBase::GetValueObject() const
{
	return FRCVirtualPropertyCastHelpers::GetObjectValue(this);
}

FString URCVirtualPropertyBase::GetDisplayValueAsString() const
{
	const FProperty* Property = GetProperty();

	if (const FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
	{
		bool Value;
		GetValueBool(Value);

		return Value ? TEXT("true") : TEXT("false");
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		uint8 Value;
		GetValueByte(Value);

		return FString::Printf(TEXT("%d"), Value);
	}
	else if (const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
	{
		FText Value;
		GetValueText(Value);

		return Value.ToString();
	}
	else if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
	{
		FString Value;
		GetValueString(Value);

		return Value;
	}
	else if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
	{
		FName Value;
		GetValueName(Value);

		return Value.ToString();
	}
	else if (const FIntProperty* IntProperty = CastField<FIntProperty>(Property))
	{
		int32 Value;
		GetValueInt32(Value);

		return FString::Printf(TEXT("%d"), Value);
	}
	else if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
	{
		float Value;
		GetValueFloat(Value);

		return FString::Printf(TEXT("%f"), Value);
	}
	else if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
	{
		double Value;
		GetValueDouble(Value);

		return FString::Printf(TEXT("%lf"), Value);
	}
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
	{
		if (const UObject* Object = GetValueObject())
		{
			return Object->GetPathName();
		}
	}
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		if (UEnum* Enum = EnumProperty->GetEnum())
		{
			uint8 Value;
			GetValueEnum(Value);

			return Enum->GetDisplayNameTextByValue(static_cast<int64>(Value)).ToString();
		}
	}
	else if (IsVectorType())
	{
		FVector Vector;
		GetValueVector(Vector);

		return Vector.ToString();
	}
	else if (IsVector2DType())
	{
		FVector2D Vector2D;
		GetValueVector2D(Vector2D);

		return Vector2D.ToString();
	}
	else if (IsRotatorType())
	{
		FRotator Rotator;
		GetValueRotator(Rotator);

		return Rotator.ToString();
	}
	else if (IsColorType())
	{
		FColor Color;
		GetValueColor(Color);

		return Color.ToString();
	}
	else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		FString Value;
		StructProperty->ExportTextItem_Direct(
			Value,
			GetValuePtr(),
			/* Default value ptr */ nullptr,
			/* Parent */ nullptr,
			/* Port flags */ 0
		);

		return Value;
	}

	// Unimplemented type!
	ensureAlwaysMsgf(false, TEXT("Unimplemented type %s passed to GetDisplayValueAsString"), *Property->GetName());
		 
	return "Unimplemented type";
}

bool URCVirtualPropertyBase::SetValueBool(bool InBoolValue)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FBoolProperty, bool>(this, InBoolValue);
}

bool URCVirtualPropertyBase::SetValueInt8(const int8 InInt8)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FInt8Property, int8>(this, InInt8);
}

bool URCVirtualPropertyBase::SetValueByte(const uint8 InByte)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FByteProperty, uint8>(this, InByte);
}

bool URCVirtualPropertyBase::SetValueInt16(const int16 InInt16)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FInt16Property, int16>(this, InInt16);
}

bool URCVirtualPropertyBase::SetValueUint16(const uint16 InUInt16)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FUInt16Property, uint16>(this, InUInt16);
}

bool URCVirtualPropertyBase::SetValueInt32(const int32 InInt32)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FIntProperty, int32>(this, InInt32);
}

bool URCVirtualPropertyBase::SetValueUInt32(const uint32 InUInt32)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FUInt32Property, uint32>(this, InUInt32);
}

bool URCVirtualPropertyBase::SetValueInt64(const int64 InInt64)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FInt64Property, int64>(this, InInt64);
}

bool URCVirtualPropertyBase::SetValueUint64(const uint64 InUInt64)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FUInt64Property, uint64>(this, InUInt64);
}

bool URCVirtualPropertyBase::SetValueFloat(const float InFloat)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FFloatProperty, float>(this, InFloat);
}

bool URCVirtualPropertyBase::SetValueDouble(const double InDouble)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FDoubleProperty, double>(this, InDouble);
}

bool URCVirtualPropertyBase::SetValueString(const FString& InStringValue)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FStrProperty, FString>(this, InStringValue);
}

bool URCVirtualPropertyBase::SetValueName(const FName& InNameValue)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FNameProperty, FName>(this, InNameValue);
}

bool URCVirtualPropertyBase::SetValueText(const FText& InTextValue)
{
	return FRCVirtualPropertyCastHelpers::SetPrimitiveValue<FTextProperty, FText>(this, InTextValue);
}

bool URCVirtualPropertyBase::SetValueNumericInteger(const int64 InInt64Value)
{
	uint8* ValuePtr = GetValuePtr();
	if (!ValuePtr)
	{
		return false;
	}
		
	const FProperty* Property = GetProperty();
		
	if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
	{
		if (NumericProperty->IsInteger())
		{
			NumericProperty->SetIntPropertyValue(ValuePtr, InInt64Value);
		}
	}

	return false;
}

bool URCVirtualPropertyBase::SetValueVector(const FVector& InVector)
{
	return FRCVirtualPropertyCastHelpers::SetStructValue<FVector>(this, TBaseStructure<FVector>::Get(), InVector);
}

bool URCVirtualPropertyBase::SetValueVector2D(const FVector2D& InVector2D)
{
	return FRCVirtualPropertyCastHelpers::SetStructValue<FVector2D>(this, TBaseStructure<FVector2D>::Get(), InVector2D);
}

bool URCVirtualPropertyBase::SetValueRotator(const FRotator& InRotator)
{
	return FRCVirtualPropertyCastHelpers::SetStructValue<FRotator>(this, TBaseStructure<FRotator>::Get(), InRotator);
}

bool URCVirtualPropertyBase::SetValueColor(const FColor& InColor)
{
	return FRCVirtualPropertyCastHelpers::SetStructValue<FColor>(this, TBaseStructure<FColor>::Get(), InColor);
}

bool URCVirtualPropertyBase::SetValueLinearColor(const FLinearColor& InLinearColor)
{
	return FRCVirtualPropertyCastHelpers::SetStructValue<FLinearColor>(this, TBaseStructure<FLinearColor>::Get(), InLinearColor);
}

FName URCVirtualPropertyBase::GetPropertyName() const
{
	return PropertyName;
}

FName URCVirtualPropertyBase::GetVirtualPropertyTypeDisplayName(const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject)
{
	const UEnum* Enum = FindObjectChecked<UEnum>(nullptr, TEXT("/Script/StructUtils.EPropertyBagPropertyType"));
	FString PropertyTypeString = Enum->GetValueAsName(InValueType).ToString();
	PropertyTypeString.RemoveFromStart("EPropertyBagPropertyType::");

	// User-friendly typename alias
	if (PropertyTypeString == "Bool")
	{
		PropertyTypeString = "Boolean";
	}
	else if (PropertyTypeString == "Int32")
	{
		PropertyTypeString = TEXT("Integer");
	}
	
	// For Structs and Objects we should derive the prefix from the subobject type (Vector / Color / Rotator / etc)
	// or Object type (StaticMesh, MaterialInterface/etc) rather than the property bag type (which is always "Struct" or "Object")
	if (InValueTypeObject)
	{
		PropertyTypeString = InValueTypeObject->GetName();

		if (PropertyTypeString == TEXT("StaticMesh"))
		{
			PropertyTypeString = TEXT("Mesh");
		}
		else if (PropertyTypeString == TEXT("MaterialInterface"))
		{
			PropertyTypeString = TEXT("Material");
		}
	}

	return *PropertyTypeString;
}

void URCVirtualPropertyBase::SetDisplayIndex(const int32 InDisplayIndex)
{
	DisplayIndex = InDisplayIndex;
}

const FInstancedPropertyBag* URCVirtualPropertyInContainer::GetPropertyBagInstance() const
{
	if (ContainerWeakPtr.Get())
	{
		return &ContainerWeakPtr->Bag;
	}
	
	return Super::GetPropertyBagInstance();
}

void URCVirtualPropertySelfContainer::AddProperty(const FName InPropertyName, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject)
{
	if (Bag.GetNumPropertiesInBag() > 0)
	{
		return;
	}
	
	PropertyName = InPropertyName;
	Id = FGuid::NewGuid();
	
	Bag.AddProperty(PropertyName, InValueType, InValueTypeObject);
}

bool URCVirtualPropertySelfContainer::DuplicateProperty(const FName& InPropertyName, const FProperty* InSourceProperty)
{
	if (Bag.GetNumPropertiesInBag() > 0)
	{
		return false;
	}

	PropertyName = InPropertyName;
	Id = FGuid::NewGuid();
	
	Bag.AddProperty(InPropertyName, InSourceProperty);
	const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(InPropertyName);
	check(BagPropertyDesc);

	return true;
}

bool URCVirtualPropertySelfContainer::DuplicatePropertyWithCopy(const FName& InPropertyName, const FProperty* InSourceProperty, const uint8* InSourceContainerPtr)
{
	if (!DuplicateProperty(InPropertyName, InSourceProperty))
	{
		return false;
	}
	
	if (InSourceContainerPtr)
	{
		return Bag.SetValue(PropertyName, InSourceProperty, InSourceContainerPtr) == EPropertyBagResult::Success;	
	}
	return false;
}

bool URCVirtualPropertySelfContainer::DuplicatePropertyWithCopy(URCVirtualPropertyBase* InVirtualProperty)
{
	if (!DuplicateProperty(InVirtualProperty->PropertyName, InVirtualProperty->GetProperty()))
	{
		return false;
	}
	
	if (InVirtualProperty->GetContainerPtr())
	{
		return Bag.SetValue(PropertyName, InVirtualProperty->GetProperty(), InVirtualProperty->GetContainerPtr()) == EPropertyBagResult::Success;	
	}

	return false;
}

bool URCVirtualPropertySelfContainer::UpdateValueWithProperty(const FProperty* InProperty, const void* InPropertyContainer)
{
	const EPropertyBagResult Result = Bag.SetValue(PropertyName, InProperty, InPropertyContainer);
	return Result == EPropertyBagResult::Success;
}

bool URCVirtualPropertySelfContainer::UpdateValueWithProperty(const URCVirtualPropertyBase* InVirtualProperty)
{
	if (InVirtualProperty && InVirtualProperty->GetContainerPtr())
	{
		return Bag.SetValue(PropertyName, InVirtualProperty->GetProperty(), InVirtualProperty->GetContainerPtr()) == EPropertyBagResult::Success;	
	}

	return false;
}

void URCVirtualPropertySelfContainer::Reset()
{
	PropertyName = NAME_None;
	FieldId = NAME_None;
	Id = FGuid();
	Bag.Reset();
}

const FInstancedPropertyBag* URCVirtualPropertySelfContainer::GetPropertyBagInstance() const
{
	return &Bag;
}

TSharedPtr<FStructOnScope> URCVirtualPropertySelfContainer::CreateStructOnScope()
{
	TSharedRef<FStructOnScope> StructOnScope = MakeShared<FStructOnScope>(Bag.GetPropertyBagStruct(), Bag.GetMutableValue().GetMemory());
	if (PresetWeakPtr.IsValid() && PresetWeakPtr->GetPackage())
	{
		StructOnScope->SetPackage(PresetWeakPtr->GetPackage());
	}
	return StructOnScope;
}
