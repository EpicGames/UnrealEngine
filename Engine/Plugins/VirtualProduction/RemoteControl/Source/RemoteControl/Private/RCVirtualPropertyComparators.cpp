// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCVirtualProperty.h"

/** Generates functions that Compare this object with a given Virtual Property, using a given Comparator */
#define IMPLEMENT_RCLOGIC_COMPARATOR( CompareFunctionName, Comparator)  \
\
bool URCVirtualPropertyBase::CompareFunctionName(URCVirtualPropertyBase* InVirtualProperty) const\
{ \
	const FProperty* Property = GetProperty(); \
	                                                                      \
/** Note: Currently numeric Controller types are limited to int32 and float, so only these are implemented*/ \
	if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property)) \
	{ \
        /** Numeric Integers (potentially covers int32, uint32, int8, int16, uint16, int32, uint32, int64, uint64) */ \
		if (NumericProperty->IsInteger()) \
		{ \
            /** int32 */ \
			if (const FIntProperty* IntProperty = CastField<FIntProperty>(Property)) \
			{ \
				int32 LHSValue, RHSValue; \
				this->GetValueInt32(LHSValue); \
				InVirtualProperty->GetValueInt32(RHSValue); \
				                                                                          \
				/** Comparison */                                              \
				return LHSValue Comparator RHSValue; \
			} \
		} \
		else /** Numeric Floating Point(potentially covers float and double) */ \
		{ \
            /** float */ \
			if (const FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property)) \
			{ \
				float LHSValue, RHSValue; \
				this->GetValueFloat(LHSValue); \
				InVirtualProperty->GetValueFloat(RHSValue); \
				                                                                         \
				/** Comparison */                                             \
				return LHSValue Comparator RHSValue; \
			} \
            /** double */ \
			if (const FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property)) \
			{ \
				double LHSValue, RHSValue; \
				this->GetValueDouble(LHSValue); \
				InVirtualProperty->GetValueDouble(RHSValue); \
				                                                                         \
				/** Comparison */                                             \
				return LHSValue Comparator RHSValue; \
			} \
		} \
	} \
	else \
	{ \
		ensureAlwaysMsgf(false, TEXT("Only numeric types are supported for value comparison")); \
	} \
	ensureAlwaysMsgf(false, TEXT("Unimplemented Numeric Type %s passed for value comparison!/nSee above for currently supported numeric types"), *Property->GetName()); \
	return false; \
} \

IMPLEMENT_RCLOGIC_COMPARATOR(IsValueGreaterThan, >)
IMPLEMENT_RCLOGIC_COMPARATOR(IsValueGreaterThanOrEqualTo, >= )
IMPLEMENT_RCLOGIC_COMPARATOR(IsValueLesserThan, < )
IMPLEMENT_RCLOGIC_COMPARATOR(IsValueLesserThanOrEqualTo, <= )

#undef IMPLEMENT_RCLOGIC_COMPARATOR