// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCVirtualPropertyContainer.h"

#include "RCVirtualProperty.h"
#include "Templates/SubclassOf.h"
#include "UObject/StructOnScope.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "URCVirtualPropertyInContainer"

namespace UE::RemoteControl::Private
{
	void SplitNameAndNumber(FString& InOutName, FString& OutNumber)
	{
		for (int32 Index = InOutName.Len() - 1; Index >= 0; Index--)
		{
			// Stop on first index that isn't a digit
			if (!FChar::IsDigit(InOutName[Index]))
			{
				const FStringView NumberView = FStringView(InOutName).RightChop(Index + 1);
				if (NumberView.Len() > 0)
				{
					// Process number (right-side) first before name gets modified inlined to only contain the left side
					OutNumber = NumberView;
					InOutName.LeftInline(Index + 1);
					return;
				}
				// Fallback in case there was no digit found
				OutNumber = TEXT("1");
				return;
			}
		}

		// The entire string is a number (never stopped at the first non-digit index)
		OutNumber = MoveTemp(InOutName);
		InOutName.Reset();
	}

	const FString::ElementType* IncrementNumber(FString& InNumber)
	{
		int32 LeadingZeros = 0;

		// Count leading zeros
		for (const FString::ElementType& Digit : InNumber)
		{
			if (Digit == TEXT('0'))
			{
				++LeadingZeros;
			}
			else
			{
				break;
			}
		}

		// If the first non-zero digit is a 9, it means the incremented number will take up the space of a leading zero
		if (LeadingZeros < InNumber.Len() && InNumber[LeadingZeros] == TEXT('9'))
		{
			--LeadingZeros;
		}

		TArray<FString::ElementType> LeadingZeroString;
		if (LeadingZeros > 0)
		{
			LeadingZeroString.Init(TEXT('0'), LeadingZeros);
		}

		InNumber = FString(MoveTemp(LeadingZeroString)) + FString::FromInt(FCString::Atoi(*InNumber) + 1);
		return *InNumber;
	}
}

void URCVirtualPropertyContainerBase::AddVirtualProperty(URCVirtualPropertyBase* InVirtualProperty)
{
	if(ensure(InVirtualProperty))
	{
		// Add property to Set
		VirtualProperties.Add(InVirtualProperty);
	}
}

URCVirtualPropertyInContainer* URCVirtualPropertyContainerBase::AddProperty(const FName& InPropertyName, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject, TArray<FPropertyBagPropertyDescMetaData> MetaData/* = TArray<FPropertyBagPropertyDescMetaData>()*/)
{
	const FName PropertyName = GenerateUniquePropertyName(InPropertyName, InValueType, InValueTypeObject, this);

	FPropertyBagPropertyDesc PropertyBagDesc = FPropertyBagPropertyDesc(PropertyName, InValueType, InValueTypeObject);

#if WITH_EDITORONLY_DATA
	PropertyBagDesc.MetaData = MetaData;
#endif

	// Create Property in Container
	URCVirtualPropertyInContainer* VirtualPropertyInContainer = NewObject<URCVirtualPropertyInContainer>(this, InPropertyClass.Get(), NAME_None, RF_Transactional);
	VirtualPropertyInContainer->PropertyName = PropertyName;
	VirtualPropertyInContainer->DisplayName = PropertyName;
	VirtualPropertyInContainer->PresetWeakPtr = PresetWeakPtr;
	VirtualPropertyInContainer->ContainerWeakPtr = this;
	VirtualPropertyInContainer->Id = FGuid::NewGuid();

	VirtualPropertyInContainer->InitPropertyBagPropertyDescription(PropertyBagDesc);

	Bag.AddProperties({ PropertyBagDesc });

	// Ensure that the property has been successfully added to the Bag
	const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(PropertyName);
	if (!ensure(BagPropertyDesc))
	{
		return nullptr;
	}
	
	ControllerLabelToIdCache.Add(PropertyName, VirtualPropertyInContainer->Id);
	AddVirtualProperty(VirtualPropertyInContainer);

	return VirtualPropertyInContainer;
}

URCVirtualPropertyInContainer* URCVirtualPropertyContainerBase::DuplicateProperty(const FName& InPropertyName, const FProperty* InSourceProperty, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass)
{
	// Ensure that the property being duplicated is not already a part of the Bag (not to be confused with a similar looking ensure performed above in AddProperty, this is the exact inverse!)
	const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(InPropertyName);
	if (!ensure(!BagPropertyDesc))
	{
		return nullptr;
	}

	Bag.AddProperty(InPropertyName, InSourceProperty);
	
	URCVirtualPropertyInContainer* VirtualPropertyInContainer = NewObject<URCVirtualPropertyInContainer>(this, InPropertyClass.Get());
	VirtualPropertyInContainer->PropertyName = InPropertyName;
	VirtualPropertyInContainer->DisplayName = GenerateUniqueDisplayName(VirtualPropertyInContainer->DisplayName, this);;
	VirtualPropertyInContainer->PresetWeakPtr = PresetWeakPtr;
	VirtualPropertyInContainer->ContainerWeakPtr = this;
	VirtualPropertyInContainer->Id = FGuid::NewGuid();

	ControllerLabelToIdCache.Add(InPropertyName, VirtualPropertyInContainer->Id);
	AddVirtualProperty(VirtualPropertyInContainer);

	return VirtualPropertyInContainer;
}

URCVirtualPropertyInContainer* URCVirtualPropertyContainerBase::DuplicatePropertyWithCopy(const FName& InPropertyName, const FProperty* InSourceProperty, const uint8* InSourceContainerPtr, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass)
{
	if (InSourceContainerPtr == nullptr)
	{
		return nullptr;
	}
	
	URCVirtualPropertyInContainer* VirtualPropertyInContainer = DuplicateProperty(InPropertyName, InSourceProperty, InPropertyClass);
	if (VirtualPropertyInContainer == nullptr)
	{
		return nullptr;
	}

	const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(InPropertyName);
	check(BagPropertyDesc); // Property bag should be exists after DuplicateProperty()

	ensure(Bag.SetValue(InPropertyName, InSourceProperty, InSourceContainerPtr) == EPropertyBagResult::Success);

	return VirtualPropertyInContainer;
}

URCVirtualPropertyInContainer* URCVirtualPropertyContainerBase::DuplicateVirtualProperty(URCVirtualPropertyInContainer* InVirtualProperty)
{
	if (URCVirtualPropertyInContainer* NewVirtualProperty = DuplicateObject<URCVirtualPropertyInContainer>(InVirtualProperty, InVirtualProperty->GetOuter()))
	{
		NewVirtualProperty->PropertyName = GenerateUniquePropertyName(InVirtualProperty->PropertyName, this);
		NewVirtualProperty->DisplayName = GenerateUniqueDisplayName(InVirtualProperty->DisplayName, this);
		NewVirtualProperty->Id = FGuid::NewGuid();

		// Sync Property Bag
		Bag.AddProperty(NewVirtualProperty->PropertyName, InVirtualProperty->GetProperty());

		// Ensure that the property has been successfully added to the Bag
		const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(NewVirtualProperty->PropertyName);
		if (!ensure(BagPropertyDesc))
		{
			return nullptr;
		}

		// Sync Virtual Properties List
		AddVirtualProperty(NewVirtualProperty);

		// Sync Cache
		ControllerLabelToIdCache.Add(NewVirtualProperty->DisplayName, NewVirtualProperty->Id);

		return NewVirtualProperty;
	}

	return nullptr;
}

bool URCVirtualPropertyContainerBase::HasVirtualProperty(const URCVirtualPropertyBase* InVirtualProperty) const
{
	return InVirtualProperty && VirtualProperties.Contains(InVirtualProperty);
}

bool URCVirtualPropertyContainerBase::RemoveProperty(const FName& InPropertyName)
{
	Bag.RemovePropertyByName(InPropertyName);

	for (TSet<TObjectPtr<URCVirtualPropertyBase>>::TIterator PropertiesIt = VirtualProperties.CreateIterator(); PropertiesIt; ++PropertiesIt)
	{
		if (const URCVirtualPropertyBase* VirtualProperty = *PropertiesIt)
		{
			if (VirtualProperty->PropertyName == InPropertyName)
			{
				ControllerLabelToIdCache.Remove(VirtualProperty->DisplayName);
				PropertiesIt.RemoveCurrent();
				return true;
			}
		}
	}

	return false;
}

void URCVirtualPropertyContainerBase::Reset()
{
	VirtualProperties.Empty();
	ControllerLabelToIdCache.Reset();
	Bag.Reset();
}

URCVirtualPropertyBase* URCVirtualPropertyContainerBase::GetVirtualProperty(const FName InPropertyName) const
{
	for (URCVirtualPropertyBase* VirtualProperty : VirtualProperties)
	{
		if (!ensure(VirtualProperty))
		{
			continue;
		}

		if (VirtualProperty->PropertyName == InPropertyName)
		{
			return VirtualProperty;
		}
	}
	
	return nullptr;
}

URCVirtualPropertyBase* URCVirtualPropertyContainerBase::GetVirtualProperty(const FGuid& InId) const
{
	for (URCVirtualPropertyBase* VirtualProperty : VirtualProperties)
	{
		if (VirtualProperty->Id == InId)
		{
			return VirtualProperty;
		}
	}

	return nullptr;
}

URCVirtualPropertyBase* URCVirtualPropertyContainerBase::GetVirtualPropertyByDisplayName(const FName InDisplayName) const
{
	if (const FGuid* ControllerId = ControllerLabelToIdCache.Find(InDisplayName))
	{
		if (URCVirtualPropertyBase* Controller = GetVirtualProperty(*ControllerId))
		{
			return Controller;
		}
	}

	for (URCVirtualPropertyBase* VirtualProperty : VirtualProperties)
	{
		if (VirtualProperty->DisplayName == InDisplayName)
		{
			return VirtualProperty;
		}
	}

	return nullptr;
}

URCVirtualPropertyBase* URCVirtualPropertyContainerBase::GetVirtualPropertyByFieldId(const FName InFieldId) const
{
	for (URCVirtualPropertyBase* VirtualProperty : VirtualProperties)
	{
		if (VirtualProperty->FieldId == InFieldId)
		{
			return VirtualProperty;
		}
	}

	return nullptr;
}

URCVirtualPropertyBase* URCVirtualPropertyContainerBase::GetVirtualPropertyByFieldId(const FName InFieldId, const EPropertyBagPropertyType InType) const
{
	const TArray<URCVirtualPropertyBase*>& VirtualPropertiesByFieldId = GetVirtualPropertiesByFieldId(InFieldId);

	for (URCVirtualPropertyBase* VirtualProperty : VirtualPropertiesByFieldId)
	{
		if (VirtualProperty->GetValueType() == InType)
		{
			return VirtualProperty;
		}
	}

	return nullptr;
}

TArray<URCVirtualPropertyBase*> URCVirtualPropertyContainerBase::GetVirtualPropertiesByFieldId(const FName InFieldId) const
{
	TArray<URCVirtualPropertyBase*> VirtualPropertiesByFieldId;
	
	for (URCVirtualPropertyBase* VirtualProperty : VirtualProperties)
	{
		if (VirtualProperty->FieldId == InFieldId)
		{
			VirtualPropertiesByFieldId.Add(VirtualProperty);
		}
	}

	return VirtualPropertiesByFieldId;
}

int32 URCVirtualPropertyContainerBase::GetNumVirtualProperties() const
{
	const int32 NumPropertiesInBag = Bag.GetNumPropertiesInBag();
	const int32 NumVirtualProperties = VirtualProperties.Num();

	check(NumPropertiesInBag == NumVirtualProperties);

	return NumPropertiesInBag;
}

const UPropertyBag* URCVirtualPropertyContainerBase::GetPropertyBagStruct() const
{
	return Bag.GetPropertyBagStruct();
}

FStructView URCVirtualPropertyContainerBase::GetPropertyBagMutableValue()
{
	return Bag.GetMutableValue();
}

TSharedPtr<FStructOnScope> URCVirtualPropertyContainerBase::CreateStructOnScope()
{
	return MakeShared<FStructOnScope>(Bag.GetPropertyBagStruct(), Bag.GetMutableValue().GetMemory());
}

FName URCVirtualPropertyContainerBase::SetControllerDisplayName(FGuid InGuid, FName InNewName)
{
	if (URCVirtualPropertyBase* Controller = GetVirtualProperty(InGuid))
	{
		Controller->Modify();

		ControllerLabelToIdCache.Remove(Controller->DisplayName);
		Controller->DisplayName = GenerateUniqueDisplayName(InNewName, this);
		ControllerLabelToIdCache.Add(Controller->DisplayName, Controller->Id);

		return Controller->DisplayName;
	}

	return NAME_None;
}

FName URCVirtualPropertyContainerBase::GenerateUniquePropertyName(const FName& InPropertyName, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject, const URCVirtualPropertyContainerBase* InContainer)
{
	FName BaseName = InPropertyName;
	if (BaseName.IsNone())
	{
		BaseName = URCVirtualPropertyBase::GetVirtualPropertyTypeDisplayName(InValueType, InValueTypeObject);
	}

	return GenerateUniquePropertyName(BaseName, InContainer);
}

FName URCVirtualPropertyContainerBase::GenerateUniquePropertyName(const FName& InPropertyName, const URCVirtualPropertyContainerBase* InContainer)
{
	using namespace UE::RemoteControl;

	FString PropertyName = InPropertyName.ToString();
	FString Prefix = PropertyName;
	FString Number;
	Private::SplitNameAndNumber(Prefix, Number);

	// Recursively search for an available name by incrementing number until a unique name is found
	while (InContainer->Bag.FindPropertyDescByName(*PropertyName))
	{
		PropertyName = FString::Printf(TEXT("%s%s"), *Prefix, Private::IncrementNumber(Number));
	}

	return *PropertyName;
}

FName URCVirtualPropertyContainerBase::GenerateUniqueDisplayName(const FName& InPropertyName, const URCVirtualPropertyContainerBase* InContainer)
{
	using namespace UE::RemoteControl;

	FString DisplayName = InPropertyName.ToString();
	FString Prefix = DisplayName;
	FString Number;
	Private::SplitNameAndNumber(Prefix, Number);

	// Recursively search for an available name by incrementing number until a unique name is found
	while (InContainer->ControllerLabelToIdCache.Contains(*DisplayName))
	{
		DisplayName = FString::Printf(TEXT("%s%s"), *Prefix, Private::IncrementNumber(Number));
	}

	return *DisplayName;
}

void URCVirtualPropertyContainerBase::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	for (const TObjectPtr<URCVirtualPropertyBase>& VirtualProperty : VirtualProperties)
	{
		if (VirtualProperty)
		{
			VirtualProperty->UpdateEntityIds(InEntityIdMap);
		}
	}
}

void URCVirtualPropertyContainerBase::CacheControllersLabels()
{
	ControllerLabelToIdCache.Reset();
	for (const URCVirtualPropertyBase* Controller : VirtualProperties)
	{
		if (Controller)
		{
			if (!ControllerLabelToIdCache.Contains(Controller->DisplayName))
			{
				ControllerLabelToIdCache.Add(Controller->DisplayName, Controller->Id);
			}
		}
	}
}

void URCVirtualPropertyContainerBase::FixAndCacheControllersLabels()
{
	ControllerLabelToIdCache.Reset();
	for (URCVirtualPropertyBase* Controller : VirtualProperties)
	{
		if (Controller)
		{
			if (!ControllerLabelToIdCache.Contains(Controller->DisplayName))
			{
				ControllerLabelToIdCache.Add(Controller->DisplayName, Controller->Id);
			}
			// Cases for older presets where you could have the same DisplayName, and if it is already saved then we need to change it now before caching it
			else
			{
				const FName NewControllerName = GenerateUniqueDisplayName(Controller->DisplayName, this);
				Controller->DisplayName = NewControllerName;
				ControllerLabelToIdCache.Add(NewControllerName, Controller->Id);
			}
		}
	}
}

#if WITH_EDITOR
void URCVirtualPropertyContainerBase::PostEditUndo()
{
	Super::PostEditUndo();

	OnVirtualPropertyContainerModifiedDelegate.Broadcast();
}

void URCVirtualPropertyContainerBase::OnModifyPropertyValue(const FPropertyChangedEvent& PropertyChangedEvent)
{
	MarkPackageDirty();
}
#endif

#undef LOCTEXT_NAMESPACE
