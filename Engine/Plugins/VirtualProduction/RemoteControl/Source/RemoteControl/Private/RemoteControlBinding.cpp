// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlBinding.h"

#include "Components/BillboardComponent.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "HAL/IConsoleManager.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "UObject/Package.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/SoftObjectPtr.h"

#if WITH_EDITOR
#include "Components/ActorComponent.h"
#include "Editor.h"
#include "Misc/App.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlBinding"


namespace UE::RemoteControlBinding
{
	bool IsValidObjectForRebinding(UObject* InObject, UWorld* PresetWorld)
	{
		return InObject
			&& !InObject->IsA<UBillboardComponent>()
			&& InObject->GetName().Find(TEXT("TRASH_")) == INDEX_NONE
			&& InObject->GetPackage() != GetTransientPackage()
			&& InObject->GetWorld() == PresetWorld;
	}

	bool IsValidActorForRebinding(AActor* InActor, UWorld* PresetWorld)
	{
		return IsValidObjectForRebinding(InActor, PresetWorld) &&
#if WITH_EDITOR
			InActor->IsEditable() &&
			InActor->IsListedInSceneOutliner() &&
#endif
			!InActor->IsTemplate() &&
			InActor->GetClass() != ABrush::StaticClass() && // Workaround Brush being listed as visible in the scene outliner even though it's not.
			!InActor->HasAnyFlags(RF_Transient);
	}

	bool IsValidSubObjectForRebinding(UObject* InComponent, UWorld* PresetWorld)
	{
		if (!IsValidObjectForRebinding(InComponent, PresetWorld))
		{
			return false;
		}

		if (AActor* OuterActor = InComponent->GetTypedOuter<AActor>())
		{
			if (!IsValidActorForRebinding(OuterActor, PresetWorld))
			{
				return false;
			}
		}

		return true;
	}

	FString ExtractObjectFromPath(const FSoftObjectPath& OriginalObjectPath)
	{
		const static FString PersistentLevelString("PersistentLevel.");
		const int32 PersistentLevelStringLength = PersistentLevelString.Len();
		// /Game/MapName.MapName:PersistentLevel.StaticMeshActor_42.StaticMeshComponent becomes PersistentLevel.StaticMeshActor_42.StaticMeshComponent
		const FString& SubPathString = OriginalObjectPath.GetSubPathString();
		const int32 IndexOfPersistentLevelInfo = SubPathString.Find(PersistentLevelString, ESearchCase::CaseSensitive);
		if (IndexOfPersistentLevelInfo == INDEX_NONE)
		{
			return TEXT("");
		}

		return SubPathString.RightChop(PersistentLevelStringLength);
	}

}

namespace
{
	/**
	 * Find the Object in Editor
	 * 
	 * @TODO Does this need to be updated to possibly include a non-editor related preset?
	 */
	UObject* FindObjectInEditorWorld(UObject* InObject)
	{
		UObject* CounterpartObject = nullptr;
#if WITH_EDITOR
		if (InObject && GEditor)
		{
			if (AActor* Actor = Cast<AActor>(InObject))
			{
				CounterpartObject = EditorUtilities::GetEditorWorldCounterpartActor(Actor);
			}
			else if(AActor* Owner = InObject->GetTypedOuter<AActor>())
			{
				if (AActor* CounterpartWorldOwner = EditorUtilities::GetEditorWorldCounterpartActor(Owner))
				{
					CounterpartObject = FindObject<UObject>(CounterpartWorldOwner, *InObject->GetName());
				}
			}
		}
#endif
		return CounterpartObject ? CounterpartObject : InObject;
	}

	void DumpBindings(const TArray<FString>& Args)
	{
		if (Args.Num())
		{
			if (URemoteControlPreset* Preset = FindFirstObject<URemoteControlPreset>(*Args[0], EFindFirstObjectOptions::EnsureIfAmbiguous))
			{
				TStringBuilder<1000> Output;
				for (URemoteControlBinding* Binding : Preset->Bindings)
				{
					if (GEngine)
					{
						GEngine->Exec(nullptr, *FString::Format(TEXT("obj dump {0} hide=\"actor,object,lighting,movement\""), { Binding->GetPathName() }));
					}
				}
			}
		}
	}
}

static FAutoConsoleCommand CCmdDumpBindings = FAutoConsoleCommand(
	TEXT("RemoteControl.DumpBindings"),
	TEXT("Dumps all binding info on the preset passed as argument."),
	FConsoleCommandWithArgsDelegate::CreateStatic(&DumpBindings)
);

FSoftObjectPath URemoteControlBinding::GetLastBoundObjectPath() const
{
	return LastBoundObjectPath;
}

void URemoteControlLevelIndependantBinding::SetBoundObject(const TSoftObjectPtr<UObject>& InObject)
{
	BoundObject = InObject;
}

void URemoteControlLevelIndependantBinding::UnbindObject(const TSoftObjectPtr<UObject>& InBoundObject)
{
	if (BoundObject == InBoundObject)
	{
		BoundObject.Reset();
	}
}

UObject* URemoteControlLevelIndependantBinding::Resolve() const
{
	return BoundObject.Get();
}

UObject* URemoteControlLevelIndependantBinding::ResolveForWorld(const UWorld* InWorld) const
{
	return BoundObject.Get();
}

bool URemoteControlLevelIndependantBinding::IsValid() const
{
	return BoundObject.IsValid();
}

bool URemoteControlLevelIndependantBinding::IsBound(const TSoftObjectPtr<UObject>& Object) const
{
	return BoundObject == Object;
}

bool URemoteControlLevelIndependantBinding::PruneDeletedObjects()
{
	if (!BoundObject.IsValid())
	{
		Modify();
		BoundObject.ResetWeakPtr();
		return true;
	}

	return false;
}

void URemoteControlLevelIndependantBinding::PostLoad()
{
	Super::PostLoad();
	if (!LastBoundObjectPath.IsValid())
	{
		LastBoundObjectPath = BoundObject.ToSoftObjectPath();
	}
}

void URemoteControlLevelDependantBinding::SetBoundObject(const TSoftObjectPtr<UObject>& InObject)
{
	if (ensure(InObject))
	{
		UObject* EditorObject = FindObjectInEditorWorld(InObject.Get());
		ULevel* OuterLevel = EditorObject->GetTypedOuter<ULevel>();
		BoundObjectMapByPath.FindOrAdd(OuterLevel) = EditorObject;
		SubLevelSelectionMapByPath.FindOrAdd(OuterLevel->GetWorld()) = OuterLevel;
		
		Name = EditorObject->GetName();
		LastBoundObjectPath = InObject.ToSoftObjectPath();
		
		UpdateBindingContext(InObject.Get());
	}
}

void URemoteControlLevelDependantBinding::SetBoundObject_OverrideContext(const TSoftObjectPtr<UObject>& InObject)
{
	SetBoundObject(InObject);
}

void URemoteControlLevelDependantBinding::InitializeForNewLevel()
{
	static const int32 PersistentLevelStrLength = FCString::Strlen(TEXT("PersistentLevel"));

	if (!LevelWithLastSuccessfulResolve.IsNull())
	{
		if (UWorld* CurrentWorld = GetCurrentWorld())
		{
			ULevel* CurrentLevel = CurrentWorld->PersistentLevel;
			if (BoundObjectMapByPath.Contains(CurrentLevel))
			{
				// If there is already a binding for this level, don't overwrite it.
				return;
			}

			if (TSoftObjectPtr<UObject>* BoundObjectPtr = BoundObjectMapByPath.Find(LevelWithLastSuccessfulResolve.ToSoftObjectPath()))
			{
				// Try to find the bound object in the current world by reparenting its path to the current level.
				FSoftObjectPath NewPath = CurrentLevel->GetPathName() + BoundObjectPtr->ToSoftObjectPath().GetSubPathString().RightChop(PersistentLevelStrLength);
				if (NewPath.ResolveObject())
				{
					BoundObjectMapByPath.Add(CurrentLevel, TSoftObjectPtr<UObject>{NewPath});
				}
			}
		}
	}
}

void URemoteControlLevelDependantBinding::UnbindObject(const TSoftObjectPtr<UObject>& InBoundObject)
{
	for (auto It = BoundObjectMapByPath.CreateIterator(); It; ++It)
	{
		if (It.Value() == InBoundObject)
		{
			if (InBoundObject)
			{
				SubLevelSelectionMapByPath.Remove(InBoundObject->GetWorld());
			}

			It.RemoveCurrent();
		}
	}
}

UObject* URemoteControlLevelDependantBinding::Resolve() const
{
	bool bAllowPIE = true;

	// Disallow PIE for Embedded Presets as there should be an Embedded Preset counterpart in PIE taking care of this
	const URemoteControlPreset* Preset = Cast<URemoteControlPreset>(GetOuter());
	if (Preset && Preset->IsEmbeddedPreset())
	{
		bAllowPIE = false;
	}

	UObject* Object = ResolveForCurrentWorld(bAllowPIE).Get();

	if (Object)
	{
		if (Object->GetWorld() && Object->GetWorld()->WorldType == EWorldType::PIE)
		{
			// Don't update path if we manually resolved to a PIE object. (Can happen if editor world gets unloaded)
			return Object;
		}
		// Make sure we don't resolve on a subobject of a dying parent actor.
		if (const AActor* OwnerActor = Object->GetTypedOuter<AActor>())
		{
			if (!::IsValid(OwnerActor))
			{
				return nullptr;
			}
		}

		LevelWithLastSuccessfulResolve = Object->GetTypedOuter<ULevel>();
		if (!LastBoundObjectPath.IsValid())
		{
			LastBoundObjectPath = FSoftObjectPath(Object);
		}

		if (BindingContext.OwnerActorName.IsNone() || (!Object->IsA<AActor>() && !BindingContext.HasValidSubObjectPath()))
		{
			UpdateBindingContext(Object);
		}
	}

#if WITH_EDITOR
	if (bAllowPIE)
	{
		if (Preset && Preset->SelectedWorld.IsValid())
		{
			if (UObject* ObjectToReturn = ResolveForWorld(Preset->SelectedWorld.Get()))
			{
				return ObjectToReturn;
			}
		}
	}
#endif
	return Object;
}

UObject* URemoteControlLevelDependantBinding::ResolveForWorld(const UWorld* InWorld) const
{
	if (!InWorld)
	{
		return nullptr;
	}

	if (const UPackage* WorldPackage = InWorld->GetPackage())
	{
		const UObject* CurrentResolvedObject = ResolveForCurrentWorld(true).Get();
		FSoftObjectPath ObjectPath = CurrentResolvedObject;
		ObjectPath = FSoftObjectPath(FTopLevelAssetPath(WorldPackage->GetFName(), ObjectPath.GetAssetFName()), ObjectPath.GetSubPathString());

		if (UObject* ResolvedObject = ObjectPath.ResolveObject())
		{
			return ResolvedObject;
		}

		if (UObject* LoadedObject = ObjectPath.TryLoad())
		{
			return LoadedObject;
		}
	}

	return nullptr;
}

bool URemoteControlLevelDependantBinding::IsValid() const
{
	return BoundObjectMapByPath.Num() > 0;
}

bool URemoteControlLevelDependantBinding::IsBound(const TSoftObjectPtr<UObject>& Object) const
{
	for (const TPair<FSoftObjectPath, TSoftObjectPtr<UObject>>& Pair : BoundObjectMapByPath)
	{
		if (Pair.Value == Object)
		{
			return true;
		}
	}

	return false;
}

bool URemoteControlLevelDependantBinding::PruneDeletedObjects()
{
	if (!ResolveForCurrentWorld())
	{
		if (UWorld* World = GetCurrentWorld())
		{
			if (const TSoftObjectPtr<ULevel>* LastLevelForBinding = SubLevelSelectionMapByPath.Find(World))
			{
				const FSoftObjectPath& LevelPath = LastLevelForBinding->ToSoftObjectPath();
				const TSoftObjectPtr<UObject>* BoundObject = BoundObjectMapByPath.Find(LevelPath);

				if (!BoundObject || !*BoundObject)
				{
					Modify();
					BoundObjectMapByPath.Remove(LevelPath);
					SubLevelSelectionMapByPath.Remove(World);
					return true;
				}
			}
		}
	}

	return false;
}

void URemoteControlLevelDependantBinding::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (BoundObjectMap_DEPRECATED.Num() > 0)
	{
		for (const TPair<TSoftObjectPtr<ULevel>, TSoftObjectPtr<UObject>>& Pair : BoundObjectMap_DEPRECATED)
		{
			BoundObjectMapByPath.Add(Pair.Key.ToSoftObjectPath(), Pair.Value);
		}
		BoundObjectMap_DEPRECATED.Empty();
	}

	if (SubLevelSelectionMap_DEPRECATED.Num() > 0)
	{
		for (const TPair<TSoftObjectPtr<UWorld>, TSoftObjectPtr<ULevel>>& Pair : SubLevelSelectionMap_DEPRECATED)
		{
			SubLevelSelectionMapByPath.Add(Pair.Key.ToSoftObjectPath(), Pair.Value);
		}
		SubLevelSelectionMap_DEPRECATED.Empty();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

	if (!LastBoundObjectPath.IsValid())
	{
		LastBoundObjectPath = BoundObjectMapByPath.FindRef(LevelWithLastSuccessfulResolve.ToSoftObjectPath()).ToSoftObjectPath();
	}
}

TSoftObjectPtr<UObject> URemoteControlLevelDependantBinding::ResolveForCurrentWorld(bool bAllowPIE) const
{
	// Note that we use Find rather than FindRef throughout this function. This lets us dereference
	// the BoundObjectMapByPath/SubLevelSelectionMapByPath's values directly so that their internal weak pointers
	// are updated when stale (rather than dereferencing copies and leaving stale pointers in the originals).

	if (UWorld* World = GetCurrentWorld(bAllowPIE))
	{
		FSoftObjectPath LevelPath = World;
#if WITH_EDITOR
		// Try finding the object using the sub level selection map first.
		if (bAllowPIE && World->WorldType == EWorldType::PIE)
		{
			FString PIEPackageName = World->GetPackage()->GetPathName();
			FString OriginalPackage = UWorld::StripPIEPrefixFromPackageName(PIEPackageName, World->StreamingLevelsPrefix);
			FTopLevelAssetPath TopLevelAssetPath(FName(OriginalPackage), World->GetFName());

			LevelPath = FSoftObjectPath(TopLevelAssetPath);
		}
#endif

		if (TSoftObjectPtr<ULevel>* LastBindingLevel = SubLevelSelectionMapByPath.Find(LevelPath))
		{
			if (const TSoftObjectPtr<UObject>* ObjectPtr = BoundObjectMapByPath.Find(LastBindingLevel->ToSoftObjectPath()))
			{
				if (*ObjectPtr)
				{
					return *ObjectPtr;
				}

#if WITH_EDITOR
				if (bAllowPIE && World->WorldType == EWorldType::PIE)
				{
					bool bPathToActor = true;

					FString ActorName = UE::RemoteControlBinding::ExtractObjectFromPath(ObjectPtr->ToSoftObjectPath());
					if (!ActorName.IsEmpty())
					{
						if (UObject* SimWorldObject = FindObject<UObject>(World->PersistentLevel, *ActorName, EFindObjectFlags::None))
						{
							if ((SimWorldObject->IsA<AActor>() && GEditor->ObjectsThatExistInEditorWorld.Get(SimWorldObject))
								|| (GEditor->ObjectsThatExistInEditorWorld.Get(SimWorldObject->GetTypedOuter<AActor>())))
							{
								return SimWorldObject;
							}
						}
					}

				}
#endif
				
			}
		}
		
		// Resort to old method where we use the first level we find in the bound object map,
		// and add an entry in the sub level selection map.
		for (auto LevelIt = World->GetLevelIterator(); LevelIt; ++LevelIt)
		{
			const TSoftObjectPtr<ULevel>& WeakLevel = *LevelIt;
			if (const TSoftObjectPtr<UObject>* ObjectPtr = BoundObjectMapByPath.Find(WeakLevel.ToSoftObjectPath()))
			{
				if (*ObjectPtr)
				{
					SubLevelSelectionMapByPath.FindOrAdd(World) = WeakLevel;
					return *ObjectPtr;
				}
			}
		}
	}

	return nullptr;
}

void URemoteControlLevelDependantBinding::UpdateBindingContext(UObject* InObject) const
{
	if (InObject)
	{
		BindingContext.SupportedClass = InObject->GetClass();

		if (InObject->IsA<AActor>())
		{
			BindingContext.OwnerActorClass = InObject->GetClass();
			BindingContext.OwnerActorName = InObject->GetFName();
		}
		else
		{
			AActor* Owner = InObject->GetTypedOuter<AActor>();
			check(Owner);
			BindingContext.ComponentName = InObject->GetFName();
			BindingContext.SubObjectPath = InObject->GetPathName(Owner);
			BindingContext.OwnerActorClass = Owner->GetClass();
			BindingContext.OwnerActorName = Owner->GetFName();
		}
	}
}

UClass* URemoteControlLevelDependantBinding::GetSupportedOwnerClass() const
{
	return BindingContext.OwnerActorClass.LoadSynchronous();
}

UWorld* URemoteControlLevelDependantBinding::GetCurrentWorld(bool bAllowPIE) const
{
	URemoteControlPreset* Preset = Cast<URemoteControlPreset>(GetOuter());

	return URemoteControlPreset::GetWorld(Preset, bAllowPIE);
}

void URemoteControlLevelDependantBinding::SetBoundObject(const TSoftObjectPtr<ULevel>& Level, const TSoftObjectPtr<UObject>& BoundObject)
{
	BoundObjectMapByPath.FindOrAdd(Level.ToSoftObjectPath()) = BoundObject;
	const FSoftObjectPath& Path = BoundObject.ToSoftObjectPath();
	const FString& SubPath = Path.GetSubPathString();
	FString LeftPart;
	FString ObjectName;
	SubPath.Split(TEXT("."), &LeftPart, &ObjectName, ESearchCase::Type::IgnoreCase, ESearchDir::FromEnd);
	ensure(ObjectName.Len());
	Name = MoveTemp(ObjectName);
	LastBoundObjectPath = BoundObject.ToSoftObjectPath();
}

#undef LOCTEXT_NAMESPACE

