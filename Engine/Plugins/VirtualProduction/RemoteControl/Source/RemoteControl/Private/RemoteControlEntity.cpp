// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlEntity.h"

#include "Algo/Transform.h"
#include "RemoteControlBinding.h"
#include "RemoteControlPreset.h"

TArray<UObject*> FRemoteControlEntity::GetBoundObjects() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlEntity::GetBoundObjects);
	TArray<UObject*> ResolvedObjects;
	ResolvedObjects.Reserve(Bindings.Num());
	Algo::TransformIf(Bindings, ResolvedObjects,
		[](TWeakObjectPtr<URemoteControlBinding> WeakBinding) { return WeakBinding.IsValid(); },
		[](TWeakObjectPtr<URemoteControlBinding> WeakBinding) { return WeakBinding->Resolve(); });

	return ResolvedObjects.FilterByPredicate([](const UObject* Object){ return !!Object; });
}

UObject* FRemoteControlEntity::GetBoundObject() const
{
	TArray<UObject*> ResolvedObjects;
	ResolvedObjects.Reserve(Bindings.Num());
	Algo::TransformIf(Bindings, ResolvedObjects,
		[](TWeakObjectPtr<URemoteControlBinding> WeakBinding) { return WeakBinding.IsValid(); },
		[](TWeakObjectPtr<URemoteControlBinding> WeakBinding) { return WeakBinding->Resolve(); });

	UObject** Obj = ResolvedObjects.FindByPredicate([](const UObject* Object) { return !!Object; });
	return Obj ? *Obj : nullptr;
}

UObject* FRemoteControlEntity::GetBoundObjectForWorld(const UWorld* InWorld) const
{
	if (Bindings.IsEmpty())
	{
		return nullptr;
	}

	if (!Bindings[0].IsValid() || Bindings[0].IsStale())
	{
		return nullptr;
	}

	return Bindings[0]->ResolveForWorld(InWorld);
}

const TArray<TWeakObjectPtr<URemoteControlBinding>>& FRemoteControlEntity::GetBindings() const
{
	return Bindings;
}

const TMap<FName, FString>& FRemoteControlEntity::GetMetadata() const
{
	return UserMetadata;
}

void FRemoteControlEntity::RemoveMetadataEntry(FName Key)
{
	UserMetadata.Remove(Key);
	OnEntityModifiedDelegate.ExecuteIfBound(Id);
}

void FRemoteControlEntity::SetMetadataValue(FName Key, FString Value)
{
	UserMetadata.FindOrAdd(Key) = Value;
	OnEntityModifiedDelegate.ExecuteIfBound(Id);
}

void FRemoteControlEntity::BindObject(UObject* InObjectToBind)
{
	if (!InObjectToBind)
	{
		return;
	}

	if (Bindings.Num() == 0)
	{
		Bindings.Emplace(Owner->FindOrAddBinding(InObjectToBind));
	}
	else
	{
		// Don't modify the binding itself since that would modify all other properties pointing to this binding.
		// Try to first find a binding that has the same bound object map to preserve that information.
		if (URemoteControlBinding* MatchingBinding = Owner->FindMatchingBinding(Bindings[0].Get(), InObjectToBind))
		{
			Bindings[0] = MatchingBinding;
		}
		else
		{
			Bindings[0] = DuplicateObject<URemoteControlBinding>(Bindings[0].Get(), Owner.Get());
			Owner->Bindings.Add(Bindings[0].Get());
			Bindings[0]->SetBoundObject(InObjectToBind);
		}

	}

	// The order of this delegate needs to be this way
	// The first will update the Entity widget with the new data
	// The second one will refresh the List
	Owner->OnEntityRebind().Broadcast(Id);
	OnEntityModifiedDelegate.ExecuteIfBound(Id);
}

bool FRemoteControlEntity::IsBound() const
{
	return GetBoundObjects().Num() > 0;
}

FSoftObjectPath FRemoteControlEntity::GetLastBindingPath() const
{
	FSoftObjectPath Path;
	for (const TWeakObjectPtr<URemoteControlBinding>& Binding : Bindings)
	{
		if (Binding.IsValid())
		{
			Path = Binding->GetLastBoundObjectPath();
			break;
		}
	}

	return Path;
}

bool FRemoteControlEntity::operator==(const FRemoteControlEntity& InEntity) const
{
	return Id == InEntity.Id;
}

bool FRemoteControlEntity::operator==(FGuid InEntityId) const
{
	return Id == InEntityId;
}

FRemoteControlEntity::FRemoteControlEntity(URemoteControlPreset* InPreset, FName InLabel, const TArray<URemoteControlBinding*>& InBindings)
	: Owner(InPreset)
	, Label(InLabel)
	, Id(FGuid::NewGuid())
{
	Bindings.Append(InBindings);
}

const UScriptStruct* FRemoteControlEntity::GetStruct() const
{
	if (URemoteControlPreset* Preset = Owner.Get())
	{
		return Preset->GetExposedEntityType(Id);
	}
	return nullptr;
}

FName FRemoteControlEntity::Rename(FName NewLabel)
{
	if (URemoteControlPreset* Preset = Owner.Get())
	{
		Preset->Modify();
		FName NewName = Preset->RenameExposedEntity(Id, NewLabel);
		return NewName;
	}

	checkNoEntry();
	return NAME_None;
}

uint32 GetTypeHash(const FRemoteControlEntity& InEntity)
{
	return GetTypeHash(InEntity.Id);
}