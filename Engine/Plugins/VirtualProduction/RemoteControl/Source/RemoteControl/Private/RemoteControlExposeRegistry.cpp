// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlExposeRegistry.h"

#include "Algo/Find.h"
#include "Algo/RemoveIf.h"
#include "Algo/Transform.h"
#include "HAL/UnrealMemory.h"
#include "Misc/Guid.h"
#include "RemoteControlObjectVersion.h"
#include "Serialization/Archive.h"
#include "UObject/Class.h"
#include "UObject/SoftObjectPath.h"

TArray<TSharedPtr<const FRemoteControlEntity>> URemoteControlExposeRegistry::GetExposedEntities() const
{
	TArray<TSharedPtr<const FRemoteControlEntity>> Entities;

	Algo::TransformIf(ExposedEntitiesArray, Entities,
		[](const FRCEntityWrapper& Wrapper)
		{
			return Wrapper.IsValid();
		},
		[](const FRCEntityWrapper& Wrapper)
		{
			return Wrapper.Get();
		});

	return Entities;
}

TArray<TSharedPtr<const FRemoteControlEntity>> URemoteControlExposeRegistry::GetExposedEntities(UScriptStruct* EntityType) const
{
	check(EntityType->IsChildOf(FRemoteControlEntity::StaticStruct()));
	TArray<TSharedPtr<const FRemoteControlEntity>> Entities;

	for (const FRCEntityWrapper& Wrapper : ExposedEntitiesArray)
	{
		if (Wrapper.IsValid() && Wrapper.GetType()->IsChildOf(EntityType))
		{
			Entities.Add(Wrapper.Get());
		}
	}

	return Entities;
}

TArray<TSharedPtr<FRemoteControlEntity>> URemoteControlExposeRegistry::GetExposedEntities(UScriptStruct* EntityType)
{
	check(EntityType->IsChildOf(FRemoteControlEntity::StaticStruct()));
	TArray<TSharedPtr<FRemoteControlEntity>> Entities;

	for (FRCEntityWrapper& Wrapper : ExposedEntitiesArray)
	{
		if (Wrapper.IsValid() && Wrapper.GetType()->IsChildOf(EntityType))
		{
			Entities.Add(Wrapper.Get());
		}
	}

	return Entities;
}

TSharedPtr<const FRemoteControlEntity> URemoteControlExposeRegistry::GetExposedEntity(const FGuid& ExposedEntityId, const UScriptStruct* EntityType) const
{
	return const_cast<URemoteControlExposeRegistry*>(this)->GetExposedEntity(ExposedEntityId, EntityType);
}

TSharedPtr<FRemoteControlEntity> URemoteControlExposeRegistry::GetExposedEntity(const FGuid& ExposedEntityId, const UScriptStruct* EntityType)
{
	if (FRCEntityWrapper* Wrapper = Algo::Find(ExposedEntitiesArray, ExposedEntityId))
	{
		if (Wrapper->IsValid() && Wrapper->GetType()->IsChildOf(EntityType))
		{
			return Wrapper->Get();
		}
	}

	return nullptr;
}

const UScriptStruct* URemoteControlExposeRegistry::GetExposedEntityType(const FGuid& ExposedEntityId) const
{
	if (const FRCEntityWrapper* Wrapper = Algo::Find(ExposedEntitiesArray, ExposedEntityId))
	{
		return Wrapper->GetType();
	}

	return nullptr;
}

const bool URemoteControlExposeRegistry::IsEmpty() const
{
    return ExposedEntitiesArray.IsEmpty();
}

TSharedPtr<FRemoteControlEntity> URemoteControlExposeRegistry::AddExposedEntity(FRemoteControlEntity&& EntityToExpose, UScriptStruct* EntityType)
{
	LabelToIdCache.Add(EntityToExpose.GetLabel(), EntityToExpose.GetId());
	FRCEntityWrapper Wrapper{ MoveTemp(EntityToExpose), EntityType};
	TSharedPtr<FRemoteControlEntity> Entity = Wrapper.Get();
	ExposedEntitiesArray.Add(MoveTemp(Wrapper));
	ExposedTypes.Add(EntityType);
	return Entity;
}

void URemoteControlExposeRegistry::RemoveExposedEntity(const FGuid& Id)
{
	uint32 Hash = GetTypeHash(Id);
	if (const FRCEntityWrapper* Wrapper = Algo::Find(ExposedEntitiesArray, Id))
	{
		if (TSharedPtr<const FRemoteControlEntity> Entity = Wrapper->Get())
		{
			LabelToIdCache.Remove(Entity->GetLabel());
		}
		ExposedEntitiesArray.SetNum(Algo::StableRemoveIf(ExposedEntitiesArray,
			[&Id](const FRCEntityWrapper& EntityWrapper)
			{
				return EntityWrapper.Get()->GetId() == Id;
			}));
	}
}

FName URemoteControlExposeRegistry::RenameExposedEntity(const FGuid& Id, FName NewLabel)
{
	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity(Id))
	{
		LabelToIdCache.Remove(Entity->GetLabel());
		Entity->Label = GenerateUniqueLabel(NewLabel);
		LabelToIdCache.Add(Entity->GetLabel(), Id);
		return Entity->GetLabel();
	}
	return NAME_None;
}

FGuid URemoteControlExposeRegistry::GetExposedEntityId(FName EntityLabel) const
{
	if (const FGuid* EntityId = LabelToIdCache.Find(EntityLabel))
	{
		return *EntityId;
	}
	return FGuid();
}

FName URemoteControlExposeRegistry::GenerateUniqueLabel(FName BaseName) const
{
	// Try using the field name itself
	if (!LabelToIdCache.Contains(BaseName))
	{
		return BaseName;
	}

	// Then try the field name with a suffix
	for (uint32 Index = 1; Index < 1000; ++Index)
	{
		const FName Candidate = FName(*FString::Printf(TEXT("%s (%d)"), *BaseName.ToString(), Index));
		if (!LabelToIdCache.Contains(Candidate))
		{
			return Candidate;
		}
	}

	// Something went wrong if we end up here.
	checkNoEntry();
	return NAME_None;
}

void URemoteControlExposeRegistry::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRemoteControlObjectVersion::GUID);

#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FRemoteControlObjectVersion::GUID) < FRemoteControlObjectVersion::ChangedExposedEntitiesSetToAnOrderedTArray)
		{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Algo::Transform(ExposedEntities_DEPRECATED, ExposedEntitiesArray,
			[](const FRCEntityWrapper& EntityWrapper)
			{
				return EntityWrapper;
			});
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif // WITH_EDITORONLY_DATA
}

void URemoteControlExposeRegistry::PostLoad()
{
	Super::PostLoad();
	CacheLabels();
}

void URemoteControlExposeRegistry::PostDuplicate(bool bDuplicateForPIE)
{
	UObject::PostDuplicate(bDuplicateForPIE);
	CacheLabels();
}

TSharedPtr<FRemoteControlEntity> URemoteControlExposeRegistry::GetEntity(const FGuid& EntityId)
{
	/** Get a raw pointer to an entity using its id. */
	if (FRCEntityWrapper* Wrapper = Algo::Find(ExposedEntitiesArray, EntityId))
	{
		return Wrapper->Get();
	}

	return nullptr;
}

TSharedPtr<const FRemoteControlEntity> URemoteControlExposeRegistry::GetEntity(const FGuid& EntityId) const
{
	return const_cast<URemoteControlExposeRegistry*>(this)->GetEntity(EntityId);
}

void URemoteControlExposeRegistry::CacheLabels()
{
	LabelToIdCache.Reset();
	for (const FRCEntityWrapper& Wrapper : ExposedEntitiesArray)
	{
		if (TSharedPtr<const FRemoteControlEntity> Entity = Wrapper.Get())
		{
			LabelToIdCache.Add(Entity->GetLabel(), Entity->GetId());
		}
	}
}

FRCEntityWrapper::FRCEntityWrapper(const FRemoteControlEntity& InEntity, UScriptStruct* InEntityType)
{
	WrappedEntity = MakeShareable<FRemoteControlEntity>((FRemoteControlEntity*)FMemory::Malloc(InEntityType->GetStructureSize(), InEntityType->GetMinAlignment()));
	InEntityType->InitializeStruct(WrappedEntity.Get());
	InEntityType->CopyScriptStruct(WrappedEntity.Get(), &InEntity);
	EntityType = InEntityType;
}

TSharedPtr<FRemoteControlEntity> FRCEntityWrapper::Get()
{
	return WrappedEntity;
}

TSharedPtr<const FRemoteControlEntity> FRCEntityWrapper::Get() const
{
	return WrappedEntity;
}

const UScriptStruct* FRCEntityWrapper::GetType() const
{
	return EntityType;
}

bool FRCEntityWrapper::IsValid() const
{
	return WrappedEntity && EntityType;
}

bool FRCEntityWrapper::Serialize(FArchive& Ar)
{
	if (Ar.IsSaving())
	{
		if (IsValid())
		{
			FSoftObjectPath Path{ EntityType };
			Path.Serialize(Ar);
			EntityType->SerializeItem(Ar, (uint8*)WrappedEntity.Get(), nullptr);
		}
	}
	else if (Ar.IsLoading())
	{
		FSoftObjectPath StructPath;
		StructPath.Serialize(Ar);

		if (UObject* StructObject = StructPath.TryLoad())
		{
			if (UScriptStruct* ScriptStruct = static_cast<UScriptStruct*>(StructObject))
			{
				EntityType = ScriptStruct;
				WrappedEntity = MakeShareable<FRemoteControlEntity>((FRemoteControlEntity*)FMemory::Malloc(EntityType->GetStructureSize(), EntityType->GetMinAlignment()));
				EntityType->InitializeStruct(WrappedEntity.Get());
				EntityType->SerializeItem(Ar, (uint8*)WrappedEntity.Get(), nullptr);
			}
		}
	}

	return true;
}

bool FRCEntityWrapper::operator==(const FGuid& WrappedId) const
{
	if (!IsValid())
	{
		return false;
	}

	return WrappedEntity->GetId() == WrappedId;
}

bool FRCEntityWrapper::operator==(const FRCEntityWrapper& Other) const
{
	if (IsValid() && Other.IsValid())
	{
		return WrappedEntity->GetId() == Other.WrappedEntity->GetId();
	}
	return false;
}

uint32 GetTypeHash(const FRCEntityWrapper& Wrapper)
{
	return Wrapper.EntityType->GetStructTypeHash(Wrapper.WrappedEntity.Get());
}

