// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "CoreMinimal.h"
#include "RemoteControlEntity.h"
#include "UObject/Class.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTypeTraits.h"

#include "RemoteControlExposeRegistry.generated.h"

class FArchive;
struct FGuid;

/** Wrapper class used to serialize exposable entities in a generic way. */
USTRUCT()
struct FRCEntityWrapper
{
	GENERATED_BODY()

	FRCEntityWrapper() = default;

	FRCEntityWrapper(const FRemoteControlEntity& InEntity, UScriptStruct* InEntityType);

	/** Get the wrapped entity. */
	TSharedPtr<FRemoteControlEntity> Get();

	/** Get the wrapped entity. */
	TSharedPtr<const FRemoteControlEntity> Get() const;

	/** Get the type of the wrapped entity. */
	const UScriptStruct* GetType() const;

	/** Returns whether the type and the underlying data is valid. */
	bool IsValid() const;

public:
	bool Serialize(FArchive& Ar);
	friend uint32 GetTypeHash(const FRCEntityWrapper& Wrapper);
	bool operator==(const FGuid& WrappedId) const;
	bool operator==(const FRCEntityWrapper& Other) const;


private:
	/** Holds the type of the wrapped entity. */
	UScriptStruct* EntityType = nullptr;
	/** Pointer to underlying remote control property. */
	TSharedPtr<FRemoteControlEntity> WrappedEntity;
};

template<> struct TStructOpsTypeTraits<FRCEntityWrapper> : public TStructOpsTypeTraitsBase2<FRCEntityWrapper>
{
	enum
	{
		WithSerializer = true,
		WithIdenticalViaEquality = true
	};
};

UCLASS()
class URemoteControlExposeRegistry : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Get all the exposed entities.
	 */
	TArray<TSharedPtr<const FRemoteControlEntity>> GetExposedEntities() const;

	/**
	 * Get the exposed entities of a certain type.
	 */
	TArray<TSharedPtr<const FRemoteControlEntity>> GetExposedEntities(UScriptStruct* EntityType) const;

	/**
	 * Get the exposed entities of a certain type.
	 */
	TArray<TSharedPtr<FRemoteControlEntity>> GetExposedEntities(UScriptStruct* EntityType);
	
	/**
	 * Get the exposed entities of a certain type.
	 */
	template <typename EntityType = FRemoteControlEntity>
	TArray<TSharedPtr<EntityType>> GetExposedEntities()
	{
		static_assert(TIsDerivedFrom<EntityType, FRemoteControlEntity>::Value, "EntityType must derive from FRemoteControlEntity.");
		TArray<TSharedPtr<FRemoteControlEntity>> Entities = GetExposedEntities(EntityType::StaticStruct());
		TArray<TSharedPtr<EntityType>> CastEntities;
		CastEntities.Reserve(Entities.Num());
		Algo::Transform(Entities, CastEntities, [](const TSharedPtr<FRemoteControlEntity>& InEntity){ return StaticCastSharedPtr<EntityType>(InEntity);});
		return CastEntities;		
	}

	/**
	 * Get the exposed entities of a certain type.
	 */
	template <typename EntityType>
    TArray<TSharedPtr<EntityType>> GetExposedEntities() const
	{
		return const_cast<URemoteControlExposeRegistry*>(this)->GetExposedEntities<EntityType>();
	}

	/**
	 * Get an exposed entity from the registry.
	 * @param ExposedEntityId the id of the entity to get.
	 * @param EntityType the type of entity to search for.
	 * @return the exposed entity pointer, or an invalid pointer otherwise.
	 */
	TSharedPtr<const FRemoteControlEntity> GetExposedEntity(const FGuid& ExposedEntityId, const UScriptStruct* EntityType = FRemoteControlEntity::StaticStruct()) const;

	/**
	 * Get an exposed entity from the registry.
	 * @param ExposedEntityId the id of the entity to get.
	 * @param EntityType the type of entity to search for.
	 * @return the exposed entity pointer, or an invalid pointer otherwise.
	 */
	TSharedPtr<FRemoteControlEntity> GetExposedEntity(const FGuid& ExposedEntityId, const UScriptStruct* EntityType = FRemoteControlEntity::StaticStruct());

	/**
	 * Get an exposed entity from the registry.
	 * @param ExposedEntityId the id of the entity to get.
	 * @return the exposed entity pointer, or an invalid pointer otherwise.
	 */
	template <typename EntityType>
	TSharedPtr<const EntityType> GetExposedEntity(const FGuid& ExposedEntityId) const
	{
		return const_cast<URemoteControlExposeRegistry*>(this)->GetExposedEntity<EntityType>(ExposedEntityId);
	}

	/**
	 * Get an exposed entity from the registry.
	 * @param ExposedEntityId the id of the entity to get.
	 * @return the exposed entity pointer, or an invalid pointer otherwise.
	 */
	template <typename EntityType>
	TSharedPtr<EntityType> GetExposedEntity(const FGuid& ExposedEntityId)
	{
		static_assert(TIsDerivedFrom<EntityType, FRemoteControlEntity>::Value, "EntityType must derive from FRemoteControlEntity.");
		UScriptStruct* EntityStruct = EntityType::StaticStruct();
		return StaticCastSharedPtr<EntityType>(GetExposedEntity(ExposedEntityId, EntityStruct));
	}

	/**
	 * Get the type of an exposed entity.
	 * @param ExposedEntityId The id of the entity to find.
	 * @return the entity type (ie. FRemoteControlActor) or nullptr if not found.
	 */
	const UScriptStruct* GetExposedEntityType(const FGuid& ExposedEntityId) const;

	/**
	 * Get all the exposed types in the registry (ie. FRemoteControlActor)
	 */
	const TSet<TObjectPtr<UScriptStruct>>& GetExposedEntityTypes() const
	{
		return ExposedTypes;
	}

	/** Returns true when Exposed Entities is empty. */
	const bool IsEmpty() const;

	/**
	 * Add an entity to the set of exposed entities of its type.
	 * @param EntityToExpose the entity to expose.
	 * @param EntityType EntityType the type of the entity to expose.
	 */
	TSharedPtr<FRemoteControlEntity> AddExposedEntity(FRemoteControlEntity&& EntityToExpose, UScriptStruct* EntityType);

	/**
	 * Remove an exposed entity from the registry using its id.
	 */
	void RemoveExposedEntity(const FGuid& Id);

	/**
	 * Rename an exposed entity.
	 * @param Id the entity's id
	 * @param NewLabel the new label to assign to the entity.
	 * @return The assigned label, which might be suffixed if the label already exists in the registry
	 *		   or NAME_None if the entity was not found.
	 */
	FName RenameExposedEntity(const FGuid& Id, FName NewLabel);

	/**
	 * Get an entity's id using it's label.
	 * @param EntityLabel the entity's label.
	 * @return The entity's id if found, otherwise an invalid guid.
	 */
	FGuid GetExposedEntityId(FName EntityLabel) const;

	/**
	 * Generate a unique label using a base name.
	 * @param BaseName The base name to use.
	 * @param The generated name.
	 */
	FName GenerateUniqueLabel(FName BaseName) const;

	//~ Begin UObject interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	//~ End UObject interface

	/** Cache labels for all exposed entities. */
	void CacheLabels();

private:
	/** Get a raw pointer to an entity using its id. */
	TSharedPtr<FRemoteControlEntity> GetEntity(const FGuid& EntityId);

	/** Get a raw pointer to an entity using its id. */
	TSharedPtr<const FRemoteControlEntity> GetEntity(const FGuid& EntityId) const;

private:
	/** Holds the exposed entities. */
	UPROPERTY()
	TArray<FRCEntityWrapper> ExposedEntitiesArray;

#if WITH_EDITORONLY_DATA
	/** DEPRECATED 5.5 Holds the exposed entities. Instead refer to the new TArray<FRCEntityWrapper> URemoteControlExposeRegistry::ExposedEntities */
	UE_DEPRECATED(5.5, "Changed to an ordered TArray.")
	UPROPERTY()
	TSet<FRCEntityWrapper> ExposedEntities_DEPRECATED;
#endif //

	/** Cache of label to ids. */
	UPROPERTY(Transient)
	TMap<FName, FGuid> LabelToIdCache;

	/** Holds the types of entities exposed in the registry. */
	UPROPERTY()
	TSet<TObjectPtr<UScriptStruct>> ExposedTypes;
};
