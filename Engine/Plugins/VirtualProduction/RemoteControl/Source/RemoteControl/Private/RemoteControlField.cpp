// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlField.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "Math/NumericLimits.h"
#include "Misc/App.h"
#include "RemoteControlObjectVersion.h"
#include "RemoteControlFieldPath.h"
#include "RemoteControlBinding.h"
#include "RemoteControlPreset.h"
#include "RemoteControlPropertyHandle.h"
#include "UObject/Class.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/StructOnScope.h"
#include "UObject/UnrealType.h"


namespace RemoteControlResolvers
{
	FName DisplayClusterActorName = "DisplayClusterRootActor";
	FName ConfigDataPropertyName = "CurrentConfigData";

	UObject* NDisplayConfigDataResolver(const UObject* NDisplayActor)
	{
		if (NDisplayActor)
		{
			UClass* SuperClass = NDisplayActor->GetClass()->GetSuperClass();
			if (SuperClass && SuperClass->GetFName() == DisplayClusterActorName)
			{
				if (FObjectProperty* ConfigData = CastField<FObjectProperty>(SuperClass->FindPropertyByName(ConfigDataPropertyName)))
				{
					return ConfigData->GetObjectPropertyValue_InContainer(NDisplayActor);
				}
			}
		}

		return nullptr;
	}

	static TMap<FName, TFunction<UObject* (const UObject*)>> CustomResolvers = {
		{ DisplayClusterActorName, NDisplayConfigDataResolver }
	};
}

FRemoteControlField::FRemoteControlField(URemoteControlPreset* InPreset, EExposedFieldType InType, FName InLabel, FRCFieldPathInfo InFieldPathInfo, const TArray<URemoteControlBinding*> InBindings)
	: FRemoteControlEntity(InPreset, InLabel, InBindings)
	, FieldType(InType)
	, FieldName(InFieldPathInfo.GetFieldName())
	, FieldPathInfo(MoveTemp(InFieldPathInfo))
{
}

TArray<UObject*> FRemoteControlField::ResolveFieldOwners(const TArray<UObject*>& SectionObjects) const
{
	TArray<UObject*> ResolvedObjects;
#if WITH_EDITORONLY_DATA
	if (ComponentHierarchy_DEPRECATED.Num())
	{
		ResolvedObjects = ResolveFieldOwnersUsingComponentHierarchy(SectionObjects);
	}
	else
#endif
	{
		ResolvedObjects = SectionObjects;
	}

	return ResolvedObjects;
}

void FRemoteControlField::BindObject(UObject* InObjectToBind)
{
	if (!InObjectToBind)
	{
		return;
	}
	
	if (UClass* ResolvedOwnerClass = GetSupportedBindingClass())
	{
		if (InObjectToBind->GetClass()->IsChildOf(ResolvedOwnerClass))
		{
			FRemoteControlEntity::BindObject(InObjectToBind);
		}
		else if (AActor* Actor = Cast<AActor>(InObjectToBind))
		{
			// Attempt to bind to the root component since it is a very common case.
			const USceneComponent* RootComponent = Actor->GetRootComponent();
			if (RootComponent && RootComponent->GetClass() == ResolvedOwnerClass)
			{
				FRemoteControlEntity::BindObject(Actor->GetRootComponent());
			}
			else
			{
				UClass* Class = InObjectToBind->GetClass();
				if (Class->GetSuperClass())
				{
					if (TFunction<UObject* (const UObject*)>* Resolver = RemoteControlResolvers::CustomResolvers.Find(Class->GetSuperClass()->GetFName()))
					{
						UObject* CustomResolvedObject = (*Resolver)(InObjectToBind);
						if (CustomResolvedObject && CustomResolvedObject->GetClass() == ResolvedOwnerClass)
						{
							FRemoteControlEntity::BindObject(CustomResolvedObject);
							return;
						}
					}
				}

				// Search for a matching component if the root component was not a match.
				FRemoteControlEntity::BindObject(Actor->GetComponentByClass(ResolvedOwnerClass));
			}
		}
	}
}

bool FRemoteControlField::CanBindObject(const UObject* InObjectToBind) const
{
	if (InObjectToBind)
	{
		if (UClass* ResolvedOwnerClass = GetSupportedBindingClass())
		{
			if (InObjectToBind->GetClass()->IsChildOf(ResolvedOwnerClass))
			{
				return true;
			}
			
			if (ResolvedOwnerClass->IsChildOf<UActorComponent>())
			{
				if (const AActor* Actor = Cast<AActor>(InObjectToBind))
				{
					return !!Actor->GetComponentByClass(ResolvedOwnerClass);
				}
				return false;
			}

			UClass* Class = InObjectToBind->GetClass();
			if (Class->GetSuperClass())
			{
				if (TFunction<UObject* (const UObject*)>* Resolver = RemoteControlResolvers::CustomResolvers.Find(Class->GetSuperClass()->GetFName()))
				{
					UObject* CustomResolvedObject = (*Resolver)(InObjectToBind);
					return CustomResolvedObject && CustomResolvedObject->GetClass() == ResolvedOwnerClass;
				}
			}
		}
	}
	return false;
}

void FRemoteControlField::ClearMask(ERCMask InMaskBit)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ActiveMasks_DEPRECATED &= ~(uint8)InMaskBit;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FRemoteControlField::EnableMask(ERCMask InMaskBit)
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ActiveMasks_DEPRECATED |= (uint8)InMaskBit;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool FRemoteControlField::HasMask(ERCMask InMaskBit) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return (ActiveMasks_DEPRECATED & (uint8)InMaskBit) != (uint8)ERCMask::NoMask;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

ERCMask FRemoteControlField::GetActiveMasks() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return (ERCMask)ActiveMasks_DEPRECATED;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void FRemoteControlField::PostSerialize(const FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		int32 CustomVersion = Ar.CustomVer(FRemoteControlObjectVersion::GUID);
		if (CustomVersion < FRemoteControlObjectVersion::AddedRebindingFunctionality)
		{
			if (OwnerClass.IsNull())
			{
				OwnerClass = GetSupportedBindingClass();
			}
		}
	}
}

#if WITH_EDITORONLY_DATA
TArray<UObject*> FRemoteControlField::ResolveFieldOwnersUsingComponentHierarchy(const TArray<UObject*>& SectionObjects) const
{
	TArray<UObject*> FieldOwners;
	FieldOwners.Reserve(SectionObjects.Num());

	for (UObject* Object : SectionObjects)
	{
		//If component hierarchy is not empty, we need to walk it to find the child object
		if (ComponentHierarchy_DEPRECATED.Num() > 0)
		{
			UObject* Outer = Object;

			for (const FString& Component : ComponentHierarchy_DEPRECATED)
			{
				if (UObject* ResolvedFieldOwner = FindObject<UObject>(Outer, *Component))
				{
					Outer = ResolvedFieldOwner;
				}
				else
				{
					// This can happen when one of the grouped actors has a component named DefaultSceneRoot and one has a component StaticMeshComponent.
					// @todo: Change to a log if this situation can occur under normal conditions. (ie. Blueprint reinstanced)
					ensureAlwaysMsgf(false, TEXT("Could not resolve field owner for field %s"), *Object->GetName());
					Outer = nullptr;
					break;
				}
			}


			if (Outer)
			{
				FieldOwners.Add(Outer);
			}
		}
		else
		{
			FieldOwners.Add(Object);
		}
	}

	return FieldOwners;
}
#endif

FName FRemoteControlProperty::MetadataKey_Min = "Min";
FName FRemoteControlProperty::MetadataKey_Max = "Max";

FRemoteControlProperty::FRemoteControlProperty(FName InLabel, FRCFieldPathInfo FieldPathInfo, TArray<FString> InComponentHierarchy)
	: FRemoteControlField(nullptr, EExposedFieldType::Property, InLabel, MoveTemp(FieldPathInfo), {})
{
}

FRemoteControlProperty::FRemoteControlProperty(URemoteControlPreset* InPreset, FName InLabel, FRCFieldPathInfo InFieldPathInfo, const TArray<URemoteControlBinding*>& InBindings)
	: FRemoteControlField(InPreset, EExposedFieldType::Property, InLabel, MoveTemp(InFieldPathInfo), InBindings)
{
	InitializeMetadata();
	OwnerClass = GetSupportedBindingClass();

	if (FProperty* Property = GetProperty())
	{
		bIsEditorOnly = Property->HasAnyPropertyFlags(CPF_EditorOnly);
		bIsEditableInPackaged = !Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly);
	}
}

uint32 FRemoteControlProperty::GetUnderlyingEntityIdentifier() const
{
	return FieldPathInfo.PathHash;
}

FProperty* FRemoteControlProperty::GetProperty() const
{
	// Make a copy in order to preserve constness.
	FRCFieldPathInfo FieldPathCopy = FieldPathInfo;
	TArray<UObject*> Objects = GetBoundObjects();
	if (Objects.Num() && FieldPathCopy.Resolve(Objects[0]))
	{
		FRCFieldResolvedData Data = FieldPathCopy.GetResolvedData();
		return Data.Field;
	}
	return nullptr;
}

void* FRemoteControlProperty::GetFieldContainerAddress() const
{
	// Make a copy in order to preserve constness.
	FRCFieldPathInfo FieldPathCopy = FieldPathInfo;
	TArray<UObject*> Objects = GetBoundObjects();
	if (Objects.Num() && FieldPathCopy.Resolve(Objects[0]))
	{
		const FRCFieldResolvedData Data = FieldPathCopy.GetResolvedData();
		return Data.ContainerAddress;
	}
	return nullptr;
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlProperty::GetPropertyHandle() const 
{
	TSharedPtr<FRemoteControlProperty> ThisPtr = Owner->GetExposedEntity<FRemoteControlProperty>(GetId()).Pin();
	FProperty* Property = GetProperty();
	const int32 ArrayIndex = Property->ArrayDim != 1 ? -1 : 0;
	constexpr FProperty* ParentProperty = nullptr;
	const TCHAR* ParentFieldPath = TEXT("");
	return FRemoteControlPropertyHandle::GetPropertyHandle(ThisPtr, Property, ParentProperty, ParentFieldPath, ArrayIndex);
}

void FRemoteControlProperty::EnableEditCondition()
{
#if WITH_EDITOR
	if (FProperty* Property = GetProperty())
	{
		FRCFieldPathInfo EditConditionPath;

		if (CachedEditConditionPath.GetSegmentCount())
		{
			EditConditionPath = CachedEditConditionPath;
		}
		else
		{
			const FString EditConditionPropertyName = Property->GetMetaData("EditCondition");
			if (!EditConditionPropertyName.IsEmpty())
			{
				EditConditionPath = FieldPathInfo;
				EditConditionPath.Segments.Pop();
				EditConditionPath.Segments.Emplace(EditConditionPropertyName);
				CachedEditConditionPath = EditConditionPath;
			}
		}

		for (UObject* Object : GetBoundObjects())
		{
			if (EditConditionPath.Resolve(Object))
			{
				FRCFieldResolvedData Data = EditConditionPath.GetResolvedData();
				if (ensure(Data.IsValid() && Data.Field->IsA(FBoolProperty::StaticClass())))
				{
					if (!Data.Field->HasAnyPropertyFlags(CPF_EditConst))
					{
						CastFieldChecked<FBoolProperty>(Data.Field)->SetPropertyValue_InContainer(Data.ContainerAddress, true);
					}
				}
			}
		}
	}
#endif
}

bool FRemoteControlProperty::IsEditableInPackaged(FString* OutError) const
{
	// Inaccurate: bIsEditableInPackaged = !Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly) -> leads to false positives
	return IRemoteControlModule::Get().PropertySupportsRawModification(GetProperty(), GetBoundObject(), false, OutError);
}

bool FRemoteControlProperty::IsEditableInEditor(FString* OutError) const
{
	return IRemoteControlModule::Get().PropertySupportsRawModification(GetProperty(), GetBoundObject(), true, OutError);
}

bool FRemoteControlProperty::IsEditable(FString* OutError) const
{
	if (FApp::IsGame())
	{
		return IsEditableInPackaged(OutError); 
	}
	return IsEditableInEditor(OutError);
}

bool FRemoteControlProperty::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FRemoteControlObjectVersion::GUID);
	FRemoteControlProperty::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)this, FRemoteControlProperty::StaticStruct(), nullptr);

	return true;
}

void FRemoteControlProperty::PostSerialize(const FArchive& Ar)
{
	FRemoteControlField::PostSerialize(Ar);

	if (Ar.IsLoading())
	{
		if (FProperty* Property = GetProperty())
		{
			int32 CustomVersion = Ar.CustomVer(FRemoteControlObjectVersion::GUID);
			if (CustomVersion < FRemoteControlObjectVersion::AddedFieldFlags)
			{
				bIsEditorOnly = Property->HasAnyPropertyFlags(CPF_EditorOnly);
				bIsEditableInPackaged = !Property->HasAnyPropertyFlags(CPF_BlueprintReadOnly);
			}
		}
		
		if (UserMetadata.Num() == 0)
		{
			InitializeMetadata();
		}
	}
}

UClass* FRemoteControlProperty::GetSupportedBindingClass() const
{
	if (UClass* Class = OwnerClass.TryLoadClass<UObject>())
	{
		return Class;	
	}
	
	if (FProperty* Property = GetProperty())
	{
		if (UClass* PropertyOwnerClass = Property->GetOwnerClass())
		{
			return PropertyOwnerClass;
		}
		else if (!OwnerClass.IsValid() && FieldPathInfo.GetSegmentCount() > 0 && FieldPathInfo.GetFieldSegment(0).IsResolved())
		{
			return FieldPathInfo.GetFieldSegment(0).ResolvedData.Field->GetOwnerClass();
		}
	}
	return nullptr;
}

bool FRemoteControlProperty::IsBound() const
{
	return !!GetProperty();
}

bool FRemoteControlProperty::SupportsMasking() const
{
	return IRemoteControlModule::Get().SupportsMasking(GetProperty());
}

bool FRemoteControlProperty::CheckIsBoundToPropertyPath(const FString& InPath) const
{
	return FieldPathInfo.ToPathPropertyString() == InPath;
}

bool FRemoteControlProperty::CheckIsBoundToString(const FString& InPath) const
{
	return FieldPathInfo.ToString() == InPath;
}

bool FRemoteControlProperty::ContainsBoundObjects(TArray<UObject*> InObjects) const
{
	const TArray<UObject*> BoundObjects = GetBoundObjects();

	if (BoundObjects.Num() == 0 || InObjects.Num() == 0)
	{
		return false;
	}

	for (UObject* BoundObject : BoundObjects)
	{
		if (InObjects.Contains(BoundObject))
		{
			return true;
		}
	}
	
	return false;
}

void FRemoteControlProperty::InitializeMetadata()
{
#if WITH_EDITOR
	auto SupportsMinMax = [] (FProperty* Property)
	{
		if (!Property)
		{
			return false;
		}

		if (Property->IsA<FNumericProperty>())
		{
			return !(Property->IsA<FByteProperty>() && CastFieldChecked<FByteProperty>(Property)->IsEnum());
		}

		if (Property->IsA<FStructProperty>())
		{
			UStruct* Struct = CastFieldChecked<FStructProperty>(Property)->Struct;
			return Struct->IsChildOf(TBaseStructure<FVector>::Get())
				|| Struct->IsChildOf(TBaseStructure<FVector2D>::Get())
				|| Struct->IsChildOf(TBaseStructure<FRotator>::Get());
		}

		return false;
	};
	
	if (FProperty* Property = GetProperty())
	{
		if (SupportsMinMax(Property))
		{
			const FString& UIMin = Property->GetMetaData(TEXT("UIMin"));
			const FString& UIMax = Property->GetMetaData(TEXT("UIMax"));
			const FString& ClampMin = Property->GetMetaData(TEXT("ClampMin"));
			const FString& ClampMax = Property->GetMetaData(TEXT("ClampMax"));

			const FString& NewMinEntry = !UIMin.IsEmpty() ? UIMin : ClampMin;
			const FString& NewMaxEntry = !UIMax.IsEmpty() ? UIMax : ClampMax;

			// Add the metadata even if empty in case the user wants to specify it themself.
			UserMetadata.FindOrAdd(MetadataKey_Min) = NewMinEntry;
			UserMetadata.FindOrAdd(MetadataKey_Max) = NewMaxEntry;
		}
	}
#endif
}

FRemoteControlFunction::FRemoteControlFunction(FName InLabel, FRCFieldPathInfo FieldPathInfo, UFunction* InFunction)
	: FRemoteControlField(nullptr, EExposedFieldType::Function, InLabel, MoveTemp(FieldPathInfo), {})
	, FunctionPath(InFunction)
{
	checkSlow(InFunction);
	FunctionArguments = MakeShared<FStructOnScope>(InFunction);
	InFunction->InitializeStruct(FunctionArguments->GetStructMemory());
	AssignDefaultFunctionArguments();
	OwnerClass = GetSupportedBindingClass();

#if WITH_EDITOR
	CachedFunctionArgsHash = HashFunctionArguments(InFunction);
#endif
}

FRemoteControlFunction::FRemoteControlFunction(URemoteControlPreset* InPreset, FName InLabel, FRCFieldPathInfo InFieldPathInfo, UFunction* InFunction, const TArray<URemoteControlBinding*>& InBindings)
	: FRemoteControlField(InPreset, EExposedFieldType::Function, InLabel, MoveTemp(InFieldPathInfo), InBindings)
	, FunctionPath(InFunction)
{
	check(InFunction);
	FunctionArguments = MakeShared<FStructOnScope>(InFunction);
	InFunction->InitializeStruct(FunctionArguments->GetStructMemory());
	CachedFunction = InFunction;
	AssignDefaultFunctionArguments();
	OwnerClass = GetSupportedBindingClass();

	bIsEditorOnly = InFunction->HasAnyFunctionFlags(FUNC_EditorOnly);
	bIsCallableInPackaged = InFunction->HasAnyFunctionFlags(FUNC_BlueprintCallable);

#if WITH_EDITOR
	CachedFunctionArgsHash = HashFunctionArguments(InFunction);
#endif
}

uint32 FRemoteControlFunction::GetUnderlyingEntityIdentifier() const
{
	if (UFunction* Function = GetFunction())
	{
		return GetTypeHash(Function->GetName());
	}
	
	return 0;
}

bool FRemoteControlFunction::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		Ar << *this;
	}
	return true;
}

void FRemoteControlFunction::PostSerialize(const FArchive& Ar)
{
	FRemoteControlField::PostSerialize(Ar);

	if (Ar.IsLoading())
	{
		int32 CustomVersion = Ar.CustomVer(FRemoteControlObjectVersion::GUID);

		if (CustomVersion < FRemoteControlObjectVersion::AddedFieldFlags)
		{
			if (UFunction* Function = GetFunction())
			{
				bIsEditorOnly = Function->HasAnyFunctionFlags(FUNC_EditorOnly);
				bIsCallableInPackaged = Function->HasAnyFunctionFlags(FUNC_BlueprintCallable);
			}
		}

#if WITH_EDITOR
		CachedFunctionArgsHash = HashFunctionArguments(GetFunction());
#endif
	}
}

UClass* FRemoteControlFunction::GetSupportedBindingClass() const
{
	if (UClass* Class = OwnerClass.TryLoadClass<UObject>())
	{
		return Class;	
	}

	if (UFunction* Function = GetFunction())
	{
		return Function->GetOwnerClass();
	}
	return nullptr;
}

bool FRemoteControlFunction::IsBound() const
{
	if (UFunction* Function = GetFunction())
	{
		TArray<UObject*> BoundObjects = GetBoundObjects();
		if (!BoundObjects.Num())
		{
			return false;
		}
		
		if (UClass* SupportedClass = GetSupportedBindingClass())
		{
			return BoundObjects.ContainsByPredicate([SupportedClass](UObject* Object){ return Object->GetClass() && Object->GetClass()->IsChildOf(SupportedClass);});
		}
	}
	
	return false;
}

UFunction* FRemoteControlFunction::GetFunction() const
{
	UObject* ResolvedObject = FunctionPath.ResolveObject();
	
	if (!ResolvedObject)
	{
		ResolvedObject = FunctionPath.TryLoad();
	}

	UFunction* ResolvedFunction = Cast<UFunction>(ResolvedObject);
	if (ResolvedFunction)
	{
		CachedFunction = ResolvedFunction;
	}
	
	return ResolvedFunction;
}

#if WITH_EDITOR
void FRemoteControlFunction::RegenerateArguments()
{
	// Recreate the function arguments with the function from the new BP and copy the old ones on top of it.
	if (UFunction* Function = GetFunction())
	{
		FStructOnScope NewFunctionOnScope{ Function };

		// Only port the old values if the new function has the same arguments.
		// We use a hash to determine compatibility because the old function may not be available after a recompile.
		const uint32 NewHash = HashFunctionArguments(Function);
		if (CachedFunctionArgsHash == NewHash)
		{
			for (TFieldIterator<FProperty> It(Function); It; ++It)
			{
				It->CopyCompleteValue_InContainer(NewFunctionOnScope.GetStructMemory(), FunctionArguments->GetStructMemory());
			}
		}
		else
		{
			CachedFunctionArgsHash = NewHash; 
		}
		
		*FunctionArguments = MoveTemp(NewFunctionOnScope);
	}
}
#endif

void FRemoteControlFunction::AssignDefaultFunctionArguments()
{
#if WITH_EDITOR
	if (UFunction* Function = GetFunction())
	{
		for (TFieldIterator<FProperty> It(Function); It; ++It)
		{
			if (!It->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				const FName DefaultPropertyKey = *FString::Printf(TEXT("CPP_Default_%s"), *It->GetName());
				const FString& PropertyDefaultValue = Function->GetMetaData(DefaultPropertyKey);
				if (!PropertyDefaultValue.IsEmpty())
				{
					It->ImportText_Direct(*PropertyDefaultValue, It->ContainerPtrToValuePtr<uint8>(FunctionArguments->GetStructMemory()), NULL, PPF_None);
				}
			}
		}
	}
#endif
}

#if WITH_EDITOR
uint32 FRemoteControlFunction::HashFunctionArguments(UFunction* InFunction)
{
	if (!InFunction)
	{
		return 0;
	}

	uint32 Hash = GetTypeHash(InFunction->NumParms);

	for (TFieldIterator<FProperty> It(InFunction); It; ++It)
	{
		const bool Param = It->HasAnyPropertyFlags(CPF_Parm);
		const bool OutParam = It->HasAnyPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_ConstParm);
		const bool ReturnParam = It->HasAnyPropertyFlags(CPF_ReturnParm);

		if (!Param || OutParam || ReturnParam)
		{
			continue;
		}

		Hash = HashCombine(Hash, GetTypeHash(It->GetClass()->GetFName()));
		Hash = HashCombine(Hash, GetTypeHash(It->GetSize()));
	}

	return Hash;
}
#endif

FArchive& operator<<(FArchive& Ar, FRemoteControlFunction& RCFunction)
{
	Ar.UsingCustomVersion(FRemoteControlObjectVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	FRemoteControlFunction::StaticStruct()->SerializeTaggedProperties(Ar, (uint8*)&RCFunction, FRemoteControlFunction::StaticStruct(), nullptr);

	if (Ar.IsLoading())
	{
#if WITH_EDITOR
		if (RCFunction.Function_DEPRECATED && RCFunction.FunctionPath.IsValid())
		{
			RCFunction.FunctionPath = RCFunction.Function_DEPRECATED;
		}
#endif

		int32 CustomVersion = Ar.CustomVer(FReleaseObjectVersion::GUID);
		int64 NumSerializedBytesFromArchive = 0;

		if (CustomVersion >= FReleaseObjectVersion::RemoteControlSerializeFunctionArgumentsSize)
		{
			Ar << NumSerializedBytesFromArchive;
		}
		
		int64 ArgsBegin = Ar.Tell();
		if (UFunction* Function = RCFunction.GetFunction())
		{
			RCFunction.FunctionArguments = MakeShared<FStructOnScope>(Function);
			Function->SerializeTaggedProperties(Ar, RCFunction.FunctionArguments->GetStructMemory(), Function, nullptr);

			if (ArgsBegin != INDEX_NONE
				&& NumSerializedBytesFromArchive != 0
				&& (Ar.Tell() - ArgsBegin) != NumSerializedBytesFromArchive)
			{
				UE_LOGF(LogRemoteControl, Warning, "Arguments size mismatch from size serialized in asset. Expected %lld, Actual: %lld", NumSerializedBytesFromArchive, Ar.Tell() - ArgsBegin);
				Ar.SetError();
			}
		}
		else
		{
			UE_LOGF(LogRemoteControl, Warning, "%ls could not be loaded while deserialzing a Remote Control Function.", *RCFunction.FunctionPath.ToString());
			if (NumSerializedBytesFromArchive > 0)
			{
				// Skip over chunk of data if we were unable to resolve the class.
				Ar.Seek(Ar.Tell() + NumSerializedBytesFromArchive);
			}
		}
	}
	else
	{
		// The following code serializes the size of the arguments to serialize so that when loading,
		// we can skip over it if the function could not be loaded.
		const int64 ArgumentsSizePos = Ar.Tell();

		// Serialize a temporary value for the delta in order to end up with an archive of the right size.
		int64 ArgumentsSize = 0;
		Ar << ArgumentsSize;

		// Then serialize the arguments in order to get its size.
		const int64 ArgsBegin = Ar.Tell();
		if (UFunction* Function = RCFunction.CachedFunction.Get())
		{
			Function->SerializeTaggedProperties(Ar, RCFunction.FunctionArguments->GetStructMemory(), Function, nullptr);
		}

		// Only go back and serialize the number of argument bytes if there is actually an underlying buffer to seek to.
		if (ArgumentsSizePos != INDEX_NONE)
		{
			const int64 ArgsEnd = Ar.Tell();
			ArgumentsSize = (ArgsEnd - ArgsBegin);

			// Come back to the temporary value we wrote and overwrite it with the arguments size we just calculated.
			Ar.Seek(ArgumentsSizePos);
			Ar << ArgumentsSize;

			// And finally seek back to the end.
			Ar.Seek(ArgsEnd);
		}
	}
	return Ar;
}
