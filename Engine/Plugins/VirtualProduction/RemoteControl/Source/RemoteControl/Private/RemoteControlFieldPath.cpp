// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlFieldPath.h"
#include "StructUtils/InstancedStruct.h"

namespace RemoteControlFieldUtils
{
	void ResolveSegment(const FRCFieldPathSegment& Segment, UStruct* Owner, void* ContainerAddress, FRCFieldResolvedData& OutResolvedData)
	{
		// When resolving for Instanced Struct, use the underlying Struct/Memory
		if (Owner == TBaseStructure<FInstancedStruct>::Get())
		{
			FInstancedStruct& InstancedStruct = *static_cast<FInstancedStruct*>(ContainerAddress);
			Owner = const_cast<UScriptStruct*>(InstancedStruct.GetScriptStruct());
			ContainerAddress = InstancedStruct.GetMutableMemory();
		}

		if (FProperty* FoundField = FindFProperty<FProperty>(Owner, Segment.Name))
		{
			OutResolvedData.ContainerAddress = ContainerAddress;
			OutResolvedData.Field = FoundField;
			OutResolvedData.Struct = Owner;

			// Convert the map key to an index if the key was provided.
			if (!Segment.MapKey.IsEmpty() && FoundField->IsA<FMapProperty>())
			{
				FScriptMapHelper_InContainer MapHelper(CastField<FMapProperty>(FoundField), ContainerAddress);
				OutResolvedData.MapIndex = MapHelper.FindMapIndexWithKey(&Segment.MapKey);
			}
		}
		else
		{
			OutResolvedData.ContainerAddress = nullptr;
			OutResolvedData.Field = nullptr;
			OutResolvedData.Struct = nullptr;
		}
	}
}


bool FRCFieldPathInfo::ResolveInternalRecursive(UStruct* OwnerType, void* ContainerAddress, int32 SegmentIndex)
{
	// NOTE: There are other struct types like FSharedStruct, FInstancedStructContainer that could need support here in a similar way.
	if (OwnerType == TBaseStructure<FInstancedStruct>::Get() && Segments[SegmentIndex].Name == TEXT("Struct"))
	{
		// Instanced Structs generate an additional "Struct." segment from property handles path generation, as they use FStructurePropertyNode
		// This "Struct." segment is not a property to resolve so needs to be skipped.
		// see FStructurePropertyNode::GetQualifiedName.
		++SegmentIndex;
		if (!ensureMsgf(SegmentIndex < Segments.Num(), TEXT("Instanced Structs expect more segments after 'Struct'. Index: %d, Num: %d"), SegmentIndex, Segments.Num()))
		{
			return false;
		}
	}

	const bool bLastSegment = (SegmentIndex == Segments.Num() - 1);

	//Resolve the desired segment
	FRCFieldPathSegment& Segment = Segments[SegmentIndex];
	RemoteControlFieldUtils::ResolveSegment(Segment, OwnerType, ContainerAddress, Segment.ResolvedData);

	if (bLastSegment == false)
	{
		if (Segment.IsResolved())
		{
			const int32 ArrayIndex = Segment.ArrayIndex == INDEX_NONE ? 0 : Segment.ArrayIndex;
			//Not the last segment so we'll call ourself again digging into structures / arrays / containers

			FProperty* Property = Segment.ResolvedData.Field;
			if (FStructProperty* StructureProperty = CastField<FStructProperty>(Property))
			{
				return ResolveInternalRecursive(StructureProperty->Struct, StructureProperty->ContainerPtrToValuePtr<void>(ContainerAddress, ArrayIndex), SegmentIndex + 1);
			}
			else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
			{
				//Look for the kind of array this is. Since it's not the final segment, it must be a container thing
				if (FStructProperty* ArrayInnerStructureProperty = CastField<FStructProperty>(ArrayProperty->Inner))
				{
					FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, ContainerAddress);
					if (ArrayHelper.IsValidIndex(ArrayIndex))
					{
						return ResolveInternalRecursive(ArrayInnerStructureProperty->Struct, reinterpret_cast<void*>(ArrayHelper.GetRawPtr(ArrayIndex)), SegmentIndex + 1);
					}
				}
			}
			else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
			{
				//Look for the kind of set this is. Since it's not the final segment, it must be a container thing
				if (FStructProperty* SetInnerStructureProperty = CastField<FStructProperty>(SetProperty->ElementProp))
				{
					FScriptSetHelper_InContainer SetHelper(SetProperty, ContainerAddress);
					if (SetHelper.IsValidIndex(ArrayIndex))
					{
						return ResolveInternalRecursive(SetInnerStructureProperty->Struct, reinterpret_cast<void*>(SetHelper.GetElementPtr(ArrayIndex)), SegmentIndex + 1);
					}
				}
			}
			else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
			{
				//Look for the kind of map this is. Since it's not the final segment, it must be a container thing
				if (FStructProperty* MapInnerStructureProperty = CastField<FStructProperty>(MapProperty->ValueProp))
				{
					FScriptMapHelper_InContainer MapHelper(MapProperty, ContainerAddress);
					int32 MapIndex = Segment.ResolvedData.MapIndex != INDEX_NONE ? Segment.ResolvedData.MapIndex : ArrayIndex;
					if (MapHelper.IsValidIndex(MapIndex))
					{
						return ResolveInternalRecursive(MapInnerStructureProperty->Struct, reinterpret_cast<void*>(MapHelper.GetValuePtr(MapIndex)), SegmentIndex + 1);
					}
				}
			}

			//Add support for missing types if required (SoftObjPtr, ObjectProperty, etc...)
			return false;
		}
	}

	return Segment.IsResolved();
}

FRCFieldPathInfo::FRCFieldPathInfo(const FString& PathInfo, bool bCleanDuplicates)
{
	Initialize(PathInfo, bCleanDuplicates);
}

FRCFieldPathInfo::FRCFieldPathInfo(FProperty* Property)
{
	if (ensure(Property))
	{
		Initialize(Property->GetName(), false);	
	}
}

bool FRCFieldPathInfo::Resolve(UObject* Owner)
{
	if (Owner == nullptr)
	{
		return false;
	}

	if (Segments.Num() <= 0)
	{
		return false;
	}

	void* ContainerAddress = reinterpret_cast<void*>(Owner);
	UStruct* Type = Owner->GetClass();
	return ResolveInternalRecursive(Type, ContainerAddress, 0);
}

bool FRCFieldPathInfo::IsResolved() const
{
	const int32 SegmentCount = GetSegmentCount();
	if (SegmentCount <= 0)
	{
		return false;
	}

	return GetFieldSegment(SegmentCount - 1).IsResolved();
}

bool FRCFieldPathInfo::IsEqual(FStringView OtherPath) const
{
	return GetTypeHash(OtherPath) == PathHash;
}

bool FRCFieldPathInfo::IsEqual(const FRCFieldPathInfo& OtherPath) const
{
	return OtherPath.PathHash == PathHash;
}

FString FRCFieldPathInfo::ToString(int32 EndSegment /*= INDEX_NONE*/) const
{
	const int32 LastSegment = EndSegment == INDEX_NONE ? Segments.Num() : FMath::Min(Segments.Num(), EndSegment);
	FString FullPath;
	for (int32 SegmentIndex = 0; SegmentIndex < LastSegment; ++SegmentIndex)
	{
		const FRCFieldPathSegment& Segment = GetFieldSegment(SegmentIndex);

		// Segment
		FullPath += Segment.ToString();

		// Delimiter
		if (SegmentIndex < GetSegmentCount() - 1)
		{
			FullPath += TEXT(".");
		}
	}

	return FullPath;
}

FString FRCFieldPathInfo::ToPathPropertyString(int32 EndSegment /*= INDEX_NONE*/) const
{
	const int32 LastSegment = EndSegment == INDEX_NONE ? Segments.Num() : FMath::Min(Segments.Num(), EndSegment);
	FString FullPath;
	for (int32 SegmentIndex = 0; SegmentIndex < LastSegment; ++SegmentIndex)
	{
		const FRCFieldPathSegment& Segment = GetFieldSegment(SegmentIndex);

		// Segment
		FullPath += Segment.ToString(true /*bDuplicateContainer*/);

		// Delimiter
		if (SegmentIndex < GetSegmentCount() - 1)
		{
			FullPath += TEXT(".");
		}
	}

	return FullPath;
}

const FRCFieldPathSegment& FRCFieldPathInfo::GetFieldSegment(int32 Index) const
{
	check(Segments.IsValidIndex(Index));
	return Segments[Index];
}

FRCFieldResolvedData FRCFieldPathInfo::GetResolvedData() const
{
	if (IsResolved())
	{
		return GetFieldSegment(GetSegmentCount() - 1).ResolvedData;
	}

	return FRCFieldResolvedData();
}

FName FRCFieldPathInfo::GetFieldName() const
{
	if (GetSegmentCount() <= 0)
	{
		return NAME_None;
	}

	return *GetFieldSegment(GetSegmentCount() - 1).ToString();
}

FRCFieldPathSegment::FRCFieldPathSegment(FStringView SegmentName)
{
	bool bValidSegment = false;

	int32 FieldNameEnd = MAX_int32;
	int32 OpenBracketIndex;
	if (SegmentName.FindChar('[', OpenBracketIndex))
	{
		if (OpenBracketIndex > 0)
		{
			FieldNameEnd = OpenBracketIndex;
		
			//Found an open bracket, find the closing one
			int32 CloseBracketIndex;
			if (SegmentName.FindChar(']', CloseBracketIndex) && (CloseBracketIndex > OpenBracketIndex + 1))
			{
				if (SegmentName[OpenBracketIndex + 1] == TEXT('\"') && SegmentName[CloseBracketIndex - 1] == TEXT('\"'))
				{
					// Handles map key indexing with a string.
					const int32 KeyStartIndex = OpenBracketIndex + 2;
					const int32 KeyEndIndex = CloseBracketIndex - 2;
					// Only pick the part between the quotes
					MapKey = SegmentName.Mid(KeyStartIndex, KeyEndIndex - OpenBracketIndex - 1);
					bValidSegment = true;
				}
				else
				{
					// Handles array/set indexing.
					FStringView IndexString = SegmentName.Mid(OpenBracketIndex + 1, CloseBracketIndex - OpenBracketIndex - 1);
					ArrayIndex = FCString::Atoi(IndexString.GetData());

					//  Catch case where an invalid string gets parsed as 0.
					const bool bInvalidIndex = (ArrayIndex == INDEX_NONE)
						|| (ArrayIndex == 0 && IndexString.Len() != 1);

					if (!bInvalidIndex)
					{
						bValidSegment = true;
					}

					// Check if this is a map value index
					FStringView OnlyName = SegmentName.Left(FieldNameEnd);
					static const FString ValueSuffix = TEXT("_Value");
					if (OnlyName.EndsWith(ValueSuffix))
					{
						ValuePropertyName = OnlyName.LeftChop(ValueSuffix.Len());
					}
				}
			}
		}
	}
	else
	{
		bValidSegment = true;
	}

	if (bValidSegment)
	{
		FStringView FieldName = SegmentName.Mid(0, FieldNameEnd);
		Name = FName(FieldName);
	}
}

bool FRCFieldPathSegment::IsResolved() const
{
	const bool bMapKeyResolved = MapKey.IsEmpty() || (!MapKey.IsEmpty() && ResolvedData.MapIndex != INDEX_NONE);
	return ResolvedData.IsValid()
		&& bMapKeyResolved;
}

FString FRCFieldPathSegment::ToString(bool bDuplicateContainer) const
{
	FString Output;
	FString IndexString = ArrayIndex != INDEX_NONE ? FString::FromInt(ArrayIndex) : MapKey;

	if (ArrayIndex == INDEX_NONE && MapKey.IsEmpty())
	{
		Output = FString::Printf(TEXT("%s"), *Name.ToString());
	}
	else
	{
		//Special case for GeneratePathToProperty match
		if (bDuplicateContainer)
		{
			if (ValuePropertyName.IsEmpty())
			{
				Output = FString::Printf(TEXT("%s.%s[%s]"), *Name.ToString(), *Name.ToString(), *IndexString);
			}
			else
			{
				if (ArrayIndex != INDEX_NONE)
				{
					Output = FString::Printf(TEXT("%s.%s_Value[%d]"), *Name.ToString(), *ValuePropertyName, ArrayIndex);
				}
				else if (!MapKey.IsEmpty())
				{
					// This format needs the numerical index of the entry in the map instead of the key.
					if (IsResolved() && ResolvedData.Field->IsA<FMapProperty>())
					{
						FScriptMapHelper_InContainer MapHelper(CastField<FMapProperty>(ResolvedData.Field), ResolvedData.ContainerAddress);
						int32 ElementIndex = MapHelper.FindMapIndexWithKey(&MapKey);
						if (ElementIndex != INDEX_NONE)
						{
							Output = FString::Printf(TEXT("%s.%s_Value[%d]"), *Name.ToString(), *ValuePropertyName, ElementIndex);
						}
					}
				}
			}
		}
		else
		{
			Output = FString::Printf(TEXT("%s[%s]"), *Name.ToString(), *IndexString);
		}
	}
	return Output;
}

void FRCFieldPathSegment::ClearResolvedData()
{
	ResolvedData = FRCFieldResolvedData();
}

FPropertyChangedEvent FRCFieldPathInfo::ToPropertyChangedEvent(EPropertyChangeType::Type InChangeType) const
{
	check(IsResolved());

	FPropertyChangedEvent PropertyChangedEvent(GetFieldSegment(GetSegmentCount() - 1).ResolvedData.Field, InChangeType);

	// Set a containing 'struct' if we need to
	if (GetSegmentCount() > 1)
	{
		PropertyChangedEvent.SetActiveMemberProperty(GetFieldSegment(0).ResolvedData.Field);
	}

	return PropertyChangedEvent;
}

void FRCFieldPathInfo::ToEditPropertyChain(FEditPropertyChain& OutPropertyChain) const
{
	check(IsResolved());

	//Go over the segment chain to build the property changed chain skipping duplicates
	for (int32 Index = 0; Index < GetSegmentCount(); ++Index)
	{
		const FRCFieldPathSegment& Segment = GetFieldSegment(Index);
		OutPropertyChain.AddTail(Segment.ResolvedData.Field);
	}

	OutPropertyChain.SetActivePropertyNode(OutPropertyChain.GetTail()->GetValue());
	if (GetSegmentCount() > 1)
	{
		OutPropertyChain.SetActiveMemberPropertyNode(OutPropertyChain.GetHead()->GetValue());
	}
}

#if WITH_EDITOR
TSharedRef<FPropertyPath> FRCFieldPathInfo::ToPropertyPath() const
{
	TSharedRef<FPropertyPath> Path = FPropertyPath::CreateEmpty();
	check(IsResolved());

	for (const FRCFieldPathSegment& Segment : Segments)
	{
		if (!Segment.Name.IsNone())
		{
			// Property path map entries contain duplicates.
			if (!Segment.MapKey.IsEmpty())
			{
				Path->AddProperty(FPropertyInfo(Segment.ResolvedData.Field));
			}

			if (Segment.ArrayIndex != -1)
			{
				if (FArrayProperty* Property = CastField<FArrayProperty>(Segment.ResolvedData.Field))
				{
					FPropertyInfo Info;
					Info.Property = Segment.ResolvedData.Field;

					Path->AddProperty(FPropertyInfo(Property));
					Path->AddProperty(FPropertyInfo(Property->Inner, Segment.ArrayIndex));

					continue;
				}
				
			}

			FPropertyInfo Info;
			Info.Property = Segment.ResolvedData.Field;
			Info.ArrayIndex = Segment.ArrayIndex;

			Path->AddProperty(Info);
		}
	}

	return Path;
}
#endif

void FRCFieldPathInfo::Initialize(const FString& PathInfo, bool bCleanDuplicates)
{
	TArray<FString> PathSegment;
	PathInfo.ParseIntoArray(PathSegment, TEXT("."));

	Segments.Reserve(PathSegment.Num());
	for (int32 Index = 0; Index < PathSegment.Num(); ++Index)
	{
		const FString& SegmentString = PathSegment[Index];
		FRCFieldPathSegment NewSegment(SegmentString);

		if (bCleanDuplicates && Index > 0 && NewSegment.ArrayIndex != INDEX_NONE)
		{
			FRCFieldPathSegment& PreviousSegment = Segments[Segments.Num() - 1];

			if (PreviousSegment.Name == NewSegment.Name)
			{
				//Skip duplicate entries if required for GeneratePathToProperty style (Array.Array[Index])
				PreviousSegment.ArrayIndex = NewSegment.ArrayIndex;
				continue;
			}
			else if (!NewSegment.ValuePropertyName.IsEmpty())
			{
				//Skip duplicate entries for a map entry if required for GeneratePathToProperty style (Map.Map_Value[Index])
				PreviousSegment.ValuePropertyName = NewSegment.ValuePropertyName;
				PreviousSegment.ArrayIndex = NewSegment.ArrayIndex;
				continue;
			}

		}

		Segments.Emplace(MoveTemp(NewSegment));
	}

	PathHash = GetTypeHash(PathInfo);
}
