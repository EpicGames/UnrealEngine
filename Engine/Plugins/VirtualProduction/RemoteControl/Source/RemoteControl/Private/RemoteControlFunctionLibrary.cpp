// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlFunctionLibrary.h"
#include "IRemoteControlModule.h"

namespace RemoteControlFunctionLibrary
{
	FGuid GetOrCreateGroup(URemoteControlPreset* Preset, const FString& GroupName)
	{
		FGuid GroupId;
		if (GroupName.IsEmpty())
		{
			GroupId = Preset->Layout.GetDefaultGroup().Id;
		}
		else
		{
			if (FRemoteControlPresetGroup* Group = Preset->Layout.GetGroupByName(*GroupName))
			{
				GroupId = Group->Id;
			}
			else
			{
				GroupId = Preset->Layout.CreateGroup(*GroupName).Id;
			}
		}
		return GroupId;
	}

	/** Convert a value from the range for a normal linear color (0-1) to the range for a color grading wheel (provided by MinValue and MaxValue). */
	void TransformLinearColorRangeToColorGradingRange(FVector4& Value, float MinValue, float MaxValue)
	{
		check(MaxValue > MinValue);

		Value *= (MaxValue - MinValue);
		Value += FVector4(MinValue, MinValue, MinValue, MinValue);
	}

	/** Convert a value from the range for a color grading wheel (provided by MinValue and MaxValue) to the range for a normal linear color (0-1). */
	void TransformColorGradingRangeToLinearColorRange(FVector4& Value, float MinValue, float MaxValue)
	{
		check(MaxValue > MinValue);

		Value -= FVector4(MinValue, MinValue, MinValue, MinValue);
		Value /= (MaxValue - MinValue);
	}

	/**
	 * Apply a wheel-based color delta to an RGB linear color.
	 */
	void ApplyWheelColorBaseDelta(FLinearColor& InOutColor, const FColorWheelColorBase& DeltaValue, const FColorWheelColorBase& ReferenceColor, float MinValue, float MaxValue)
	{
		// Convert to HSV
		InOutColor = InOutColor.LinearRGBToHSV();

		FVector2D Position;

		if (InOutColor.B > UE_DOUBLE_KINDA_SMALL_NUMBER)
		{
			// Determine direction as a unit vector based on the calculated hue
			const double HueRadians = FMath::DegreesToRadians(InOutColor.R);
			Position = FVector2D(FMath::Cos(HueRadians), FMath::Sin(HueRadians));

			// Multiply the unit vector by saturation to determine the current position in the color wheel
			Position *= InOutColor.G;
		}
		else
		{
			// Color's value is too low to determine the hue and saturation. Fall back to the reference color's position
			Position = ReferenceColor.Position;
		}

		// Apply the delta to the position, then convert back to hue and saturation
		Position += FVector2D(DeltaValue.Position.X, DeltaValue.Position.Y);
		InOutColor.R = FMath::Fmod(FMath::RadiansToDegrees(FMath::Atan2(Position.Y, Position.X)) + 360.0, 360.0);
		InOutColor.G = FMath::Clamp(Position.Length(), MinValue, MaxValue);

		// We're operating on a color transformed to a (0, 1) range from its full range, so scale the value delta by the full range
		check(MaxValue > MinValue);
		float ScaledDeltaValue = DeltaValue.Value / (MaxValue - MinValue);
		InOutColor.B = FMath::Clamp(InOutColor.B + ScaledDeltaValue, MinValue, MaxValue);

		InOutColor = InOutColor.HSVToLinearRGB();
	}

	/** Set a property on an object and fire relevant events. */
	void SetPropertyAndFireEvents(const FRCObjectReference& InObjectRef, const void* InValue, bool bIsInteractive)
	{
#if WITH_EDITOR
		FEditPropertyChain PropertyChain;
		InObjectRef.PropertyPathInfo.ToEditPropertyChain(PropertyChain);
		InObjectRef.Object->PreEditChange(PropertyChain);
		InObjectRef.Object->Modify();
#endif

		InObjectRef.Property->SetValue_InContainer(InObjectRef.ContainerAdress, InValue);

#if WITH_EDITOR
		SnapshotTransactionBuffer(InObjectRef.Object.Get());

		FPropertyChangedEvent PropertyEvent = InObjectRef.PropertyPathInfo.ToPropertyChangedEvent();
		PropertyEvent.ChangeType = bIsInteractive ? EPropertyChangeType::Interactive : EPropertyChangeType::ValueSet;

		FPropertyChangedChainEvent PropertyChainEvent(PropertyChain, PropertyEvent);
		InObjectRef.Object->PostEditChangeChainProperty(PropertyChainEvent);
#endif
	}
}

bool URemoteControlFunctionLibrary::ExposeProperty(URemoteControlPreset* Preset, UObject* SourceObject, const FString& Property, FRemoteControlOptionalExposeArgs Args)
{
	if (Preset && SourceObject)
	{
		return Preset->ExposeProperty(SourceObject, Property, {Args.DisplayName, RemoteControlFunctionLibrary::GetOrCreateGroup(Preset, Args.GroupName)}).IsValid();
	}
	return false;
}

bool URemoteControlFunctionLibrary::ExposeFunction(URemoteControlPreset* Preset, UObject* SourceObject, const FString& Function, FRemoteControlOptionalExposeArgs Args)
{
	if (Preset && SourceObject)
	{
		if (UFunction* TargetFunction = SourceObject->FindFunction(*Function))
		{
			return Preset->ExposeFunction(SourceObject, TargetFunction, { Args.DisplayName, RemoteControlFunctionLibrary::GetOrCreateGroup(Preset, Args.GroupName) }).IsValid();
		}
	}
	return false;
}

bool URemoteControlFunctionLibrary::ExposeActor(URemoteControlPreset* Preset, AActor* Actor, FRemoteControlOptionalExposeArgs Args)
{
	if (Preset && Actor)
	{
		return Preset->ExposeActor(Actor, {Args.DisplayName, RemoteControlFunctionLibrary::GetOrCreateGroup(Preset, Args.GroupName)}).IsValid();
	}
	return false;
}

bool URemoteControlFunctionLibrary::ApplyColorWheelDelta(UObject* TargetObject, const FString& PropertyName, const FColorWheelColor& DeltaValue, const FColorWheelColor& ReferenceColor, bool bIsInteractive)
{
	FRCObjectReference ObjectRef;
	IRemoteControlModule::Get().ResolveObject(ERCAccess::WRITE_ACCESS, TargetObject->GetPathName(), PropertyName, ObjectRef);

	if (!ObjectRef.IsValid())
	{
		return false;
	}

	if (const FStructProperty* ColorProperty = CastField<FStructProperty>(ObjectRef.Property.Get()))
	{
		if (ColorProperty->Struct != TBaseStructure<FLinearColor>::Get())
		{
			// This isn't a color
			return false;
		}

		FLinearColor Color;
		ColorProperty->GetValue_InContainer(ObjectRef.ContainerAdress, &Color);

		RemoteControlFunctionLibrary::ApplyWheelColorBaseDelta(Color, DeltaValue, ReferenceColor, 0.f, 1.f);

		// Apply the alpha change directly
		Color.A = FMath::Clamp(Color.A + DeltaValue.Alpha, 0.f, 1.f);

		RemoteControlFunctionLibrary::SetPropertyAndFireEvents(ObjectRef, &Color, bIsInteractive);

		return true;
	}

	return false;
}

bool URemoteControlFunctionLibrary::ApplyColorGradingWheelDelta(UObject* TargetObject, const FString& PropertyName, const FColorGradingWheelColor& DeltaValue, const FColorGradingWheelColor& ReferenceColor, bool bIsInteractive, float MinValue, float MaxValue)
{
	if (MinValue >= MaxValue || !TargetObject)
	{
		return false;
	}

	FRCObjectReference ObjectRef;
	IRemoteControlModule::Get().ResolveObject(ERCAccess::WRITE_ACCESS, TargetObject->GetPathName(), PropertyName, ObjectRef);
	
	if (!ObjectRef.IsValid())
	{
		return false;
	}

	if (const FStructProperty* ColorProperty = CastField<FStructProperty>(ObjectRef.Property.Get()))
	{
		if (ColorProperty->Struct != TBaseStructure<FVector4>::Get())
		{
			// This isn't an FVector4 representation of a color
			return false;
		}

		// Get the vector value and convert the color portion of it to HSV
		FVector4 VectorValue;
		ColorProperty->GetValue_InContainer(ObjectRef.ContainerAdress, &VectorValue);

		const float OldLuminance = VectorValue.W;

		RemoteControlFunctionLibrary::TransformColorGradingRangeToLinearColorRange(VectorValue, MinValue, MaxValue);

		FLinearColor Color(VectorValue.X, VectorValue.Y, VectorValue.Z);
		RemoteControlFunctionLibrary::ApplyWheelColorBaseDelta(Color, DeltaValue, ReferenceColor, MinValue, MaxValue);
		VectorValue = FVector4(Color);

		RemoteControlFunctionLibrary::TransformLinearColorRangeToColorGradingRange(VectorValue, MinValue, MaxValue);
		VectorValue.W = OldLuminance + DeltaValue.Luminance;

		RemoteControlFunctionLibrary::SetPropertyAndFireEvents(ObjectRef, &VectorValue, bIsInteractive);

		return true;
	}

	return false;
}
