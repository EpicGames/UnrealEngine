// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlInstanceMaterial.h"

#include "IRemoteControlModule.h"
#include "Materials/MaterialInstanceConstant.h"
#include "RemoteControlPreset.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "MaterialEditor/DEditorFontParameterValue.h"
#include "MaterialEditor/DEditorScalarParameterValue.h"
#include "MaterialEditor/DEditorTextureParameterValue.h"
#include "MaterialEditor/DEditorVectorParameterValue.h"
#include "MaterialEditor/MaterialEditorInstanceConstant.h"
#endif

#if WITH_EDITOR
struct FMaterialHelper
{
private:
	/** Try to get a path for material property */
	static FString GetFieldPathCallback(const FName& InPropertyName, const FRCFieldPathInfo& InOriginalFieldPath, const int32 InIndex)
	{
		FString FieldPathString = FString::Printf(TEXT("%s[%d].ParameterValue"), *InPropertyName.ToString(), InIndex);

		if (InOriginalFieldPath.Segments.Num() > 1)
		{
			FieldPathString += TEXT(".") + InOriginalFieldPath.Segments[1].ToString();
		}

		return FieldPathString;
	}

	/** Try to get a path for font material property */
	static FString GetFontFieldPathCallback(const FName& InPropertyName, const FRCFieldPathInfo& InOriginalFieldPath, const int32 InIndex)
	{
		FString FieldPathString;

		if (InOriginalFieldPath.Segments.Num() > 1)
		{
			FieldPathString = FString::Printf(TEXT("%s[%d].%s"), *InPropertyName.ToString(), InIndex, *InOriginalFieldPath.Segments[1].ToString());
		}
		else
		{
			FieldPathString = FString::Printf(TEXT("%s[%d].FontValue"), *InPropertyName.ToString(), InIndex);
		}

		return FieldPathString;
	}
	
public:
	/** Field path Callback for getting Field path */
	using FFieldPathCallback = TFunctionRef<FString(const FName& InPropertyName, const FRCFieldPathInfo&, const int32 InIndex)>;
	
	/** Try to find parameter index by given name */
	template <typename ParameterType>
	static int32 FindParameterIndexByName(const TArray<ParameterType>& Parameters, const FHashedMaterialParameterInfo& ParameterInfo)
	{
		for (int32 ParameterIndex = 0; ParameterIndex < Parameters.Num(); ++ParameterIndex)
		{
			const ParameterType* Parameter = &Parameters[ParameterIndex];
			if (Parameter->ParameterInfo == ParameterInfo)
			{
				return ParameterIndex;
			}
		}

		return INDEX_NONE;
	}

	/** Try replace Field Path based on given input */
	template <typename ParameterType>
	static bool ReplaceFieldPath(FFieldPathCallback InFieldPathCallback, const FName& InPropertyName,
	                             const TArray<ParameterType>& Parameters,
	                             const FHashedMaterialParameterInfo& ParameterInfo,
	                             const FRCFieldPathInfo& OriginalFieldPath, FRCFieldPathInfo& OutFieldPath)
	{
		const int32 Index = FindParameterIndexByName(Parameters, ParameterInfo);
		if (Index == INDEX_NONE)
		{
			return false;
		}

		const FString FieldPathString = InFieldPathCallback(InPropertyName, OriginalFieldPath, Index);

		OutFieldPath = FieldPathString;

		return true;
	}


	/** Replace Object and Field Path */
	static bool ReplaceFieldPath(UMaterialInstance* InMaterialInstance, UDEditorParameterValue* InEditorParameterValue, const FRCFieldPathInfo& OriginalFieldPath, FRCFieldPathInfo& OutFieldPath)
	{
		bool bReplaced = false;

		const FMaterialParameterInfo& ParameterInfo = InEditorParameterValue->ParameterInfo;
			
		if (InEditorParameterValue->IsA<UDEditorScalarParameterValue>())
		{
			bReplaced = ReplaceFieldPath(GetFieldPathCallback,
				GET_MEMBER_NAME_STRING_CHECKED(UMaterialInstance, ScalarParameterValues),
				InMaterialInstance->ScalarParameterValues, ParameterInfo, OriginalFieldPath,  OutFieldPath);
		}
		else if (InEditorParameterValue->IsA<UDEditorVectorParameterValue>())
		{
			bReplaced = ReplaceFieldPath(GetFieldPathCallback,
				GET_MEMBER_NAME_STRING_CHECKED(UMaterialInstance, VectorParameterValues),
				InMaterialInstance->VectorParameterValues, ParameterInfo, OriginalFieldPath, OutFieldPath);
		}
		else if (InEditorParameterValue->IsA<UDEditorTextureParameterValue>())
		{
			bReplaced = ReplaceFieldPath(GetFieldPathCallback,
				GET_MEMBER_NAME_STRING_CHECKED(UMaterialInstance, TextureParameterValues),
				InMaterialInstance->TextureParameterValues, ParameterInfo, OriginalFieldPath, OutFieldPath);
		}
		else if (InEditorParameterValue->IsA<UDEditorFontParameterValue>())
		{			
			bReplaced = ReplaceFieldPath(GetFontFieldPathCallback,
				GET_MEMBER_NAME_STRING_CHECKED(UMaterialInstance, FontParameterValues),
				InMaterialInstance->FontParameterValues, ParameterInfo, OriginalFieldPath, OutFieldPath);
		}

		return bReplaced;
		}

	template<typename ParameterType>
	static const ParameterType* FindParameterValue(const TArray<ParameterType>& Parameters, const FName& InName)
	{
		return Parameters.FindByPredicate([&InName](const ParameterType& InValue)
		{
			return InValue.ParameterInfo.Name == InName;
		});
	}
};

#endif

#if WITH_EDITOR

FRemoteControlInstanceMaterial::FRemoteControlInstanceMaterial(URemoteControlPreset* InPreset, FName InLabel,
                                                               UDEditorParameterValue* InEditorParameterValue, const FRCFieldPathInfo& InOriginalFieldPathInfo,
                                                               UObject* InInstance, const FRCFieldPathInfo& InFieldPath,
                                                               const TArray<URemoteControlBinding*>& InBindings)
	: FRemoteControlProperty(InPreset, InLabel, InFieldPath, InBindings)
	, OriginalFieldPathInfo(InOriginalFieldPathInfo)
	, InstancePath(InInstance)
{
	OriginalClass = InEditorParameterValue->GetClass();
	ParameterInfo = InEditorParameterValue->ParameterInfo;
}
#endif

#if WITH_EDITOR
void FRemoteControlInstanceMaterial::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent&)
{
	UMaterialInstance* InputMaterialInstance =  Cast<UMaterialInstance>(InObject);
	UMaterialInstance* MaterialInstance =  Cast<UMaterialInstance>(InstancePath.ResolveObject());
	
	if (!InputMaterialInstance || !MaterialInstance || InputMaterialInstance != MaterialInstance)
	{
		return;
	}

	UDEditorParameterValue* DEditorParameterValue = [InputMaterialInstance, this]()
	{
		UDEditorParameterValue* ReturnParameterValue = nullptr;
		
		for (TObjectIterator<UDEditorParameterValue> It; It; ++It)
		{
			UDEditorParameterValue* ParameterValue = *It;
			if (!ParameterValue || !ParameterValue->IsValidLowLevelFast())
			{
				continue;
			}

			if (ParameterValue->ParameterInfo.Name != ParameterInfo.Name)
			{
				continue;
			}

			if(UMaterialEditorInstanceConstant* MaterialEditorInstanceConstant = Cast<UMaterialEditorInstanceConstant>(ParameterValue->GetOuter()))
			{
				if (MaterialEditorInstanceConstant->SourceInstance == InputMaterialInstance)
				{
					ReturnParameterValue = ParameterValue;
				}
			}
		}

		return ReturnParameterValue;
	}();

	if (!DEditorParameterValue)
	{
		return;
	}

	const FName EditorValueParameterName = DEditorParameterValue->ParameterInfo.Name;

	if (UDEditorScalarParameterValue* ScalarParameterValue = Cast<UDEditorScalarParameterValue>(DEditorParameterValue))
	{
		const FScalarParameterValue* FoundScalarParameterValue = FMaterialHelper::FindParameterValue(MaterialInstance->ScalarParameterValues, EditorValueParameterName);

		if (FoundScalarParameterValue)
		{
			if (!FMath::IsNearlyEqual(ScalarParameterValue->ParameterValue ,FoundScalarParameterValue->ParameterValue))
			{
				ScalarParameterValue->ParameterValue = FoundScalarParameterValue->ParameterValue;
			}
		}
	}
	else if (UDEditorVectorParameterValue* VectorParameterValue = Cast<UDEditorVectorParameterValue>(DEditorParameterValue))
	{
		const FVectorParameterValue* FoundVectorParameterValue = FMaterialHelper::FindParameterValue(MaterialInstance->VectorParameterValues, EditorValueParameterName);

		if (FoundVectorParameterValue)
		{
			if (VectorParameterValue->ParameterValue != FoundVectorParameterValue->ParameterValue)
			{
				VectorParameterValue->ParameterValue = FoundVectorParameterValue->ParameterValue;
			}
		}
	}
	else if (UDEditorTextureParameterValue* TextureParameterValue = Cast<UDEditorTextureParameterValue>(DEditorParameterValue))
	{
		const FTextureParameterValue* FoundTextureParameterValue = FMaterialHelper::FindParameterValue(MaterialInstance->TextureParameterValues, EditorValueParameterName);

		if (FoundTextureParameterValue)
		{
			if (TextureParameterValue->ParameterValue != FoundTextureParameterValue->ParameterValue)
			{
				TextureParameterValue->ParameterValue = FoundTextureParameterValue->ParameterValue;
			}
		}
	}
	else if (UDEditorFontParameterValue* FontParameterValue = Cast<UDEditorFontParameterValue>(DEditorParameterValue))
	{
		const FFontParameterValue* FoundFontParameterValue = FMaterialHelper::FindParameterValue(MaterialInstance->FontParameterValues, EditorValueParameterName);

		if (FoundFontParameterValue)
		{
			if (FontParameterValue->ParameterValue.FontValue != FoundFontParameterValue->FontValue)
			{
				FontParameterValue->ParameterValue.FontValue = FoundFontParameterValue->FontValue;
				FontParameterValue->ParameterValue.FontPage = FoundFontParameterValue->FontPage;
			}
		}
	}
}
#endif

bool FRemoteControlInstanceMaterial::CheckIsBoundToPropertyPath(const FString& InPath) const
{
	return OriginalFieldPathInfo.ToString() == InPath;
}

bool FRemoteControlInstanceMaterial::ContainsBoundObjects(TArray<UObject*> InObjects) const
{
#if WITH_EDITOR	
	TArray<UObject*> ReplacesObjects;

	for (UObject* Object : InObjects)
	{
		UDEditorParameterValue* EditorParameterValue = Cast<UDEditorParameterValue>(Object);

		if (!EditorParameterValue)
		{
			return false;
		}

		if (OriginalClass != EditorParameterValue->GetClass())
		{
			return false;
		}
	
		UMaterialInstance* ParameterMaterialInstance = nullptr;
		if(UMaterialEditorInstanceConstant* MaterialEditorInstanceConstant = Cast<UMaterialEditorInstanceConstant>(EditorParameterValue->GetOuter()))
		{
			ParameterMaterialInstance = MaterialEditorInstanceConstant->SourceInstance;
		}

		if (!ParameterMaterialInstance)
		{
			return false;
		}
	
		UMaterialInstance* MaterialInstance =  Cast<UMaterialInstance>(InstancePath.ResolveObject());
		if (!MaterialInstance || MaterialInstance != ParameterMaterialInstance)
		{
			return false;
		}

		ReplacesObjects.Add(MaterialInstance);
	}

	return FRemoteControlProperty::ContainsBoundObjects(ReplacesObjects);
#else
	return false;
#endif
}

void FRemoteControlInstanceMaterial::PostLoad()
{
#if WITH_EDITOR
	// Preload material instance asset in editor
	InstancePath.TryLoad();
#endif
}

TSharedRef<IRemoteControlPropertyFactory> FRemoteControlInstanceMaterialFactory::MakeInstance()
{
	return MakeShared<FRemoteControlInstanceMaterialFactory>();
}

TSharedPtr<FRemoteControlProperty> FRemoteControlInstanceMaterialFactory::CreateRemoteControlProperty(URemoteControlPreset* Preset, UObject* Object, FRCFieldPathInfo FieldPath, FRemoteControlPresetExposeArgs Args)
{
#if WITH_EDITOR
	if (UDEditorParameterValue* EditorParameterValue = Cast<UDEditorParameterValue>(Object))
	{
		FRCFieldPathInfo OriginalFieldPath = FieldPath;

		UMaterialEditorInstanceConstant* MaterialEditorInstanceConstant = Cast<UMaterialEditorInstanceConstant>(EditorParameterValue->GetOuter());
		if (!MaterialEditorInstanceConstant)
		{
			return nullptr;
		}
		
		UMaterialInstance* MaterialInstance = MaterialEditorInstanceConstant->SourceInstance;
		if (!MaterialInstance)
		{
			return nullptr;
		}

		if (EditorParameterValue->bOverride == false)
		{
			// Override the property value in material
			EditorParameterValue->bOverride = true;
			EditorParameterValue->PostEditChange();

			FPropertyChangedEvent OverrideEvent(nullptr);
			MaterialEditorInstanceConstant->PostEditChangeProperty( OverrideEvent );
		}

		// If we are dealing with DEditor Parameter Value it should have a Object and Field Path replacement
		// For Material Instance it should be replacement of UDEditorParameterValue with UMaterial Instance
		FRCFieldPathInfo ReplacedFieldPath;
		if (ensure(FMaterialHelper::ReplaceFieldPath(MaterialInstance, EditorParameterValue, FieldPath, ReplacedFieldPath)))
		{
			if (!ReplacedFieldPath.Resolve(MaterialInstance))
			{
				return nullptr;
			}
			
			FName DesiredName = *Args.Label;
			if (OriginalFieldPath.Segments.Num() > 1)
			{
				DesiredName = Preset->GetEntityName(DesiredName, MaterialInstance, ReplacedFieldPath);
			}
			if (DesiredName == NAME_None)
			{
				DesiredName = *FString::Printf(TEXT("%s (%s)"), *EditorParameterValue->ParameterInfo.Name.ToString(), *MaterialInstance->GetName());
			}
				
			FRemoteControlInstanceMaterial RCProperty{
				Preset, Preset->GenerateUniqueLabel(DesiredName), EditorParameterValue, MoveTemp(OriginalFieldPath),
				MaterialInstance, MoveTemp(ReplacedFieldPath), {Preset->FindOrAddBinding(MaterialInstance)}
			};
        		
			return StaticCastSharedPtr<FRemoteControlProperty>(Preset->Expose(MoveTemp(RCProperty), FRemoteControlInstanceMaterial::StaticStruct(), Args.GroupId));
		}
	}
	
#endif
	
	return nullptr;
}

void FRemoteControlInstanceMaterialFactory::PostSetObjectProperties(UObject* InObject, bool bInSuccess) const
{
	if (bInSuccess && !GIsEditor)
	{
		if (UMaterialInstance* MaterialInstance = Cast<UMaterialInstance>(InObject))
		{
			// This modifies material variables used for rendering
			MaterialInstance->InitStaticPermutation();
		}
	}
}

bool FRemoteControlInstanceMaterialFactory::SupportExposedClass(UClass* InClass) const
{
#if WITH_EDITOR
	return InClass->IsChildOf(UDEditorParameterValue::StaticClass());
#else
	return false;
#endif
}