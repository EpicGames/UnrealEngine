// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlInterceptionCommands.h"
#include "IRemoteControlModule.h"


// Internal helper for ERCIAccess->ERCAccess conversion
constexpr ERCAccess ToInternal(ERCIAccess Value)
{
	switch (Value)
	{
	case ERCIAccess::NO_ACCESS:
		return ERCAccess::NO_ACCESS;

	case ERCIAccess::READ_ACCESS:
		return ERCAccess::READ_ACCESS;

	case ERCIAccess::WRITE_ACCESS:
		return ERCAccess::WRITE_ACCESS;

	case ERCIAccess::WRITE_TRANSACTION_ACCESS:
		return ERCAccess::WRITE_TRANSACTION_ACCESS;

	case ERCIAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
		return ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS;
	}

	return ERCAccess::NO_ACCESS;
}

// Internal helper for ERCIPayloadType->ERCPayloadType conversion
constexpr ERCPayloadType ToInternal(ERCIPayloadType Value)
{
	switch (Value)
	{
	case ERCIPayloadType::Cbor:
		return ERCPayloadType::Cbor;

	case ERCIPayloadType::Json:
		return ERCPayloadType::Json;
	}

	return ERCPayloadType::Cbor;
}

// Internal helper for ERCIModifyOperation->ERCModifyOperation conversion
constexpr ERCModifyOperation ToInternal(ERCIModifyOperation Value)
{
	switch (Value)
	{
	case ERCIModifyOperation::EQUAL:
		return ERCModifyOperation::EQUAL;

	case ERCIModifyOperation::ADD:
		return ERCModifyOperation::ADD;

	case ERCIModifyOperation::SUBTRACT:
		return ERCModifyOperation::SUBTRACT;

	case ERCIModifyOperation::MULTIPLY:
		return ERCModifyOperation::MULTIPLY;

	case ERCIModifyOperation::DIVIDE:
		return ERCModifyOperation::DIVIDE;
	}

	return ERCModifyOperation::EQUAL;
}

// Internal helper for ERCAccess->ERCIAccess conversion
constexpr ERCIAccess ToExternal(ERCAccess Value)
{
	switch (Value)
	{
	case ERCAccess::NO_ACCESS:
		return ERCIAccess::NO_ACCESS;

	case ERCAccess::READ_ACCESS:
		return ERCIAccess::READ_ACCESS;

	case ERCAccess::WRITE_ACCESS:
		return ERCIAccess::WRITE_ACCESS;

	case ERCAccess::WRITE_TRANSACTION_ACCESS:
		return ERCIAccess::WRITE_TRANSACTION_ACCESS;

	case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
		return ERCIAccess::WRITE_MANUAL_TRANSACTION_ACCESS;
	}

	return ERCIAccess::NO_ACCESS;
}

// Internal helper for ERCIPayloadType->ERCPayloadType conversion
constexpr ERCIPayloadType ToExternal(ERCPayloadType Value)
{
	switch (Value)
	{
	case ERCPayloadType::Cbor:
		return ERCIPayloadType::Cbor;

	case ERCPayloadType::Json:
		return ERCIPayloadType::Json;
	}

	return ERCIPayloadType::Cbor;
}

// Internal helper for ERCModifyOperation->ERCIModifyOperation conversion
constexpr ERCIModifyOperation ToExternal(ERCModifyOperation Value)
{
	switch (Value)
	{
	case ERCModifyOperation::EQUAL:
		return ERCIModifyOperation::EQUAL;

	case ERCModifyOperation::ADD:
		return ERCIModifyOperation::ADD;

	case ERCModifyOperation::SUBTRACT:
		return ERCIModifyOperation::SUBTRACT;

	case ERCModifyOperation::MULTIPLY:
		return ERCIModifyOperation::MULTIPLY;

	case ERCModifyOperation::DIVIDE:
		return ERCIModifyOperation::DIVIDE;
	}

	return ERCIModifyOperation::EQUAL;
}
