// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlInterceptionProcessor.h"

// Interception related headers
#include "IRemoteControlModule.h"
#include "RemoteControlInterceptionHelpers.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Serialization/MemoryReader.h"
#include "StructDeserializer.h"
#include "UObject/FieldPath.h"

void FRemoteControlInterceptionProcessor::SetObjectProperties(FRCIPropertiesMetadata& PropsMetadata)
{
	// Set object reference
	FRCObjectReference ObjectRef;
	ObjectRef.Property = TFieldPath<FProperty>(*PropsMetadata.PropertyPath).Get();
	ObjectRef.Access = ToInternal(PropsMetadata.Access);
	ObjectRef.PropertyPathInfo = FRCFieldPathInfo(PropsMetadata.PropertyPathInfo);
	UObject* LoadedObject = StaticLoadObject(UObject::StaticClass(), nullptr, *PropsMetadata.ObjectPath, nullptr, LOAD_None, nullptr, true);
	check(ObjectRef.Property.IsValid());
	check(LoadedObject);

	// Resolve object property
	const bool bResult = IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, LoadedObject, ObjectRef.PropertyPathInfo, ObjectRef);
	if (bResult)
	{
		// Create a backend based on SerializationMethod
		FMemoryReader BackendMemoryReader(PropsMetadata.Payload);
		TSharedPtr<IStructDeserializerBackend> StructDeserializerBackend = nullptr;
		if (PropsMetadata.PayloadType == ERCIPayloadType::Cbor)
		{
			StructDeserializerBackend = MakeShared<FCborStructDeserializerBackend>(BackendMemoryReader);
		}
		else if (PropsMetadata.PayloadType == ERCIPayloadType::Json)
		{
			StructDeserializerBackend = MakeShared<FJsonStructDeserializerBackend>(BackendMemoryReader);
		}

		check(StructDeserializerBackend.IsValid());

		// Deserialize without replication
		IRemoteControlModule::Get().SetObjectProperties(ObjectRef, *StructDeserializerBackend, ERCPayloadType::Json, TArray<uint8>(), ToInternal(PropsMetadata.Operation));
	}
}

void FRemoteControlInterceptionProcessor::ResetObjectProperties(FRCIObjectMetadata& InObject)
{
	// Set object reference
	FRCObjectReference ObjectRef;
	ObjectRef.Property = TFieldPath<FProperty>(*InObject.PropertyPath).Get();
	ObjectRef.Access = ToInternal(InObject.Access);
	ObjectRef.PropertyPathInfo = FRCFieldPathInfo(InObject.PropertyPathInfo);
	UObject* LoadedObject = StaticLoadObject(UObject::StaticClass(), nullptr, *InObject.ObjectPath, nullptr, LOAD_None, nullptr, true);
	check(ObjectRef.Property.IsValid());
	check(LoadedObject);

	// Resolve object property
	const bool bResult = IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, LoadedObject, ObjectRef.PropertyPathInfo, ObjectRef);
	if (bResult)
	{
		// Reset object property without replication
		IRemoteControlModule::Get().ResetObjectProperties(ObjectRef, false);
	}
}

void FRemoteControlInterceptionProcessor::InvokeCall(FRCIFunctionMetadata& InFunction)
{
	// This is invoke on a final render node
	UObject* LoadedObject = StaticLoadObject(UObject::StaticClass(), nullptr, *InFunction.ObjectPath, nullptr, LOAD_None, nullptr, true);
	UFunction* LoadedFunction = Cast<UFunction>(StaticLoadObject(UObject::StaticClass(), nullptr, *InFunction.FunctionPath, nullptr, LOAD_None, nullptr, true));
	
	if (!ensure(LoadedObject) || !ensure(LoadedFunction))
	{
		return;
	}

	FStructOnScope FunctionArgs{ LoadedFunction };
	
	// Create the backend based on SerializationMethod
	FMemoryReader BackendMemoryReader(InFunction.Payload);
	TSharedPtr<IStructDeserializerBackend> StructDeserializerBackend = nullptr;
	if (InFunction.PayloadType == ERCIPayloadType::Cbor)
	{
		StructDeserializerBackend = MakeShared<FCborStructDeserializerBackend>(BackendMemoryReader);
	}
	else if (InFunction.PayloadType == ERCIPayloadType::Json)
	{
		StructDeserializerBackend = MakeShared<FJsonStructDeserializerBackend>(BackendMemoryReader);
	}
	else
	{
		// It should never be the case
		checkNoEntry();
	}

	if (FStructDeserializer::Deserialize(FunctionArgs.GetStructMemory(), *const_cast<UStruct*>(FunctionArgs.GetStruct()), *StructDeserializerBackend, FStructDeserializerPolicies()))
	{
		FRCCallReference CallRef;
		CallRef.Object = LoadedObject;
		CallRef.Function = LoadedFunction;
		
		FRCCall Call;
		Call.CallRef = MoveTemp(CallRef);
		Call.TransactionMode = InFunction.bGenerateTransaction ? ERCTransactionMode::AUTOMATIC : ERCTransactionMode::NONE;
		Call.ParamStruct = FStructOnScope(FunctionArgs.GetStruct(), FunctionArgs.GetStructMemory());

		IRemoteControlModule::Get().InvokeCall(Call);
	}
}

void FRemoteControlInterceptionProcessor::SetPresetController(FRCIControllerMetadata& InController)
{
	FMemoryReader Reader(InController.Payload);
	FJsonStructDeserializerBackend Backend(Reader);

	constexpr bool bAllowIntercept = false; // Run without replication

	IRemoteControlModule::Get().SetPresetController(InController.Preset, InController.Controller, Backend, TArray<uint8>(), bAllowIntercept);
}

