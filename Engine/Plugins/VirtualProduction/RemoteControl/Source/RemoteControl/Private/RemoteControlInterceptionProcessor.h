// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlInterceptionFeature.h"


/**
 * Remote control interception processor feature
 */
class FRemoteControlInterceptionProcessor :
	public IRemoteControlInterceptionFeatureProcessor
{
public:
	FRemoteControlInterceptionProcessor() = default;
	virtual ~FRemoteControlInterceptionProcessor() = default;

public:
	// IRemoteControlInterceptionCommands interface
	virtual void SetObjectProperties(FRCIPropertiesMetadata& InProperties) override;
	virtual void ResetObjectProperties(FRCIObjectMetadata& InObject) override;
	virtual void InvokeCall(FRCIFunctionMetadata& InFunction) override;
	virtual void SetPresetController(FRCIControllerMetadata& InController) override;
	// ~IRemoteControlInterceptionCommands interface
};
