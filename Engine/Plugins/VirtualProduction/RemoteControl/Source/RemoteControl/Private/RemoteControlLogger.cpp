// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlLogger.h"

#include "Modules/ModuleManager.h"
#include "Widgets/SNullWidget.h"

#if WITH_EDITOR
#include "IMessageLogListing.h"
#include "MessageLogModule.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlLogger"

FRemoteControlLogger::FRemoteControlLogger()
{
#if WITH_EDITOR
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");

	FMessageLogInitializationOptions LogOptions;

	// Don't show Pages so that user is never allowed to clear log messages
	LogOptions.bShowPages   = false;
	LogOptions.bShowFilters = false;
	LogOptions.bAllowClear  = true;
	LogOptions.MaxPageCount = 1;

	MessageLogListing = MessageLogModule.CreateLogListing("Remote control logging", LogOptions);
#endif
}

void FRemoteControlLogger::Log(const FName& InputType, FLogCallback InLogTextCallback, EVerbosityLevel Verbosity)
{
#if WITH_EDITOR
	if (!bIsEnabled)
	{
		return;
	}
	
	if (!ensure(MessageLogListing.IsValid()))
	{
		return;
	}
		
	TArray<TSharedRef<FTokenizedMessage>> Messages;
	TSharedRef<FTokenizedMessage> Line = FTokenizedMessage::Create(
		Verbosity == EVerbosityLevel::Log ? EMessageSeverity::Info : 
			(Verbosity == EVerbosityLevel::Warning ? EMessageSeverity::Warning : EMessageSeverity::Error));

	const FText Message = InLogTextCallback();
	Line->AddToken(FTextToken::Create(
		FText::Format(LOCTEXT("null", "{0}"), FText::FromString(FDateTime::Now().ToString(TEXT("%H:%M:%S - "))))));
	Line->AddToken(FTextToken::Create(FText::Format(LOCTEXT("null", "{0}"), FText::FromName(InputType))));
	Line->AddToken(FTextToken::Create(Message));
	Messages.Add(Line);

	constexpr bool bMirrorToOutputLog = false;
	MessageLogListing->AddMessages(MoveTemp(Messages), bMirrorToOutputLog);

	// Always select last message, that keep the UI widget scrolling
	constexpr bool bSelected = true;
	MessageLogListing->SelectMessage(Line, bSelected);
#endif
}

void FRemoteControlLogger::EnableLog(const bool bEnable)
{
	bIsEnabled = bEnable;
}

void FRemoteControlLogger::ClearLog() const
{
#if WITH_EDITOR
	MessageLogListing->ClearMessages();
#endif
}

#undef LOCTEXT_NAMESPACE
