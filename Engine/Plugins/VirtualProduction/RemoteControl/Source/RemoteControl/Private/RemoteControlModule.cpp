// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Components/ActorComponent.h"
#include "Components/LightComponent.h"
#include "Components/MeshComponent.h"
#include "Factories/RCDefaultValueFactories.h"
#include "Factories/RemoteControlMaskingFactories.h"
#include "Features/IModularFeatures.h"
#include "IRemoteControlInterceptionFeature.h"
#include "IRemoteControlModule.h"
#include "IStructDeserializerBackend.h"
#include "IStructSerializerBackend.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/ScopeExit.h"
#include "PropertyIdHandler/BasePropertyIdHandler.h"
#include "PropertyIdHandler/EnumPropertyIdHandler.h"
#include "PropertyIdHandler/ObjectPropertyIdHandler.h"
#include "PropertyIdHandler/StructPropertyIdHandler.h"
#include "RCPropertyUtilities.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlFieldPath.h"
#include "RemoteControlInstanceMaterial.h"
#include "RemoteControlInterceptionHelpers.h"
#include "RemoteControlInterceptionProcessor.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolEntityProcessor.h"
#include "RemoteControlSettings.h"
#include "SceneInterface.h"
#include "Serialization/PropertyMapStructDeserializerBackendWrapper.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Tests/RemoteControlInterceptionForwardInterceptor.h"
#include "UObject/Class.h"
#include "UObject/FieldPath.h"
#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Editor/Transactor.h"
#include "Editor/UnrealEdEngine.h"
#include "HAL/IConsoleManager.h"
#include "ScopedTransaction.h"
#include "UnrealEdGlobals.h"
#else
#include "Misc/ScopeExit.h"
#endif

DEFINE_LOG_CATEGORY(LogRemoteControl);
#define LOCTEXT_NAMESPACE "RemoteControl"

IMPLEMENT_MODULE(FRemoteControlModule, RemoteControl);

struct FRCInterceptionPayload
{
	TArray<uint8> Payload;
	ERCPayloadType Type;
};

#if WITH_EDITOR
static TAutoConsoleVariable<int32> CVarRemoteControlEnableOngoingChangeOptimization(TEXT("RemoteControl.EnableOngoingChangeOptimization"), 1, TEXT("Enable an optimization that keeps track of the ongoing remote control change in order to improve performance."));
#endif // WITH_EDITOR

#define REMOTE_CONTROL_LOG_ONCE(Verbosity, Format, ...) (LogOnce(ELogVerbosity::Verbosity, *FString::Printf(Format, ##__VA_ARGS__), FString(__FILE__), __LINE__))

namespace RemoteControlUtil
{
	const FName NAME_DisplayName(TEXT("DisplayName"));
	const FName NAME_ScriptName(TEXT("ScriptName"));
	const FName NAME_ScriptNoExport(TEXT("ScriptNoExport"));
	const FName NAME_DeprecatedFunction(TEXT("DeprecatedFunction"));
	const FName NAME_BlueprintGetter(TEXT("BlueprintGetter"));
	const FName NAME_BlueprintSetter(TEXT("BlueprintSetter"));
	const FName NAME_AllowPrivateAccess(TEXT("AllowPrivateAccess"));
	const FName NAME_RelativeLocation(TEXT("RelativeLocation"));

	bool CompareFunctionName(const FString& FunctionName, const FString& ScriptName)
	{
		int32 SemiColonIndex = INDEX_NONE;
		if (ScriptName.FindChar(TEXT(';'), SemiColonIndex))
		{
			return FCString::Strncmp(*ScriptName, *FunctionName, SemiColonIndex) == 0;
		}
		return ScriptName.Equals(FunctionName);
	}

	UFunction* FindFunctionByNameOrMetaDataName(UObject* Object, const FString& FunctionName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemoteControl::FindFunctionByNameOrMetaDataName);

		UFunction* Function = Object->FindFunction(FName(*FunctionName));
#if WITH_EDITOR
		// if the function wasn't found through the function map, try finding it through its `ScriptName` or `DisplayName` metadata
		if (Function == nullptr)
		{
			for (TFieldIterator<UFunction> FuncIt(Object->GetClass(), EFieldIteratorFlags::IncludeSuper, EFieldIteratorFlags::ExcludeDeprecated, EFieldIteratorFlags::IncludeInterfaces); FuncIt; ++FuncIt)
			{
				if (FuncIt->HasMetaData(NAME_ScriptName) && CompareFunctionName(FunctionName, FuncIt->GetMetaData(NAME_ScriptName)))
				{
					Function = *FuncIt;
					break;
				}
				else if (FuncIt->HasMetaData(NAME_DisplayName) && CompareFunctionName(FunctionName, FuncIt->GetMetaData(NAME_DisplayName)))
				{
					Function = *FuncIt;
					break;
				}
			}
		}
#endif
		return Function;
	}

	/** Returns whether the access is a write access regardless of if it generates a transaction. */
	bool IsWriteAccess(ERCAccess Access)
	{
		return Access == ERCAccess::WRITE_ACCESS || Access == ERCAccess::WRITE_TRANSACTION_ACCESS || Access == ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS;
	}

	/** Returns whether the access is a read access. */
	bool IsReadAccess(ERCAccess Access)
	{
		return Access == ERCAccess::READ_ACCESS;
	}

#if WITH_EDITOR
	/**
	 * For an editor build to restrict access only to what is available in packaged game.
	 */
	static bool GEmulatePackagedAccess = false;

	bool IsEmulatePackaged()
	{
		return GEmulatePackagedAccess;
	}
#endif

	// Todo: Temporary fork of RemoteControlPropertyUtilities::FindSetterFunctionInternal for hotfix. Merge in main.
	static UFunction* FindSetterFunctionInternal_TMP(const FProperty* Property, UClass* OwnerClass)
	{
		using namespace RemoteControlPropertyUtilities;

		// Check if the property setter is already cached.
		const TWeakObjectPtr<UFunction> SetterPtr = CachedSetterFunctions.FindRef(Property);
		if (SetterPtr.IsValid())
		{
			return SetterPtr.Get();
		}

		static const FName RelativeLocationPropertyName = USceneComponent::GetRelativeLocationPropertyName();
		static const FName RelativeRotationPropertyName = USceneComponent::GetRelativeRotationPropertyName();
		static const FName RelativeScalePropertyName = USceneComponent::GetRelativeScale3DPropertyName();

		static TMap<FName, FName> CommonPropertiesToFunctions = { 
			{ RelativeLocationPropertyName, "K2_SetRelativeLocation"},
			{ RelativeRotationPropertyName, "K2_SetRelativeRotation" },
			{ RelativeScalePropertyName, "SetRelativeScale3D" }
		};

		FString SetterName;
		UFunction* SetterFunction = nullptr;
		if(OwnerClass->IsChildOf<ULightComponent>())
		{
			const FName FoundSetterName = FindLightSetterFunctionInternal(Property, OwnerClass);
			if(!FoundSetterName.IsNone())
			{
				SetterName = FoundSetterName.ToString();				
			}
		}
#if WITH_EDITOR
		if (!IsEmulatePackaged()) // -- Added for hotfix
		{
			if(SetterName.IsEmpty())
			{
				SetterName = Property->GetMetaData(*NAME_BlueprintSetter.ToString());
			}
			if (!SetterName.IsEmpty())
			{
				SetterFunction = OwnerClass->FindFunctionByName(*SetterName);
			}
			else
			{
				if (CommonPropertiesToFunctions.Contains(Property->GetFName()))
				{
					SetterFunction = OwnerClass->FindFunctionByName(*CommonPropertiesToFunctions[Property->GetFName()].ToString());
				}
			}
		}
#endif

		if(!SetterFunction)
		{
			FString PropertyName = Property->GetName();
			if (Property->IsA<FBoolProperty>())
			{
				PropertyName.RemoveFromStart("b", ESearchCase::CaseSensitive);
			}

			static const TArray<FString> SetterPrefixes = {
				FString("Set"),
				FString("K2_Set"),
				FString("BP_Set")
			};

			for (const FString& Prefix : SetterPrefixes)
			{
				const FName SetterFunctionName = FName(Prefix + PropertyName);
				SetterFunction = OwnerClass->FindFunctionByName(SetterFunctionName);
				if (SetterFunction)
				{
					break;
				}
			}
		}

		if (SetterFunction && FindSetterArgument(SetterFunction, Property))
		{
			CachedSetterFunctions.Add(Property, SetterFunction);
		}
		else
		{
			// Arguments are not compatible so don't use this setter.
			SetterFunction = nullptr;
		}

		return SetterFunction;
	}

	// Todo: temporary fork of RemoteControlPropertyUtilities::FindSetterFunction for hotfix. Merge in main.
	static UFunction* FindSetterFunction_TMP(const FProperty* Property, UClass* OwnerClass)
	{
		// UStruct properties cannot have setters.
		if (!ensure(Property) || !Property->GetOwnerClass())
		{
			return nullptr;
		}

		UFunction* SetterFunction = nullptr;
		if (OwnerClass && OwnerClass != Property->GetOwnerClass())
		{
			SetterFunction = FindSetterFunctionInternal_TMP(Property, OwnerClass);
		}
		
		if (!SetterFunction)
		{
			SetterFunction = FindSetterFunctionInternal_TMP(Property, Property->GetOwnerClass());
		}
		
		return SetterFunction;
	}

	bool PropertyModificationShouldUseSetter(const UObject* Object, FProperty* Property)
	{
		if (!Property || !Object)
		{
			return false;
		}

		return Property->HasSetter() || !!FindSetterFunction_TMP(Property, Object->GetClass());
	}

	bool IsPropertyAllowed(const FProperty* InProperty, ERCAccess InAccessType, const UObject* InObject, bool bObjectInGamePackage, FString* OutError)
	{
		FTextBuilder Error;
		ON_SCOPE_EXIT
		{
			if (OutError)
			{
				*OutError = Error.ToText().ToString();
			}
		};

		auto PreError = [InAccessType]()-> FText
		{
			if (IsWriteAccess(InAccessType))
			{
				return LOCTEXT("RCWritingAccess", "Property is not writeable because");
			}
			if (IsReadAccess(InAccessType))
			{
				return LOCTEXT("RCReadingAccess", "Property is not readable because");
			}
			return LOCTEXT("RCNoAccess", "Property");
		};

#if WITH_EDITOR
		const URemoteControlSettings* RCSettings = GetMutableDefault<URemoteControlSettings>();
		
		if (!IsEmulatePackaged())
		{
			// Override this flag to false if we are running in Editor Mode with -game flag.
			bObjectInGamePackage = false;
		}
#endif

		// The property is allowed to be accessed if it exists...
		if (!InProperty)
		{
			if (OutError)
			{
				Error.AppendLine(FText::Format(LOCTEXT("RCPropertyNotValid", "{0} it is not valid."), PreError()));
			}
			return false;
		}

#if WITH_EDITOR
		// it doesn't have exposed getter AND setter that should be used instead or if it should ignore them
		const bool bHasBlueprintGetterOrSetterOrIgnored = !InProperty->HasMetaData(RemoteControlUtil::NAME_BlueprintGetter) ||
												 		  !InProperty->HasMetaData(RemoteControlUtil::NAME_BlueprintSetter) ||
												 		  RCSettings->bIgnoreGetterSetterCheck;
		if (!bHasBlueprintGetterOrSetterOrIgnored && OutError)
		{
			Error.AppendLine(FText::Format(LOCTEXT("RCBlueprintHasGetterSetterOrNotIgnored", "{0} it has blueprint getter and setter to use.\nEnable in the RemoteControl settings IgnoreGetterSetterCheck if you want to ignore it."), PreError()));
		}

		// it isn't private or protected, except if AllowPrivateAccess is true and if only for Protected whenever it should be ignored
		const bool bAllowPrivateAccessForPrivate =  !InProperty->HasAnyPropertyFlags(CPF_NativeAccessSpecifierPrivate) ||
													InProperty->GetBoolMetaData(RemoteControlUtil::NAME_AllowPrivateAccess);

		const bool bAllowPrivateAccessForProtected = !InProperty->HasAnyPropertyFlags(CPF_NativeAccessSpecifierProtected) ||
													 RCSettings->bIgnoreProtectedCheck ||
													 InProperty->GetBoolMetaData(RemoteControlUtil::NAME_AllowPrivateAccess);
		
		const bool bIsAccessibleEditor = !IsEmulatePackaged() ?
			bHasBlueprintGetterOrSetterOrIgnored &&
			bAllowPrivateAccessForPrivate &&
			bAllowPrivateAccessForProtected : true;
		
#endif

		const bool bPropertyHasSetterToUse = IsWriteAccess(InAccessType) ? PropertyModificationShouldUseSetter(InObject, const_cast<FProperty*>(InProperty)) : true;
		const bool bPropertyHasGetterToUse = IsReadAccess(InAccessType) ? InProperty->HasGetter() : true;

		bool bAppendSettingInformation = false;
		if (!bPropertyHasSetterToUse && OutError)
		{
#if WITH_EDITOR
			if (!IsEmulatePackaged())
			{
				if (!bAllowPrivateAccessForPrivate)
				{
					Error.AppendLine(FText::Format(LOCTEXT("RCPropertyPrivateWithoutSetter", "{0} it is private without the AllowPrivateAccess specifier set to true or a setter assigned."), PreError()));
				}

				if (!bAllowPrivateAccessForProtected)
				{
					Error.AppendLine(FText::Format(LOCTEXT("RCPropertyProtectedWithoutSetterOrIgnored", "{0} it is protected without the AllowPrivateAccess specifier set to true or a setter assigned."), PreError()));
					bAppendSettingInformation = true;
				}
			}
			else
#endif
			{
				Error.AppendLine(FText::Format(LOCTEXT("RCPropertyNoSetter", "{0} it doesn't have a setter specifier assigned to it."), PreError()));
			}
		}

		if (!bPropertyHasGetterToUse && OutError)
		{
#if WITH_EDITOR
			if (!IsEmulatePackaged())
			{
				if (!bAllowPrivateAccessForPrivate)
				{
					Error.AppendLine(FText::Format(LOCTEXT("RCPropertyPrivateWithoutGetter", "{0} it is private without the AllowPrivateAccess specifier set to true or a getter assigned."), PreError()));
				}

				if (!bAllowPrivateAccessForProtected)
				{
					Error.AppendLine(FText::Format(LOCTEXT("RCPropertyProtectedWithoutGetterOrIgnored", "{0} it is protected without the AllowPrivateAccess specifier set to true or a getter assigned."), PreError()));
					bAppendSettingInformation = true;
				}
			}
			else
#endif
			{
				Error.AppendLine(FText::Format(LOCTEXT("RCPropertyNoGetter", "{0} it doesn't have a getter specifier assigned to it."), PreError()));
			}
		}

#if WITH_EDITOR
		if (bAppendSettingInformation && OutError)
		{
			Error.AppendLine(LOCTEXT("RCSettingsIgnoreProtected","You can enable the RemoteControl setting IgnoreProtectedCheck to ignore it for protected properties."));
		}
#endif
		// it isn't blueprint private
		const bool bIsBlueprintPublic = !InProperty->HasAnyPropertyFlags(CPF_DisableEditOnInstance);

		if (!bIsBlueprintPublic && OutError)
		{
			Error.AppendLine(FText::Format(LOCTEXT("RCBlueprintPropertyPrivate", "{0} it has flag DisableEditOnInstance.\nEnable the InstanceEditable flag of the property in the blueprint editor."), PreError()));
		}

		// and it's either blueprint visible if in game or editable if in editor and it isn't read only if the access type is write
		bool bIsAccessible;

		if (bObjectInGamePackage)
		{
			bIsAccessible = InProperty->HasAnyPropertyFlags(CPF_BlueprintVisible) &&
							 (InAccessType == ERCAccess::READ_ACCESS ||
							 !InProperty->HasAnyPropertyFlags(CPF_BlueprintReadOnly));

			if (!bIsAccessible && OutError)
			{
				Error.AppendLine(FText::Format(LOCTEXT("RCBlueprintNotAccessibleInGamePackage", "{0} it doesn't have the BlueprintVisible flag and/or you are trying to write to it but it has the BlueprintReadOnly flag."), PreError()));
			}
		}
		else
		{
			bIsAccessible = InAccessType == ERCAccess::READ_ACCESS ||
							(InProperty->HasAnyPropertyFlags(CPF_Edit) &&
							!InProperty->HasAnyPropertyFlags(CPF_EditConst));

			if (!bIsAccessible && OutError)
			{
				Error.AppendLine(FText::Format(LOCTEXT("RCBlueprintNotAccessibleInNonGamePackage", "{0} it has the EditConst flag."), PreError()));
			}
		}
		
		return (bIsAccessible &&
#if WITH_EDITOR
			bIsAccessibleEditor &&
#endif
			bIsBlueprintPublic) ||
			(bPropertyHasGetterToUse && bPropertyHasSetterToUse);
	}

	FARFilter GetBasePresetFilter()
	{
		FARFilter Filter;
		Filter.bIncludeOnlyOnDiskAssets = false;
		Filter.ClassPaths = {URemoteControlPreset::StaticClass()->GetClassPathName()};
		Filter.bRecursivePaths = true;

		return Filter;
	}

	void GetAllPresetAssets(TArray<FAssetData>& OutAssets)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.GetAssets(GetBasePresetFilter(), OutAssets);
	}

	URemoteControlPreset* GetFirstPreset(const FARFilter& Filter)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::GetFirstPreset);
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		TArray<FAssetData> Assets;
		AssetRegistry.GetAssets(Filter, Assets);

		return Assets.Num() ? CastChecked<URemoteControlPreset>(Assets[0].GetAsset()) : nullptr;
	}

	URemoteControlPreset* GetPresetById(const FGuid& Id)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::GetPresetId);
		FARFilter Filter = GetBasePresetFilter();
		Filter.TagsAndValues.Add(FName("PresetId"), Id.ToString());
		return GetFirstPreset(Filter);
	}

	URemoteControlPreset* FindPresetByName(FName PresetName)
	{
		IAssetRegistry& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

		TArray<FAssetData> Assets;
		AssetRegistry.GetAssetsByClass(URemoteControlPreset::StaticClass()->GetClassPathName(), Assets);

		FAssetData* FoundAsset = Assets.FindByPredicate([&PresetName](const FAssetData& InAsset)
		{
			return InAsset.AssetName == PresetName;
		});

		return FoundAsset ? Cast<URemoteControlPreset>(FoundAsset->GetAsset()) : nullptr;
	}

	URemoteControlPreset* GetPresetByName(FName PresetName)
	{
		FARFilter Filter = GetBasePresetFilter();
		Filter.PackageNames = {PresetName};
		URemoteControlPreset* FoundPreset = GetFirstPreset(Filter);
		return FoundPreset ? FoundPreset : FindPresetByName(PresetName);
	}

	FGuid GetPresetId(const FAssetData& PresetAsset)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::GetPresetId);
		FGuid Id;
		FAssetDataTagMapSharedView::FFindTagResult Result = PresetAsset.TagsAndValues.FindTag(FName("PresetId"));
		if (Result.IsSet())
		{
			Id = FGuid{Result.GetValue()};
		}

		return Id;
	}

	/** Helper function to check if a value is 0 without always calling FMath::IsNearlyZero, since that results in an ambiguous call for non-float values. */
	template <typename T>
	bool IsNearlyZero(T Value)
	{
		return Value == 0;
	}

	template <>
	bool IsNearlyZero(double Value)
	{
		return FMath::IsNearlyZero(Value);
	}

	/**
	 * Apply a property modification based on the given type's overloaded operators.
	 * 
	 * @param Operation The operation to perform.
	 * @param BasePropertyData A pointer to the original value before modification.
	 * @param DeltaPropertyData A pointer to the value of the operand and which will store the result.
	 * @param Getter Function which, given a pointer to property data, will return the property's value.
	 * @param Setter Function which, given a pointer to property data and a value, will set the property's value to the new value.
	 */
	template <typename T>
	bool ApplySimpleDeltaOperation(ERCModifyOperation Operation, const void* BasePropertyData, void* DeltaPropertyData, FProperty* Property, TFunction<T(const void*)> Getter, TFunction<void(void*, T)> Setter)
	{
		const T BaseValue = Getter(BasePropertyData);
		const T DeltaValue = Getter(DeltaPropertyData);

		T NewValue;

		switch (Operation)
		{
		case ERCModifyOperation::ADD:
			NewValue = BaseValue + DeltaValue;
			break;

		case ERCModifyOperation::SUBTRACT:
			NewValue = BaseValue - DeltaValue;
			break;

		case ERCModifyOperation::MULTIPLY:
			NewValue = BaseValue * DeltaValue;
			break;

		case ERCModifyOperation::DIVIDE:
			if (IsNearlyZero<T>(DeltaValue))
			{
				return false;
			}
			NewValue = BaseValue / DeltaValue;
			break;

		default:
			// Unsupported operation
			return false;
		}

#if WITH_EDITORONLY_DATA
		// Respect property's clamp settings if they exist
		const FString& ClampMinString = Property->GetMetaData(TEXT("ClampMin"));
		if (!ClampMinString.IsEmpty())
		{
			T ClampMin;
			TTypeFromString<T>::FromString(ClampMin, *ClampMinString);
			NewValue = FMath::Max(NewValue, ClampMin);
		}

		const FString& ClampMaxString = Property->GetMetaData(TEXT("ClampMax"));
		if (!ClampMaxString.IsEmpty())
		{
			T ClampMax;
			TTypeFromString<T>::FromString(ClampMax, *ClampMaxString);
			NewValue = FMath::Min(NewValue, ClampMax);
		}
#endif

		Setter(DeltaPropertyData,  NewValue);
		return true;
	}
}

namespace RemoteControlSetterUtils
{
	struct FConvertToFunctionCallArgs
	{
		FConvertToFunctionCallArgs(const FRCObjectReference& InObjectReference, IStructDeserializerBackend& InReaderBackend, FRCCall& OutCall, const void* InValuePtrOverride = nullptr)
			: ObjectReference(InObjectReference)
			, ReaderBackend(InReaderBackend)
			, Call(OutCall)
			, ValuePtrOverride(InValuePtrOverride)
		{ }

		const FRCObjectReference& ObjectReference;
		IStructDeserializerBackend& ReaderBackend;
		FRCCall& Call;
		const void* ValuePtrOverride;
	};

	void CreateRCCall(FConvertToFunctionCallArgs& InOutArgs, UFunction* InFunction, FStructOnScope&& InFunctionArguments, FRCInterceptionPayload& OutPayload)
	{
		ensure(InFunctionArguments.GetStruct() && InFunctionArguments.GetStruct()->IsA<UFunction>());

		// Create the output payload for interception purposes.
		FMemoryWriter Writer{OutPayload.Payload};
		FCborStructSerializerBackend WriterBackend{Writer, EStructSerializerBackendFlags::Default};
		FStructSerializer::Serialize(InFunctionArguments.GetStructMemory(), *const_cast<UStruct*>(InFunctionArguments.GetStruct()), WriterBackend, FStructSerializerPolicies());
		OutPayload.Type = ERCPayloadType::Cbor;

		switch (InOutArgs.ObjectReference.Access)
		{
		case ERCAccess::WRITE_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
			break;

		case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::MANUAL;
			break;

		default:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::NONE;
			break;
		}

		InOutArgs.Call.CallRef.Function = InFunction;
		InOutArgs.Call.CallRef.Object = InOutArgs.ObjectReference.Object;
		InOutArgs.Call.ParamStruct = MoveTemp(InFunctionArguments);
	}

	void CreateRCCall(FConvertToFunctionCallArgs& InOutArgs, UFunction* InFunction, FStructOnScope&& InFunctionArguments)
	{
		ensure(InFunctionArguments.GetStruct() && InFunctionArguments.GetStruct()->IsA<UFunction>());

		switch (InOutArgs.ObjectReference.Access)
		{
		case ERCAccess::WRITE_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
			break;

		case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::MANUAL;
			break;

		default:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::NONE;
			break;
		}

		InOutArgs.Call.CallRef.Function = InFunction;
		InOutArgs.Call.CallRef.Object = InOutArgs.ObjectReference.Object;
		InOutArgs.Call.ParamStruct = MoveTemp(InFunctionArguments);
	}

	void CreateRCCall(FConvertToFunctionCallArgs& InOutArgs, FProperty* InPropertyWithSetter, FStructOnScope&& InFunctionArguments, FRCInterceptionPayload& OutPayload)
	{
		ensure(InFunctionArguments.GetStruct());
		
		// Create the output payload for interception purposes.
		FMemoryWriter Writer{OutPayload.Payload};
		FCborStructSerializerBackend WriterBackend{Writer, EStructSerializerBackendFlags::Default};
		FStructSerializer::Serialize(InFunctionArguments.GetStructMemory(), *const_cast<UStruct*>(InFunctionArguments.GetStruct()), WriterBackend, FStructSerializerPolicies());
		OutPayload.Type = ERCPayloadType::Cbor;


		switch (InOutArgs.ObjectReference.Access)
		{
		case ERCAccess::WRITE_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
			break;

		case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::MANUAL;
			break;

		default:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::NONE;
			break;
		}

		InOutArgs.Call.CallRef.PropertyWithSetter = InPropertyWithSetter;
		InOutArgs.Call.CallRef.Object = InOutArgs.ObjectReference.Object;
		InOutArgs.Call.ParamStruct = MoveTemp(InFunctionArguments);
		
		InOutArgs.Call.ParamData.SetNumUninitialized(InPropertyWithSetter->GetSize());
		InPropertyWithSetter->InitializeValue(InOutArgs.Call.ParamData.GetData());
		const void* ValuePtr = InPropertyWithSetter->ContainerPtrToValuePtr<uint8>(InOutArgs.Call.ParamStruct.GetStructMemory());
		InPropertyWithSetter->CopyCompleteValue(InOutArgs.Call.ParamData.GetData(), ValuePtr);
	}

	void CreateRCCall(FConvertToFunctionCallArgs& InOutArgs, FProperty* InPropertyWithSetter, FStructOnScope&& InFunctionArguments)
	{
		ensure(InFunctionArguments.GetStruct());

		switch (InOutArgs.ObjectReference.Access)
		{
		case ERCAccess::WRITE_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
			break;

		case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::MANUAL;
			break;

		default:
			InOutArgs.Call.TransactionMode = ERCTransactionMode::NONE;
			break;
		}

		InOutArgs.Call.CallRef.PropertyWithSetter = InPropertyWithSetter;
		InOutArgs.Call.CallRef.Object = InOutArgs.ObjectReference.Object;
		InOutArgs.Call.ParamStruct = MoveTemp(InFunctionArguments);

		InOutArgs.Call.ParamData.SetNumUninitialized(InPropertyWithSetter->GetSize());
		InPropertyWithSetter->InitializeValue(InOutArgs.Call.ParamData.GetData());
		const void* ValuePtr = InPropertyWithSetter->ContainerPtrToValuePtr<uint8>(InOutArgs.Call.ParamStruct.GetStructMemory());
		InPropertyWithSetter->CopyCompleteValue(InOutArgs.Call.ParamData.GetData(), ValuePtr);
	}

	/** Create the payload to pass to be passed to a property's setter function. */
	TOptional<FStructOnScope> CreateSetterFunctionPayload(UFunction* InSetterFunction, FConvertToFunctionCallArgs& InOutArgs)
	{
		TOptional<FStructOnScope> OptionalArgsOnScope;

		bool bSuccess = false;
		if (FProperty* SetterArgument = RemoteControlPropertyUtilities::FindSetterArgument(InSetterFunction, InOutArgs.ObjectReference.Property.Get()))
		{
			FStructOnScope ArgsOnScope{ InSetterFunction };

			// Temporarily rename the setter argument in order to copy/deserialize the incoming property modification on top of it
			// regardless of the argument name.
			FName OldSetterArgumentName = SetterArgument->GetFName();
			SetterArgument->Rename(InOutArgs.ObjectReference.Property->GetFName());
			{
				ON_SCOPE_EXIT
				{
					SetterArgument->Rename(OldSetterArgumentName);
				};

				if (InOutArgs.ValuePtrOverride)
				{
					// We already have a pointer directly to the data we want, so copy the data into the function arguments
					const UStruct* ArgsStruct = ArgsOnScope.GetStruct();

					check(ArgsStruct);

					uint8* SetterArgData = SetterArgument->ContainerPtrToValuePtr<uint8>(ArgsOnScope.GetStructMemory());
					InOutArgs.ObjectReference.Property->CopyCompleteValue(SetterArgData, InOutArgs.ValuePtrOverride);

					bSuccess = true;
				}
				else
				{
					// Data needs to be deserialized from the passed reader backend

					// First put the complete property value from the object in the struct on scope
					// in case the user only a part of the incoming structure (ie. Providing only { "x": 2 } in the case of a vector.
					const uint8* ContainerAddress = InOutArgs.ObjectReference.Property->ContainerPtrToValuePtr<uint8>(InOutArgs.ObjectReference.ContainerAdress);
					InOutArgs.ObjectReference.Property->CopyCompleteValue(ArgsOnScope.GetStructMemory(), ContainerAddress);

					// Deserialize on top of the setter argument
					bSuccess = FStructDeserializer::Deserialize((void*)ArgsOnScope.GetStructMemory(), *const_cast<UStruct*>(ArgsOnScope.GetStruct()), InOutArgs.ReaderBackend, FStructDeserializerPolicies());
				}

				if (bSuccess)
				{
					OptionalArgsOnScope = MoveTemp(ArgsOnScope);
				}
			}
		}

		return OptionalArgsOnScope;
	}

	/** Create the payload to pass to be passed to a property's native setter. */
	TOptional<FStructOnScope> CreateSetterFunctionPayload(FProperty* InPropertyWithSetter, FConvertToFunctionCallArgs& InOutArgs)
	{
		TOptional<FStructOnScope> OptionalArgsOnScope;

		FStructOnScope ArgsOnScope{ InOutArgs.ObjectReference.ContainerType.Get() };

		bool bSuccess = false;
		if (InOutArgs.ValuePtrOverride)
		{
			// We already have a pointer directly to the data we want, so copy the data into the function arguments
			const UStruct* ArgsStruct = ArgsOnScope.GetStruct();

			check(ArgsStruct);

			uint8* SetterArgData = InOutArgs.ObjectReference.Property->ContainerPtrToValuePtr<uint8>(ArgsOnScope.GetStructMemory());
			InOutArgs.ObjectReference.Property->CopyCompleteValue(SetterArgData, InOutArgs.ValuePtrOverride);

			bSuccess = true;
		}
		else
		{
			// Data needs to be deserialized from the passed reader backend

			// First put the complete property value from the object in the struct on scope
			// in case the user only a part of the incoming structure (ie. Providing only { "x": 2 } in the case of a vector.
			const uint8* ContainerAddress = InOutArgs.ObjectReference.Property->ContainerPtrToValuePtr<uint8>(InOutArgs.ObjectReference.ContainerAdress);
			InOutArgs.ObjectReference.Property->CopyCompleteValue(ArgsOnScope.GetStructMemory(), ContainerAddress);

			// Deserialize on top of the setter argument
			bSuccess = FStructDeserializer::Deserialize((void*)ArgsOnScope.GetStructMemory(), *const_cast<UStruct*>(ArgsOnScope.GetStruct()), InOutArgs.ReaderBackend, FStructDeserializerPolicies());
		}

		if (bSuccess)
		{
			OptionalArgsOnScope = MoveTemp(ArgsOnScope);
		}

		return OptionalArgsOnScope;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InOutArgs, UFunction* InSetterFunction, FRCInterceptionPayload& OutPayload)
	{
		if (TOptional<FStructOnScope> ArgsOnScope = CreateSetterFunctionPayload(InSetterFunction, InOutArgs))
		{
			CreateRCCall(InOutArgs, InSetterFunction, MoveTemp(*ArgsOnScope), OutPayload);
			return true;
		}

		return false;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InOutArgs, UFunction* InSetterFunction)
	{
		if (TOptional<FStructOnScope> ArgsOnScope = CreateSetterFunctionPayload(InSetterFunction, InOutArgs))
		{
			CreateRCCall(InOutArgs, InSetterFunction, MoveTemp(*ArgsOnScope));
			return true;
		}

		return false;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InOutArgs, FProperty* InPropertyWithSetter, FRCInterceptionPayload& OutPayload)
	{
		if (TOptional<FStructOnScope> ArgsOnScope = CreateSetterFunctionPayload(InPropertyWithSetter, InOutArgs))
		{
			CreateRCCall(InOutArgs, InPropertyWithSetter, MoveTemp(*ArgsOnScope), OutPayload);
			return true;
		}

		return false;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InOutArgs, FProperty* InPropertyWithSetter)
	{
		if (TOptional<FStructOnScope> ArgsOnScope = CreateSetterFunctionPayload(InPropertyWithSetter, InOutArgs))
		{
			CreateRCCall(InOutArgs, InPropertyWithSetter, MoveTemp(*ArgsOnScope));
			return true;
		}

		return false;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InOutArgs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemoteControlSetterUtils::ConvertModificationToFunctionCall);
		if (!InOutArgs.ObjectReference.Object.IsValid() || !InOutArgs.ObjectReference.Property.IsValid())
		{
			return false;
		}

		if (InOutArgs.ObjectReference.Property->HasSetter())
		{
			return ConvertModificationToFunctionCall(InOutArgs, InOutArgs.ObjectReference.Property.Get());
		}
		else if (UFunction* SetterFunction = RemoteControlPropertyUtilities::FindSetterFunction(InOutArgs.ObjectReference.Property.Get(), InOutArgs.ObjectReference.Object->GetClass()))
		{
			return ConvertModificationToFunctionCall(InOutArgs, SetterFunction);
		}

		return false;
	}

	bool ConvertModificationToFunctionCall(FConvertToFunctionCallArgs& InArgs, FRCInterceptionPayload& OutPayload)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(RemoteControlSetterUtils::ConvertModificationToFunctionCall);
		if (!InArgs.ObjectReference.Object.IsValid() || !InArgs.ObjectReference.Property.IsValid())
		{
			return false;
		}

		if (InArgs.ObjectReference.Property->HasSetter())
		{
			return ConvertModificationToFunctionCall(InArgs, InArgs.ObjectReference.Property.Get(), OutPayload);
		}
		if (UFunction* SetterFunction = RemoteControlPropertyUtilities::FindSetterFunction(InArgs.ObjectReference.Property.Get(), InArgs.ObjectReference.Object->GetClass()))
		{
			return ConvertModificationToFunctionCall(InArgs, SetterFunction, OutPayload);
		}

		return false;
	}
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FRemoteControlModule::StartupModule()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();

	AssetRegistry.OnAssetAdded().AddRaw(this, &FRemoteControlModule::OnAssetAdded);
	AssetRegistry.OnAssetRemoved().AddRaw(this, &FRemoteControlModule::OnAssetRemoved);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FRemoteControlModule::OnAssetRenamed);

	// Instantiate the RCI processor feature on module start
	RCIProcessor = MakeUnique<FRemoteControlInterceptionProcessor>();
	// Register the interceptor feature
	IModularFeatures::Get().RegisterModularFeature(IRemoteControlInterceptionFeatureProcessor::GetName(), RCIProcessor.Get());

	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddRaw(this, &FRemoteControlModule::CachePresets);
	}

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FRemoteControlModule::HandleEnginePostInit);
	FCoreUObjectDelegates::PreLoadMap.AddRaw(this, &FRemoteControlModule::HandleMapPreLoad);
#endif // WITH_EDITOR
	
	// Register Default Value Factories
	RegisterDefaultValueFactories();

	// Register Property Factories
	RegisterEntityFactory(FRemoteControlInstanceMaterial::StaticStruct()->GetFName(), FRemoteControlInstanceMaterialFactory::MakeInstance());

	// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
	RegisterMaskingFactories();

	// Register PropertyIdHandler
	RegisterPropertyIdHandler();

	PopulateDisallowedFunctions();

	ConsoleObjects.Add(IConsoleManager::Get().RegisterConsoleCommand(
		TEXT("RemoteControl.EnableLocalForwardingInterceptor"),
		TEXT("Enables/Disables the local forwarding interceptor debugging mode.\n")
		TEXT("This allows to emulate remote control interception and processing locally.\n")
		TEXT("Usage: \"RemoteControl.EnableLocalForwardingInterceptor [true/false] or [0/1]\""),
		FConsoleCommandWithArgsDelegate::CreateLambda([this](const TArray<FString>& InArgs)
		{
			const bool bEnable = InArgs.Num() > 0 ? InArgs[0].ToBool() : true;
			if (bEnable)
			{
				RegisterLocalForwardingInterceptor();
				UE_LOGF(LogRemoteControl, Log, "Local Forwarding Interception is enabled.");
			}
			else
			{
				UnregisterLocalForwardingInterceptor();
				UE_LOGF(LogRemoteControl, Log, "Local Forwarding Interception is disabled.");
			}
		}),
		ECVF_Default
		));
}

void FRemoteControlModule::ShutdownModule()
{
	UnregisterConsoleObjects();

#if WITH_EDITOR
	FCoreUObjectDelegates::PreLoadMap.RemoveAll(this);
	UnregisterEditorDelegates();
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif
	if (FModuleManager::Get().IsModuleLoaded(AssetRegistryConstants::ModuleName))
	{
		IAssetRegistry& AssetRegistry = FModuleManager::GetModuleChecked<FAssetRegistryModule>(AssetRegistryConstants::ModuleName).Get();
		AssetRegistry.OnFilesLoaded().RemoveAll(this);
		AssetRegistry.OnAssetRenamed().RemoveAll(this);
		AssetRegistry.OnAssetAdded().RemoveAll(this);
		AssetRegistry.OnAssetRemoved().RemoveAll(this);
	}

	// Unregister the interceptor feature on module shutdown
	IModularFeatures::Get().UnregisterModularFeature(IRemoteControlInterceptionFeatureProcessor::GetName(), RCIProcessor.Get());
	
	UnregisterLocalForwardingInterceptor();

	if (UObjectInitialized())
	{
		// Unregister Property Factories
		UnregisterEntityFactory(FRemoteControlInstanceMaterial::StaticStruct()->GetFName());
		
		// Unregister Default Value
		DefaultValueFactories.Empty();

		// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
		MaskingFactories.Empty();
	}
}

IRemoteControlModule::FOnPresetRegistered& FRemoteControlModule::OnPresetRegistered()
{
	return OnPresetRegisteredDelegate;
}

IRemoteControlModule::FOnPresetUnregistered& FRemoteControlModule::OnPresetUnregistered()
{
	return OnPresetUnregisteredDelegate;
}

IRemoteControlModule::FOnError& FRemoteControlModule::OnError()
{
	return OnErrorDelegate;
}

/** Register the preset with the module, enabling using the preset remotely using its name. */
bool FRemoteControlModule::RegisterPreset(FName Name, URemoteControlPreset* Preset)
{
	return false;
}

/** Unregister the preset */
void FRemoteControlModule::UnregisterPreset(FName Name)
{
}

bool FRemoteControlModule::RegisterEmbeddedPreset(URemoteControlPreset* Preset, bool bReplaceExisting)
{
	if (!Preset)
	{
		return false;
	}

	FName PresetName = Preset->GetPresetName();

	if (PresetName == NAME_None)
	{
		return false;
	}

	if (const TWeakObjectPtr<URemoteControlPreset>* FoundPreset = EmbeddedPresets.Find(PresetName))
	{
		if (FoundPreset->IsValid() && !bReplaceExisting)
		{
			return false;
		}
	}

	EmbeddedPresets.Emplace(PresetName, TWeakObjectPtr<URemoteControlPreset>(Preset));

	FGuid PresetId = Preset->GetPresetId();

	if (PresetId.IsValid())
	{
		CachedPresetNamesById.Emplace(PresetId, PresetName);
	}

	return true;
}

void FRemoteControlModule::UnregisterEmbeddedPreset(FName Name)
{
	if (Name == NAME_None)
	{
		return;
	}

	// Check the cached preset ids and remove it if our name matches the stored id
	TWeakObjectPtr<URemoteControlPreset>* FoundPreset = EmbeddedPresets.Find(Name);

	// Remark: the weak ptr will be invalid during GC and the object will not be unregistered.
	if (FoundPreset && FoundPreset->IsValid())
	{
		UnregisterEmbeddedPreset(FoundPreset->Get());
	}
}

void FRemoteControlModule::UnregisterEmbeddedPreset(URemoteControlPreset* Preset)
{
	if (!Preset)
	{
		return;
	}

	// Remark: the PresetName is unreliable. When unregistering during the garbage collection,
	// the name can be "None". In that case we can recover the name from CachedPresetNamesById
	// by using the PresetId.
	// The other case is when the package has been renamed (save level as), in that case
	// the PresetName is the renamed one, but both CachedPresetNamesById and EmbeddedPresets
	// still have the old name.
	const FName PresetName = Preset->GetPresetName();
	bool bPresetRemoved = false;

	if (PresetName != NAME_None)
	{
		// Attempt to remove with current name (may not work if the package has been renamed).
		bPresetRemoved = (EmbeddedPresets.Remove(PresetName) > 0) ? true : false;
	}
	
	const FGuid PresetId = Preset->GetPresetId();
	
	if (PresetId.IsValid())
	{
		if (!bPresetRemoved)
		{
			if (const FName* FoundPresetName = CachedPresetNamesById.Find(PresetId))
			{
				// Attempt to remove from recovered the name from the id.
				EmbeddedPresets.Remove(*FoundPresetName);
			}
		}
		CachedPresetNamesById.Remove(PresetId);
	}
}

bool FRemoteControlModule::CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const
{
	if (!InObject || !InArgs.Property)
	{
		return false;
	}

	// NOTE : Get the owning class to which the property actually belongs.
	if (UClass* OwningClass = InArgs.Property->GetOwnerClass())
	{
		const FString DefaultValueKey = OwningClass->GetName() + TEXT(".") + InArgs.Property->GetName();
		
		if (const TSharedPtr<IRCDefaultValueFactory>* DefaultValueFactory = DefaultValueFactories.Find(*DefaultValueKey))
		{
			return (*DefaultValueFactory)->CanResetToDefaultValue(InObject, InArgs);
		}
	}

	return false;
}

bool FRemoteControlModule::HasDefaultValueCustomization(const UObject* InObject, const FProperty* InProperty) const
{
	if (!InObject || !InProperty)
	{
		return false;
	}

	// NOTE : Get the owning class to which the property actually belongs.
	if (UClass* OwningClass = InProperty->GetOwnerClass())
	{
		const FString DefaultValueKey = OwningClass->GetName() + TEXT(".") + InProperty->GetName();

		if (const TSharedPtr<IRCDefaultValueFactory>* DefaultValueFactory = DefaultValueFactories.Find(*DefaultValueKey))
		{
			return (*DefaultValueFactory)->SupportsClass(InObject->GetClass()) && (*DefaultValueFactory)->SupportsProperty(InProperty);
		}
	}
	
	return false;
}

void FRemoteControlModule::ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs)
{
	if (!InObject || !InArgs.Property)
	{
		return;
	}

	// NOTE : Get the owning class to which the property actually belongs.
	if (UClass* OwningClass = InArgs.Property->GetOwnerClass())
	{
		const FString DefaultValueKey = OwningClass->GetName() + TEXT(".") + InArgs.Property->GetName();

		if (const TSharedPtr<IRCDefaultValueFactory>* DefaultValueFactory = DefaultValueFactories.Find(*DefaultValueKey))
		{
#if WITH_EDITOR
			
			if (InArgs.bCreateTransaction && GEditor)
			{
				GEditor->BeginTransaction(LOCTEXT("RemoteControlModule","Remote Control Custom Reset To Default"));
			}

#endif // WITH_EDITOR

			(*DefaultValueFactory)->ResetToDefaultValue(InObject, InArgs);

#if WITH_EDITOR

			if (InArgs.bCreateTransaction && GEditor)
			{
				GEditor->EndTransaction();
			}

#endif // WITH_EDITOR
		}
	}
}

void FRemoteControlModule::PerformMasking(const TSharedRef<FRCMaskingOperation>& InMaskingOperation)
{
	// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::PerformMasking);

	if (!InMaskingOperation->IsValid())
	{
		return;
	}

	// Since we do not have any masks we do not need to process anything.
	if (InMaskingOperation->Masks == ERCMask::NoMask || !SupportsMasking(InMaskingOperation->ObjectRef.Property.Get()))
	{
		return;
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InMaskingOperation->ObjectRef.Property.Get()))
	{
		if (const TSharedPtr<IRemoteControlMaskingFactory>* MaskingFactory = MaskingFactories.Find(StructProperty->Struct))
		{
			if (ActiveMaskingOperations.Contains(InMaskingOperation))
			{
				constexpr bool bIsInteractive = true;

				(*MaskingFactory)->ApplyMaskedValues(InMaskingOperation, bIsInteractive);

				ActiveMaskingOperations.Remove(InMaskingOperation);
			}
			else
			{
				ActiveMaskingOperations.Add(InMaskingOperation);

				(*MaskingFactory)->CacheRawValues(InMaskingOperation);
			}
		}
	}
}

void FRemoteControlModule::RegisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType, const TSharedPtr<IRemoteControlMaskingFactory>& InMaskingFactory)
{
	// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
	if (!MaskingFactories.Contains(RemoteControlPropertyType))
	{
		MaskingFactories.Add(RemoteControlPropertyType, InMaskingFactory);
	}
}

void FRemoteControlModule::UnregisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType)
{	
	// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
	MaskingFactories.Remove(RemoteControlPropertyType);
}

bool FRemoteControlModule::SupportsMasking(const UScriptStruct* InStruct) const
{
	using namespace UE::RemoteControl;
	return ProtocolEntityProcessor::DoesScriptStructSupportMasking(InStruct);
}

bool FRemoteControlModule::SupportsMasking(const FProperty* InProperty) const
{
	using namespace UE::RemoteControl;
	return ProtocolEntityProcessor::DoesPropertySupportMasking(InProperty);
}

bool FRemoteControlModule::ResolveCall(const FString& ObjectPath, const FString& FunctionName, FRCCallReference& OutCallRef, FString* OutErrorText)
{
	bool bSuccess = true;
	FString ErrorText;
	if (!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		// Resolve the object
		UObject* Object = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectPath);
		if (Object)
		{
			if (CanBeAccessedRemotely(Object))
			{
				// Find the function to call
				UFunction* Function = RemoteControlUtil::FindFunctionByNameOrMetaDataName(Object, FunctionName);

				if (!Function)
				{
					ErrorText = FString::Printf(TEXT("Function: %s does not exist on object: %s"), *FunctionName, *ObjectPath);
					bSuccess = false;
				}
				else if (!IsFunctionAllowed(Function)
						|| (!Function->HasAllFunctionFlags(FUNC_BlueprintCallable | FUNC_Public) && !Function->HasAllFunctionFlags(FUNC_BlueprintEvent))
	#if WITH_EDITOR
						|| Function->HasMetaData(RemoteControlUtil::NAME_DeprecatedFunction)
						|| Function->HasMetaData(RemoteControlUtil::NAME_ScriptNoExport)
	#endif
				)
				{
					ErrorText = FString::Printf(TEXT("Function: %s is deprecated or unavailable remotely on object: %s"), *FunctionName, *ObjectPath);
					bSuccess = false;
				}
				else
				{
					OutCallRef.Object = Object;
					OutCallRef.Function = Function;
				}
			}
			else
			{
				ErrorText = FString::Printf(TEXT("Object %s cannot be accessed remotely, check remote control project settings."), *Object->GetName());
				bSuccess = false;
			}
		}
		else
		{
			ErrorText = FString::Printf(TEXT("Object: %s does not exist."), *ObjectPath);
			bSuccess = false;
		}
	}
	else
	{
		ErrorText = FString::Printf(TEXT("Can't resolve object: %s while saving or garbage collecting."), *ObjectPath);
		bSuccess = false;
	}

	if (!bSuccess && !ErrorText.IsEmpty())
	{
		IRemoteControlModule::BroadcastError(ErrorText);
	}

	if (OutErrorText && !ErrorText.IsEmpty())
	{
		*OutErrorText = MoveTemp(ErrorText);
	}
	
	return bSuccess;
}

bool FRemoteControlModule::InvokeCall(FRCCall& InCall, ERCPayloadType InPayloadType, const TArray<uint8>& InInterceptPayload)
{
	UE_LOGF(LogRemoteControl, VeryVerbose, "Invoke function");

	if (InCall.IsValid())
	{
		const bool bGenerateTransaction = InCall.TransactionMode == ERCTransactionMode::AUTOMATIC;

		// Check the replication path before apply property values
		if (InInterceptPayload.Num() != 0 && CanInterceptFunction(InCall))
		{
			FRCIFunctionMetadata FunctionMetadata(InCall.CallRef.Object->GetPathName(), InCall.CallRef.Function->GetPathName(), bGenerateTransaction, ToExternal(InPayloadType), InInterceptPayload);

			// Initialization
			IModularFeatures& ModularFeatures = IModularFeatures::Get();
			const FName InterceptorFeatureName = IRemoteControlInterceptionFeatureInterceptor::GetName();
			const int32 InterceptorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureInterceptor::GetName());

			// Pass interception command data to all available interceptors
			bool bShouldIntercept = false;
			for (int32 InterceptorIdx = 0; InterceptorIdx < InterceptorsAmount; ++InterceptorIdx)
			{
				IRemoteControlInterceptionFeatureInterceptor* const Interceptor = static_cast<IRemoteControlInterceptionFeatureInterceptor*>(ModularFeatures.GetModularFeatureImplementation(InterceptorFeatureName, InterceptorIdx));
				if (Interceptor)
				{
					// Update response flag
					UE_LOGF(LogRemoteControl, VeryVerbose, "Invoke function - Intercepted");
					bShouldIntercept |= (Interceptor->InvokeCall(FunctionMetadata) == ERCIResponse::Intercept);
				}
			}

			// Don't process the RC message if any of interceptors returned ERCIResponse::Intercept
			if (bShouldIntercept)
			{
				return true;
			}
		}

#if WITH_EDITOR
		const bool bUseOngoingChangeOptimization = CVarRemoteControlEnableOngoingChangeOptimization.GetValueOnAnyThread() == 1;
		if (bUseOngoingChangeOptimization)
		{
			EndOngoingModificationIfMismatched(GetTypeHash(InCall.CallRef));
		}

		const bool bIsNewTransaction = bGenerateTransaction && (!bUseOngoingChangeOptimization || !OngoingModification);
		if (bIsNewTransaction && GEditor)
		{
			GEditor->BeginTransaction(LOCTEXT("RemoteCallTransaction", "Remote Call Transaction Wrap"));
		}

		const bool bIsManualTransaction = InCall.TransactionMode == ERCTransactionMode::MANUAL;
		if ((bIsNewTransaction || bIsManualTransaction) && ensure(InCall.CallRef.Object.IsValid()))
		{
			InCall.CallRef.Object->Modify();
		}
			
#endif
		FEditorScriptExecutionGuard ScriptGuard;
		if (ensure(InCall.CallRef.Object.IsValid()))
		{
			if (InCall.CallRef.PropertyWithSetter.IsValid())
			{
				InCall.CallRef.PropertyWithSetter->CallSetter(InCall.CallRef.Object.Get(), InCall.ParamData.GetData());
			}
			else
			{
				InCall.CallRef.Object->ProcessEvent(InCall.CallRef.Function.Get(), InCall.ParamStruct.GetStructMemory());
			}
		}

#if WITH_EDITOR
		if (bUseOngoingChangeOptimization)
		{
			// If we've called the same function recently, refresh the triggered flag and snapshot the object to the transaction buffer
			if (OngoingModification && GetTypeHash(*OngoingModification) == GetTypeHash(InCall.CallRef))
			{
				OngoingModification->bWasTriggeredSinceLastPass = true;
				if (OngoingModification->bHasStartedTransaction)
				{
					SnapshotTransactionBuffer(InCall.CallRef.Object.Get());
					
					if (UActorComponent* Component = Cast<UActorComponent>(InCall.CallRef.Object.Get()))
					{
						SnapshotTransactionBuffer(Component->GetOwner());
						Component->MarkRenderStateDirty();
						Component->UpdateComponentToWorld();
					}
				}
			}
			else if (!bIsManualTransaction)
			{
				OngoingModification = InCall.CallRef;
				OngoingModification->bHasStartedTransaction = bGenerateTransaction;
			}
		}
		else if (GEditor && bGenerateTransaction)
		{
			GEditor->EndTransaction();
		}
#endif
		return true;
	}
	return false;
}

bool FRemoteControlModule::ResolveObject(ERCAccess AccessType, const FString& ObjectPath, const FString& PropertyName, FRCObjectReference& OutObjectRef, FString* OutErrorText)
{
	bool bSuccess = true;
	FString ErrorText;
	if (!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		// Resolve the object
		UObject* Object = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectPath);
		if (Object)
		{
			if (CanBeAccessedRemotely(Object))
			{
				bSuccess = ResolveObjectProperty(AccessType, Object, PropertyName, OutObjectRef, OutErrorText);
			}
			else
			{
				ErrorText = FString::Printf(TEXT("Object %s cannot be accessed remotely, check remote control project settings."), *Object->GetName());
				bSuccess = false;
			}
		}
		else
		{
			ErrorText = FString::Printf(TEXT("Object: %s does not exist when trying to resolve property: %s"), *ObjectPath, *PropertyName);
			bSuccess = false;
		}
	}
	else
	{
		ErrorText = FString::Printf(TEXT("Can't resolve object: %s while saving or garbage collecting."), *ObjectPath);
		bSuccess = false;
	}

	if (!bSuccess && !ErrorText.IsEmpty())
	{
		IRemoteControlModule::BroadcastError(ErrorText);
	}

	if (OutErrorText && !ErrorText.IsEmpty())
	{
		*OutErrorText = MoveTemp(ErrorText);
	}

	return bSuccess;
}

bool FRemoteControlModule::ResolveObjectProperty(ERCAccess AccessType, UObject* Object, FRCFieldPathInfo PropertyPath, FRCObjectReference& OutObjectRef, FString* OutErrorText)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::ResolveObjectProperty);
	bool bSuccess = true;
	FString ErrorText;
	if (!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		if (Object)
		{
			const bool bObjectInGame = !GIsEditor || IsRunningGame();

			if (PropertyPath.GetSegmentCount() != 0)
			{
				//Build a FieldPathInfo using property name to facilitate resolving
				if (PropertyPath.Resolve(Object))
				{
					FProperty* ResolvedProperty = PropertyPath.GetResolvedData().Field;

					if (RemoteControlUtil::IsPropertyAllowed(ResolvedProperty, AccessType, Object, bObjectInGame, &ErrorText))
					{
						OutObjectRef = FRCObjectReference{AccessType, Object, MoveTemp(PropertyPath)};
					}
					else
					{
						bSuccess = false;
					}
				}
				else
				{
					ErrorText = FString::Printf(TEXT("Object property: %s could not be resolved on object: %s"), *PropertyPath.GetFieldName().ToString(), *Object->GetPathName());
					bSuccess = false;
				}
			}
			else
			{
				OutObjectRef = FRCObjectReference{AccessType, Object};
			}
		}
		else
		{
			ErrorText = FString::Printf(TEXT("Invalid object to resolve property '%s'"), *PropertyPath.GetFieldName().ToString());
			bSuccess = false;
		}
	}
	else
	{
		ErrorText = FString::Printf(TEXT("Can't resolve object '%s' properties: %s while saving or garbage collecting."), *Object->GetPathName(), *PropertyPath.GetFieldName().ToString());
		bSuccess = false;
	}

	if (!bSuccess && !ErrorText.IsEmpty())
	{
		IRemoteControlModule::BroadcastError(ErrorText);

		if (OutErrorText != nullptr)
		{
			*OutErrorText = MoveTemp(ErrorText);
		}
	}

	return bSuccess;
}

bool FRemoteControlModule::GetObjectProperties(const FRCObjectReference& ObjectAccess, IStructSerializerBackend& Backend)
{
	if (ObjectAccess.IsValid() && ObjectAccess.Access == ERCAccess::READ_ACCESS)
	{
		bool bCanSerialize = true;
		UObject* Object = ObjectAccess.Object.Get();
		UStruct* ContainerType = ObjectAccess.ContainerType.Get();

		FStructSerializerPolicies Policies;
		if (ObjectAccess.Property.IsValid())
		{
			if (ObjectAccess.PropertyPathInfo.IsResolved())
			{
				Policies.MapSerialization = EStructSerializerMapPolicies::Array;
				Policies.PropertyFilter = [&ObjectAccess](const FProperty* CurrentProp, const FProperty* ParentProp)
				{
					return CurrentProp == ObjectAccess.Property || ParentProp != nullptr;
				};
			}
			else
			{
				bCanSerialize = false;
			}
		}
		else
		{
			bool bObjectInGame = !GIsEditor || Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
			Policies.PropertyFilter = [&ObjectAccess, bObjectInGame](const FProperty* CurrentProp, const FProperty* ParentProp)
			{
				FString ErrorText;
				const bool bIsPropertyAllowed =	RemoteControlUtil::IsPropertyAllowed(CurrentProp, ObjectAccess.Access, ObjectAccess.Object.Get(), bObjectInGame, &ErrorText);
				if (bIsPropertyAllowed || ParentProp != nullptr)
				{
					return true;
				}

				IRemoteControlModule::BroadcastError(ErrorText + TEXT("\nAlso parent property is not valid."));
				return false;
			};
		}

		if (bCanSerialize)
		{
			//Serialize the element if we're looking for a member or serialize the full object if not
			if (ObjectAccess.PropertyPathInfo.IsResolved())
			{
				const FRCFieldPathSegment& LastSegment = ObjectAccess.PropertyPathInfo.GetFieldSegment(ObjectAccess.PropertyPathInfo.GetSegmentCount() - 1);
				int32 Index = LastSegment.ArrayIndex != INDEX_NONE ? LastSegment.ArrayIndex : LastSegment.ResolvedData.MapIndex;

				FStructSerializer::SerializeElement(ObjectAccess.ContainerAdress, LastSegment.ResolvedData.Field, Index, Backend, Policies);
			}
			else
			{
				FStructSerializer::Serialize(ObjectAccess.ContainerAdress, *ObjectAccess.ContainerType, Backend, Policies);
			}
			return true;
		}
	}
	return false;
}

#if WITH_EDITOR
bool FRemoteControlModule::StartPropertyTransaction(FRCObjectReference& ObjectReference, const FText& InChangeDescription, bool& bOutGeneratedTransaction)
{
	const bool bUseOngoingChangeOptimization = CVarRemoteControlEnableOngoingChangeOptimization.GetValueOnAnyThread() == 1;
	if (bUseOngoingChangeOptimization)
	{
		const FString ObjectPath = ObjectReference.Object->GetPathName();

		// If we have a change that hasn't yet generated a post edit change property, do that before handling the next change.
		EndOngoingModificationIfMismatched(GetTypeHash(ObjectReference));

		// This step is necessary because the object might get recreated by a PostEditChange called in TestOrFinalizeOngoingChange.
		if (!ObjectReference.Object.IsValid())
		{
			ObjectReference.Object = StaticFindObject(UObject::StaticClass(), nullptr, *ObjectPath);
			if (ObjectReference.Object.IsValid() && ObjectReference.PropertyPathInfo.IsResolved())
			{
				// Update ContainerAddress as well if the path was resolved.
				if (ObjectReference.PropertyPathInfo.Resolve(ObjectReference.Object.Get()))
				{
					ObjectReference.ContainerAdress = ObjectReference.PropertyPathInfo.GetResolvedData().ContainerAddress;
				}
			}
			else
			{
				return false;
			}
		}
	}

	bOutGeneratedTransaction = ObjectReference.Access == ERCAccess::WRITE_TRANSACTION_ACCESS;
	const bool bIsNewTransaction = bOutGeneratedTransaction && (!bUseOngoingChangeOptimization || !OngoingModification);

	if (bIsNewTransaction && GEditor)
	{
		GEditor->BeginTransaction(InChangeDescription);

		if (bUseOngoingChangeOptimization)
		{
			// Call modify since it's not called by PreEditChange until the end of the ongoing change.
			ObjectReference.Object->Modify();
		}
	}

	if (!bOutGeneratedTransaction)
	{
		FEditPropertyChain PreEditChain;
		ObjectReference.PropertyPathInfo.ToEditPropertyChain(PreEditChain);
		ObjectReference.Object->PreEditChange(PreEditChain);
	}

	return true;
}
#endif

#if WITH_EDITOR
void FRemoteControlModule::SnapshotOrEndTransaction(FRCObjectReference& ObjectReference, bool bGeneratedTransaction)
{
	const bool bUseOngoingChangeOptimization = CVarRemoteControlEnableOngoingChangeOptimization.GetValueOnAnyThread() == 1;
	const bool bHasOngoingTransaction = bUseOngoingChangeOptimization && OngoingModification
		&& GetTypeHash(*OngoingModification) == GetTypeHash(ObjectReference) && OngoingModification->bHasStartedTransaction;

	if (bHasOngoingTransaction || ObjectReference.Access == ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS)
	{
		SnapshotTransactionBuffer(ObjectReference.Object.Get());
		FPropertyChangedEvent PropertyEvent(ObjectReference.PropertyPathInfo.ToPropertyChangedEvent());
		PropertyEvent.ChangeType = EPropertyChangeType::Interactive;
		ObjectReference.Object->PostEditChangeProperty(PropertyEvent);

		if (UActorComponent* Component = Cast<UActorComponent>(ObjectReference.Object.Get()))
		{
			Component->MarkRenderStateDirty();
			Component->UpdateComponentToWorld();
		}
	}

	if (bUseOngoingChangeOptimization)
	{
		// If we have modified the same object and property in the last frames,
		// update the triggered flag and snapshot the object to the transaction buffer.
		if (OngoingModification && GetTypeHash(*OngoingModification) == GetTypeHash(ObjectReference))
		{
			OngoingModification->bWasTriggeredSinceLastPass = true;

			// Update the world lighting if we're modifying a color.
			if (ObjectReference.IsValid())
			{
				if (ObjectReference.Object->IsA<ULightComponent>())
				{
					UClass* OwnerClass = ObjectReference.Property->GetOwnerClass();
					if (OwnerClass && OwnerClass->IsChildOf(ULightComponentBase::StaticClass()))
					{
						UWorld* World = ObjectReference.Object->GetWorld();
						if (World && World->Scene)
						{
							World->Scene->UpdateLightColorAndBrightness(Cast<ULightComponent>(ObjectReference.Object.Get()));
						}
					}
				}
			}
		}
		else if (ObjectReference.Access != ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS)
		{
			OngoingModification = ObjectReference;
			OngoingModification->bHasStartedTransaction = bGeneratedTransaction;
		}
	}
	else
	{
		if (GEditor && bGeneratedTransaction)
		{
			GEditor->EndTransaction();
		}

		FPropertyChangedEvent PropertyEvent(ObjectReference.PropertyPathInfo.ToPropertyChangedEvent());
		ObjectReference.Object->PostEditChangeProperty(PropertyEvent);
	}
}
#endif

bool FRemoteControlModule::SetObjectProperties(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InPayload, ERCModifyOperation Operation, const ERCModifyOperationFlags ModifyOperationFlags)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::SetObjectProperties);
	UE_LOGF(LogRemoteControl, VeryVerbose, "Set Object Properties");

	// Check the replication path before applying property values
	if (InPayload.Num() != 0 && ObjectAccess.Object.IsValid())
	{
		// Convert raw property modifications to setter function calls if necessary.
		if (RemoteControlUtil::PropertyModificationShouldUseSetter(ObjectAccess.Object.Get(), ObjectAccess.Property.Get()))
		{
			FRCCall Call;
			FRCInterceptionPayload InterceptionPayload;
			constexpr bool bCreateInterceptionPayload = true;
			RemoteControlSetterUtils::FConvertToFunctionCallArgs Args(ObjectAccess, Backend, Call);

			// If this is a delta operation, deserialize the value into and apply the operation on a temporary buffer
			TArray<uint8> DeltaData;
			if (Operation != ERCModifyOperation::EQUAL)
			{
				if (!DeserializeDeltaModificationData(ObjectAccess, Backend, Operation, DeltaData))
				{
					return false;
				}

				Args.ValuePtrOverride = ObjectAccess.Property->ContainerPtrToValuePtr<void>(DeltaData.GetData());
			}

			if (RemoteControlSetterUtils::ConvertModificationToFunctionCall(Args, InterceptionPayload))
			{
				const bool bResult = InvokeCall(Call, InterceptionPayload.Type, InterceptionPayload.Payload);

				if (bResult)
				{
					RefreshEditorPostSetObjectProperties(ObjectAccess);
				}

				return bResult;
			}
		}

		// If a setter wasn't used, verify if the property should be allowed.
		bool bObjectInGame = !GIsEditor;
		FString ErrorText;

		if (!RemoteControlUtil::IsPropertyAllowed(ObjectAccess.Property.Get(), ObjectAccess.Access, ObjectAccess.Object.Get() ,bObjectInGame, &ErrorText))
		{
			IRemoteControlModule::BroadcastError(ErrorText);
			return false;
		}

		// Build interception command
		FString PropertyPathString = TFieldPath<FProperty>(ObjectAccess.Property.Get()).ToString();
		FRCIPropertiesMetadata PropsMetadata(ObjectAccess.Object->GetPathName(), PropertyPathString, ObjectAccess.PropertyPathInfo.ToString(), ToExternal(ObjectAccess.Access), ToExternal(InPayloadType), ToExternal(Operation), InPayload);

		// Initialization
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const FName InterceptorFeatureName = IRemoteControlInterceptionFeatureInterceptor::GetName();
		const int32 InterceptorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureInterceptor::GetName());

		// Pass interception command data to all available interceptors
		bool bShouldIntercept = false;
		for (int32 InterceptorIdx = 0; InterceptorIdx < InterceptorsAmount; ++InterceptorIdx)
		{
			IRemoteControlInterceptionFeatureInterceptor* const Interceptor = static_cast<IRemoteControlInterceptionFeatureInterceptor*>(ModularFeatures.GetModularFeatureImplementation(InterceptorFeatureName, InterceptorIdx));
			if (Interceptor)
			{
				// Update response flag
				UE_LOGF(LogRemoteControl, VeryVerbose, "Set Object Properties - Intercepted");
				bShouldIntercept |= (Interceptor->SetObjectProperties(PropsMetadata) == ERCIResponse::Intercept);
			}
		}

		// Don't process the RC message if any of interceptors returned ERCIResponse::Intercept
		if (bShouldIntercept)
		{
			return true;
		}
	}

	// Convert raw property modifications to setter function calls if necessary.
	if (RemoteControlUtil::PropertyModificationShouldUseSetter(ObjectAccess.Object.Get(), ObjectAccess.Property.Get()))
	{
		FRCCall Call;
		RemoteControlSetterUtils::FConvertToFunctionCallArgs Args(ObjectAccess, Backend, Call);

		// If this is a delta operation, deserialize the value into and apply the operation on a temporary buffer
		TArray<uint8> DeltaData;
		if (Operation != ERCModifyOperation::EQUAL)
		{
			if (!DeserializeDeltaModificationData(ObjectAccess, Backend, Operation, DeltaData))
			{
				return false;
			}

			Args.ValuePtrOverride = ObjectAccess.Property->ContainerPtrToValuePtr<void>(DeltaData.GetData());
		}

		if (RemoteControlSetterUtils::ConvertModificationToFunctionCall(Args))
		{
			const bool bResult = InvokeCall(Call);

			if (bResult)
			{
				RefreshEditorPostSetObjectProperties(ObjectAccess);
			}

			return bResult;
		}
	}

	// Setting object properties require a property and can't be done at the object level. Property must be valid to move forward
	if (ObjectAccess.IsValid()
		&& (RemoteControlUtil::IsWriteAccess(ObjectAccess.Access))
		&& ObjectAccess.Property.IsValid()
		&& ObjectAccess.PropertyPathInfo.IsResolved())
	{
		UStruct* ContainerType = ObjectAccess.ContainerType.Get();
		FRCObjectReference MutableObjectReference = ObjectAccess;

#if WITH_EDITOR
		const bool bWithPropertyChangedEvents = !EnumHasAnyFlags(ModifyOperationFlags, ERCModifyOperationFlags::SkipPropertyChangeEvents);
		
		bool bGeneratedTransaction = false;
		if (bWithPropertyChangedEvents &&
			!StartPropertyTransaction(MutableObjectReference, LOCTEXT("RemoteSetPropertyTransaction", "Remote Set Object Property"), bGeneratedTransaction))
		{
			return false;
		}
#endif

		FStructDeserializerPolicies Policies;
		if (MutableObjectReference.Property.IsValid())
		{
			Policies.PropertyFilter = [&MutableObjectReference](const FProperty* CurrentProp, const FProperty* ParentProp)
			{
				return CurrentProp == MutableObjectReference.Property || ParentProp != nullptr;
			};
		}
		else
		{
			bool bObjectInGame = !GIsEditor || MutableObjectReference.Object->GetOutermost()->HasAnyPackageFlags(PKG_PlayInEditor);
			Policies.PropertyFilter = [&MutableObjectReference, bObjectInGame](const FProperty* CurrentProp, const FProperty* ParentProp)
			{
				FString ErrorText;
				const bool bIsPropertyAllowed =	RemoteControlUtil::IsPropertyAllowed(CurrentProp, MutableObjectReference.Access, MutableObjectReference.Object.Get(), bObjectInGame, &ErrorText);
				if (bIsPropertyAllowed || ParentProp != nullptr)
				{
					return true;
				}

				IRemoteControlModule::BroadcastError(ErrorText + TEXT("\nAlso parent property is not valid."));
				return false;
			};
		}

		bool bSuccess = false;

		if (Operation == ERCModifyOperation::EQUAL)
		{
			//Serialize the element if we're looking for a member or serialize the full object if not
			if (MutableObjectReference.PropertyPathInfo.IsResolved())
			{
				const FRCFieldPathSegment& LastSegment = MutableObjectReference.PropertyPathInfo.GetFieldSegment(MutableObjectReference.PropertyPathInfo.GetSegmentCount() - 1);
				int32 Index = LastSegment.ArrayIndex != INDEX_NONE ? LastSegment.ArrayIndex : LastSegment.ResolvedData.MapIndex;
				bSuccess = FStructDeserializer::DeserializeElement(MutableObjectReference.ContainerAdress, *LastSegment.ResolvedData.Struct, Index, Backend, Policies);
			}
			else
			{
				bSuccess = FStructDeserializer::Deserialize(MutableObjectReference.ContainerAdress, *ContainerType, Backend, Policies);
			}
		}
		else
		{
			// This is a delta operation, so deserialize the value into and apply the operation on a temporary buffer
			TArray<uint8> DeltaData;
			if (!DeserializeDeltaModificationData(MutableObjectReference, Backend, Operation, DeltaData))
			{
				return false;
			}

			// Copy data to the actual object
			MutableObjectReference.Property->CopyCompleteValue_InContainer(MutableObjectReference.ContainerAdress, DeltaData.GetData());
			bSuccess = true;
		}

#if WITH_EDITOR
		if (bWithPropertyChangedEvents)
		{
			SnapshotOrEndTransaction(MutableObjectReference, bGeneratedTransaction);
		}
#endif

		for (const TPair<FName, TSharedPtr<IRemoteControlPropertyFactory>>& EntityFactoryPair : EntityFactories)
		{
			EntityFactoryPair.Value->PostSetObjectProperties(MutableObjectReference.Object.Get(), bSuccess);
		}

		if (bSuccess)
		{
			RefreshEditorPostSetObjectProperties(ObjectAccess);
		}

		return bSuccess;
	}
	return false;
}

bool FRemoteControlModule::AppendToObjectArrayProperty(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InPayload)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::AppendToObjectArrayProperty);
	UE_LOGF(LogRemoteControl, VeryVerbose, "Append to Object Array Property");

	return AddToArrayProperty(ObjectAccess, Backend, [](FScriptArrayHelper& ArrayHelper)
	{
		return ArrayHelper.AddValue();
	});
}

bool FRemoteControlModule::InsertToObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InPayload)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::InsertToObjectArrayProperty);
	UE_LOGF(LogRemoteControl, VeryVerbose, "Insert to Object Array Property");

	return AddToArrayProperty(ObjectAccess, Backend, [Index](FScriptArrayHelper& ArrayHelper)
	{
		if (!ArrayHelper.IsValidIndex(Index))
		{
			return (int32)INDEX_NONE;
		}

		ArrayHelper.InsertValues(Index, 1);
		return Index;
	});
}

bool FRemoteControlModule::RemoveFromObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::RemoveFromObjectArrayProperty);
	UE_LOGF(LogRemoteControl, VeryVerbose, "Remove from Object Array Property");

	return ModifyArrayProperty(ObjectAccess, [Index](FScriptArrayHelper& ArrayHelper)
	{
		if (!ArrayHelper.IsValidIndex(Index))
		{
			return false;
		}

		ArrayHelper.RemoveValues(Index);
		return true;
	});
}

void FRemoteControlModule::RefreshEditorPostSetObjectProperties(const FRCObjectReference& ObjectAccess)
{
#if WITH_EDITOR
	UObject* Object = ObjectAccess.Object.Get();

	if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
	{
		if (FProperty* Property = ObjectAccess.Property.Get())
		{
			// Check if the property is the RelativeLocation or if a child by checking the Segment[0]
			const bool bCheckSegmentParent = ObjectAccess.PropertyPathInfo.Segments.Num() ? ObjectAccess.PropertyPathInfo.Segments[0].Name == RemoteControlUtil::NAME_RelativeLocation : false;
			if(Property->GetFName() == RemoteControlUtil::NAME_RelativeLocation || bCheckSegmentParent)
			{
				if (AActor* Actor = SceneComponent->GetOwner())
				{
					if (Actor->IsSelectedInEditor())
					{
						GUnrealEd->UpdatePivotLocationForSelection();
					}
				}
			}
		}
	}
#endif
}

bool FRemoteControlModule::SetPresetController(const FName PresetName, const FName ControllerName, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept)
{
	URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(PresetName);
	if (!ensure(Preset))
	{
		return false;
	}

	URCVirtualPropertyBase* VirtualProperty = Preset->GetController(ControllerName);
	if (!ensure(VirtualProperty))
	{
		return false;
	}

	return SetPresetController(PresetName, VirtualProperty, Backend, InPayload, bAllowIntercept);
}

// Note: The actual Controller implementation (URCController) resides in the RemoteControlLogic module
// and is not referenced directly here to avoid circular dependency between RemoteControl and RemoteControlLogic modules.
// Controllers are simply accessed as their parent "URCVirtualPropertyBase" objects 

bool FRemoteControlModule::SetPresetController(const FName PresetName, class URCVirtualPropertyBase* VirtualProperty, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept)
{
	if (!ensure(VirtualProperty))
	{
		return false;
	}

	URemoteControlPreset* Preset = VirtualProperty->PresetWeakPtr.Get();
	if (!ensure(Preset))
	{
		return false;
	}

	if (bAllowIntercept)
	{
		// Build interception command
		FRCIControllerMetadata ControllersMetadata(PresetName, VirtualProperty->PropertyName, InPayload);

		// Initialization
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const FName InterceptorFeatureName = IRemoteControlInterceptionFeatureInterceptor::GetName();
		const int32 InterceptorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureInterceptor::GetName());

		UE_LOGF(LogRemoteControl, VeryVerbose, "SetPresetController - Num Interceptors: %d", InterceptorsAmount);

		// Pass interception command data to all available interceptors
		bool bShouldIntercept = false;
		for (int32 InterceptorIdx = 0; InterceptorIdx < InterceptorsAmount; ++InterceptorIdx)
		{
			IRemoteControlInterceptionFeatureInterceptor* const Interceptor = static_cast<IRemoteControlInterceptionFeatureInterceptor*>(ModularFeatures.GetModularFeatureImplementation(InterceptorFeatureName, InterceptorIdx));
			if (Interceptor)
			{
				// Update response flag
				bShouldIntercept |= (Interceptor->SetPresetController(ControllersMetadata) == ERCIResponse::Intercept);
			}
		}

		// Don't process the RC message if any of interceptors returned ERCIResponse::Intercept
		if (bShouldIntercept)
		{
			return true;
		}
	}

	bool bSuccess = VirtualProperty->DeserializeFromBackend(Backend);
	if (bSuccess)
	{
		Preset->OnControllerModified().Broadcast(Preset, {VirtualProperty->Id});
	}
	return bSuccess;
}

bool FRemoteControlModule::ResetObjectProperties(const FRCObjectReference& ObjectAccess, const bool bAllowIntercept)
{
	// Check the replication path before reset the property on this instance
	if (bAllowIntercept && ObjectAccess.Object.IsValid())
	{
		// Build interception command
		FString PropertyPathString = TFieldPath<FProperty>(ObjectAccess.Property.Get()).ToString();
		FRCIObjectMetadata ObjectMetadata(ObjectAccess.Object->GetPathName(), PropertyPathString, ObjectAccess.PropertyPathInfo.ToString(), ToExternal(ObjectAccess.Access));

		// Initialization
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const FName InterceptorFeatureName = IRemoteControlInterceptionFeatureInterceptor::GetName();
		const int32 InterceptorsAmount = ModularFeatures.GetModularFeatureImplementationCount(IRemoteControlInterceptionFeatureInterceptor::GetName());

		// Pass interception command data to all available interceptors
		bool bShouldIntercept = false;
		for (int32 InterceptorIdx = 0; InterceptorIdx < InterceptorsAmount; ++InterceptorIdx)
		{
			IRemoteControlInterceptionFeatureInterceptor* const Interceptor = static_cast<IRemoteControlInterceptionFeatureInterceptor*>(ModularFeatures.GetModularFeatureImplementation(InterceptorFeatureName, InterceptorIdx));
			if (Interceptor)
			{
				// Update response flag
				bShouldIntercept |= (Interceptor->ResetObjectProperties(ObjectMetadata) == ERCIResponse::Intercept);
			}
		}

		// Don't process the RC message if any of interceptors returned ERCIResponse::Intercept
		if (bShouldIntercept)
		{
			return true;
		}
	}

	if (ObjectAccess.IsValid() && RemoteControlUtil::IsWriteAccess(ObjectAccess.Access))
	{
		UObject* Object = ObjectAccess.Object.Get();
		UStruct* ContainerType = ObjectAccess.ContainerType.Get();

#if WITH_EDITOR
		bool bGenerateTransaction = ObjectAccess.Access == ERCAccess::WRITE_TRANSACTION_ACCESS;
		constexpr bool bForceEndChange = true;
		TestOrFinalizeOngoingChange(true);
		FScopedTransaction Transaction(LOCTEXT("RemoteResetPropertyTransaction", "Remote Reset Object Property"), bGenerateTransaction);
		if (bGenerateTransaction)
		{
			FEditPropertyChain PreEditChain;
			ObjectAccess.PropertyPathInfo.ToEditPropertyChain(PreEditChain);
			Object->PreEditChange(PreEditChain);
		}

		if(HasDefaultValueCustomization(Object, ObjectAccess.Property.Get()))
		{
			FRCResetToDefaultArgs Args;
			Args.Property = ObjectAccess.Property.Get();
			Args.Path = ObjectAccess.PropertyPathInfo.ToPathPropertyString();
			Args.ArrayIndex = ObjectAccess.PropertyPathInfo.Segments[0].ArrayIndex;
			Args.bCreateTransaction = false;
			ResetToDefaultValue(Object, Args);
		}
		else
#endif
		{
			FRCFieldPathInfo FieldPathInfo = ObjectAccess.PropertyPathInfo;
			void* TargetAddress = FieldPathInfo.GetResolvedData().ContainerAddress;

			UObject* DefaultObject = Object->GetClass()->GetDefaultObject();
			if (FieldPathInfo.Resolve(DefaultObject))
			{
				// Copy the value from the field on the CDO
				FRCFieldResolvedData DefaultObjectResolvedData = FieldPathInfo.GetResolvedData();
				ObjectAccess.Property->CopyCompleteValue_InContainer(TargetAddress, DefaultObjectResolvedData.ContainerAddress);
			}
			else if (UStruct* ContainerStruct = ObjectAccess.ContainerType.Get())
			{
				// Structs have no CDO, so initialize a new instance of the struct and copy the value from there
				TArray<uint8> NewStructDataArray;
				const int32 StructureSize = ContainerStruct->GetStructureSize();
				NewStructDataArray.SetNumUninitialized(StructureSize);

				void* NewStructData = NewStructDataArray.GetData();
				ContainerStruct->InitializeStruct(NewStructData);

				ObjectAccess.Property->CopyCompleteValue_InContainer(TargetAddress, NewStructData);

				ContainerStruct->DestroyStruct(NewStructData);
			}
			else
			{
				// Containing object has no default value, so fall back to the initial value for this property type
				ObjectAccess.Property->InitializeValue_InContainer(TargetAddress);
			}
		}

		// if we are generating a transaction, also generate post edit property event, event if the change ended up unsuccessful
		// this is to match the pre edit change call that can unregister components for example
#if WITH_EDITOR
		if (bGenerateTransaction)
		{
			FPropertyChangedEvent PropertyEvent(ObjectAccess.Property.Get());
			Object->PostEditChangeProperty(PropertyEvent);
		}
#endif
		return true;
	}
	return false;
}

TOptional<FExposedFunction> FRemoteControlModule::ResolvePresetFunction(const FResolvePresetFieldArgs& Args) const
{
	TOptional<FExposedFunction> ExposedFunction;

	if (URemoteControlPreset* Preset = ResolvePreset(FName(*Args.PresetName)))
	{
		ExposedFunction = Preset->ResolveExposedFunction(FName(*Args.FieldLabel));
	}

	return ExposedFunction;
}

TOptional<FExposedProperty> FRemoteControlModule::ResolvePresetProperty(const FResolvePresetFieldArgs& Args) const
{
	TOptional<FExposedProperty> ExposedProperty;

	if (URemoteControlPreset* Preset = ResolvePreset(FName(*Args.PresetName)))
	{
		ExposedProperty = Preset->ResolveExposedProperty(FName(*Args.FieldLabel));
	}

	return ExposedProperty;
}

URemoteControlPreset* FRemoteControlModule::ResolvePreset(FName PresetName) const
{
	if (const TWeakObjectPtr<URemoteControlPreset>* EmbeddedPreset = EmbeddedPresets.Find(PresetName))
	{
		if (EmbeddedPreset->IsValid())
		{
			return EmbeddedPreset->Get();
		}
	}

	if (const TArray<FAssetData>* Assets = CachedPresetsByName.Find(PresetName))
	{
		for (const FAssetData& Asset : *Assets)
		{
			if (Asset.AssetName == PresetName)
			{
				return Cast<URemoteControlPreset>(Asset.GetAsset());
			}
		}
	}

	/**
	 * No need to cache hosted preset names - they should already be in a short enough list
	 * mapped to their name.
	 */
	if (URemoteControlPreset* FoundPreset = RemoteControlUtil::GetPresetByName(PresetName))
	{
		CachedPresetsByName.FindOrAdd(PresetName).AddUnique(FoundPreset);
		return FoundPreset;
	}

	return nullptr;
}

URemoteControlPreset* FRemoteControlModule::ResolvePreset(const FGuid& PresetId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlModule::ResolvePreset);

	if (const FName* AssetName = CachedPresetNamesById.Find(PresetId))
	{
		if (const TWeakObjectPtr<URemoteControlPreset>* EmbeddedPreset = EmbeddedPresets.Find(*AssetName))
		{
			if (EmbeddedPreset->IsValid())
			{
				return EmbeddedPreset->Get();
			}
		}

		if (const TArray<FAssetData>* Assets = CachedPresetsByName.Find(*AssetName))
		{
			for (const FAssetData& Asset : *Assets)
			{
				if (RemoteControlUtil::GetPresetId(Asset) == PresetId)
				{
					return Cast<URemoteControlPreset>(Asset.GetAsset());
				}
			}
		}
		else
		{
			ensureMsgf(false, TEXT("Preset id should be cached if the asset name already is."));
		}
	}

	for (const auto& Pair : EmbeddedPresets)
	{
		if (Pair.Value.IsValid())
		{
			URemoteControlPreset* EmbeddedPreset = Pair.Value.Get();
			FGuid EmbeddedPresetId = Pair.Value->GetPresetId();

			if (EmbeddedPresetId.IsValid() && EmbeddedPresetId == PresetId)
			{
				CachedPresetNamesById.Emplace(PresetId, Pair.Key);
				return EmbeddedPreset;
			}
		}
	}

	if (URemoteControlPreset* FoundPreset = RemoteControlUtil::GetPresetById(PresetId))
	{
		CachedPresetNamesById.Emplace(PresetId, FoundPreset->GetName());
		return FoundPreset;
	}

	return nullptr;
}

URemoteControlPreset* FRemoteControlModule::CreateTransientPreset()
{
	const FName AssetName(*FString::Printf(TEXT("TransientRCPreset%d"), NextTransientPresetIndex));
	++NextTransientPresetIndex;
		
	const FString AssetPath = FString::Printf(TEXT("/Temp/%s"), *AssetName.ToString());
	if (UPackage* Package = CreatePackage(*AssetPath))
	{
		if (URemoteControlPreset* Preset = NewObject<URemoteControlPreset>(Package, AssetName, RF_Transient | RF_Public | RF_Standalone))
		{
			FAssetRegistryModule::AssetCreated(Preset);
			const FAssetData AssetData(Preset);
			const FGuid PresetId = RemoteControlUtil::GetPresetId(AssetData);

			TransientPresets.Add(AssetData);
			OnAssetAdded(AssetData);

			return Preset;
		}
	}

	return nullptr;
}

bool FRemoteControlModule::DestroyTransientPreset(FName PresetName)
{
	if (URemoteControlPreset* Preset = ResolvePreset(PresetName))
	{
		return DestroyTransientPreset(Preset);
	}

	return false;
}

bool FRemoteControlModule::DestroyTransientPreset(const FGuid& PresetId)
{
	if (URemoteControlPreset* Preset = ResolvePreset(PresetId))
	{
		return DestroyTransientPreset(Preset);
	}

	return false;
}

bool FRemoteControlModule::IsPresetTransient(FName PresetName) const
{
	if (URemoteControlPreset* Preset = ResolvePreset(PresetName))
	{
		for (const FAssetData& TransientAssetData : TransientPresets)
		{
			if (TransientAssetData.GetAsset() == Preset)
			{
				return true;
			}
		}
	}

	return false;
}

bool FRemoteControlModule::IsPresetTransient(const FGuid& PresetId) const
{
	if (URemoteControlPreset* Preset = ResolvePreset(PresetId))
	{
		for (const FAssetData& TransientAssetData : TransientPresets)
		{
			if (TransientAssetData.GetAsset() == Preset)
			{
				return true;
			}
		}
	}

	return false;
}

void FRemoteControlModule::GetPresets(TArray<TSoftObjectPtr<URemoteControlPreset>>& OutPresets) const
{
	OutPresets.Reserve(CachedPresetsByName.Num());
	for (const TPair<FName, TArray<FAssetData>>& Entry : CachedPresetsByName)
	{
		Algo::Transform(Entry.Value, OutPresets, [this](const FAssetData& AssetData) { return Cast<URemoteControlPreset>(AssetData.GetAsset()); });
	}
}

void FRemoteControlModule::GetPresetAssets(TArray<FAssetData>& OutPresetAssets, bool bIncludeTransient) const
{
	if (CachedPresetsByName.Num() == 0)
	{
		CachePresets();
	}

	OutPresetAssets.Reserve(CachedPresetsByName.Num());
	
	if (bIncludeTransient)
	{
		for (const TPair<FName, TArray<FAssetData>>& Entry : CachedPresetsByName)
		{
			OutPresetAssets.Append(Entry.Value);
		}
	}
	else
	{
		for (const TPair<FName, TArray<FAssetData>>& Entry : CachedPresetsByName)
		{
			for (const FAssetData& AssetData : Entry.Value)
			{
				if (!TransientPresets.Contains(AssetData))
				{
					OutPresetAssets.Add(AssetData);
				}
			}
		}
	}
}

void FRemoteControlModule::GetEmbeddedPresets(TArray<TWeakObjectPtr<URemoteControlPreset>>& OutEmbeddedPresets) const
{
	for (const auto& Pair : EmbeddedPresets)
	{
		if (Pair.Value.IsValid())
		{
			OutEmbeddedPresets.Add(Pair.Value);
		}
	}
}

const TMap<FName, FEntityMetadataInitializer>& FRemoteControlModule::GetDefaultMetadataInitializers() const
{
	return DefaultMetadataInitializers;
}

bool FRemoteControlModule::RegisterDefaultEntityMetadata(FName MetadataKey, FEntityMetadataInitializer MetadataInitializer)
{
	if (!DefaultMetadataInitializers.Contains(MetadataKey))
	{
		DefaultMetadataInitializers.FindOrAdd(MetadataKey) = MoveTemp(MetadataInitializer);
		return true;
	}
	return false;
}

void FRemoteControlModule::UnregisterDefaultEntityMetadata(FName MetadataKey)
{
	DefaultMetadataInitializers.Remove(MetadataKey);
}

bool FRemoteControlModule::PropertySupportsRawModificationWithoutEditor(FProperty* Property, UClass* OwnerClass) const
{
	constexpr bool bInGameOrPackage = true;
#if WITH_EDITOR
	TGuardValue EmulatePackagedGuard(RemoteControlUtil::GEmulatePackagedAccess, bInGameOrPackage);
#endif
	return Property && (RemoteControlUtil::IsPropertyAllowed(Property, ERCAccess::WRITE_ACCESS, nullptr, bInGameOrPackage, nullptr) || !!RemoteControlPropertyUtilities::FindSetterFunction(Property, OwnerClass));
}

bool FRemoteControlModule::PropertySupportsRawModification(FProperty* InProperty, const UObject* InObject, const bool bInWithEditor, FString* OutError) const
{
	FString ErrorTextSet;
	FString ErrorTextGet;
	const bool bCheckPackaged = !bInWithEditor;
	if (!InProperty)
	{
		if (OutError)
		{
			*OutError = TEXT("Property is not bound");
		}
		return false;
	}

#if WITH_EDITOR
	TGuardValue EmulatePackagedGuard(RemoteControlUtil::GEmulatePackagedAccess, bCheckPackaged);
#endif
	const bool bIsPropertyAllowedToSet = RemoteControlUtil::IsPropertyAllowed(InProperty, ERCAccess::WRITE_ACCESS, InObject, bCheckPackaged, &ErrorTextSet);
	const bool bIsPropertyAllowedToRead = RemoteControlUtil::IsPropertyAllowed(InProperty, ERCAccess::READ_ACCESS, InObject, bCheckPackaged, &ErrorTextGet);

	if (bIsPropertyAllowedToSet && bIsPropertyAllowedToRead)
	{
		return true;
	}

	if (OutError)
	{
		if (!bIsPropertyAllowedToSet)
		{
			if (!bIsPropertyAllowedToRead)
			{
				*OutError = FString::Printf(TEXT("%s\n\n%s"), *ErrorTextSet, *ErrorTextGet);
			}
			else
			{
				*OutError = ErrorTextSet;
			}
		}
		else if (!bIsPropertyAllowedToRead)
		{
			*OutError = ErrorTextGet;
		}
	}
	return false;
}

void FRemoteControlModule::RegisterEntityFactory(const FName InFactoryName, const TSharedRef<IRemoteControlPropertyFactory>& InFactory)
{
	if( InFactoryName != NAME_None )
	{
		EntityFactories.Add(InFactoryName, InFactory);
	}
	else
	{
		UE_LOGF(LogRemoteControl, Error, "Factory should have a name");
		ensure(false);
	}
}

void FRemoteControlModule::UnregisterEntityFactory(const FName InFactoryName)
{
	EntityFactories.Remove(InFactoryName);
}

FGuid FRemoteControlModule::BeginManualEditorTransaction(const FText& InDescription, uint32 TypeHash)
{
#if WITH_EDITOR
	if (CVarRemoteControlEnableOngoingChangeOptimization.GetValueOnAnyThread() == 1)
	{
		EndOngoingModificationIfMismatched(TypeHash);

		if (OngoingModification)
		{
			// We weren't able to finish the ongoing modification, so we won't start a new transaction
			return FGuid();
		}
	}

	if (GEditor && GEditor->Trans)
	{
		if (GEditor->BeginTransaction(InDescription) != INDEX_NONE)
		{
			const FTransaction* Transaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - 1);
			if (ensure(Transaction))
			{
				return Transaction->GetId();
			}
		}
	}
#endif

	return FGuid();
}

int32 FRemoteControlModule::EndManualEditorTransaction(const FGuid& TransactionId)
{
#if WITH_EDITOR
	if (GEditor && GEditor->Trans && GEditor->Trans->IsActive())
	{
		if (const FTransaction* CurrentTransaction = GEditor->Trans->GetTransaction(GEditor->Trans->GetQueueLength() - 1))
		{
			if (CurrentTransaction->GetId() == TransactionId)
			{
				return GEditor->EndTransaction();
			}
		}
	}
#endif

	return INDEX_NONE;
}

bool FRemoteControlModule::CanBeAccessedRemotely(UObject* Object) const
{
	check(Object);
		
	if (!GetDefault<URemoteControlSettings>()->bEnableRemotePythonExecution)
	{
		static const FName PythonScriptName = "PythonScriptLibrary";
		static const FName PythonScriptSettingsName = "PythonScriptPluginSettings";

		const FName ObjectClassName = Object->GetClass()->GetFName();
		if (ObjectClassName == PythonScriptName || ObjectClassName == PythonScriptSettingsName)
		{
			return false;
		}
	}

	if (Object == GetDefault<URemoteControlSettings>())
	{
		return false;
	}

	return true;
}

TSharedPtr<IPropertyIdHandler> FRemoteControlModule::GetPropertyIdHandlerFor(FProperty* InProperty)
{
	for (TSharedPtr<IPropertyIdHandler> Handler : PropertyIdPropertyHandlers)
	{
		if (Handler->IsPropertySupported(InProperty))
		{
			return Handler;
		}
	}
	return nullptr;
}

void FRemoteControlModule::RegisterPropertyIdPropertyHandlerImpl(const TSharedRef<IPropertyIdHandler>& InPropertyIdPropertyHandler)
{
	PropertyIdPropertyHandlers.Add(InPropertyIdPropertyHandler);
}

void FRemoteControlModule::CachePresets() const
{
	TArray<FAssetData> Assets;
	RemoteControlUtil::GetAllPresetAssets(Assets);

	for (const FAssetData& AssetData : Assets)
	{
		CachedPresetsByName.FindOrAdd(AssetData.AssetName).AddUnique(AssetData);

		const FGuid PresetAssetId = RemoteControlUtil::GetPresetId(AssetData);
		if (PresetAssetId.IsValid())
		{
			CachedPresetNamesById.Add(PresetAssetId, AssetData.AssetName);
		}
		else if (URemoteControlPreset* Preset = Cast<URemoteControlPreset>(AssetData.GetAsset()))
		{
			// Handle the case where the preset asset data does not contain the ID yet.
			// This can happen with old assets that haven't been resaved yet.
			const FGuid PresetId = Preset->GetPresetId();
			if (PresetId.IsValid())
			{
				CachedPresetNamesById.Add(PresetId, AssetData.AssetName);
				CachedPresetsByName.FindOrAdd(AssetData.AssetName).AddUnique(AssetData);
			}
		}
	}
}

void FRemoteControlModule::OnAssetAdded(const FAssetData& AssetData)
{
	if (AssetData.AssetClassPath != URemoteControlPreset::StaticClass()->GetClassPathName())
	{
		return;
	}

	const FGuid PresetAssetId = RemoteControlUtil::GetPresetId(AssetData);
	if (PresetAssetId.IsValid())
	{
		CachedPresetNamesById.Add(PresetAssetId, AssetData.AssetName);
			CachedPresetsByName.FindOrAdd(AssetData.AssetName).AddUnique(AssetData);
		}
	}

void FRemoteControlModule::OnAssetRemoved(const FAssetData& AssetData)
{
	if (AssetData.AssetClassPath != URemoteControlPreset::StaticClass()->GetClassPathName())
	{
		return;
	}

	TransientPresets.Remove(AssetData);

	const FGuid PresetId = RemoteControlUtil::GetPresetId(AssetData);
	if (FName* PresetName = CachedPresetNamesById.Find(PresetId))
	{
		if (TArray<FAssetData>* Assets = CachedPresetsByName.Find(*PresetName))
		{
			for (auto It = Assets->CreateIterator(); It; ++It)
			{
				if (RemoteControlUtil::GetPresetId(*It) == PresetId)
				{
					It.RemoveCurrent();
					break;
				}
			}
		}
	}
}

void FRemoteControlModule::OnAssetRenamed(const FAssetData& AssetData, const FString& OldName)
{
	if (AssetData.AssetClassPath != URemoteControlPreset::StaticClass()->GetClassPathName())
	{
		// OldName comes in full path format /path/package.object
		// Assets are registered in package format /path/package
		FString PackageName = OldName;
		int32 DotIdx = INDEX_NONE;
		
		if (PackageName.FindChar('.', DotIdx))
		{
			PackageName = PackageName.Left(DotIdx);
		}

		FName PackageFName = FName(PackageName);
		
		// Find already registered preset with old package name
		TWeakObjectPtr<URemoteControlPreset> EmbeddedPreset;

		for (const auto& Pair : EmbeddedPresets)
		{
			if (Pair.Key == PackageFName)
			{
				EmbeddedPreset = Pair.Value;

				// Remove found asset
				EmbeddedPresets.Remove(Pair.Key);
				bool bRemovedId = false;
				
				// If preset is still valid and it has a valid id, just remove it.
				if (EmbeddedPreset.IsValid())
				{
					FGuid EmbeddedPresetId = EmbeddedPreset->GetPresetId();

					if (EmbeddedPresetId.IsValid())
					{
						CachedPresetNamesById.Remove(EmbeddedPresetId);
						bRemovedId = true;
					}
				}

				// If it's not valid or doesn't have a valid id, search for it and remove it.
				if (!bRemovedId)
				{
					for (auto It = CachedPresetNamesById.CreateIterator(); It; ++It)
					{
						if (It.Value() == Pair.Key)
						{
							It.RemoveCurrent();
						}
					}
				}

				break;
			}
		}

		// Reregister embedded asset if it's still valid.
		if (EmbeddedPreset.IsValid())
		{
			RegisterEmbeddedPreset(EmbeddedPreset.Get(), true);
		}

		return;
	}

	// Update regular asset-based preset
	const FGuid PresetId = RemoteControlUtil::GetPresetId(AssetData);
	if (FName* OldPresetName = CachedPresetNamesById.Find(PresetId))
	{
		if (TArray<FAssetData>* Assets = CachedPresetsByName.Find(*OldPresetName))
		{
			for (auto It = Assets->CreateIterator(); It; ++It)
			{
				if (RemoteControlUtil::GetPresetId(*It) == PresetId)
				{
					It.RemoveCurrent();
					break;
				}
			}

			if (Assets->Num() == 0)
			{
				CachedPresetsByName.Remove(*OldPresetName);
			}
		}
	}

	CachedPresetNamesById.Remove(PresetId);
	CachedPresetNamesById.Add(PresetId, AssetData.AssetName);

	CachedPresetsByName.FindOrAdd(AssetData.AssetName).AddUnique(AssetData);
}

bool FRemoteControlModule::DestroyTransientPreset(URemoteControlPreset* Preset)
{
	for (const FAssetData& TransientAssetData : TransientPresets)
	{
		if (TransientAssetData.GetAsset() == Preset)
		{
			// This call will also remove the asset from TransientPresets
			OnAssetRemoved(TransientAssetData);

			FAssetRegistryModule::AssetDeleted(Preset);

			if (UPackage* Package = Preset->GetPackage())
			{
				Package->MarkAsGarbage();
			}

			Preset->ClearFlags(RF_Public | RF_Standalone);
			Preset->MarkAsGarbage();

			return true;
		}
	}

	return false;
}

bool FRemoteControlModule::DeserializeDeltaModificationData(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCModifyOperation Operation, TArray<uint8>& OutData)
{
	// Allocate data to deserialize the request data into
	const int32 StructureSize = ObjectAccess.ContainerType->GetStructureSize();
	OutData.SetNumUninitialized(StructureSize);
	void* OutContainerAddress = OutData.GetData();

	// Copy existing property data into the delta container so unchanged values remain the same
	ObjectAccess.Property->CopyCompleteValue_InContainer(OutContainerAddress, ObjectAccess.ContainerAdress);

	// Deserialize the data on top of what we just copied so that modified properties contain delta values
	FStructDeserializerPolicies Policies;
	Policies.PropertyFilter = [&ObjectAccess](const FProperty* CurrentProp, const FProperty* ParentProp)
	{
		return CurrentProp == ObjectAccess.Property || ParentProp != nullptr;
	};

	// Wrap the backend so we can track which properties were actually changed by deserialization
	FPropertyMapStructDeserializerBackendWrapper BackendWrapper(Backend);

	bool bSuccess;
	if (ObjectAccess.PropertyPathInfo.IsResolved())
	{
		const FRCFieldPathSegment& LastSegment = ObjectAccess.PropertyPathInfo.GetFieldSegment(ObjectAccess.PropertyPathInfo.GetSegmentCount() - 1);
		int32 Index = LastSegment.ArrayIndex != INDEX_NONE ? LastSegment.ArrayIndex : LastSegment.ResolvedData.MapIndex;
		bSuccess = FStructDeserializer::DeserializeElement(OutContainerAddress, *LastSegment.ResolvedData.Struct, Index, BackendWrapper, Policies);
	}
	else
	{
		bSuccess = FStructDeserializer::Deserialize(OutContainerAddress, *ObjectAccess.ContainerType, BackendWrapper, Policies);
	}

	if (!bSuccess)
	{
		return false;
	}

	// The offset between the container we're modifying and the object it belongs to
	const ptrdiff_t ContainerFromBaseOffset = ((const uint8*)ObjectAccess.ContainerAdress) - ((const uint8*)ObjectAccess.Object.Get());

	// Apply delta operation to each property that was changed (where possible)
	for (const FPropertyMapStructDeserializerBackendWrapper::FReadPropertyData& ReadProperty : BackendWrapper.GetReadProperties())
	{
		// Pointer to the property in OutData
		void* OutPropertyValue = ReadProperty.Data;

		// Offset from start of OutData struct to the property that was changed
		const ptrdiff_t Offset = ((const uint8*)OutPropertyValue) - ((const uint8*)OutContainerAddress);

		// Pointer to the equivalent property in the original object
		const void* BasePropertyValue = ((uint8*)ObjectAccess.Object.Get()) + ContainerFromBaseOffset + Offset;

		if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(ReadProperty.Property))
		{
			// Float property
			if (NumericProperty->IsFloatingPoint())
			{
				bSuccess &= RemoteControlUtil::ApplySimpleDeltaOperation<double>(Operation, BasePropertyValue, OutPropertyValue, ReadProperty.Property,
					[&NumericProperty](const void* Data) { return NumericProperty->GetFloatingPointPropertyValue(Data); },
					[&NumericProperty](void* Data, double Value) { NumericProperty->SetFloatingPointPropertyValue(Data, Value); });
			}
			// Integer property
			else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
			{
				bSuccess &= RemoteControlUtil::ApplySimpleDeltaOperation<int64>(Operation, BasePropertyValue, OutPropertyValue, ReadProperty.Property,
					[&NumericProperty](const void* Data) { return NumericProperty->GetSignedIntPropertyValue(Data); },
					[&NumericProperty](void* Data, int64 Value) { NumericProperty->SetIntPropertyValue(Data, Value); });
			}
		}

		if (!bSuccess)
		{
			break;
		}
	}

	return bSuccess;
}

void FRemoteControlModule::RegisterDefaultValueFactoryForType(UClass* RemoteControlPropertyType, const FName PropertyName, const TSharedPtr<IRCDefaultValueFactory>& InDefaultValueFactory)
{
	const FString DefaultValueKey = RemoteControlPropertyType->GetName() + TEXT(".") + PropertyName.ToString();

	if (!DefaultValueFactories.Contains(*DefaultValueKey))
	{
		DefaultValueFactories.Add(*DefaultValueKey, InDefaultValueFactory);
	}
}

void FRemoteControlModule::UnregisterDefaultValueFactoryForType(UClass* RemoteControlPropertyType, const FName PropertyName)
{
	const FString DefaultValueKey = RemoteControlPropertyType->GetName() + TEXT(".") + PropertyName.ToString();

	DefaultValueFactories.Remove(*DefaultValueKey);
}

void FRemoteControlModule::RegisterDefaultValueFactories()
{
	// NOTE : Use the class to which the property actually belongs.
	RegisterDefaultValueFactoryForType(ULightComponentBase::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponentBase, Intensity), FLightIntensityDefaultValueFactory::MakeInstance());
	RegisterDefaultValueFactoryForType(UMeshComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials), FOverrideMaterialsDefaultValueFactory::MakeInstance());
}

void FRemoteControlModule::RegisterMaskingFactories()
{
	// DEPRECATED 5.5, here to keep support of the old implementation while it cannot be removed yet.
	RegisterMaskingFactoryForType(TBaseStructure<FVector>::Get(), FVectorMaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FVector4>::Get(), FVector4MaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FIntVector>::Get(), FIntVectorMaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FIntVector4>::Get(), FIntVector4MaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FRotator>::Get(), FRotatorMaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FColor>::Get(), FColorMaskingFactory::MakeInstance());
	RegisterMaskingFactoryForType(TBaseStructure<FLinearColor>::Get(), FLinearColorMaskingFactory::MakeInstance());
}

void FRemoteControlModule::RegisterPropertyIdHandler()
{
	RegisterPropertyIdPropertyHandler<FBasePropertyIdHandler>();
	RegisterPropertyIdPropertyHandler<FStructPropertyIdHandler>();
	RegisterPropertyIdPropertyHandler<FEnumPropertyIdHandler>();
	RegisterPropertyIdPropertyHandler<FObjectPropertyIdHandler>();
}

void FRemoteControlModule::PopulateDisallowedFunctions()
{
	FunctionDisallowList.Add(GetDefault<UKismetSystemLibrary>()->FindFunction(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, LaunchURL)));
}

bool FRemoteControlModule::CanInterceptFunction(const FRCCall& RCCall) const
{
	if (RCCall.IsValid())
	{
		if (!RCCall.CallRef.Function.IsValid())
		{
			return false;
		}

		for (TFieldIterator<FProperty> It(RCCall.CallRef.Function.Get()); It; ++It)
		{
			// At the moment interceptors do not support return values.
			if (It->HasAnyPropertyFlags(CPF_ReturnParm) || (It->HasAnyPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_ConstParm)))
			{
#if WITH_EDITOR
				FString FunctionName = RCCall.CallRef.Function->GetDisplayNameText().ToString();
#else
				FString FunctionName = RCCall.CallRef.Function->GetName();
#endif
				REMOTE_CONTROL_LOG_ONCE(Log, TEXT("Function \"%s\" on object \"%s\" could not be intercepted because it contains out parameters."), *FunctionName, *RCCall.CallRef.Object->GetName());
				return false;
			}
		}
	}

	return true;
}

void FRemoteControlModule::LogOnce(ELogVerbosity::Type InVerbosity, const FString& LogDetails, const FString& FileName, int32 LineNumber) const
{
	// This code was taken from AudioMixer.h
#if !NO_LOGGING
	// Log once to avoid Spam.
	static FCriticalSection Cs;
	static TSet<uint32> LogHistory;

	FScopeLock Lock(&Cs);
	const FString MessageToHash = FString::Printf(TEXT("%s (File %s, Line %d)"), *LogDetails, *FileName, LineNumber);

	uint32 Hash = GetTypeHash(MessageToHash);
	if (!LogHistory.Contains(Hash))
	{
		switch (InVerbosity)
		{
		case ELogVerbosity::Fatal:
			UE_LOGF(LogRemoteControl, Fatal, "%ls", *LogDetails);
			break;

		case ELogVerbosity::Error:
			UE_LOGF(LogRemoteControl, Error, "%ls", *LogDetails);
			break;

		case ELogVerbosity::Warning:
			UE_LOGF(LogRemoteControl, Warning, "%ls", *LogDetails);
			break;

		case ELogVerbosity::Display:
			UE_LOGF(LogRemoteControl, Display, "%ls", *LogDetails);
			break;

		case ELogVerbosity::Log:
			UE_LOGF(LogRemoteControl, Log, "%ls", *LogDetails);
			break;

		case ELogVerbosity::Verbose:
			UE_LOGF(LogRemoteControl, Verbose, "%ls", *LogDetails);
			break;

		case ELogVerbosity::VeryVerbose:
			UE_LOGF(LogRemoteControl, VeryVerbose, "%ls", *LogDetails);
			break;

		default:
			UE_LOGF(LogRemoteControl, Error, "%ls", *LogDetails);
			{
				static_assert(static_cast<uint8>(ELogVerbosity::NumVerbosity) == 8, "Missing ELogVerbosity case coverage");
			}
			break;
		}

		LogHistory.Add(Hash);
		if (InVerbosity == ELogVerbosity::Fatal || InVerbosity == ELogVerbosity::Error || InVerbosity == ELogVerbosity::Warning)
		{
			BroadcastError(LogDetails, InVerbosity);
		}
	}
#endif
}

bool FRemoteControlModule::AddToArrayProperty(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, TFunctionRef<int32(FScriptArrayHelper&)> ModifyFunction)
{
	return ModifyArrayProperty(ObjectAccess, [&ObjectAccess, &Backend, &ModifyFunction](FScriptArrayHelper& ArrayHelper) {
		const int32 Index = ModifyFunction(ArrayHelper);

		if (Index == INDEX_NONE)
		{
			return false;
		}

		const FRCFieldPathSegment& LastSegment = ObjectAccess.PropertyPathInfo.GetFieldSegment(ObjectAccess.PropertyPathInfo.GetSegmentCount() - 1);

		FStructDeserializerPolicies Policies;
		Policies.PropertyFilter = [&ObjectAccess](const FProperty* CurrentProp, const FProperty* ParentProp)
		{
			return CurrentProp == ObjectAccess.Property || ParentProp != nullptr;
		};

		return FStructDeserializer::DeserializeElement(ObjectAccess.ContainerAdress, *LastSegment.ResolvedData.Struct, Index, Backend, Policies);
	});
}

bool FRemoteControlModule::ModifyArrayProperty(const FRCObjectReference& ObjectAccess, TFunctionRef<bool(FScriptArrayHelper&)> ModifyFunction)
{
	if (ObjectAccess.IsValid()
		&& (RemoteControlUtil::IsWriteAccess(ObjectAccess.Access))
		&& ObjectAccess.Property.IsValid()
		&& ObjectAccess.PropertyPathInfo.IsResolved())
	{
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ObjectAccess.Property.Get()))
		{
#if WITH_EDITOR
			FRCObjectReference MutableObjectReference = ObjectAccess;

			bool bGeneratedTransaction;
			if (!StartPropertyTransaction(MutableObjectReference, LOCTEXT("RemoteArrayAppendTransaction", "Remote Append to Object Array Property"), bGeneratedTransaction))
			{
				return false;
			}
#endif

			FScriptArrayHelper ArrayHelper(ArrayProperty, ArrayProperty->ContainerPtrToValuePtr<void>(ObjectAccess.ContainerAdress));

			const bool bSuccess = ModifyFunction(ArrayHelper);

#if WITH_EDITOR
			SnapshotOrEndTransaction(MutableObjectReference, bGeneratedTransaction);

			for (const TPair<FName, TSharedPtr<IRemoteControlPropertyFactory>>& EntityFactoryPair : EntityFactories)
			{
				EntityFactoryPair.Value->PostSetObjectProperties(MutableObjectReference.Object.Get(), bSuccess);
			}
#endif

			if (bSuccess)
			{
				RefreshEditorPostSetObjectProperties(ObjectAccess);
			}

			return bSuccess;
		}
	}

	return false;
}

bool FRemoteControlModule::IsFunctionAllowed(UFunction* Function)
{
	return !FunctionDisallowList.Contains(Function);
}

#if WITH_EDITOR
void FRemoteControlModule::EndOngoingModificationIfMismatched(uint32 TypeHash)
{
	if (OngoingModification && GetTypeHash(*OngoingModification) != TypeHash)
	{
		TestOrFinalizeOngoingChange(true);
	}
}

void FRemoteControlModule::TestOrFinalizeOngoingChange(bool bForceEndChange)
{
	if (OngoingModification)
	{
		if (bForceEndChange || !OngoingModification->bWasTriggeredSinceLastPass)
		{
			// Call PreEditChange for the last call
			if (OngoingModification->Reference.IsType<FRCObjectReference>())
			{
				FRCObjectReference& Reference = OngoingModification->Reference.Get<FRCObjectReference>();
				if (Reference.IsValid())
				{
					FEditPropertyChain PreEditChain;
					Reference.PropertyPathInfo.ToEditPropertyChain(PreEditChain);
					Reference.Object->PreEditChange(PreEditChain);
				}
			}
			
			// If not, trigger the post edit change (and end the transaction if one was started)
			if (GEditor && OngoingModification->bHasStartedTransaction)
			{
				GEditor->EndTransaction();
			}

			if (OngoingModification->Reference.IsType<FRCObjectReference>())
			{
				const FRCObjectReference& ObjectReference  = OngoingModification->Reference.Get<FRCObjectReference>();

				if (UObject* const Object = ObjectReference.Object.Get())
				{
					FPropertyChangedEvent PropertyEvent = ObjectReference.PropertyPathInfo.ToPropertyChangedEvent(EPropertyChangeType::ValueSet);

					FEditPropertyChain EditPropertyChain;
					ObjectReference.PropertyPathInfo.ToEditPropertyChain(EditPropertyChain);

					if (EditPropertyChain.IsEmpty())
					{
						Object->PostEditChangeProperty(PropertyEvent);
					}
					else
					{
						FPropertyChangedChainEvent ChainEvent(EditPropertyChain, PropertyEvent);

						TArray<TMap<FString, int32>> ArrayIndicesPerObject;
						{
							TMap<FString, int32> ArrayIndices;
							ArrayIndices.Reserve(ObjectReference.PropertyPathInfo.Segments.Num());

							for (const FRCFieldPathSegment& Segment : ObjectReference.PropertyPathInfo.Segments)
							{
								ArrayIndices.Add(Segment.Name.ToString(), Segment.ArrayIndex);
							}

							ArrayIndicesPerObject.Add(MoveTemp(ArrayIndices));
						}

						ChainEvent.ObjectIteratorIndex = 0;
						ChainEvent.SetArrayIndexPerObject(ArrayIndicesPerObject);

						Object->PostEditChangeChainProperty(ChainEvent);
						RefreshEditorPostSetObjectProperties(ObjectReference);
					}
				}
			}

			OngoingModification.Reset();
		}
		else
		{
			// If the change has occured for this change, effectively reset the counter for it.
			OngoingModification->bWasTriggeredSinceLastPass = false;
		}
	}
}

void FRemoteControlModule::HandleEnginePostInit()
{
	CachePresets();
	RegisterEditorDelegates();
}

void FRemoteControlModule::HandleMapPreLoad(const FString& MapName)
{
	constexpr bool bForceFinalizeChange = true;
	TestOrFinalizeOngoingChange(bForceFinalizeChange);
}
	
void FRemoteControlModule::RegisterEditorDelegates()
{
	if (GEditor)
	{
		GEditor->GetTimerManager()->SetTimer(OngoingChangeTimer, FTimerDelegate::CreateRaw(this, &FRemoteControlModule::TestOrFinalizeOngoingChange, false), SecondsBetweenOngoingChangeCheck, true);
	}
	else
	{
		FallbackOngoingChangeTimer = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([&](float DeltaTime)
			{
				TestOrFinalizeOngoingChange(false);

				return true;
			}
		), SecondsBetweenOngoingChangeCheck);
	}
}
	
void FRemoteControlModule::UnregisterEditorDelegates()
{
	if (GEditor)
	{ 
		GEditor->GetTimerManager()->ClearTimer(OngoingChangeTimer);
	}
	else
	{
		FTSTicker::RemoveTicker(FallbackOngoingChangeTimer);
	}
}

FRemoteControlModule::FOngoingChange::FOngoingChange(FRCObjectReference InReference)
{
	Reference.Set<FRCObjectReference>(MoveTemp(InReference));
}
		
FRemoteControlModule::FOngoingChange::FOngoingChange(FRCCallReference InReference)
{
	Reference.Set<FRCCallReference>(MoveTemp(InReference));
}

#endif

void FRemoteControlModule::RegisterLocalForwardingInterceptor()
{
	if (!RCILocalForwardingInterceptor)
	{
		RCILocalForwardingInterceptor =  MakeUnique<FRemoteControlInterceptionForwardInterceptor>();
		IModularFeatures::Get().RegisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), RCILocalForwardingInterceptor.Get());
	}
}

void FRemoteControlModule::UnregisterLocalForwardingInterceptor()
{
	if (RCILocalForwardingInterceptor)
	{
		IModularFeatures::Get().UnregisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), RCILocalForwardingInterceptor.Get());
		RCILocalForwardingInterceptor.Reset();
	}
}

void FRemoteControlModule::UnregisterConsoleObjects()
{
	for (IConsoleObject* ConsoleObject : ConsoleObjects)
	{
		IConsoleManager::Get().UnregisterConsoleObject(ConsoleObject);
	}
	ConsoleObjects.Empty();
}

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
