// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/Ticker.h"
#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Factories/IRCDefaultValueFactory.h"
#include "Factories/IRemoteControlMaskingFactory.h"
#include "IRemoteControlInterceptionFeature.h"
#include "IRemoteControlModule.h"

/**
 * Implementation of the RemoteControl interface
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS

class FRemoteControlModule : public IRemoteControlModule
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IRemoteControlModule
	virtual FOnPresetRegistered& OnPresetRegistered() override;
	virtual FOnPresetUnregistered& OnPresetUnregistered() override;
	virtual FOnError& OnError() override;
	virtual bool RegisterPreset(FName Name, URemoteControlPreset* Preset) override;
	virtual void UnregisterPreset(FName Name) override;
	virtual bool RegisterEmbeddedPreset(URemoteControlPreset* Preset, bool bReplaceExisting) override;
	virtual void UnregisterEmbeddedPreset(FName Name) override;
	virtual void UnregisterEmbeddedPreset(URemoteControlPreset* Preset) override;
	virtual bool CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const override;
	virtual bool HasDefaultValueCustomization(const UObject* InObject, const FProperty* InProperty) const override;
	virtual void ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs) override;
	UE_DEPRECATED(5.5, "PerformMasking is deprecated, masking is now applied where appropriate automatically.")
	virtual void PerformMasking(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) override;
	virtual void RegisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType, const TSharedPtr<IRemoteControlMaskingFactory>& InMaskingFactory) override;
	virtual void UnregisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType) override;
	virtual bool SupportsMasking(const UScriptStruct* InStruct) const override;
	virtual bool SupportsMasking(const FProperty* InProperty) const override;
	virtual bool ResolveCall(const FString& ObjectPath, const FString& FunctionName, FRCCallReference& OutCallRef, FString* OutErrorText) override;
	virtual bool InvokeCall(FRCCall& InCall, ERCPayloadType InPayloadType = ERCPayloadType::Json, const TArray<uint8>& InInterceptPayload = TArray<uint8>()) override;
	virtual bool ResolveObject(ERCAccess AccessType, const FString& ObjectPath, const FString& PropertyName, FRCObjectReference& OutObjectRef, FString* OutErrorText = nullptr) override;
	virtual bool ResolveObjectProperty(ERCAccess AccessType, UObject* Object, FRCFieldPathInfo PropertyPath, FRCObjectReference& OutObjectRef, FString* OutErrorText = nullptr) override;
	virtual bool GetObjectProperties(const FRCObjectReference& ObjectAccess, IStructSerializerBackend& Backend) override;
	virtual bool SetObjectProperties(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InPayload, ERCModifyOperation Operation, const ERCModifyOperationFlags ModifyOperationFlags = ERCModifyOperationFlags::None) override;
	virtual bool ResetObjectProperties(const FRCObjectReference& ObjectAccess, const bool bAllowIntercept) override;
	virtual bool InsertToObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InInterceptPayload) override;
	virtual bool RemoveFromObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess) override;
	virtual bool AppendToObjectArrayProperty(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType, const TArray<uint8>& InInterceptPayload) override;
	virtual bool SetPresetController(const FName PresetName, const FName ControllerName, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept) override;
	virtual bool SetPresetController(const FName PresetName, class URCVirtualPropertyBase* VirtualProperty, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept) override;
	virtual TOptional<FExposedFunction> ResolvePresetFunction(const FResolvePresetFieldArgs& Args) const override;
	virtual TOptional<FExposedProperty> ResolvePresetProperty(const FResolvePresetFieldArgs& Args) const override;
	virtual URemoteControlPreset* ResolvePreset(FName PresetName) const override;
	virtual URemoteControlPreset* ResolvePreset(const FGuid& PresetId) const override;
	virtual URemoteControlPreset* CreateTransientPreset() override;
	virtual bool DestroyTransientPreset(FName PresetName) override;
	virtual bool DestroyTransientPreset(const FGuid& PresetId) override;
	virtual bool IsPresetTransient(FName PresetName) const override;
	virtual bool IsPresetTransient(const FGuid& PresetId) const override;
	virtual void GetPresets(TArray<TSoftObjectPtr<URemoteControlPreset>>& OutPresets) const override;
	virtual void GetPresetAssets(TArray<FAssetData>& OutPresetAssets, bool bIncludeTransient = true) const override;
	virtual void GetEmbeddedPresets(TArray<TWeakObjectPtr<URemoteControlPreset>>& OutEmbeddedPresets) const override;
	virtual const TMap<FName, FEntityMetadataInitializer>& GetDefaultMetadataInitializers() const override;
	virtual bool RegisterDefaultEntityMetadata(FName MetadataKey, FEntityMetadataInitializer MetadataInitializer) override;
	virtual void UnregisterDefaultEntityMetadata(FName MetadataKey) override;
	virtual bool PropertySupportsRawModificationWithoutEditor(FProperty* Property, UClass* OwnerClass = nullptr) const override;
	virtual bool PropertySupportsRawModification(FProperty* InProperty, const UObject* InObject, const bool bInWithEditor, FString* OutError = nullptr) const override;
	virtual void RegisterEntityFactory( const FName InFactoryName, const TSharedRef<IRemoteControlPropertyFactory>& InFactory) override;
	virtual void UnregisterEntityFactory( const FName InFactoryName ) override;
	virtual FGuid BeginManualEditorTransaction(const FText& InDescription, uint32 TypeHash) override;
	virtual int32 EndManualEditorTransaction(const FGuid& TransactionId) override;
	virtual const TMap<FName, TSharedPtr<IRemoteControlPropertyFactory>>& GetEntityFactories() const override { return EntityFactories; };
	virtual bool CanBeAccessedRemotely(UObject* Object) const override;
	virtual TSharedPtr<IPropertyIdHandler> GetPropertyIdHandlerFor(FProperty* InProperty) override;
	//~ End IRemoteControlModule

protected:
	virtual void RegisterPropertyIdPropertyHandlerImpl(const TSharedRef<IPropertyIdHandler>& InPropertyIdPropertyHandler) override;

private:
	/** Refreshes Editor related visuals like location Gizmo for relevant properties (like Location of a SceneComponent) */
	void RefreshEditorPostSetObjectProperties(const FRCObjectReference& ObjectAccess);

	/** Cache all presets in the project for the ResolvePreset function. */
	void CachePresets() const;
	
	//~ Asset registry callbacks
	void OnAssetAdded(const FAssetData& AssetData);
	void OnAssetRemoved(const FAssetData& AssetData);
	void OnAssetRenamed(const FAssetData& AssetData, const FString&);
	void OnEmbeddedPresetRenamed(URemoteControlPreset* Preset);

	/** Destroy a transient preset using an object reference. */
	bool DestroyTransientPreset(URemoteControlPreset* Preset);

	/**
	 * Deserialize data for a non-EQUAL modification request and apply the operation to the resulting data.
	 * 
	 * @param ObjectAccess Data about the object/property for which modification was requested.
	 * @param Backend Deserialization backend for the modification request.
	 * @param Operation Type of operation to apply to the value.
	 * @param OutData Buffer used to deserialize data from the backend and contain the result value.
	 * @return True if the data was successfully deserialized and modified.
	 */
	static bool DeserializeDeltaModificationData(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCModifyOperation Operation, TArray<uint8>& OutData);
	
	/**
	 * Register a default factory to handle that default value for the supported properties.
	 */
	void RegisterDefaultValueFactoryForType(UClass* RemoteControlPropertyType, const FName PropertyName, const TSharedPtr<IRCDefaultValueFactory>& InDefaultValueFactory);
	
	/**
	 * Unregister a previously registered default value factory.
	 */
	void UnregisterDefaultValueFactoryForType(UClass* RemoteControlPropertyType, const FName PropertyName);

	/**
	 * Register(s) default value factories of supported types.
	 */
	void RegisterDefaultValueFactories();
	
	/**
	 * Register(s) masking factories of supported types.
	 */
	void RegisterMaskingFactories();

	/**
 	 * Register(s) masking factories of supported types.
 	 */
	void RegisterPropertyIdHandler();

	/**
	 * Populate the list of functions that cannot be called remotely.
	 */
	void PopulateDisallowedFunctions();

	/**
	 * Whether function can be intercepted by a remote control interceptor.
	 */
	bool CanInterceptFunction(const FRCCall& RCCall) const;

	/**
	 * Used to log once per program execution. Called by the REMOTE_CONTROL_LOG_ONCE macro.
	 */
	void LogOnce(ELogVerbosity::Type InVerbosity, const FString& LogDetails, const FString& FileName, int32 LineNumber) const;


	/**
	 * Add an item to an array property, handling transactions and editor events automatically.
	 * @param ObjectAccess Reference to the array property to be modified.
	 * @param Backend Deserialization backend for the value to be added to the array. This will be deserialized into the array at the index provided by ModifyFunction.
	 * @param ModifyFunction Function which takes an array helper, attempts to modify the array, and returns the index of the new item (or INDEX_NONE if the request is invalid).
	 */
	bool AddToArrayProperty(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, TFunctionRef<int32(FScriptArrayHelper&)> ModifyFunction);

	/**
	 * Modify an array property, handling transactions and editor events automatically.
	 * @param ObjectAccess Reference to the array property to be modified.
	 * @param ModifyFunction Function which takes an array helper, attempts to modify the array, and returns true if the request was valid.
	 */
	bool ModifyArrayProperty(const FRCObjectReference& ObjectAccess, TFunctionRef<bool(FScriptArrayHelper&)> ModifyFunction);

	/** Returns whether the function is allowed to be called remotely. */
	bool IsFunctionAllowed(UFunction* Function);

#if WITH_EDITOR

	/**
	 * Create a transaction to modify an object's property, taking into account any ongoing transactions before this one.
	 * @param ObjectReference A reference to the object to be modified.
	 * @param InChangeDescription The description of the transaction to be created.
	 * @param bOutGeneratedTransaction Set to true if a transaction needed to be created, else false.
	 * @return true if the property is ready to be modified, else false.
	 */
	bool StartPropertyTransaction(FRCObjectReference& ObjectReference, const FText& InChangeDescription, bool& bOutGeneratedTransaction);

	/**
	 * Depending on transaction mode, either snapshot the object to the transaction buffer or end the transaction.
	 * This should be called after StartPropertyTransaction and then modifying the object's properties.
	 * @param ObjectReference A reference to the object to be modified.
	 * @param bGeneratedTransaction The value of bOutGeneratedTransaction provided by StartPropertyTransaction.
	 */
	void SnapshotOrEndTransaction(FRCObjectReference& ObjectReference, bool bGeneratedTransaction);

	/**
	 * End the ongoing modification if one exists and is a mismatch for the new object to edit.
	 * @param TypeHash The type hash of the object we want to modify after this check.
	 */
	void EndOngoingModificationIfMismatched(uint32 TypeHash);

	/** Finalize an ongoing change, triggering post edit change on the tracked object. */
	void TestOrFinalizeOngoingChange(bool bForceEndChange = false);

	// End ongoing change on map preload.
	void HandleMapPreLoad(const FString& MapName);
	
	/** Callback to handle registering delegates once the engine has finished its startup. */
	void HandleEnginePostInit();

	//~ Register/Unregister editor delegates.
	void RegisterEditorDelegates();
	void UnregisterEditorDelegates();

#endif // WITH_EDITOR

	/** Registers the local forwarding interceptor. */
	void RegisterLocalForwardingInterceptor();

	/** Unregisters the local forwarding interceptor. */
	void UnregisterLocalForwardingInterceptor();

	/** Unregisters the console objects from the console manager. */
	void UnregisterConsoleObjects();
	
private:
	/** Cache of preset names to preset assets */
	mutable TMap<FName, TArray<FAssetData>> CachedPresetsByName;

	/** Cache of ids to preset names. */
	mutable TMap<FGuid, FName> CachedPresetNamesById;

	/**
	 * Listed of hosted (non-asset based) presets. 
	 * We can assume that all hosted presets are "active," so the list should be relatively short.
	 **/
	TMap<FName, TWeakObjectPtr<URemoteControlPreset>> EmbeddedPresets;

	/** Temporary presets that aren't saved as assets or directly visible to the editor's user. */
	TSet<FAssetData> TransientPresets;

	/** Index of the next created transient preset, to avoid naming collisions. */
	uint32 NextTransientPresetIndex = 0;

	/** Map of registered default metadata initializers. */
	TMap<FName, FEntityMetadataInitializer> DefaultMetadataInitializers;

	/** Delegate for preset registration */
	FOnPresetRegistered OnPresetRegisteredDelegate;

	/** Delegate for preset unregistration */
	FOnPresetUnregistered OnPresetUnregisteredDelegate;

	/** Delegate for errors to allow external custom handling. */
	FOnError OnErrorDelegate;

	/** RC Processor feature instance */
	TUniquePtr<IRemoteControlInterceptionFeatureProcessor> RCIProcessor;

	/**
	 * RC Local Forwarding interceptor instance.
	 * When this is enabled it can locally emulate interception and forwarding to the RCI processor.
	 */
	TUniquePtr<IRemoteControlInterceptionFeatureInterceptor> RCILocalForwardingInterceptor;

#if WITH_EDITOR
	/** Flags for a given RC change. */
	struct FOngoingChange
	{
		FOngoingChange(FRCObjectReference InReference);
		FOngoingChange(FRCCallReference InReference);
		
		friend uint32 GetTypeHash(const FOngoingChange& Modification)
		{
			if (Modification.Reference.IsType<FRCObjectReference>())
			{
				return GetTypeHash(Modification.Reference.Get<FRCObjectReference>());
			}
			else
			{
				return GetTypeHash(Modification.Reference.Get<FRCCallReference>());
			}
		}

		/** Reference to either the property we're modifying or the function we're calling. */
		TVariant<FRCObjectReference, FRCCallReference> Reference;
		/** Whether this change was triggered with a transaction or not. */
		bool bHasStartedTransaction = false;
		/** Whether this change was triggered since the last tick of OngoingChangeTimer. */
		bool bWasTriggeredSinceLastPass = true;
	};
	
	/** Ongoing change that needs to end its transaction and call PostEditChange in the case of a property modification. */
	TOptional<FOngoingChange> OngoingModification;

	/** Handle to the timer that that ends the ongoing change in regards to PostEditChange and transactions. */
	FTimerHandle OngoingChangeTimer;
	
	/** Handle to the timer that that ends the ongoing change in regards to PostEditChange and transactions. */
	FTSTicker::FDelegateHandle FallbackOngoingChangeTimer;

	/** Delay before we check if a modification is no longer ongoing. */
	static constexpr float SecondsBetweenOngoingChangeCheck = 0.2f;

#endif // WITH_EDITOR

	/** Map of the factories which is responsible for the Remote Control property creation */
	TMap<FName, TSharedPtr<IRemoteControlPropertyFactory>> EntityFactories;

	/** Map of the factories which is responsible for the Remote Control property masking. */
	TMap<TWeakObjectPtr<UScriptStruct>, TSharedPtr<IRemoteControlMaskingFactory>> MaskingFactories;

	/** Holds the set of active masking operations. */
	TSet<TSharedPtr<FRCMaskingOperation>> ActiveMaskingOperations;

	/** Map of the factories which is responsible for resetting the Remote Control property to its default value. */
	TMap<FName, TSharedPtr<IRCDefaultValueFactory>> DefaultValueFactories;

	TSet<TSharedPtr<IPropertyIdHandler>> PropertyIdPropertyHandlers; 

	/** List of functions that can't be called remotely. */
	TSet<TWeakObjectPtr<UFunction>> FunctionDisallowList;

	/** Keep track of the registered console objects for automatic unregistration on module shutdown. */
	TArray<IConsoleObject*> ConsoleObjects;
};

PRAGMA_ENABLE_DEPRECATION_WARNINGS

