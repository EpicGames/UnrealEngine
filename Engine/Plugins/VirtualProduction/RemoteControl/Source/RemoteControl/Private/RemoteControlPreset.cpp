// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPreset.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"
#include "Components/ActorComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Factories/IRCDefaultValueFactory.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"
#include "IRemoteControlModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "Misc/TransactionObjectEvent.h"
#include "RCSignatureRegistry.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlActor.h"
#include "RemoteControlBinding.h"
#include "RemoteControlEntityFactory.h"
#include "RemoteControlExposeRegistry.h"
#include "RemoteControlFieldPath.h"
#include "RemoteControlLogger.h"
#include "RemoteControlObjectVersion.h"
#include "RemoteControlPresetRebindingManager.h"
#include "RemoteControlPropertyIdRegistry.h"
#include "RemoteControlTransactionListenerHelper.h"

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Package.h"
#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "Editor.h"
#include "EngineAnalytics.h"
#include "Engine/Blueprint.h"
#include "RemoteControlSettings.h"
#include "TimerManager.h"
#include "UObject/PackageReload.h"
#endif

URemoteControlPreset::FOnPostLoadRemoteControlPreset URemoteControlPreset::OnPostLoadRemoteControlPreset;
URemoteControlPreset::FOnPostInitPropertiesRemoteControlPreset URemoteControlPreset::OnPostInitPropertiesRemoteControlPreset;

#define LOCTEXT_NAMESPACE "RemoteControlPreset"

static TAutoConsoleVariable<int32> CVarRemoteControlEnablePropertyWatchInEditor(TEXT("RemoteControl.EnablePropertyWatchInEditor"), 0, TEXT("Whether or not to manually compare certain properties to detect property changes while in editor."));
static TAutoConsoleVariable<int32> CVarRemoteControlFramesBetweenPropertyWatch(TEXT("RemoteControl.FramesBetweenPropertyWatch"), 5, TEXT("The number of frames between every property value comparison when manually watching for property changes."));

namespace
{
	FGuid DefaultGroupId = FGuid(0x5DFBC958, 0xF3B311EA, 0x9A3F00EE, 0xFB2CA371);
	FName NAME_DefaultLayoutGroup = FName("All");
	FName NAME_DefaultNewGroup = FName("New Group");
	const FString DefaultObjectPrefix = TEXT("Default__");
	bool bAllowPresetGuidRenewal = true; 

	UClass* FindCommonBase(const TArray<UObject*>& ObjectsToTest)
	{
		TArray<UClass*> Classes;
		Classes.Reserve(ObjectsToTest.Num());
		Algo::TransformIf(ObjectsToTest, Classes, [](const UObject* Object) { return !!Object; }, [](const UObject* Object) { return Object->GetClass(); });
		return UClass::FindCommonBase(Classes);
	}

	/** Create a unique name. */
	FName MakeUniqueName(FName InBase, TFunctionRef<bool(FName)> NamePoolContains, const FString& InSeparatorLeft = TEXT(" ("), const FString& InSeparatorRight = TEXT(")"))
	{
		// Try using the field name itself
		if (!NamePoolContains(InBase))
		{
			return InBase;
		}

		// Then try the field name with a suffix
		for (uint32 Index = 1; Index < 1000; ++Index)
		{
			const FName Candidate = FName(*FString::Printf(TEXT("%s%s%d%s"), *InBase.ToString(), *InSeparatorLeft, Index, *InSeparatorRight));
			if (!NamePoolContains(Candidate))
			{
				return Candidate;
			}
		}

		// Something went wrong if we end up here.
		checkNoEntry();
		return NAME_None;
	}

	FName GenerateExposedFieldLabel(const FString& FieldName, UObject* FieldOwner)
	{
		FName OutputName;
		
		if (ensure(FieldOwner))
		{
			FString ObjectName;
	#if WITH_EDITOR
			if (AActor* Actor = Cast<AActor>(FieldOwner))
			{
				ObjectName = Actor->GetActorLabel();
			}
			else if(UActorComponent* Component = Cast<UActorComponent>(FieldOwner))
			{
				ObjectName = Component->GetOwner()->GetActorLabel();
			}
			else
	#endif
			{
				// Get the class name when dealing with BP libraries and subsystems. 
				ObjectName = FieldOwner->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ? FieldOwner->GetClass()->GetName() : FieldOwner->GetName();
			}

			OutputName = *FString::Printf(TEXT("%s"), *FieldName);
		}
		return OutputName;
	}

	enum class ELaunchConfiguration : uint8
	{
		Editor,
		DashGame,
		Packaged
	};

	ELaunchConfiguration GetLaunchConfiguration()
	{
		ELaunchConfiguration CurrentConfiguration = ELaunchConfiguration::Packaged;

#if WITH_EDITOR
		if (GEditor)
		{
			CurrentConfiguration = ELaunchConfiguration::Editor;
		}
		else 
		{
			CurrentConfiguration = ELaunchConfiguration::DashGame;
		}
#endif

		return CurrentConfiguration;
	}

	template<typename InValueType>
	void RehashMap(TMap<FGuid, InValueType>& InOutMapToRehash, const TMap<FGuid, FGuid>& InGuidMap)
	{
		TMap<FGuid, InValueType> RehashedMap;
		RehashedMap.Reserve(InOutMapToRehash.Num());
	
		for (TPair<FGuid, InValueType>& Entry : InOutMapToRehash)
		{
			RehashedMap.Add(InGuidMap.Contains(Entry.Key) ? InGuidMap[Entry.Key] : Entry.Key, MoveTemp(Entry.Value));
		}

		InOutMapToRehash = MoveTemp(RehashedMap);
	}

	void RehashSet(TSet<FGuid>& InOutSetToRehash, const TMap<FGuid, FGuid>& InGuidMap)
	{
		TSet<FGuid> RehashedSet;
		RehashedSet.Reserve(InOutSetToRehash.Num());
		for (const FGuid& Key : InOutSetToRehash)
		{
			RehashedSet.Add(InGuidMap.Contains(Key) ? InGuidMap[Key] : Key);
		}
		InOutSetToRehash = MoveTemp(RehashedSet);
	}
}

FRemoteControlPresetExposeArgs::FRemoteControlPresetExposeArgs()
	: GroupId(DefaultGroupId)
{
}

FRemoteControlPresetExposeArgs::FRemoteControlPresetExposeArgs(FString InLabel, FGuid InGroupId, bool bEnableEditCondition)
	: Label(MoveTemp(InLabel))
	, GroupId(MoveTemp(InGroupId))
	, bEnableEditCondition(bEnableEditCondition)
{
}

FRCPresetGuidRenewGuard::FRCPresetGuidRenewGuard()
{
	bPreviousValue = bAllowPresetGuidRenewal;
	bAllowPresetGuidRenewal = false;
}

FRCPresetGuidRenewGuard::~FRCPresetGuidRenewGuard()
{
	bAllowPresetGuidRenewal = bPreviousValue;
}

bool FRCPresetGuidRenewGuard::IsAllowingPresetGuidRenewal()
{
	return bAllowPresetGuidRenewal;
}

const TArray<FGuid>& FRemoteControlPresetGroup::GetFields() const
{
	return Fields;
}

TArray<FGuid>& FRemoteControlPresetGroup::AccessFields()
{
	return Fields;
}

FRemoteControlPresetLayout::FRemoteControlPresetLayout(URemoteControlPreset* OwnerPreset)
	: Owner(OwnerPreset)
{
	Groups.Emplace(NAME_DefaultLayoutGroup, DefaultGroupId);
}

FRemoteControlPresetGroup& FRemoteControlPresetLayout::GetDefaultGroup()
{
	if (FRemoteControlPresetGroup* Group = GetGroup(DefaultGroupId))
	{
		return *Group;
	}
	else
	{
		return CreateGroupInternal(NAME_DefaultLayoutGroup, DefaultGroupId);
	}
}

bool FRemoteControlPresetLayout::IsDefaultGroup(FGuid GroupId) const
{
	return GroupId == DefaultGroupId;
}

FLinearColor FRemoteControlPresetLayout::GetTagColor(FGuid GroupId)
{
	if (IsDefaultGroup(GroupId))
	{
		return FLinearColor::Transparent;
	}

	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		return Group->TagColor;
	}

	return FLinearColor::White;
}

FRemoteControlPresetGroup* FRemoteControlPresetLayout::GetGroup(FGuid GroupId)
{
	return Groups.FindByPredicate([GroupId](const FRemoteControlPresetGroup& Group) { return Group.Id == GroupId; });
}

FRemoteControlPresetGroup* FRemoteControlPresetLayout::GetGroupByName(FName GroupName)
{
	return Groups.FindByPredicate([GroupName](const FRemoteControlPresetGroup& Group) { return Group.Name == GroupName; });
}

FRemoteControlPresetGroup& FRemoteControlPresetLayout::CreateGroup(FName GroupName, FGuid GroupId)
{
	FRemoteControlPresetGroup& Group = Groups.Emplace_GetRef(GroupName, GroupId);
	Owner->CacheLayoutData();
	OnGroupAddedDelegate.Broadcast(Group);

	FRCTransactionListenerHelper<FRemoteControlPresetGroup>(ERCTransaction::Undo, Owner->GetPresetId(), OnGroupDeleted(), Group);
	FRCTransactionListenerHelper<const FRemoteControlPresetGroup&>(ERCTransaction::Redo, Owner->GetPresetId(), OnGroupAdded(), Group);

	Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

	FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());

	return Group;
}

FRemoteControlPresetGroup& FRemoteControlPresetLayout::CreateGroup(FName GroupName)
{
	if (GroupName == NAME_None)
	{
		GroupName = NAME_DefaultNewGroup;
	}

	return CreateGroupInternal(MakeUniqueName(GroupName, [this](FName Candidate) { return !!GetGroupByName(Candidate); }), FGuid::NewGuid());
}

FRemoteControlPresetGroup* FRemoteControlPresetLayout::FindGroupFromField(FGuid FieldId)
{
	if (FRCCachedFieldData* CachedData = Owner->FieldCache.Find(FieldId))
	{
		return GetGroup(CachedData->LayoutGroupId);
	}

	return nullptr;
}

void FRemoteControlPresetLayout::SwapGroups(FGuid OriginGroupId, FGuid TargetGroupId)
{
	FRemoteControlPresetGroup* OriginGroup = GetGroup(OriginGroupId);
	FRemoteControlPresetGroup* TargetGroup = GetGroup(TargetGroupId);

	if (OriginGroup && TargetGroup)
	{
		int32 OriginGroupIndex = Groups.IndexOfByKey(*OriginGroup);
		int32 TargetGroupIndex = Groups.IndexOfByKey(*TargetGroup);

		if (TargetGroupIndex > OriginGroupIndex)
		{
			TargetGroupIndex += 1;
		}
		else
		{
			OriginGroupIndex += 1;
		}

		TArray<FGuid> GroupIdsBeforeTransaction;
		GroupIdsBeforeTransaction.Reserve(Groups.Num());
		Algo::Transform(Groups, GroupIdsBeforeTransaction, &FRemoteControlPresetGroup::Id);
		FRCTransactionListenerHelper<const TArray<FGuid>&>(ERCTransaction::Undo, Owner->GetPresetId(), OnGroupOrderChanged(), GroupIdsBeforeTransaction);

		Groups.Insert(FRemoteControlPresetGroup{ *OriginGroup }, TargetGroupIndex);
		Groups.Swap(TargetGroupIndex, OriginGroupIndex);
		Groups.RemoveAt(OriginGroupIndex);

		TArray<FGuid> GroupIds;
		GroupIds.Reserve(Groups.Num());
		Algo::Transform(Groups, GroupIds, &FRemoteControlPresetGroup::Id);
		OnGroupOrderChangedDelegate.Broadcast(GroupIds);

		FRCTransactionListenerHelper<const TArray<FGuid>&>(ERCTransaction::Redo, Owner->GetPresetId(), OnGroupOrderChanged(), GroupIds);

		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

void FRemoteControlPresetLayout::SwapFields(const FFieldSwapArgs& InFieldSwapArgs)
{
	FRemoteControlPresetGroup* DragOriginGroup = GetGroup(InFieldSwapArgs.OriginGroupId);
	FRemoteControlPresetGroup* DragTargetGroup = GetGroup(InFieldSwapArgs.TargetGroupId);

	if (!DragOriginGroup || !DragTargetGroup || InFieldSwapArgs.DraggedFieldsIds.Num() < 1)
	{
		return;
	}

	int32 DragOriginFieldIndex = DragOriginGroup->AccessFields().IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[0]);
	int32 DragTargetInitialFieldIndex = DragTargetGroup->AccessFields().IndexOfByKey(InFieldSwapArgs.TargetFieldId);

	if (DragOriginFieldIndex == INDEX_NONE)
	{
		return;
	}

	if (InFieldSwapArgs.OriginGroupId == InFieldSwapArgs.TargetGroupId && DragTargetInitialFieldIndex != INDEX_NONE)
	{
		// Here we don't want to trigger add/delete delegates since the fields just get moved around.
		TArray<FGuid>& Fields = DragTargetGroup->AccessFields();

		FRCTransactionListenerHelper<const FGuid&, const TArray<FGuid>&>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldOrderChanged(), InFieldSwapArgs.TargetGroupId, Fields);

		int32 TargetFieldOffset = 0;
		int32 DragOriginSelectedFieldIndex = Fields.IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[0]);
		int32 DragTargetFieldIndex = Fields.IndexOfByKey(InFieldSwapArgs.TargetFieldId);
		const bool bInitialTargetIsGreaterThanOrigin = DragTargetFieldIndex >= DragOriginSelectedFieldIndex;
		for (FGuid SelectedField : InFieldSwapArgs.DraggedFieldsIds)
		{
			DragOriginSelectedFieldIndex = Fields.IndexOfByKey(SelectedField);
			DragTargetFieldIndex = Fields.IndexOfByKey(InFieldSwapArgs.TargetFieldId);
			if (DragTargetFieldIndex == DragOriginSelectedFieldIndex)
			{
				continue;
			}
			if (bInitialTargetIsGreaterThanOrigin)
			{
				TargetFieldOffset++;
				DragTargetFieldIndex += TargetFieldOffset;
			}
			if (DragTargetFieldIndex < DragOriginSelectedFieldIndex)
			{
				DragOriginSelectedFieldIndex += 1;
			}
			Fields.Insert(SelectedField, DragTargetFieldIndex);
			Fields.Swap(DragTargetFieldIndex, DragOriginSelectedFieldIndex);
			Fields.RemoveAt(DragOriginSelectedFieldIndex);
		}
	
		Owner->CacheLayoutData();
		OnFieldOrderChangedDelegate.Broadcast(InFieldSwapArgs.TargetGroupId, Fields);
		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<const FGuid&, const TArray<FGuid>&>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldOrderChanged(), InFieldSwapArgs.TargetGroupId, Fields);

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
	else
	{
		// We do it reverse so that the order is maintained
		for (int32 Index = InFieldSwapArgs.DraggedFieldsIds.Num() - 1; Index >= 0; --Index)
		{
			TArray<FGuid>& Fields = DragOriginGroup->AccessFields();
			DragOriginFieldIndex = Fields.IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[Index]);

			DragOriginGroup->AccessFields().RemoveAt(DragOriginFieldIndex);

			DragTargetInitialFieldIndex = DragTargetInitialFieldIndex == INDEX_NONE ? 0 : DragTargetInitialFieldIndex;
			DragTargetGroup->AccessFields().Insert(InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);

			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldDeleted(), InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);
			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldAdded(), InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);

			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldDeleted(), InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);
			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldAdded(), InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);

			Owner->CacheLayoutData();
			OnFieldDeletedDelegate.Broadcast(InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);
			OnFieldAddedDelegate.Broadcast(InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);
			Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

			FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
			FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		}
	}
}

void FRemoteControlPresetLayout::SwapFieldsDefaultGroup(const FFieldSwapArgs& InFieldSwapArgs, const FGuid InFieldRealGroup, TArray<FGuid> InEntities)
{
	FRemoteControlPresetGroup* DragOriginGroup = GetGroup(InFieldRealGroup);
	FRemoteControlPresetGroup* DragTargetGroup = GetGroup(InFieldSwapArgs.TargetGroupId);

	if (!DragOriginGroup || !DragTargetGroup || InFieldSwapArgs.DraggedFieldsIds.Num() < 1)
	{
		return;
	}

	int32 DragOriginFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[0]);
	int32 DragTargetInitialFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.TargetFieldId);

	if (DragOriginFieldIndex == INDEX_NONE)
	{
		return;
	}

	if (DragTargetInitialFieldIndex != INDEX_NONE)
	{
		// Here we don't want to trigger add/delete delegates since the fields just get moved around.
		TArray<FGuid>& Fields = InEntities;

		FRCTransactionListenerHelper<const FGuid&, const TArray<FGuid>&>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldOrderChanged(), InFieldSwapArgs.TargetGroupId, Fields);

		int32 TargetFieldOffset = 0;
		int32 DragOriginSelectedFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[0]);
		int32 DragTargetFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.TargetFieldId);
		const bool bInitialTargetIsGreaterThanOrigin = DragTargetFieldIndex >= DragOriginSelectedFieldIndex;
		for (FGuid SelectedField : InFieldSwapArgs.DraggedFieldsIds)
		{
			DragOriginSelectedFieldIndex = InEntities.IndexOfByKey(SelectedField);
			DragTargetFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.TargetFieldId);
			if (DragTargetFieldIndex == DragOriginSelectedFieldIndex)
			{
				continue;
			}
			if (bInitialTargetIsGreaterThanOrigin)
			{
				TargetFieldOffset++;
				DragTargetFieldIndex += TargetFieldOffset;
			}
			if (DragTargetFieldIndex < DragOriginSelectedFieldIndex)
			{
				DragOriginSelectedFieldIndex += 1;
			}
			Fields.Insert(SelectedField, DragTargetFieldIndex);
			Fields.Swap(DragTargetFieldIndex, DragOriginSelectedFieldIndex);
			Fields.RemoveAt(DragOriginSelectedFieldIndex);
		}
	
		Owner->CacheLayoutData();
		OnFieldOrderChangedDelegate.Broadcast(GetDefaultGroup().Id, Fields);
		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<const FGuid&, const TArray<FGuid>&>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldOrderChanged(), InFieldSwapArgs.TargetGroupId, Fields);

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
	else if (!IsDefaultGroup(DragTargetGroup->Id))
	{
		// We do it reverse so that the order is maintained
		for (int32 Index = InFieldSwapArgs.DraggedFieldsIds.Num() - 1; Index >= 0; --Index)
		{
			if (DragTargetGroup->AccessFields().Contains(InFieldSwapArgs.DraggedFieldsIds[Index]))
			{
				// Skip if the Target group already contains the Field.
				continue;
			}
			DragOriginFieldIndex = InEntities.IndexOfByKey(InFieldSwapArgs.DraggedFieldsIds[Index]);

			DragOriginGroup->AccessFields().RemoveSwap(InFieldSwapArgs.DraggedFieldsIds[Index], EAllowShrinking::Yes);

			DragTargetInitialFieldIndex = DragTargetInitialFieldIndex == INDEX_NONE ? 0 : DragTargetInitialFieldIndex;
			DragTargetGroup->AccessFields().Insert(InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);

			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldDeleted(), InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);
			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldAdded(), InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);

			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldDeleted(), InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);
			FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldAdded(), InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);

			Owner->CacheLayoutData();
			OnFieldDeletedDelegate.Broadcast(InFieldSwapArgs.OriginGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragOriginFieldIndex);
			OnFieldAddedDelegate.Broadcast(InFieldSwapArgs.TargetGroupId, InFieldSwapArgs.DraggedFieldsIds[Index], DragTargetInitialFieldIndex);
			Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

			FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
			FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		}
	}
}

void FRemoteControlPresetLayout::DeleteGroup(FGuid GroupId)
{

	int32 Index = Groups.IndexOfByPredicate([GroupId](const FRemoteControlPresetGroup& Group) { return Group.Id == GroupId; });
	if (Index != INDEX_NONE)
	{
		FRemoteControlPresetGroup DeletedGroup = MoveTemp(Groups[Index]);
		Groups.RemoveAt(Index);

		for (const FGuid& FieldId : DeletedGroup.GetFields())
		{
			Owner->Unexpose(FieldId);
		}
		
		Owner->CacheLayoutData();

		FRCTransactionListenerHelper<const FRemoteControlPresetGroup&>(ERCTransaction::Undo, Owner->GetPresetId(), OnGroupAdded(), CopyTemp(DeletedGroup));
		FRCTransactionListenerHelper<FRemoteControlPresetGroup>(ERCTransaction::Redo, Owner->GetPresetId(), OnGroupDeleted(), CopyTemp(DeletedGroup));

		OnGroupDeletedDelegate.Broadcast(MoveTemp(DeletedGroup));
		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

void FRemoteControlPresetLayout::RenameGroup(FGuid GroupId, FName NewGroupName)
{
	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		FRCTransactionListenerHelper<const FGuid&, FName>(ERCTransaction::Undo, Owner->GetPresetId(), OnGroupRenamed(), GroupId, Group->Name);
		FRCTransactionListenerHelper<const FGuid&, FName>(ERCTransaction::Redo, Owner->GetPresetId(), OnGroupRenamed(), GroupId, NewGroupName);

		Group->Name = NewGroupName;
		OnGroupRenamedDelegate.Broadcast(GroupId, NewGroupName);
		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

const TArray<FRemoteControlPresetGroup>& FRemoteControlPresetLayout::GetGroups() const
{
	return Groups;
}

TArray<FRemoteControlPresetGroup>& FRemoteControlPresetLayout::AccessGroups()
{
	return Groups;
}

void FRemoteControlPresetLayout::AddField(FGuid GroupId, FGuid FieldId)
{
	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		Group->AccessFields().Add(FieldId);
		OnFieldAddedDelegate.Broadcast(GroupId, FieldId, Group->AccessFields().Num() - 1);

		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldDeleted(), GroupId, FieldId, Group->AccessFields().Num() - 1);
		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldAdded(), GroupId, FieldId, Group->AccessFields().Num() - 1);

		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

void FRemoteControlPresetLayout::InsertFieldAt(FGuid GroupId, FGuid FieldId, int32 Index)
{
	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		Group->AccessFields().Insert(FieldId, Index);
		OnFieldAddedDelegate.Broadcast(GroupId, FieldId, Index);

		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldDeleted(), GroupId, FieldId, Index);
		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldAdded(), GroupId, FieldId, Index);

		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

void FRemoteControlPresetLayout::RemoveField(FGuid GroupId, FGuid FieldId)
{
	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		int32 Index = Group->AccessFields().IndexOfByKey(FieldId);
		RemoveFieldAt(GroupId, Index);
	}
}

void FRemoteControlPresetLayout::RemoveFieldAt(FGuid GroupId, int32 Index)
{
	if (FRemoteControlPresetGroup* Group = GetGroup(GroupId))
	{
		
		TArray<FGuid>& Fields = Group->AccessFields();
		if(!Fields.IsValidIndex(Index))
		{
			return;
		}

		FGuid FieldId = Fields[Index];
		Fields.RemoveAt(Index);

		OnFieldDeletedDelegate.Broadcast(GroupId, FieldId, Index);

		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Undo, Owner->GetPresetId(), OnFieldAdded(), GroupId, FieldId, Index);
		FRCTransactionListenerHelper<const FGuid&, const FGuid&, int32>(ERCTransaction::Redo, Owner->GetPresetId(), OnFieldDeleted(), GroupId, FieldId, Index);

		Owner->OnPresetLayoutModified().Broadcast(Owner.Get());

		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Undo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
		FRCTransactionListenerHelper<URemoteControlPreset*>(ERCTransaction::Redo, Owner->GetPresetId(), Owner->OnPresetLayoutModified(), Owner.Get());
	}
}

URemoteControlPreset* FRemoteControlPresetLayout::GetOwner()
{
	return Owner.Get();
}

void FRemoteControlPresetLayout::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	for (FRemoteControlPresetGroup& Group : Groups)
	{
		for (FGuid& FieldId : Group.AccessFields())
		{
			if (InEntityIdMap.Contains(FieldId))
			{
				FieldId = InEntityIdMap[FieldId];
			}
		}
	}
}

FRemoteControlPresetGroup& FRemoteControlPresetLayout::CreateGroupInternal(FName GroupName, FGuid GroupId)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return CreateGroup(GroupName, GroupId);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

URemoteControlPreset::URemoteControlPreset()
	: Layout(FRemoteControlPresetLayout{ this })
	, PresetId(FGuid::NewGuid())
	, RebindingManager(MakePimpl<FRemoteControlPresetRebindingManager>())
	, NewControllerCategory(FString())
{
	Registry = CreateDefaultSubobject<URemoteControlExposeRegistry>(FName("ExposeRegistry"));
	Signatures = CreateDefaultSubobject<URCSignatureRegistry>(TEXT("Signatures"));

	PropertyIdRegistry = CreateDefaultSubobject<URemoteControlPropertyIdRegistry>(FName("PropertyIdRegistry"));
	PropertyIdRegistry->Initialize();
}

void URemoteControlPreset::PostInitProperties()
{
	Super::PostInitProperties();
	if (!HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		if (IsInAsyncLoadingThread())
		{
			bPendingPostInitNotify = true;
		}
		else
		{
			RegisterDelegates();
			OnPostInitPropertiesRemoteControlPreset.Broadcast(this);
		}
	}
}

#if WITH_EDITOR
void URemoteControlPreset::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(URemoteControlPreset, Metadata))
	{
		OnMetadataModified().Broadcast(this);
	}
}
#endif /*WITH_EDITOR*/

void URemoteControlPreset::PostLoad()
{
	Super::PostLoad();

	if (bPendingPostInitNotify)
	{
		OnPostInitPropertiesRemoteControlPreset.Broadcast(this);
		bPendingPostInitNotify = false;
	}
	OnPostLoadRemoteControlPreset.Broadcast(this);

	RegisterDelegates();

	CacheFieldLayoutData();

	FixAndCacheControllersLabels();

	InitializeEntitiesMetadata();

	RegisterEntityDelegates();

	CreatePropertyWatchers();

	PostLoadProperties();

	RemoveUnusedBindings();

	PropertyIdRegistry->Initialize();

	UpdateEntityUsage();
}

void URemoteControlPreset::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
	if (!bDuplicateForPIE && bAllowPresetGuidRenewal)
	{
		PresetId = FGuid::NewGuid();

		if (IsEmbeddedPreset())
		{
			RenewEntityIds();
			RenewControllerIds();
		}
	}
}

void URemoteControlPreset::BeginDestroy()
{
	UnregisterDelegates();
	Super::BeginDestroy();
}

FName URemoteControlPreset::GetPresetName() const
{
	if (!IsEmbeddedPreset())
	{
		return GetFName();
	}

	return GetPackage()->GetFName();
}

TWeakPtr<FRemoteControlActor> URemoteControlPreset::ExposeActor(AActor* Actor, FRemoteControlPresetExposeArgs Args)
{
	check(Actor);

#if WITH_EDITOR
	const FName UniqueLabel = Registry->GenerateUniqueLabel(Args.Label.IsEmpty() ? *Actor->GetActorLabel() : *Args.Label);
#else
	const FName UniqueLabel = Registry->GenerateUniqueLabel(Args.Label.IsEmpty() ? *Actor->GetName() : *Args.Label);
#endif

	FText LogText = FText::Format(LOCTEXT("ExposedActor", "Exposed actor ({0})"), FText::FromString(Actor->GetName()));
    FRemoteControlLogger::Get().Log(TEXT("RemoteControlPreset"), [Text = MoveTemp(LogText)](){ return Text; });

	FRemoteControlActor RCActor{this, UniqueLabel, { FindOrAddBinding(Actor)} };
	return StaticCastSharedPtr<FRemoteControlActor>(Expose(MoveTemp(RCActor), FRemoteControlActor::StaticStruct(), Args.GroupId));
}

TSharedPtr<FRemoteControlProperty> URemoteControlPreset::FindExposedProperty(UObject* InOuterObject, const FRCFieldPathInfo& InFieldPath) const
{
	if (!InOuterObject || !Registry)
	{
		return nullptr;
	}

	const FString FieldPathStr = InFieldPath.ToString();

	const TArray<UObject*> OuterObjects = { InOuterObject };

	for (TSharedPtr<FRemoteControlProperty> Property : Registry->GetExposedEntities<FRemoteControlProperty>())
	{
		if (Property->FieldPathInfo.ToString() == FieldPathStr && Property->ContainsBoundObjects(OuterObjects))
		{
			return Property;
		}
	}

	return nullptr;
}

FName URemoteControlPreset::GenerateUniqueLabel(const FName InDesiredName) const
{
	return Registry->GenerateUniqueLabel(InDesiredName);
}

URCVirtualPropertyBase* URemoteControlPreset::GetController(const FName InPropertyName) const
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	return ControllerContainer->GetVirtualProperty(InPropertyName);
}

URCVirtualPropertyBase* URemoteControlPreset::GetController(const FGuid& InId) const
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	return ControllerContainer->GetVirtualProperty(InId);
}

TArray<URCVirtualPropertyBase*> URemoteControlPreset::GetControllers() const
{
	return ControllerContainer->VirtualProperties.Array();
}

URCVirtualPropertyBase* URemoteControlPreset::GetControllerByDisplayName(const FName InDisplayName) const
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	return ControllerContainer->GetVirtualPropertyByDisplayName(InDisplayName);
}

URCVirtualPropertyBase* URemoteControlPreset::GetControllerByFieldId(const FName InFieldId) const
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	return ControllerContainer->GetVirtualPropertyByFieldId(InFieldId);
}

URCVirtualPropertyBase* URemoteControlPreset::GetControllerByFieldId(const FName InFieldId,	const EPropertyBagPropertyType InType) const
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	return ControllerContainer->GetVirtualPropertyByFieldId(InFieldId, InType);
}

TArray<URCVirtualPropertyBase*> URemoteControlPreset::GetControllersByFieldId(const FName InFieldId) const
{
	if (!ensure(ControllerContainer))
	{
		return {};
	}

	return ControllerContainer->GetVirtualPropertiesByFieldId(InFieldId);
}

TArray<EPropertyBagPropertyType> URemoteControlPreset::GetControllersTypesByFieldId(const FName InFieldId) const
{	
	TArray<EPropertyBagPropertyType> ValueTypes;
	
	for (const URCVirtualPropertyBase* Controller : GetControllersByFieldId(InFieldId))
	{
		if (Controller)
		{
			ValueTypes.Add(Controller->GetValueType());
		}
	}

	return ValueTypes;
}

URCVirtualPropertyInContainer* URemoteControlPreset::AddController(TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject /*= nullptr*/, const FName InPropertyName /*= NAME_None*/)
{
	if (!ensure(ControllerContainer))
	{
		return nullptr;
	}

	// New Property Name
	FName NewPropertyName = InPropertyName;
	if (NewPropertyName.IsNone())
	{
		// Property name must always be set. Fallback to generic numeric naming
		// Ensuring uniqueness is dealt in URCVirtualPropertyContainerBase::AddProperty
		NewPropertyName = TEXT("0001");
	}

#if WITH_EDITOR
	ControllerContainer->Modify();
#endif

	URCVirtualPropertyInContainer* NewController = ControllerContainer->AddProperty(NewPropertyName, InPropertyClass, InValueType, InValueTypeObject);
	if (NewController)
	{
		OnControllerAdded().Broadcast(this, NewPropertyName, NewController->Id);

		FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnControllerRemoved(), this, NewController->Id);
		FRCTransactionListenerHelper<URemoteControlPreset*, const FName, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnControllerAdded(), this, NewPropertyName, NewController->Id);

		InitializeEntityMetadata(NewController);
	}
	
	return NewController;
}

bool URemoteControlPreset::HasController(const URCVirtualPropertyBase* InController) const
{
	if (!ensure(ControllerContainer))
	{
		return false;
	}

	return ControllerContainer->HasVirtualProperty(InController);
}

bool URemoteControlPreset::RemoveController(const FName& InPropertyName)
{
	if (ensure(ControllerContainer))
	{
#if WITH_EDITOR
		ControllerContainer->Modify();
#endif

		if (const URCVirtualPropertyBase* ControllerToDelete = ControllerContainer->GetVirtualProperty(InPropertyName))
		{
			OnControllerRemoved().Broadcast(this, ControllerToDelete->Id);
			ControllerContainer->RemoveProperty(ControllerToDelete->GetPropertyName());

			FRCTransactionListenerHelper<URemoteControlPreset*, const FName, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnControllerAdded(), this, ControllerToDelete->DisplayName, ControllerToDelete->Id);
			FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnControllerRemoved(), this, ControllerToDelete->Id);

			return true;
		}
	}

	return false;
}

URCVirtualPropertyInContainer* URemoteControlPreset::DuplicateController(URCVirtualPropertyInContainer* InVirtualProperty)
{
	if (ensure(ControllerContainer))
	{
		URCVirtualPropertyInContainer* DuplicatedController = ControllerContainer->DuplicateVirtualProperty(InVirtualProperty);
		return DuplicatedController;
	}

	return nullptr;
}

void URemoteControlPreset::ResetControllers()
{
	if (ensure(ControllerContainer))
	{
#if WITH_EDITOR
		ControllerContainer->Modify();
#endif

		// Broadcasting each removal.
		for (const TObjectPtr<URCVirtualPropertyBase>& Controller : ControllerContainer->VirtualProperties)
		{
			OnControllerRemoved().Broadcast(this, Controller->Id);

			FRCTransactionListenerHelper<URemoteControlPreset*, const FName, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnControllerAdded(), this, Controller->DisplayName, Controller->Id);
			FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnControllerRemoved(), this, Controller->Id);
		}
		ControllerContainer->Reset();
	}
}

int32 URemoteControlPreset::GetNumControllers() const
{
	if (ensure(ControllerContainer))
	{
		return ControllerContainer->GetNumVirtualProperties();
	}

	return 0;
}

TSharedPtr<FStructOnScope> URemoteControlPreset::GetControllerContainerStructOnScope()
{
	if (ensure(ControllerContainer))
	{
		return ControllerContainer->CreateStructOnScope();
	}

	return nullptr;
}

void URemoteControlPreset::SetControllerContainer(URCVirtualPropertyContainerBase* InControllerContainer)
{
	ControllerContainer = InControllerContainer;
}

#if WITH_EDITOR

void URemoteControlPreset::OnNotifyPreChangeVirtualProperty(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ensure(ControllerContainer))
	{
		ControllerContainer->OnPreChangePropertyValue(PropertyChangedEvent);
	}
}

void URemoteControlPreset::OnModifyController(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ensure(ControllerContainer))
	{
		URCVirtualPropertyBase* ModifiedController = ControllerContainer->GetVirtualProperty(PropertyChangedEvent.MemberProperty->GetFName());

		if (!ModifiedController)
		{
			ModifiedController = ControllerContainer->GetVirtualProperty(PropertyChangedEvent.Property->GetFName());
		}

		if (ModifiedController)
		{
			ControllerContainer->OnModifyPropertyValue(PropertyChangedEvent);
			OnControllerModified().Broadcast(this, {ModifiedController->Id});

			FRCTransactionListenerHelper<URemoteControlPreset*, const TSet<FGuid>&>(ERCTransaction::Undo, GetPresetId(), OnControllerModified(), this, {ModifiedController->Id});
			FRCTransactionListenerHelper<URemoteControlPreset*, const TSet<FGuid>&>(ERCTransaction::Redo, GetPresetId(), OnControllerModified(), this, {ModifiedController->Id});
		}
	}	
} 
#endif

FOnVirtualPropertyContainerModified& URemoteControlPreset::OnVirtualPropertyContainerModified() const
{
	check(ControllerContainer);

	return ControllerContainer->OnVirtualPropertyContainerModified();
}

TWeakPtr<FRemoteControlProperty> URemoteControlPreset::ExposeProperty(UObject* Object, FRCFieldPathInfo FieldPath, FRemoteControlPresetExposeArgs Args)
{
	if (!Object)
	{
		return nullptr;
	}
	
	if (!IRemoteControlModule::Get().CanBeAccessedRemotely(Object))
    {
    	UE_LOGF(LogRemoteControl, Warning, "Object %ls cannot be accessed remotely. Check project settings.", *Object->GetName());
		return nullptr;
    }

	TSharedPtr<FRemoteControlProperty> RCPropertyPtr;
	const TMap<FName, TSharedPtr<IRemoteControlPropertyFactory>>& PropertyFactories = IRemoteControlModule::Get().GetEntityFactories();
	for (const TPair<FName, TSharedPtr<IRemoteControlPropertyFactory>>& EntityFactoryPair : PropertyFactories)
	{
		if (EntityFactoryPair.Value->SupportExposedClass(Object->GetClass()))
		{
			RCPropertyPtr = EntityFactoryPair.Value->CreateRemoteControlProperty(this, Object, FieldPath, Args);
			break;
		}
	}

	// Create default one
	if (!RCPropertyPtr)
	{
		if (!FieldPath.Resolve(Object))
		{
			return nullptr;
		}

		const FName FieldName = GetEntityName(*Args.Label, Object, FieldPath);

		URemoteControlBinding* Binding = FindOrAddBinding(Object);
		if (!Binding->Resolve())
		{
			// The binding exists, and the object exists at this path, but the binding can't resolve to it.
			// Due to a bug (UE-187216), this can happen if:
			// 1. The object was created, but not saved
			// 2. The property was bound
			// 3. The object was destroyed
			// 4. Another object was created at the same path
			// This can all happen without FSoftObjectPath::CurrentTag being invalidated (usually only done when assets load), meaning the binding's
			// pointer cache won't update to the new object at that path.
			// But we know the new object here, so just replace the bound object and use it going forward.
			Binding->SetBoundObject(Object);
		}

		FRemoteControlProperty RCProperty{ this, Registry->GenerateUniqueLabel(FieldName), MoveTemp(FieldPath), { Binding } };

		RCPropertyPtr = StaticCastSharedPtr<FRemoteControlProperty>(Expose(MoveTemp(RCProperty), FRemoteControlProperty::StaticStruct(), Args.GroupId));
	}

	if (!RCPropertyPtr)
	{
		return nullptr;
	}

	if (Args.bEnableEditCondition)
	{
		RCPropertyPtr->EnableEditCondition();
	}

	if (PropertyShouldBeWatched(RCPropertyPtr))
	{
		CreatePropertyWatcher(RCPropertyPtr);
	}

	FText LogText = FText::Format(LOCTEXT("ExposedProperty", "Exposed property ({0}) on object {1}"), FText::FromString(RCPropertyPtr->FieldPathInfo.ToString()), FText::FromString(Object->GetPathName()));
	FRemoteControlLogger::Get().Log(TEXT("RemoteControlPreset"), [Text = MoveTemp(LogText)](){ return Text; });

	return RCPropertyPtr;
}

TWeakPtr<FRemoteControlFunction> URemoteControlPreset::ExposeFunction(UObject* Object, UFunction* Function, FRemoteControlPresetExposeArgs Args)
{
	if (!Object || !Function || !Object->GetClass() || !Object->GetClass()->FindFunctionByName(Function->GetFName()))
	{
		return nullptr;
	}

	if (!IRemoteControlModule::Get().CanBeAccessedRemotely(Object))
	{
		UE_LOGF(LogRemoteControl, Warning, "Object %ls cannot be accessed remotely. Check project settings.", *Object->GetName());
		return nullptr;
	}

	FName DesiredName = *Args.Label;

	if (DesiredName == NAME_None)
	{
		FString FunctionName;
#if WITH_EDITOR
		FunctionName = Function->GetDisplayNameText().ToString(); 
#else
		FunctionName = Function->GetName();
#endif
		
		DesiredName = GenerateExposedFieldLabel(FunctionName, Object);
	}

	FRemoteControlFunction RCFunction{ this, Registry->GenerateUniqueLabel(DesiredName), Function->GetName(), Function, { FindOrAddBinding(Object) } };
	TSharedPtr<FRemoteControlFunction> RCFunctionPtr = StaticCastSharedPtr<FRemoteControlFunction>(Expose(MoveTemp(RCFunction), FRemoteControlFunction::StaticStruct(), Args.GroupId));

	RegisterOnCompileEvent(RCFunctionPtr);

	FText LogText = FText::Format(LOCTEXT("ExposedFunction", "Exposed function ({0}) on object {1}"), FText::FromString(Function->GetPathName()), FText::FromString(Object->GetPathName()));
	FRemoteControlLogger::Get().Log(TEXT("RemoteControlPreset"), [Text = MoveTemp(LogText)](){ return Text; });
	
	return RCFunctionPtr;
}

TSharedPtr<FRemoteControlEntity> URemoteControlPreset::Expose(FRemoteControlEntity&& Entity, UScriptStruct* EntityType, const FGuid& GroupId)
{
	Registry->Modify();
	PropertyIdRegistry->Modify();

#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		check(EntityType);
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ExposedEntityType"), EntityType->GetName()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("RemoteControl.EntityExposed"), EventAttributes);	
	}
#endif
	
	TSharedPtr<FRemoteControlEntity> RCEntity = Registry->AddExposedEntity(MoveTemp(Entity), EntityType);
	InitializeEntityMetadata(RCEntity);

	if (const TSharedPtr<FRemoteControlField> RCField = StaticCastSharedPtr<FRemoteControlField>(RCEntity))
	{
		PropertyIdRegistry->AddIdentifiedField(RCField.ToSharedRef());
	}

	RCEntity->OnEntityModifiedDelegate.BindUObject(this, &URemoteControlPreset::OnEntityModified);
	FRemoteControlPresetGroup* Group = Layout.GetGroup(GroupId);
	if (!Group)
	{
		Group = &Layout.GetDefaultGroup();
	}

	FRCCachedFieldData& CachedData = FieldCache.Add(RCEntity->GetId());
	CachedData.LayoutGroupId = Group->Id;

	Layout.AddField(Group->Id, RCEntity->GetId());
	
	OnEntityExposed().Broadcast(this, RCEntity->GetId());

	FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnEntityUnexposed(), this, RCEntity->GetId());
	FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnEntityExposed(), this, RCEntity->GetId());

	return RCEntity;
}

void URemoteControlPreset::PerformChainReaction(const FRemoteControlPropertyIdArgs& InArgs) const
{
	if (PropertyIdRegistry->IsEmpty())
	{
		return;
	}
	PropertyIdRegistry->PerformChainReaction(InArgs);
}

void URemoteControlPreset::UpdateIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify) const
{
	if (PropertyIdRegistry->IsEmpty())
	{
		return;
	}
	PropertyIdRegistry->UpdateIdentifiedField(InFieldToIdentify);
}

URemoteControlBinding* URemoteControlPreset::FindOrAddBinding(const TSoftObjectPtr<UObject>& Object)
{
	if (!ensureAlways(Object.ToString().Len()))
	{
		return nullptr;
	}
	
	for (URemoteControlBinding* Binding : Bindings)
	{
		if (Binding && Binding->IsBound(Object))
		{
			return Binding;
		}
	}

	URemoteControlBinding* NewBinding = nullptr;

	if (UObject* ResolvedObject = Object.Get())
	{
		if (ResolvedObject->GetTypedOuter<ULevel>())
        {
        	NewBinding = NewObject<URemoteControlLevelDependantBinding>(this, NAME_None, RF_Transactional);	
        }
        else
        {
        	NewBinding = NewObject<URemoteControlLevelIndependantBinding>(this, NAME_None, RF_Transactional);
        }

		NewBinding->Modify();
		NewBinding->SetBoundObject(Object);
	}
	else
	{
		// Object is not currently loaded, we have to parse the path manually to find the level path.
		FString Path = Object.ToString();
		static const FString PersistentLevelText = TEXT(":PersistentLevel.");
		int32 PersistentLevelIndex = Path.Find(PersistentLevelText);
		if (PersistentLevelIndex != INDEX_NONE)
		{
			TSoftObjectPtr<ULevel> Level = TSoftObjectPtr<ULevel>{ FSoftObjectPath{ Path.Left(PersistentLevelIndex + PersistentLevelText.Len() - 1) } };
			URemoteControlLevelDependantBinding* LevelDependantBinding = NewObject<URemoteControlLevelDependantBinding>(this, NAME_None, RF_Transactional);
			LevelDependantBinding->SetBoundObject(Level, Object);
			NewBinding = LevelDependantBinding;
		}
		else
		{
			NewBinding = NewObject<URemoteControlLevelIndependantBinding>(this, NAME_None, RF_Transactional);
			NewBinding->Modify();
			NewBinding->SetBoundObject(Object);
		}
		
	}

	if (NewBinding)
	{
		Bindings.Add(NewBinding);
	}
	
	return NewBinding;
}

URemoteControlBinding* URemoteControlPreset::FindMatchingBinding(const URemoteControlBinding* InBinding, UObject* InObject)
{
	for (URemoteControlBinding* Binding : Bindings)
	{
		URemoteControlLevelDependantBinding* LevelDependantBindingIt = Cast<URemoteControlLevelDependantBinding>(Binding);
		const URemoteControlLevelDependantBinding* InLevelDependingBinding = Cast<URemoteControlLevelDependantBinding>(InBinding);

		if (!Binding 
			|| InBinding == Binding
			|| Binding->Resolve() != InObject)
		{
			continue;
		}

		if (LevelDependantBindingIt && InLevelDependingBinding)
		{
			if (LevelDependantBindingIt->BindingContext != InLevelDependingBinding->BindingContext)
			{
				continue;
			}

			TSoftObjectPtr<ULevel> CurrentWorldLevel = LevelDependantBindingIt->SubLevelSelectionMapByPath.FindRef(InObject->GetWorld());

			bool bSameBoundObjectMap = true;
			// Check if the binding it has the same bound object map except for the current level.
			for (const TPair<FSoftObjectPath, TSoftObjectPtr<UObject>>& Pair : InLevelDependingBinding->BoundObjectMapByPath)
			{
				if (Pair.Key != CurrentWorldLevel.ToSoftObjectPath())
				{
					TSoftObjectPtr<UObject>* BoundObject = LevelDependantBindingIt->BoundObjectMapByPath.Find(Pair.Key);
					if (BoundObject)
					{
						if (*BoundObject != Pair.Value)
						{
							bSameBoundObjectMap = false;
							break;
						}
					}
					else
					{
						bSameBoundObjectMap = false;
						break;
					}
				}
			}

			if (bSameBoundObjectMap)
			{
				return Binding;
			}
		}
	}

	return nullptr;
}

void URemoteControlPreset::OnEntityModified(const FGuid& EntityId)
{
	PerFrameUpdatedEntities.Add(EntityId);
	PerFrameModifiedProperties.Add(EntityId);
}

void URemoteControlPreset::InitializeEntitiesMetadata()
{
	for (const TSharedPtr<FRemoteControlEntity>& Entity : Registry->GetExposedEntities())
	{
		InitializeEntityMetadata(Entity);
	}

	for (URCVirtualPropertyBase* Controller : GetControllers())
	{
		InitializeEntityMetadata(Controller);
	}
}

void URemoteControlPreset::InitializeEntityMetadata(const TSharedPtr<FRemoteControlEntity>& Entity)
{
	if (!Entity)
	{
		return;
	}
    	
    const TMap<FName, FEntityMetadataInitializer>& Initializers = IRemoteControlModule::Get().GetDefaultMetadataInitializers();
    for (const TPair<FName, FEntityMetadataInitializer>& Entry : Initializers)
    {
    	if (Entry.Value.IsBound())
    	{
    		// Don't reset the metadata entry if already present.
    		if (!Entity->UserMetadata.Contains(Entry.Key))
    		{
    			Entity->UserMetadata.Add(Entry.Key, Entry.Value.Execute(this, Entity->GetId()));
    		}
    	}
    }
}

void URemoteControlPreset::InitializeEntityMetadata(URCVirtualPropertyBase* Controller)
{
	if (!Controller)
	{
		return;
	}
    	
    const TMap<FName, FEntityMetadataInitializer>& Initializers = IRemoteControlModule::Get().GetDefaultMetadataInitializers();
    for (const TPair<FName, FEntityMetadataInitializer>& Entry : Initializers)
    {
    	if (Entry.Value.IsBound())
    	{
    		// Don't reset the metadata entry if already present.
    		if (!Controller->Metadata.Contains(Entry.Key))
    		{
    			Controller->Metadata.Add(Entry.Key, Entry.Value.Execute(this, Controller->Id));
    		}
    	}
    }
}

void URemoteControlPreset::RegisterEntityDelegates()
{
	for (const TSharedPtr<FRemoteControlEntity>& Entity : Registry->GetExposedEntities())
	{
		Entity->OnEntityModifiedDelegate.BindUObject(this, &URemoteControlPreset::OnEntityModified);

		if (Entity->GetStruct() == FRemoteControlFunction::StaticStruct())
		{
			RegisterOnCompileEvent(StaticCastSharedPtr<FRemoteControlFunction>(Entity));
		}
	}
}

void URemoteControlPreset::RegisterOnCompileEvent(const TSharedPtr<FRemoteControlFunction>& RCFunction)
{
#if WITH_EDITOR
	if (UFunction* UnderlyingFunction = RCFunction->GetFunction())
	{
		if (UBlueprintGeneratedClass* BPClass = Cast<UBlueprintGeneratedClass>(UnderlyingFunction->GetOwnerClass()))
		{
			if (UBlueprint* Blueprint = Cast<UBlueprint>(BPClass->ClassGeneratedBy))
			{
				if (!BlueprintsWithRegisteredDelegates.Contains(Blueprint))
				{
					BlueprintsWithRegisteredDelegates.Emplace(Blueprint);
					Blueprint->OnCompiled().AddUObject(this, &URemoteControlPreset::OnBlueprintRecompiled);
				}
			}
		}
	}
#endif
}

void URemoteControlPreset::CreatePropertyWatcher(const TSharedPtr<FRemoteControlProperty>& RCProperty)
{
	if (ensure(RCProperty))
	{
		if (!PropertyWatchers.Contains(RCProperty->GetId()))
		{
			FRCPropertyWatcher Watcher{RCProperty, FSimpleDelegate::CreateLambda([this, WeakProperty = TWeakPtr<FRemoteControlProperty>(RCProperty)]()
			{
				if (TSharedPtr<FRemoteControlProperty> PinnedProperty = WeakProperty.Pin())
				{
					PerFrameModifiedProperties.Add(PinnedProperty->GetId());
				}
			})};
			
			PropertyWatchers.Add(RCProperty->GetId(), MoveTemp(Watcher));
		}
	}
}

bool URemoteControlPreset::PropertyShouldBeWatched(const TSharedPtr<FRemoteControlProperty>& RCProperty) const
{
	// We know these properties will be redirected to their setter in -game or packaged, so we won't get the object modified callback.
	static const TSet<FName> WatchedPropertyNames =
		{
			UStaticMeshComponent::GetRelativeLocationPropertyName(),
			UStaticMeshComponent::GetRelativeRotationPropertyName(),
			UStaticMeshComponent::GetRelativeScale3DPropertyName()
		};

	static const ELaunchConfiguration LaunchConfiguration = GetLaunchConfiguration();

	// In packaged, we don't have access to object modified callbacks, so we have to track them manually.
	if (LaunchConfiguration == ELaunchConfiguration::Packaged || CVarRemoteControlEnablePropertyWatchInEditor.GetValueOnAnyThread())
	{
		return true;
	}
	else if (LaunchConfiguration == ELaunchConfiguration::DashGame)
	{
		return RCProperty && WatchedPropertyNames.Contains(RCProperty->FieldName);
	}
	else
	{
		return false;
	}
}

void URemoteControlPreset::CreatePropertyWatchers()
{
	for (const TSharedPtr<FRemoteControlProperty>& ExposedProperty : Registry->GetExposedEntities<FRemoteControlProperty>())
	{
		if (PropertyShouldBeWatched(ExposedProperty))
		{
			CreatePropertyWatcher(ExposedProperty);
		}
	}
}

void URemoteControlPreset::RemoveUnusedBindings()
{
	TSet<TWeakObjectPtr<URemoteControlBinding>> ReferencedBindings;
	for (TSharedPtr<FRemoteControlEntity> Entity : Registry->GetExposedEntities())
	{
		for (TWeakObjectPtr<URemoteControlBinding> Binding : Entity->GetBindings())
		{
			ReferencedBindings.Add(Binding);
		}
	}

	for (auto It = Bindings.CreateIterator(); It; It++)
	{
		if (!ReferencedBindings.Contains(*It))
		{
			It.RemoveCurrent();
		}
	}
}

void URemoteControlPreset::PostLoadProperties()
{
	for (const TSharedPtr<FRemoteControlProperty>& ExposedProperty : Registry->GetExposedEntities<FRemoteControlProperty>())
	{
		ExposedProperty->PostLoad();
	}
}

void URemoteControlPreset::HandleDisplayClusterConfigChange(UObject* DisplayClusterConfigData)
{
#if WITH_EDITOR
	AActor* OwnerActor = DisplayClusterConfigData->GetTypedOuter<AActor>();
	FTimerManager& TimerManager = GEditor ? *GEditor->GetTimerManager() : OwnerActor->GetWorld()->GetTimerManager();
	TimerManager.SetTimerForNextTick(FTimerDelegate::CreateLambda([OwnerActor, PresetPtr = TWeakObjectPtr<URemoteControlPreset>{ this }]()
	{
		TSet<URemoteControlBinding*> ModifiedBindings;

		if (URemoteControlPreset* Preset = PresetPtr.Get())
		{
			for (URemoteControlBinding* Binding : Preset->Bindings)
			{
				UObject* NewObject = nullptr;
				UObject* ResolvedBinding = Binding->Resolve();

				static const FName NDisplayConfigurationData = "DisplayClusterConfigurationData";
				if (ResolvedBinding && ResolvedBinding->GetClass()->GetFName() == NDisplayConfigurationData)
				{
					AActor* BindingOwnerActor = ResolvedBinding->GetTypedOuter<AActor>();
					if (OwnerActor == BindingOwnerActor)
					{
						// Replace binding
						static const FName ConfigDataName = "CurrentConfigData";
						FProperty* ConfigDataProperty = BindingOwnerActor->GetClass()->FindPropertyByName(ConfigDataName);

						if (void* NewConfigData = ConfigDataProperty->ContainerPtrToValuePtr<void>(BindingOwnerActor))
						{
							if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ConfigDataProperty))
							{
								NewObject = ObjectProperty->GetObjectPropertyValue(NewConfigData);
							}
						}
					}
				}

				if (NewObject)
				{
					ModifiedBindings.Add(Binding);
					PresetPtr->Modify();
					Binding->Modify();
					Binding->SetBoundObject(NewObject);
				}

				for (const TSharedPtr<FRemoteControlField>& Entity : PresetPtr->Registry->GetExposedEntities<FRemoteControlField>())
				{
					for (TWeakObjectPtr<URemoteControlBinding> WeakBinding : Entity->Bindings)
					{
						if (!WeakBinding.IsValid())
						{
							continue;
						}

						if (ModifiedBindings.Contains(WeakBinding.Get()))
						{
							PresetPtr->PerFrameUpdatedEntities.Add(Entity->GetId());
						}
					}
				}
			}
		}
	}));
#endif
}

UWorld* URemoteControlPreset::GetWorld(const URemoteControlPreset* Preset, bool bAllowPIE)
{
	if (Preset && Preset->IsEmbeddedPreset())
	{
		UWorld* EmbeddedPresetWorld = Preset->GetEmbeddedWorld();

		if (EmbeddedPresetWorld)
		{
			return EmbeddedPresetWorld;
		}
	}

#if WITH_EDITOR
	if (GEditor)
	{
		if (bAllowPIE)
		{
			FWorldContext* PIEWorldContext = GEditor->GetPIEWorldContext();

			if (PIEWorldContext)
			{
				return PIEWorldContext->World();
			}
		}

		return GEditor->GetEditorWorldContext(false).World();
	}

	if (GWorld)
	{
		return GWorld;
	}
#endif

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		if (WorldContext.WorldType == EWorldType::Game)
		{
			return WorldContext.World();
		}
	}

	return nullptr;
}

UWorld* URemoteControlPreset::GetWorld(bool bAllowPIE) const
{
	return URemoteControlPreset::GetWorld(this, bAllowPIE);
}

UWorld* URemoteControlPreset::GetEmbeddedWorld() const
{
	if (UObject* Outer = GetOuter())
	{
		if (UWorld* OuterWorld = Outer->GetWorld())
		{
			return OuterWorld;
		}
	}

	return nullptr;
}

bool URemoteControlPreset::IsEmbeddedPreset() const
{
	UObject* Outer = GetOuter();

	if (!Outer)
	{
		return false;
	}

	// An asset
	if (Outer->IsA<UPackage>())
	{
		return false;
	}

	return true;
}

FName URemoteControlPreset::GetEntityName(const FName InDesiredName, UObject* InObject, const FRCFieldPathInfo& InFieldPath) const
{
	FName DesiredName = InDesiredName;
	
	if (DesiredName == NAME_None)
	{
		FProperty* Property = InFieldPath.GetResolvedData().Field;
		check(Property);

		FString FieldPath;

#if WITH_EDITOR
		FieldPath = Property->GetDisplayNameText().ToString();
#else
		FieldPath = InFieldPath.GetFieldName().ToString();
#endif
			
		DesiredName = *FString::Printf(TEXT("%s"), *FieldPath);
	}

	return DesiredName;
}

TOptional<FRemoteControlFunction> URemoteControlPreset::GetFunction(FName FunctionLabel) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetFunction(GetExposedEntityId(FunctionLabel));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TOptional<FRemoteControlFunction> URemoteControlPreset::GetFunction(FGuid FunctionId) const
{
	TOptional<FRemoteControlFunction> OptionalFunction;
	if (TSharedPtr<FRemoteControlFunction> RCFunction = Registry->GetExposedEntity<FRemoteControlFunction>(FunctionId))
	{
		OptionalFunction = *RCFunction;
	}

	return OptionalFunction;
}

TOptional<FRemoteControlProperty> URemoteControlPreset::GetProperty(FName PropertyLabel) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return GetProperty(GetExposedEntityId(PropertyLabel));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TOptional<FRemoteControlProperty> URemoteControlPreset::GetProperty(FGuid PropertyId) const
{
	TOptional<FRemoteControlProperty> OptionalProperty;
	if (TSharedPtr<FRemoteControlProperty> RCProperty = Registry->GetExposedEntity<FRemoteControlProperty>(PropertyId))
	{
		OptionalProperty = *RCProperty;
	}

	return OptionalProperty;
}

void URemoteControlPreset::RenameField(FName OldFieldLabel, FName NewFieldLabel)
{
	RenameExposedEntity(GetExposedEntityId(OldFieldLabel), NewFieldLabel);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TOptional<FExposedProperty> URemoteControlPreset::ResolveExposedProperty(FName PropertyLabel) const
{
	TOptional<FExposedProperty> OptionalExposedProperty;
	if (TSharedPtr<FRemoteControlProperty> RCProp = Registry->GetExposedEntity<FRemoteControlProperty>(GetExposedEntityId(PropertyLabel)))
	{
		OptionalExposedProperty = FExposedProperty();
		OptionalExposedProperty->OwnerObjects = RCProp->GetBoundObjects();
		OptionalExposedProperty->Property = RCProp->GetProperty();
	}

	return OptionalExposedProperty;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TOptional<FExposedFunction> URemoteControlPreset::ResolveExposedFunction(FName FunctionLabel) const
{
	TOptional<FExposedFunction> OptionalExposedFunction;
	if (TSharedPtr<FRemoteControlFunction> RCProp = Registry->GetExposedEntity<FRemoteControlFunction>(GetExposedEntityId(FunctionLabel)))
	{
		OptionalExposedFunction = FExposedFunction();
		OptionalExposedFunction->DefaultParameters = RCProp->FunctionArguments;
		OptionalExposedFunction->OwnerObjects = RCProp->GetBoundObjects();
		OptionalExposedFunction->Function = RCProp->GetFunction();
	}

	return OptionalExposedFunction;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void URemoteControlPreset::Unexpose(FName EntityLabel)
{
	Unexpose(GetExposedEntityId(EntityLabel));
}

void URemoteControlPreset::Unexpose(const FGuid& EntityId)
{
	if (EntityId.IsValid() && Registry->GetExposedEntity(EntityId).IsValid())
	{
		FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnEntityExposed(), this, EntityId);
		FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnEntityUnexposed(), this, EntityId);

		OnEntityUnexposedDelegate.Broadcast(this, EntityId);

		Registry->Modify();
		Registry->RemoveExposedEntity(EntityId);

		PropertyIdRegistry->Modify();
		PropertyIdRegistry->RemoveIdentifiedField(EntityId);

		if (FRCCachedFieldData* CachedData = FieldCache.Find(EntityId))
		{
			Layout.RemoveField(CachedData->LayoutGroupId, EntityId);
			FieldCache.Remove(EntityId);
			PropertyWatchers.Remove(EntityId);
		}
	}
}

void URemoteControlPreset::CacheLayoutData()
{
	CacheFieldLayoutData();
}

void URemoteControlPreset::CacheControllersLabels() const
{
	if (ControllerContainer)
	{
		ControllerContainer->CacheControllersLabels();
	}
}

TArray<UObject*> URemoteControlPreset::ResolvedBoundObjects(FName FieldLabel)
{
	TArray<UObject*> Objects;

	if (TSharedPtr<FRemoteControlField> Field = Registry->GetExposedEntity<FRemoteControlField>(GetExposedEntityId(FieldLabel)))
	{
		Objects = Field->GetBoundObjects();
	}

	return Objects;
}

void URemoteControlPreset::RebindUnboundEntities()
{
	Modify();
	RebindingManager->Rebind(this);
	Algo::Transform(Registry->GetExposedEntities(), PerFrameUpdatedEntities, [](const TSharedPtr<FRemoteControlEntity>& Entity) { return Entity->GetId(); });
	PropertyIdRegistry->Initialize();
}

void URemoteControlPreset::RebindAllEntitiesUnderSameActor(const FGuid& EntityId, AActor* NewActor, bool bUseRebindingContext)
{
	if (TSharedPtr<FRemoteControlEntity> Entity = Registry->GetExposedEntity(EntityId))
	{
		RebindingManager->RebindAllEntitiesUnderSameActor(this, Entity, NewActor, bUseRebindingContext);
	}
}

void URemoteControlPreset::RenewEntityIds()
{
	Modify();

	TArray<TSharedPtr<FRemoteControlEntity>> ExposedEntities = Registry->GetExposedEntities();

	// Keep track of the guids for delegate broadcast.
	TMap<FGuid, FGuid> EntityIdMap;	// old -> new
	EntityIdMap.Reserve(ExposedEntities.Num());
	
	for (const TSharedPtr<FRemoteControlEntity>& Entity : ExposedEntities)
	{
		if (Entity)
		{
			const FGuid OldEntityId = Entity->Id;
			Entity->Id = FGuid::NewGuid();
			EntityIdMap.Add(OldEntityId, Entity->Id);
		}
	}

	// Cache labels for all exposed entities
	Registry->CacheLabels();

	// Rehash the registries
	PropertyIdRegistry->UpdateEntityIds(EntityIdMap);
	Layout.UpdateEntityIds(EntityIdMap);
	
	// Update NameToGuidMap.
	for (TPair<FName, FGuid>& NameToGuid : NameToGuidMap)
	{
		if (const FGuid* FoundNewId = EntityIdMap.Find(NameToGuid.Value))
		{
			NameToGuid.Value = *FoundNewId;
		}
	}

	RehashMap(FieldCache, EntityIdMap);
	RehashMap(PropertyWatchers, EntityIdMap);
	RehashMap(PreObjectsModifiedCache, EntityIdMap);
	RehashMap(PreObjectsModifiedActorCache, EntityIdMap);
	RehashMap(PreMaterialModifiedCache, EntityIdMap);
	RehashSet(PerFrameModifiedProperties, EntityIdMap);
	RehashSet(PerFrameUpdatedEntities, EntityIdMap);

	if (ControllerContainer)
	{
		ControllerContainer->UpdateEntityIds(EntityIdMap);
	}

	// Fire the delegates once the registry has been rehashed to ensure proper lookup.
	OnPropertyIdsRenewed().Broadcast(this, EntityIdMap);
}

void URemoteControlPreset::RenewControllerIds()
{
	if (!ControllerContainer)
	{
		return;
	}
	
#if WITH_EDITOR
	ControllerContainer->Modify();
#endif

	// Keep track of the guids for delegate broadcast.
	TMap<FGuid, FGuid> ControllerIdMap;	// old -> new
	ControllerIdMap.Reserve(ControllerContainer->VirtualProperties.Num());
	
	for (const TObjectPtr<URCVirtualPropertyBase>& Controller : ControllerContainer->VirtualProperties)
	{
		const FGuid OldControllerId = Controller->Id;
		Controller->Id = FGuid::NewGuid();
		ControllerIdMap.Add(OldControllerId, Controller->Id);
	}

	// Re-cache after the Ids are updated
	CacheControllersLabels();

	OnControllerIdsRenewed().Broadcast(this, ControllerIdMap);
}

FName URemoteControlPreset::SetControllerDisplayName(FGuid InControllerGuid, const FName& InNewName) const
{
	if (ControllerContainer)
	{
		return ControllerContainer->SetControllerDisplayName(InControllerGuid, InNewName);
	}
	return NAME_None;
}

void URemoteControlPreset::NotifyExposedPropertyChanged(FName PropertyLabel)
{
	if (TSharedPtr<FRemoteControlProperty> ExposedProperty = GetExposedEntity<FRemoteControlProperty>(GetExposedEntityId(PropertyLabel)).Pin())
	{
		PerFrameModifiedProperties.Add(ExposedProperty->GetId());
	}
}

void URemoteControlPreset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FRemoteControlObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		if (!PresetId.IsValid())
		{
			PresetId = FGuid::NewGuid();
		}
		
#if WITH_EDITOR
		// 5.5 - Upgrade to per perset protocol modify operation flags
		if (Ar.CustomVer(FRemoteControlObjectVersion::GUID) < FRemoteControlObjectVersion::AddedPerPresetModifyOperationFlags)
		{
			const URemoteControlSettings* Settings = GetDefault<URemoteControlSettings>();

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Settings->bProtocolsGenerateTransactions_DEPRECATED)
			{
				ModifyOperationFlagsForProtocols = ERCModifyOperationFlags::None;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
#endif
	}
}

FRemoteControlField* URemoteControlPreset::GetFieldPtr(FGuid FieldId)
{
	return Registry->GetExposedEntity<FRemoteControlField>(FieldId).Get();
}

TSharedPtr<const FRemoteControlEntity> URemoteControlPreset::FindEntityById(const FGuid& EntityId, const UScriptStruct* EntityType) const
{
	return Registry->GetExposedEntity(EntityId, EntityType);
}

TSharedPtr<FRemoteControlEntity> URemoteControlPreset::FindEntityById(const FGuid& EntityId, const UScriptStruct* EntityType)
{
	return Registry->GetExposedEntity(EntityId, EntityType);
}

FGuid URemoteControlPreset::GetExposedEntityId(FName EntityLabel) const
{
	if (const FGuid* FoundGuid = NameToGuidMap.Find(EntityLabel))
	{
		return *FoundGuid;
	}
	return Registry->GetExposedEntityId(EntityLabel);
}

TArray<TSharedPtr<FRemoteControlEntity>> URemoteControlPreset::GetEntities(UScriptStruct* EntityType)
{
	return Registry->GetExposedEntities(EntityType);
}

TArray<TSharedPtr<const FRemoteControlEntity>> URemoteControlPreset::GetEntities(UScriptStruct* EntityType) const
{
	return const_cast<const URemoteControlExposeRegistry*>(ToRawPtr(Registry))->GetExposedEntities(EntityType);
}

const UScriptStruct* URemoteControlPreset::GetExposedEntityType(const FGuid& ExposedEntityId) const
{
	return Registry->GetExposedEntityType(ExposedEntityId);
}

const TSet<TObjectPtr<UScriptStruct>>& URemoteControlPreset::GetExposedEntityTypes() const
{
	return Registry->GetExposedEntityTypes();
}

const bool URemoteControlPreset::HasEntities() const
{
	return !Registry->IsEmpty();
}

FName URemoteControlPreset::RenameExposedEntity(const FGuid& ExposedEntityId, FName NewLabel)
{
	Registry->Modify();

	FName OldLabel = FName(TEXT("Invalid Label"));
	if (const TSharedPtr<FRemoteControlEntity>& Entity = Registry->GetExposedEntity(ExposedEntityId))
	{
		OldLabel = Entity->GetLabel();
	}

	const FName AssignedLabel = Registry->RenameExposedEntity(ExposedEntityId, NewLabel);

	FRCTransactionListenerHelper<URemoteControlPreset*, FName, FName>(ERCTransaction::Undo, GetPresetId(), OnFieldRenamed(), this, AssignedLabel, OldLabel);
	FRCTransactionListenerHelper<URemoteControlPreset*, FName, FName>(ERCTransaction::Redo, GetPresetId(), OnFieldRenamed(), this, OldLabel, AssignedLabel);

	return AssignedLabel;
}

bool URemoteControlPreset::IsExposed(const FGuid& ExposedEntityId) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::IsExposed);
	if (FieldCache.Contains(ExposedEntityId))
	{
		return true;
	}
	return Registry->GetExposedEntity(ExposedEntityId).IsValid();
}

TOptional<FRemoteControlField> URemoteControlPreset::GetField(FGuid FieldId) const
{
	TOptional<FRemoteControlField> Field;

	if (TSharedPtr<FRemoteControlField> FieldPtr = Registry->GetExposedEntity<FRemoteControlField>(FieldId))
	{
		Field = *FieldPtr;
	}

	return Field;
}

void URemoteControlPreset::OnExpose(const FExposeInfo& Info)
{
	FRCCachedFieldData CachedData;

	FGuid FieldGroupId = !Info.LayoutGroupId.IsValid() ? DefaultGroupId : Info.LayoutGroupId;
	FRemoteControlPresetGroup* Group = Layout.GetGroup(FieldGroupId);
	if (!Group)
	{
		Group = &Layout.GetDefaultGroup();
	}

	CachedData.LayoutGroupId = FieldGroupId;
	CachedData.OwnerObjectAlias = Info.Alias;
	FieldCache.FindOrAdd(Info.FieldId) = CachedData;

	if (FRemoteControlField* Field = GetFieldPtr(Info.FieldId))
	{
		NameToGuidMap.Add(Field->GetLabel(), Info.FieldId);
		Layout.AddField(FieldGroupId, Info.FieldId);
		OnEntityExposed().Broadcast(this, Info.FieldId);

		FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnEntityUnexposed(), this, Info.FieldId);
		FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnEntityExposed(), this, Info.FieldId);
	}
}

void URemoteControlPreset::OnUnexpose(FGuid UnexposedFieldId)
 {
	FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Undo, GetPresetId(), OnEntityExposed(), this, UnexposedFieldId);
	FRCTransactionListenerHelper<URemoteControlPreset*, const FGuid&>(ERCTransaction::Redo, GetPresetId(), OnEntityUnexposed(), this, UnexposedFieldId);

	OnEntityUnexposed().Broadcast(this, UnexposedFieldId);
	
	FRCCachedFieldData CachedData = FieldCache.FindChecked(UnexposedFieldId);

	Layout.RemoveField(CachedData.LayoutGroupId, UnexposedFieldId);

	FieldCache.Remove(UnexposedFieldId);

	FName FieldLabel;
	for (auto It = NameToGuidMap.CreateIterator(); It; ++It)
	{
		if (It.Value() == UnexposedFieldId)
		{
			FieldLabel = It.Key();
			It.RemoveCurrent();
		}
	}
}

void URemoteControlPreset::CacheFieldLayoutData()
{
	for (FRemoteControlPresetGroup& Group : Layout.AccessGroups())
	{
		for (auto It = Group.AccessFields().CreateIterator(); It; ++It)
		{
			FieldCache.FindOrAdd(*It).LayoutGroupId = Group.Id;
		}
	}
}

void URemoteControlPreset::OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event)
{
	// Objects modified should have run through the preobjectmodified. If interesting, they will be cached
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnObjectPropertyChanged);

	// ResetToDefaultValue will recall this function but re-executing it will cause a crash so we guard it to avoid this scenario
	if (bReentryGuard)
	{
		return;
	}
	TGuardValue<bool> Guard(bReentryGuard, true);

	// Handle enter / exit Multi-User session
	if (Object == this->GetPackage() && Event.ChangeType == EPropertyChangeType::Redirected)
	{
		for (const TSharedPtr<FRemoteControlEntity>& Entity : Registry->GetExposedEntities<FRemoteControlEntity>())
		{
			PerFrameUpdatedEntities.Add(Entity->GetId());
		}
		return;
	}

	if (Object == this && Event.Property && Event.Property->GetFName() == GET_MEMBER_NAME_CHECKED(URemoteControlPreset, Metadata))
	{
		// Needed because re-joining a multi user session will modify this without going through the SetMetadata function and necessitates a website refresh.
		OnMetadataModified().Broadcast(this);
		return;
	}

	if (Object && Object->GetClass()->GetName() == TEXT("DisplayClusterConfigurationData"))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnObjectPropertyChanged::HandleDisplayClusterConfigChange);
		// If a display cluster is modified, it will get invalidated so we need to update bindings to the new one.
		// Since it does not show up in the ObjectReplaced callback, we have to do it on property change.
		HandleDisplayClusterConfigChange(Object);
		return;
	}

	TMap<URemoteControlBinding*, UObject*> BoundObjectsCache;
 
	if (Event.Property == nullptr)
	{
		if(Event.MemberProperty == nullptr)
		{
			// When no property is passed to OnObjectPropertyChanged (such as by LevelSnapshot->Restore()), let's assume they all changed since we don't have more context.
			for (TSharedPtr<FRemoteControlProperty> Property : Registry->GetExposedEntities<FRemoteControlProperty>())
			{
				bool bPropertyModified = false;

				for (TWeakObjectPtr<URemoteControlBinding> Binding : Property->GetBindings())
				{
					if (!Binding.IsValid())
					{
						continue;
					}

					if (UObject** BoundObject = BoundObjectsCache.Find(Binding.Get()))
					{
						if (Object == *BoundObject)
						{
							PerFrameModifiedProperties.Add(Property->GetId());
							break;
						}
					}
					else
					{
						UObject* ResolvedObject = Binding->Resolve();
						BoundObjectsCache.Add(Binding.Get(), ResolvedObject);

						if (Object == ResolvedObject)
						{
							PerFrameModifiedProperties.Add(Property->GetId());
							break;
						}
					}
				}
			}
		}
	}
	else
	{
		for (auto Iter = PreObjectsModifiedCache.CreateIterator(); Iter; ++Iter)
		{
			FGuid& PropertyId = Iter.Key();
			FPreObjectsModifiedCache& CacheEntry = Iter.Value();
 
			if (CacheEntry.Objects.Contains(Object)
                && CacheEntry.Property == Event.Property)
			{
				if (TSharedPtr<FRemoteControlProperty> Property = Registry->GetExposedEntity<FRemoteControlProperty>(PropertyId))
				{
					UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Change detected on %ls::%ls", *GetName(), *Object->GetName(), *Event.Property->GetName());
					PerFrameModifiedProperties.Add(Property->GetId());
					Property->OnObjectPropertyChanged(Object, Event);
					Iter.RemoveCurrent();
				}
			}
		}
		for (auto Iter = PreMaterialModifiedCache.CreateIterator(); Iter; ++Iter)
		{
			FGuid& PropertyId = Iter.Key();
			FPreMaterialModifiedCache& CacheEntry = Iter.Value();
			if (TSharedPtr<FRemoteControlProperty> Property = Registry->GetExposedEntity<FRemoteControlProperty>(PropertyId))
			{
				if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(Object))
				{
					if (CacheEntry.bHadValue && MeshComponent->OverrideMaterials.IsValidIndex(CacheEntry.ArrayIndex) && MeshComponent->OverrideMaterials[CacheEntry.ArrayIndex] == NULL)
					{
						FRCResetToDefaultArgs Args;
						Args.Property = Event.Property;
						Args.Path = Property->FieldPathInfo.ToPathPropertyString();
						Args.ArrayIndex = Property->FieldPathInfo.Segments[0].ArrayIndex;
						Args.bCreateTransaction = false;
						IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();
						RemoteControlModule.ResetToDefaultValue(Object, Args);
					}
				}
			}
			Iter.RemoveCurrent();
		}
	}
 
	for (auto Iter = PreObjectsModifiedActorCache.CreateIterator(); Iter; ++Iter)
	{
		FGuid& ActorId = Iter.Key();
		FPreObjectsModifiedCache& CacheEntry = Iter.Value();
 
		if (CacheEntry.Objects.Contains(Object)
            && CacheEntry.Property == Event.Property)
		{
			if (TSharedPtr<FRemoteControlActor> RCActor = GetExposedEntity<FRemoteControlActor>(ActorId).Pin())
			{
				OnActorPropertyModified().Broadcast(this, *RCActor, Object, CacheEntry.MemberProperty);
				Iter.RemoveCurrent();
			}
		}
	}
}

void URemoteControlPreset::OnPreObjectPropertyChanged(UObject* Object, const class FEditPropertyChain& PropertyChain)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnPreObjectPropertyChanged);
	using PropertyNode = TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode;

	//Quick validation of the property chain
	PropertyNode* Head = PropertyChain.GetHead();
	if (!Head || !Head->GetValue())
	{
		return;
	}

	PropertyNode* Tail = PropertyChain.GetTail();
	if (!Tail || !Tail->GetValue())
	{
		return;
	}

	for (const TSharedPtr<FRemoteControlEntity>& Entity : Registry->GetExposedEntities(FRemoteControlActor::StaticStruct()))
	{
		if (TSharedPtr<FRemoteControlActor> RCActor = StaticCastSharedPtr<FRemoteControlActor>(Entity))
		{
			FString ActorPath = RCActor->Path.ToString();
			if (Object->GetPathName() == ActorPath || Object->GetTypedOuter<AActor>()->GetPathName() == ActorPath)
			{
				FPreObjectsModifiedCache& CacheEntry = PreObjectsModifiedActorCache.FindOrAdd(RCActor->GetId());
				
				// Don't recreate entries for a property we have already cached
				// or if the property was already cached by a child component.
				
				bool bParentObjectCached = CacheEntry.Objects.ContainsByPredicate([Object](UObject* InObjectToCompare){ return InObjectToCompare->GetTypedOuter<AActor>() == Object; }); 
				if (CacheEntry.Property == PropertyChain.GetActiveNode()->GetValue()
					|| CacheEntry.MemberProperty == PropertyChain.GetActiveMemberNode()->GetValue()
					|| bParentObjectCached)
				{
					continue;
				}
				
				CacheEntry.Objects.AddUnique(Object);
				CacheEntry.Property = PropertyChain.GetActiveNode()->GetValue();
				CacheEntry.MemberProperty = PropertyChain.GetActiveMemberNode()->GetValue();
			}
		}
	}

	for (TSharedPtr<FRemoteControlProperty> RCProperty : Registry->GetExposedEntities<FRemoteControlProperty>())
	{
		//If this property is already cached, skip it
		if (PreObjectsModifiedCache.Contains(RCProperty->GetId()))
		{
			continue;
		}
		
		TArray<UObject*> BoundObjects = RCProperty->GetBoundObjects();
		if (BoundObjects.Num() == 0)
		{
			continue;
		}

		for (UObject* BoundObject : BoundObjects)
		{
			if (BoundObject == Object || BoundObject->GetOuter() == Object)
			{
				if (FProperty* ExposedProperty = RCProperty->GetProperty())
				{
					bool bHasFound = false;
					PropertyNode* Current = Tail;
					while (Current && bHasFound == false)
					{
						//Verify if the exposed property was changed
						if (ExposedProperty == Current->GetValue())
						{
							bHasFound = true;
							
							if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(Object))
							{
								if (PropertyChain.GetActiveNode()->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials))
								{
									FPreMaterialModifiedCache& NewEntry = PreMaterialModifiedCache.FindOrAdd(RCProperty->GetId());
									NewEntry.ArrayIndex = RCProperty->FieldPathInfo.Segments[0].ArrayIndex;
									NewEntry.bHadValue = MeshComponent->OverrideMaterials.IsValidIndex(NewEntry.ArrayIndex) && MeshComponent->OverrideMaterials[NewEntry.ArrayIndex] != NULL;
								}
							}
							
							FPreObjectsModifiedCache& NewEntry = PreObjectsModifiedCache.FindOrAdd(RCProperty->GetId());
							NewEntry.Objects.AddUnique(Object);
							NewEntry.Property = PropertyChain.GetActiveNode()->GetValue();
							NewEntry.MemberProperty = PropertyChain.GetActiveMemberNode()->GetValue();
						}

						// Go backward to walk up the property hierarchy to see if an owning property is exposed.
						Current = Current->GetPrevNode();
					}
				}
			}
		}
	}
}

void URemoteControlPreset::OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnObjectTransacted);

	if (InTransactionEvent.GetEventType() != ETransactionObjectEventType::Finalized || InObject == nullptr)
	{
		return;
	}

	//When modifying a blueprint component, the root component gets trashed entirely and transaction event 
	//doesn't have information about actual properties being changed under it. We just have RootComponent
	//marked as being changed on a given actor. When that happens, we consider all properties binded to a component 
	//that belongs to that actor to be changed.
	const bool bHasRootComponentChanged = InTransactionEvent.GetChangedProperties().Contains("RootComponent");

	//Verify if have bindings to the specified object or a binding to a component owned by the modified object
	bool bIsObjectBound = false;
	for (const URemoteControlBinding* Binding : Bindings)
	{
		if (Binding && Binding->IsBound(InObject))
		{
			bIsObjectBound = true;
			break;
		}
		if (bHasRootComponentChanged && Binding)
		{
			if (UActorComponent* Component = Cast<UActorComponent>(Binding->Resolve()))
			{
				if (Component->GetOwner() == InObject)
				{
					bIsObjectBound = true;
					break;
				}
			}
		}
	}

	if (bIsObjectBound == false)
	{
		return;
	}

	//Go through all properties and verify if it was modified
	for (const TSharedPtr<FRemoteControlProperty>& Property : Registry->GetExposedEntities<FRemoteControlProperty>())
	{
		if (PerFrameModifiedProperties.Contains(Property->GetId()))
		{
			continue;
		}

		TArray<UObject*> BoundObjects = Property->GetBoundObjects();
		if (BoundObjects.Num() == 0 || Property->FieldPathInfo.Segments.Num() < 1)
		{
			continue;
		}
		
		for (UObject* BoundObject : BoundObjects)
		{
			bool bPropertyModified = false;
			if (BoundObject == InObject || BoundObject->GetOuter() == InObject)
			{
				//Before going into all transacted properties, verify if root component
				//has changed and verify if its owner matches the input object
				if (bHasRootComponentChanged)
				{
					if (UActorComponent* Component = Cast<UActorComponent>(BoundObject))
					{
						if (Component->GetOwner() == InObject)
						{
							PerFrameModifiedProperties.Add(Property->GetId());
							bPropertyModified = true;
						}
					}
				}

				//If root component checkup didn't match, go through all transacted properties and look if it's the root of the exposed property
				if (bPropertyModified == false)
				{
					for (const FName ChangedProperty : InTransactionEvent.GetChangedProperties())
					{
						//Changed properties will be a path (dot separated) to the modified property
						//All tests only returned the root structure (for nested props) so for now, only verify the first segment if it matches
						if (Property->FieldPathInfo.Segments[0].Name == ChangedProperty)
						{
							PerFrameModifiedProperties.Add(Property->GetId());
							bPropertyModified = true;
							break;		
						}
					}
				}
			}

			//If property was modified, no need to continue looking at other bindings
			if (bPropertyModified)
			{
				UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Exposed property '%ls'::'%ls' change detected during transaction of %ls", *GetName(), *BoundObject->GetName(), *Property->FieldName.ToString(), *InObject->GetName());
				break;
			}
		}
	}
}

void URemoteControlPreset::FixAndCacheControllersLabels() const
{
	if (ControllerContainer)
	{
		ControllerContainer->FixAndCacheControllersLabels();
	}
}

void URemoteControlPreset::RegisterDelegates()
{
	UnregisterDelegates();

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddUObject(this, &URemoteControlPreset::OnObjectPropertyChanged);
	FCoreUObjectDelegates::OnObjectTransacted.AddUObject(this, &URemoteControlPreset::OnObjectTransacted);

		
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddUObject(this, &URemoteControlPreset::OnPreObjectPropertyChanged);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().AddUObject(this, &URemoteControlPreset::OnActorDeleted);
	}

	FEditorDelegates::PostPIEStarted.AddUObject(this, &URemoteControlPreset::OnPieEvent);
	FEditorDelegates::EndPIE.AddUObject(this, &URemoteControlPreset::OnPieEvent);

	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &URemoteControlPreset::OnReplaceObjects);

	FEditorDelegates::MapChange.AddUObject(this, &URemoteControlPreset::OnMapChange);

	FCoreUObjectDelegates::OnPackageReloaded.AddUObject(this, &URemoteControlPreset::OnPackageReloaded);
#endif
	
	FCoreUObjectDelegates::PostLoadMapWithWorld.AddUObject(this, &URemoteControlPreset::OnMapLoadFinished);

	FCoreDelegates::OnBeginFrame.AddUObject(this, &URemoteControlPreset::OnBeginFrame);
	FCoreDelegates::OnEndFrame.AddUObject(this, &URemoteControlPreset::OnEndFrame);
}

void URemoteControlPreset::UnregisterDelegates()
{
	FCoreDelegates::OnBeginFrame.RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnPackageReloaded.RemoveAll(this);

	for (TWeakObjectPtr<UBlueprint> Blueprint : BlueprintsWithRegisteredDelegates)
	{
		if (Blueprint.IsValid())
		{
			Blueprint->OnCompiled().RemoveAll(this);
		}
	}

	FEditorDelegates::MapChange.RemoveAll(this);


	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	FEditorDelegates::EndPIE.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
	}

	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	FCoreUObjectDelegates::OnPreObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
#endif
}

#if WITH_EDITOR
void URemoteControlPreset::OnActorDeleted(AActor* Actor)
{
	UWorld* World = Actor->GetWorld();

	TSet<URemoteControlBinding*> ModifiedBindings;

	if (World && !World->IsPreviewWorld())
	{
		for (auto It = Bindings.CreateIterator(); It; ++It)
		{
			UObject* ResolvedObject = *It ? (*It)->Resolve() : nullptr;
			if (ResolvedObject && (Actor == ResolvedObject || Actor == ResolvedObject->GetTypedOuter<AActor>()))
			{
				// Defer binding clean up to next frame in case the actor deletion is actually an actor being moved to a different sub level.
				PerFrameBindingsToClean.Add(*It);
			}
		}
	}
}

void URemoteControlPreset::OnPieEvent(bool)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([PresetPtr = TWeakObjectPtr<URemoteControlPreset>{ this }]()
	{
		if (PresetPtr.IsValid() && PresetPtr->Registry)
		{
			for (TSharedPtr<FRemoteControlEntity> Entity : PresetPtr->Registry->GetExposedEntities<FRemoteControlEntity>())
			{
				if (Entity->GetStruct() == FRemoteControlProperty::StaticStruct())
				{
					StaticCastSharedPtr<FRemoteControlProperty>(Entity)->FieldPathInfo.Resolve(Entity->GetBoundObject());
				}

				PresetPtr->PerFrameUpdatedEntities.Add(Entity->GetId());
			}
		}
	}));
}

void URemoteControlPreset::OnReplaceObjects(const TMap<UObject*, UObject*>& ReplacementObjectMap)
{
	TSet<URemoteControlBinding*> ModifiedBindings;

	for (URemoteControlBinding* Binding : Bindings)
	{
		if (!Binding)
		{
			continue;
		}

		UObject* NewObject = nullptr;
		UObject* ResolvedBinding = Binding->Resolve();


		if (!NewObject)
		{
			if (UObject* Replacement = ReplacementObjectMap.FindRef(ResolvedBinding))
			{
				NewObject = Replacement;
			}
		}

		if (NewObject)
		{
			ModifiedBindings.Add(Binding);
			Modify();
			Binding->Modify();
			Binding->SetBoundObject(NewObject);
		}
	}

	for (const TSharedPtr<FRemoteControlField>& Entity : Registry->GetExposedEntities<FRemoteControlField>())
	{
		for (TWeakObjectPtr<URemoteControlBinding> Binding : Entity->Bindings)
		{
			if (!Binding.IsValid())
			{
				continue;
			}
				
			if (ModifiedBindings.Contains(Binding.Get()) || ReplacementObjectMap.FindKey(Binding->Resolve()))
			{
				PerFrameUpdatedEntities.Add(Entity->GetId());
			}
		}
	}
}

void URemoteControlPreset::OnMapChange(uint32)
{
	// Delay the refresh in order for the old actors to be invalid.
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([PresetPtr = TWeakObjectPtr<URemoteControlPreset>{this}]()
	{
		if (PresetPtr.IsValid() && PresetPtr->Registry)
		{
			Algo::Transform(PresetPtr->Registry->GetExposedEntities(), PresetPtr->PerFrameUpdatedEntities, [](const TSharedPtr<FRemoteControlEntity>& Entity) { return Entity->GetId(); });
			Algo::Transform(PresetPtr->Registry->GetExposedEntities<FRemoteControlProperty>(), PresetPtr->PerFrameModifiedProperties, [](const TSharedPtr<FRemoteControlProperty>& RCProp) { return RCProp->GetId(); });
		}
	}));
}

void URemoteControlPreset::OnBlueprintRecompiled(UBlueprint* Blueprint)
{
	if (!Blueprint)
	{
		return;
	}

	for (TSharedPtr<FRemoteControlFunction> RCFunction : Registry->GetExposedEntities<FRemoteControlFunction>())
	{
		if (UClass* Class = RCFunction->GetSupportedBindingClass())
		{
			if (Class->ClassGeneratedBy == Blueprint)
			{
				if (UFunction* OldFunction = RCFunction->GetFunction())
				{
					UClass* NewClass = Blueprint->GeneratedClass;
					if (!!NewClass->FindFunctionByName(OldFunction->GetFName()))
					{
						RCFunction->RegenerateArguments();
						PerFrameUpdatedEntities.Add(RCFunction->GetId());
					}
				}
			}
		}
	}
}

void URemoteControlPreset::OnPackageReloaded(EPackageReloadPhase Phase, FPackageReloadedEvent* Event)
{
	if (Phase == EPackageReloadPhase::PrePackageFixup && Event)
	{
		URemoteControlPreset* RepointedPreset = nullptr;
		if (Event->GetRepointedObject<URemoteControlPreset>(this, RepointedPreset) && RepointedPreset)
		{
			RepointedPreset->OnEntityExposedDelegate = OnEntityExposedDelegate;
			RepointedPreset->OnEntityUnexposedDelegate = OnEntityUnexposedDelegate;
			RepointedPreset->OnEntitiesUpdatedDelegate = OnEntitiesUpdatedDelegate;
			RepointedPreset->OnPropertyChangedDelegate = OnPropertyChangedDelegate;
			RepointedPreset->OnPropertyExposedDelegate = OnPropertyExposedDelegate;
			RepointedPreset->OnPropertyUnexposedDelegate = OnPropertyUnexposedDelegate;
			RepointedPreset->OnPresetFieldRenamed = OnPresetFieldRenamed;
			RepointedPreset->OnMetadataModifiedDelegate = OnMetadataModifiedDelegate;
			RepointedPreset->OnActorPropertyModifiedDelegate = OnActorPropertyModifiedDelegate;
			RepointedPreset->OnPresetLayoutModifiedDelegate = OnPresetLayoutModifiedDelegate;

			TSet<FGuid> PropertiesToUpdate;
			for (const TSharedPtr<FRemoteControlEntity>& Entity : RepointedPreset->Registry->GetExposedEntities<FRemoteControlEntity>())
			{
				PropertiesToUpdate.Add(Entity->GetId());
			}

			RepointedPreset->OnMetadataModifiedDelegate.Broadcast(RepointedPreset);
			RepointedPreset->OnEntitiesUpdatedDelegate.Broadcast(RepointedPreset, PropertiesToUpdate);
		}
	}
}

void URemoteControlPreset::CleanUpBindings()
{
	TSet<URemoteControlBinding*> BindingsToDelete;


	for (URemoteControlBinding* Binding : PerFrameBindingsToClean)
	{
		if (Binding)
		{
			if (Binding->PruneDeletedObjects())
			{
				BindingsToDelete.Add(Binding);
			}
		}
	}

	for (TSharedPtr<FRemoteControlEntity> Entity : Registry->GetExposedEntities<FRemoteControlEntity>())
	{
		if (Entity)
		{
			for (auto It = Entity->Bindings.CreateIterator(); It; ++It)
			{
				// Update bindings that were "touched" regardless of if they were pruned, so that the UI can re-resolve the binding
				// if one of the object was moved to a sublevel.
				if (PerFrameBindingsToClean.Contains(It->Get()))
				{
					PerFrameUpdatedEntities.Add(Entity->GetId());
				}
			}
		}
	}

	PerFrameBindingsToClean.Reset();
}
#endif

void URemoteControlPreset::OnMapLoadFinished(UWorld* LoadedWorld)
{
	if (Registry)
	{
		Algo::Transform(Registry->GetExposedEntities(), PerFrameUpdatedEntities, [](const TSharedPtr<FRemoteControlEntity>& Entity) { return Entity->GetId(); });
		Algo::Transform(Registry->GetExposedEntities<FRemoteControlProperty>(), PerFrameModifiedProperties, [](const TSharedPtr<FRemoteControlProperty>& RCProp) { return RCProp->GetId(); });
	}
}

void URemoteControlPreset::OnBeginFrame()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnBeginFrame);
	PropertyChangeWatchFrameCounter++;

	if (PropertyChangeWatchFrameCounter == CVarRemoteControlFramesBetweenPropertyWatch.GetValueOnGameThread() - 1)
	{
		PropertyChangeWatchFrameCounter = 0;
		for (TPair<FGuid, FRCPropertyWatcher>& Entry : PropertyWatchers)
		{
			Entry.Value.CheckForChange();
		}
	}

#if WITH_EDITOR
	CleanUpBindings();
#endif
}

void URemoteControlPreset::OnEndFrame()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemoteControlPreset::OnEndFrame);
	if (PerFrameUpdatedEntities.Num())
	{
		OnEntitiesUpdatedDelegate.Broadcast(this, PerFrameUpdatedEntities);
		PerFrameUpdatedEntities.Empty();
	}

	if (PerFrameModifiedProperties.Num())
	{
		OnPropertyChangedDelegate.Broadcast(this, PerFrameModifiedProperties);
		PerFrameModifiedProperties.Empty();
	}
}

URemoteControlPreset::FRCPropertyWatcher::FRCPropertyWatcher(const TSharedPtr<FRemoteControlProperty>& InWatchedProperty, FSimpleDelegate&& InOnWatchedValueChanged)
	: OnWatchedValueChanged(MoveTemp(InOnWatchedValueChanged))
	, WatchedProperty(InWatchedProperty)
{
	if (TOptional<FRCFieldResolvedData> ResolvedData = GetWatchedPropertyResolvedData())
	{
		SetLastFrameValue(*ResolvedData);
	}
}

void URemoteControlPreset::FRCPropertyWatcher::CheckForChange()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCPropertyWatcher::CheckForChange);
	if (TOptional<FRCFieldResolvedData> ResolvedData = GetWatchedPropertyResolvedData())
	{
		if (!LastFrameValue.Num())
		{
			SetLastFrameValue(*ResolvedData);
		}
		else if (ensure(ResolvedData->Field && ResolvedData->ContainerAddress))
		{
			const void* NewValueAddress = ResolvedData->Field->ContainerPtrToValuePtr<void>(ResolvedData->ContainerAddress);
			if (NewValueAddress && (ResolvedData->Field->GetSize() != LastFrameValue.Num() || !ResolvedData->Field->Identical(LastFrameValue.GetData(), NewValueAddress)))
			{
				SetLastFrameValue(*ResolvedData);
				OnWatchedValueChanged.ExecuteIfBound();
			}
		}
	}
}

TOptional<FRCFieldResolvedData> URemoteControlPreset::FRCPropertyWatcher::GetWatchedPropertyResolvedData() const
{
	TOptional<FRCFieldResolvedData> ResolvedData;
	
	if (TSharedPtr<FRemoteControlProperty> RCProperty = WatchedProperty.Pin())
	{
		if (!RCProperty->FieldPathInfo.IsResolved())
		{
			// In theory all objects should have the same value if they have an exposed property.
			TArray<UObject*> Objects = RCProperty->GetBoundObjects();
			if (Objects.Num() != 0)
			{
				RCProperty->FieldPathInfo.Resolve(Objects[0]);
			}
		}

		if (RCProperty->FieldPathInfo.IsResolved())
		{
			ResolvedData = RCProperty->FieldPathInfo.GetResolvedData();
		}
	}

	return ResolvedData;
}

void URemoteControlPreset::FRCPropertyWatcher::SetLastFrameValue(const FRCFieldResolvedData& ResolvedData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCPropertyWatcher::SetLastFrameValue);
	checkSlow(ResolvedData.Field);
	checkSlow(ResolvedData.ContainerAddress);
	
	const void* NewValueAddress = ResolvedData.Field->ContainerPtrToValuePtr<void>(ResolvedData.ContainerAddress);
	LastFrameValue.SetNumUninitialized(ResolvedData.Field->GetSize());
	ResolvedData.Field->InitializeValue(LastFrameValue.GetData());
	ResolvedData.Field->CopyCompleteValue(LastFrameValue.GetData(), NewValueAddress);
}

void URemoteControlPreset::UpdateEntityUsage()
{
	TMap<FGuid, TSet<UObject*>> EntityToBehavior;

	if (ControllerContainer)
	{
		for (URCVirtualPropertyBase* VirtualProperty : ControllerContainer->VirtualProperties)
		{
			VirtualProperty->UpdateEntityUsage(this, EntityToBehavior);
		}
	}

	BoundEntityIds.Empty(EntityToBehavior.Num());

	for (const TPair<FGuid, TSet<UObject*>>& EntityPair : EntityToBehavior)
	{
		BoundEntityIds.Add(EntityPair.Key, EntityPair.Value.Num());
	}
}

bool URemoteControlPreset::IsEntityBound(const FGuid& InEntityGuid) const
{
	if (InEntityGuid.IsValid())
	{
		return BoundEntityIds.FindRef(InEntityGuid) > 0;
	}

	return false;
}

int32 URemoteControlPreset::GetEntityBoundCount(const FGuid& InEntityGuid) const
{
	if (InEntityGuid.IsValid())
	{
		return BoundEntityIds.FindRef(InEntityGuid);
	}

	return 0;
}

int32 URemoteControlPreset::GetNextControllerDisplayIndex() const
{
	int32 HighestDisplayIndex = INDEX_NONE;

	if (ControllerContainer)
	{
		for (const FRCVirtualPropertyCategory& Category : ControllerContainer->Categories)
		{
			HighestDisplayIndex = FMath::Max(HighestDisplayIndex, Category.DisplayIndex);
		}

		for (URCVirtualPropertyBase* Controller : ControllerContainer->VirtualProperties)
		{
			HighestDisplayIndex = FMath::Max(HighestDisplayIndex, Controller->DisplayIndex);
		}
	}

	return HighestDisplayIndex + 1;
}

#undef LOCTEXT_NAMESPACE /* RemoteControlPreset */ 
