// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPresetActor.h"

ARemoteControlPresetActor::ARemoteControlPresetActor()
{
	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("RootComponent0"));
	check(RootComponent);

#if WITH_EDITOR
	RootComponent->bVisualizeComponent = true;
#endif // WITH_EDITOR

	RootComponent->SetMobility(EComponentMobility::Static);

	SetHidden(true);
	SetCanBeDamaged(false);
}
