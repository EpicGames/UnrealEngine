// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPresetRebindingManager.h"

#include "Algo/AllOf.h"
#include "Algo/Transform.h"
#include "Components/ActorComponent.h"
#include "Components/ActorComponent.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/Brush.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "Misc/Char.h"
#include "IRemoteControlModule.h"
#include "RemoteControlBinding.h"
#include "RemoteControlEntity.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectIterator.h"

#if WITH_EDITOR
#include "Editor.h"
#include "ActorEditorUtils.h"
#endif

static TAutoConsoleVariable<int32> CVarRemoteControlRebindingUseLegacyAlgo(TEXT("RemoteControl.UseLegacyRebinding"), 0, TEXT("Whether to revert to the old algorithm when doing a Rebind All."));

namespace RCPresetRebindingManager
{
	TArray<UClass*> GetRelevantClassesForObject(const TArray<UClass*>& InClasses, UObject* InObject)
	{
		return InClasses.FilterByPredicate([&InObject](UClass* InClass) { return InClass && InObject->IsA(InClass); });
	};

	/**
	 * Returns a map of actors grouped by their relevant class.
	 * (Relevant class being the base class that supports a given entity)
	 */
	TMap<UClass*, TArray<UObject*>> GetLevelObjectsGroupedByClass(const TArray<UClass*>& RelevantClasses, UWorld* PresetWorld)
	{
		TMap<UClass*, TArray<UObject*>> ObjectMap;

		auto AddRelevantClassesForObject = [&ObjectMap, &RelevantClasses, PresetWorld] (UObject* Object)
		{
			if (!Object
				|| (Object->IsA<AActor>() && !UE::RemoteControlBinding::IsValidActorForRebinding(CastChecked<AActor>(Object), PresetWorld))
				|| (Object->IsA<UActorComponent>() && !UE::RemoteControlBinding::IsValidSubObjectForRebinding(Object, PresetWorld)))
			{
				return;
			}
			
			for (UClass* Class : GetRelevantClassesForObject(RelevantClasses, Object))
			{
				ObjectMap.FindOrAdd(Class).Add(Object);
			}
		};

		if (PresetWorld)
		{
			for (TActorIterator<AActor> It(PresetWorld, AActor::StaticClass(), EActorIteratorFlags::SkipPendingKill); It; ++It)
			{
        		AddRelevantClassesForObject(*It);
			}
		}

		const EObjectFlags ExcludeFlags = RF_ClassDefaultObject | RF_ArchetypeObject | RF_Transient;
		for (TObjectIterator<UActorComponent> It(ExcludeFlags, true, EInternalObjectFlags::Garbage); It; ++It)
		{
			AddRelevantClassesForObject(*It);
		}

		return ObjectMap;
	}

	bool GetActorsOfClass(UWorld* PresetWorld, UClass* InTargetClass, TArray<AActor*>& OutActors)
	{
		auto WasGeneratedBySameBlueprintClass = [InTargetClass](UObject* Object)
		{
			UBlueprintGeneratedClass* IteratedClass = Cast<UBlueprintGeneratedClass>(Object->GetClass());
			UBlueprintGeneratedClass* TargetClass = Cast<UBlueprintGeneratedClass>(InTargetClass);

			// At the moment, only allow for ndisplay configs because I'm unsure of the repercussions of enabling this change for all BP generated classes.
			static const FName NDisplayGeneratedClass = "DisplayClusterBlueprintGeneratedClass";
			
#if WITH_EDITOR
			return IteratedClass && TargetClass
				&& InTargetClass->GetClass()->GetFName() == NDisplayGeneratedClass
				&& IteratedClass->ClassGeneratedBy && TargetClass->ClassGeneratedBy
				&& IteratedClass->ClassGeneratedBy->GetClass() == TargetClass->ClassGeneratedBy->GetClass();
#else
			return false;
#endif
		};

		auto IsObjectOfClass = [&](UObject* Object)
		{
			return Object
				&& UE::RemoteControlBinding::IsValidActorForRebinding(CastChecked<AActor>(Object), PresetWorld)
				&& (Object->IsA(InTargetClass) || WasGeneratedBySameBlueprintClass(Object));
		};

		if (PresetWorld)
		{
			for (TActorIterator<AActor> It(PresetWorld, AActor::StaticClass(), EActorIteratorFlags::SkipPendingKill); It; ++It)
			{
				if (IsObjectOfClass(*It))
				{
					OutActors.Add(*It);
				}
			}
		}

		return OutActors.Num() != 0;
	}

	TMap<UClass*, TMap<uint32, TSet<UObject*>>> CreateInitialBoundObjectMap(TConstArrayView<TSharedPtr<FRemoteControlEntity>> BoundEntities)
	{
		TMap<UClass*, TMap<uint32, TSet<UObject*>>> BoundObjectMap;
		for (const TSharedPtr<FRemoteControlEntity>& BoundEntity : BoundEntities)
		{
			if (UClass* SupportedClass = BoundEntity->GetSupportedBindingClass())
			{
				TMap<uint32, TSet<UObject*>>& BoundObjectsBySupportedOwnerClass = BoundObjectMap.FindOrAdd(SupportedClass);
				for (UObject* BoundObject : BoundEntity->GetBoundObjects())
				{
					BoundObjectsBySupportedOwnerClass.FindOrAdd(BoundEntity->GetUnderlyingEntityIdentifier()).Add(BoundObject);
				}
			}
		}
		return BoundObjectMap;
	}
}

UObject* FDefaultRebindingPolicy::FindMatch(const FFindBindingForEntityArgs& Args)
{
	UObject* const* Match = nullptr;
	
	if (const TSet<UObject*>* ObjectsBoundToEntity = Args.ExistingEntityMatches.Find(Args.Entity->GetUnderlyingEntityIdentifier()))
	{
		Match = Args.PotentialMatches.FindByPredicate([ObjectsBoundToEntity](UObject* InObject) { return !ObjectsBoundToEntity->Contains(InObject); });
	}
	else if (Args.PotentialMatches.Num())
	{
		Match = &Args.PotentialMatches[0];
	}

	return Match ? *Match : nullptr;
}

UObject* FNameBasedRebindingPolicy::FindMatch(const FFindBindingForEntityArgs& Args)
{
	UObject* const* Match = nullptr;
	
	// Do a slow search comparing object names with the name of the binding.
	if (Args.Entity->GetBindings().Num() && Args.Entity->GetBindings()[0].IsValid())
	{
		const TSet<UObject*>* ObjectsBoundToEntity = Args.ExistingEntityMatches.Find(Args.Entity->GetUnderlyingEntityIdentifier());

		// Strip digits from the end of the object name to increase chances of finding a match
		int32 LastCharIndex = Args.Entity->GetBindings()[0]->Name.FindLastCharByPredicate([](TCHAR InChar) { return FChar::IsAlpha(InChar); } );
		if (LastCharIndex != INDEX_NONE && LastCharIndex >0)
		{
			FStringView NameWithoutDigits = FStringView{ Args.Entity->GetBindings()[0]->Name };
			NameWithoutDigits.LeftInline(LastCharIndex);
			Match = Args.PotentialMatches.FindByPredicate(
	            [ObjectsBoundToEntity, &NameWithoutDigits](UObject* InObject)
	            {
					if (ObjectsBoundToEntity)
					{
						return !ObjectsBoundToEntity->Contains(InObject) && InObject->GetName().StartsWith(NameWithoutDigits.GetData());
					}

					return InObject->GetName().StartsWith(NameWithoutDigits.GetData());
	            });
		}
	}

	return Match ? *Match : FDefaultRebindingPolicy::FindMatch(Args);
}

void FRemoteControlPresetRebindingManager::Rebind(URemoteControlPreset* Preset)
{
	if (CVarRemoteControlRebindingUseLegacyAlgo.GetValueOnAnyThread() == 1)
	{
		Rebind_Legacy(Preset);
	}
	else
	{
		Rebind_NewAlgo(Preset);
	}
}
void FRemoteControlPresetRebindingManager::Rebind_Legacy(URemoteControlPreset* Preset)
{
	check(Preset);

	Context.Reset();

	ReinitializeBindings(Preset);

	TArray<TWeakPtr<FRemoteControlEntity>> WeakExposedEntities = Preset->GetExposedEntities<FRemoteControlEntity>();
	
	TArray<TSharedPtr<FRemoteControlEntity>> ExposedEntities;
	ExposedEntities.Reserve(WeakExposedEntities.Num());

	Algo::TransformIf(WeakExposedEntities, ExposedEntities,
		[](const TWeakPtr<FRemoteControlEntity>& InWeakEntity){ return InWeakEntity.IsValid(); },
		[](const TWeakPtr<FRemoteControlEntity>& InWeakEntity){ return InWeakEntity.Pin(); }
		);
	
	TArray<TSharedPtr<FRemoteControlEntity>> BoundEntities = ExposedEntities.FilterByPredicate([](const TSharedPtr<FRemoteControlEntity>& InEntity) { return InEntity->IsBound(); });
	TArray<TSharedPtr<FRemoteControlEntity>> UnboundProperties = ExposedEntities.FilterByPredicate([](const TSharedPtr<FRemoteControlEntity>& InEntity) { return !InEntity->IsBound(); });
	
	// Build a map of entities already bound in order to avoid the case where we bind an object to an exposed entity that 
	// was rebound in the ReinitializeBindings step.
	Context.BoundObjectsBySupportedOwnerClass = RCPresetRebindingManager::CreateInitialBoundObjectMap(BoundEntities);
	TMap<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>> EntitiesGroupedBySupportedOwnerClass = GroupByEntitySupportedOwnerClass(UnboundProperties);
	
	// Fetch any relevant object from the level based on if their class is relevant for the set of entities we have to rebind.
	TArray<UClass*> EntitiesOwnerClasses;
	EntitiesGroupedBySupportedOwnerClass.GenerateKeyArray(EntitiesOwnerClasses);
	constexpr bool bAllowPIE = false;
	Context.ObjectsGroupedByRelevantClass = RCPresetRebindingManager::GetLevelObjectsGroupedByClass(EntitiesOwnerClasses, Preset->GetWorld(bAllowPIE));

	for (TPair<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>>& Entry : EntitiesGroupedBySupportedOwnerClass)
	{
		if (Entry.Key->IsChildOf<AActor>())
		{
			RebindEntitiesForActorClass_Legacy(Preset, Entry.Key, Entry.Value);
		}
	}

	for (TPair<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>>& Entry : EntitiesGroupedBySupportedOwnerClass)
	{
		// Do an additional pass in case the bindings did not have the necessary info to use the previous rebinding process. 
		RebindEntitiesForClass_Legacy(Entry.Key, Entry.Value);
	}
}

void FRemoteControlPresetRebindingManager::Rebind_NewAlgo(URemoteControlPreset* Preset)
{
	check(Preset);

	// Separate bindings based on if they are bound or not.
	TArray<URemoteControlLevelDependantBinding*> ValidBindings;
	TArray<URemoteControlLevelDependantBinding*> InvalidBindings;
	
	auto ClassifyBindings = [&ValidBindings, &InvalidBindings, Preset] ()
	{
		ValidBindings.Reset();
		InvalidBindings.Reset();

		for (URemoteControlBinding* Binding : Preset->Bindings)
		{
			if (URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(Binding))
			{
				if (LevelDependantBinding->Resolve())
				{
					ValidBindings.Add(LevelDependantBinding);
				}
				else
				{
					InvalidBindings.Add(LevelDependantBinding);
				}
			}
		}
	};

	ClassifyBindings();
	const bool bAllContextsEmpty = Algo::AllOf(InvalidBindings, [](URemoteControlLevelDependantBinding* Binding) { return Binding->BindingContext.IsEmpty(); });
	if (bAllContextsEmpty)
	{
		Rebind_Legacy(Preset);
		return;
	}

	// First, try to 'rebind for all properties' bindings that are already bound.
	// to make sure that we don't accidentally rebind a property to a different actor
	for (URemoteControlLevelDependantBinding* ValidBinding : ValidBindings)
	{
		UObject* ResolvedObject = ValidBinding->Resolve();
		if (AActor* ResolvedActor = Cast<AActor>(ResolvedObject))
		{
			RebindAllEntitiesUnderSameActor(Preset, ValidBinding, ResolvedActor);
		}
		else
		{
			RebindAllEntitiesUnderSameActor(Preset, ValidBinding, ResolvedObject->GetTypedOuter<AActor>());
		}
	}

	// Reclassify bindings after we rebound them.
	ClassifyBindings();

	// Create a list of what objects and components are already bound.
	TSet<TPair<AActor*, UObject*>> BoundObjects;
	for (URemoteControlLevelDependantBinding* ValidBinding : ValidBindings)
	{
		const bool bTargetingActor = !ValidBinding->BindingContext.OwnerActorName.IsNone() && ValidBinding->BindingContext.ComponentName.IsNone();
		const bool bTargetingComponent = !ValidBinding->BindingContext.OwnerActorName.IsNone() && !ValidBinding->BindingContext.ComponentName.IsNone();

		if (bTargetingActor)
		{
			BoundObjects.Add(TPair<AActor*, UObject*>{ Cast<AActor>(ValidBinding->Resolve()), nullptr });
		}
		else if (bTargetingComponent)
		{
			UObject* Object = ValidBinding->Resolve();
			AActor* Actor = Object->GetTypedOuter<AActor>();
			BoundObjects.Add(TPair<AActor*, UObject*>{ Actor, Object });
		}
	}

	auto FindByName = [](const TArray<AActor*>& PotentialMatches, FName TargetName) -> UObject*
	{
		for (UObject* Object : PotentialMatches)
		{
			// Attempt finding by name.
			if (Object->GetFName() == TargetName)
			{
				return Object;
			}
		}
		return nullptr;
	};

	auto TryRebindPair = [&](URemoteControlBinding* InBindingToRebind, AActor* InActor, UObject* InSubObject)
	{
		TPair<AActor*, UObject*> Pair = TPair<AActor*, UObject*> { InActor, InSubObject };
		if (!BoundObjects.Contains(Pair))
		{
			BoundObjects.Add(Pair);
			InBindingToRebind->Modify();
			if (InSubObject)
			{
				InBindingToRebind->SetBoundObject(InSubObject);
			}
			else
			{
				InBindingToRebind->SetBoundObject(InActor);
			}
			return true;
		}
		return false;
	};

	auto GetSubObjectBasedOnName = [](URemoteControlLevelDependantBinding* InBinding, UObject* InOwnerObject) -> UObject*
	{
		FString SubObjectPath = InBinding->BindingContext.ComponentName.ToString();
		if (InBinding->BindingContext.HasValidSubObjectPath())
		{
			SubObjectPath = InBinding->BindingContext.SubObjectPath;
		}

		UObject* TargetSubObject = FindObject<UObject>(InOwnerObject, *SubObjectPath);
		if (TargetSubObject && TargetSubObject->GetClass()->IsChildOf(InBinding->BindingContext.SupportedClass.LoadSynchronous()))
		{
			return TargetSubObject;
		}
		return nullptr;
	};

	auto RebindComponentBasedOnClass = [&](URemoteControlLevelDependantBinding* InBinding, AActor* InActorMatch)
	{
		if (UClass* SupportedClass = InBinding->BindingContext.SupportedClass.LoadSynchronous())
		{
			if (SupportedClass->IsChildOf(UActorComponent::StaticClass()))
			{
				TArray<UActorComponent*> Components;
				Cast<AActor>(InActorMatch)->GetComponents(InBinding->BindingContext.SupportedClass.LoadSynchronous(), Components);

				UE_LOGF(LogRemoteControl, Verbose, "Potential Subobject candidates:");
				for (UActorComponent* Component : Components)
				{
					UE_LOGF(LogRemoteControl, Verbose, "\t%ls", *Component->GetName());

					if (TryRebindPair(InBinding, Component->GetOwner(), Component))
					{
						UE_LOGF(LogRemoteControl, Verbose, "Found a matching subobject (%ls) with a matching class (%ls) on actor %ls", *Component->GetName(), *Component->GetClass()->GetName(), *InActorMatch->GetName());
						return true;
					}
				}
			}
		}

		UE_LOGF(LogRemoteControl, Verbose, "Could not find an object to rebind using the binding class.");
		return false;
	};

	auto RebindUsingActorName = [&](URemoteControlLevelDependantBinding* BindingToRebind, const TArray<AActor*>& ObjectsWithSupportedClass)
	{
		if (UObject* Match = FindByName(ObjectsWithSupportedClass, BindingToRebind->BindingContext.OwnerActorName))
		{
			return TryRebindPair(BindingToRebind, Cast<AActor>(Match), nullptr);
		}
		return false;
	};

	auto RebindUsingClass = [&](URemoteControlLevelDependantBinding* BindingToRebind, const TArray<AActor*>& ObjectsWithSupportedClass)
	{
		for (UObject* PotentialMatch : ObjectsWithSupportedClass)
		{
			if (TryRebindPair(BindingToRebind, Cast<AActor>(PotentialMatch), nullptr))
			{
				return true;
			}
		}
		return false;
	};

	auto TryRebindSubObject = [&](URemoteControlLevelDependantBinding* BindingToRebind, UObject* PotentialMatch)
	{
		UE_LOGF(LogRemoteControl, Verbose, "Attempting to find subobject %ls on object %ls.", *BindingToRebind->BindingContext.ComponentName.ToString(), *PotentialMatch->GetName());
		
		if (UObject* TargetSubObject = GetSubObjectBasedOnName(BindingToRebind, PotentialMatch))
		{
			UE_LOGF(LogRemoteControl, Verbose, "Found a matching subobject (%ls) with a matching name on actor %ls", *TargetSubObject->GetName(), *PotentialMatch->GetName());
			return TryRebindPair(BindingToRebind, TargetSubObject->GetTypedOuter<AActor>(), TargetSubObject);
		}
		else
		{
			UE_LOGF(LogRemoteControl, Verbose, "Could not find a subobject using its name, trying to find one with a matching class (%ls)", *BindingToRebind->BindingContext.OwnerActorClass->GetClass()->GetName());
			return RebindComponentBasedOnClass(BindingToRebind, Cast<AActor>(PotentialMatch));
		}
	};

	auto FindSubObjectToRebind = [&](URemoteControlLevelDependantBinding* InvalidBinding, const TArray<AActor*>& ObjectsWithSupportedClass)
	{
		if (UObject* Match = FindByName(ObjectsWithSupportedClass, InvalidBinding->BindingContext.OwnerActorName))
		{
			UE_LOGF(LogRemoteControl, Verbose, "Found owner with a matching Owner Actor Name. (%ls)", *Match->GetName());
			return TryRebindSubObject(InvalidBinding, Match);
		}
		else
		{
			// Could not find an actor with the same name as the initial binding, rely only on class instead.
			UE_LOGF(LogRemoteControl, Verbose, "Could not find an actor with the same name, defaulting to the first actor that supports the binding.");
			for (UObject* Object : ObjectsWithSupportedClass)
			{
				if (TryRebindSubObject(InvalidBinding, Object))
				{
					return true;
				}
			}

			return false;
		}
	};

	auto GetPotentialActors = [&](URemoteControlLevelDependantBinding* InvalidBinding, TArray<AActor*>& OutObjectsWithSupportedClass)-> bool
	{
		UClass* OwnerClass = InvalidBinding->BindingContext.OwnerActorClass.LoadSynchronous();
		constexpr bool bAllowPIE = false;
		return RCPresetRebindingManager::GetActorsOfClass(Preset->GetWorld(bAllowPIE), OwnerClass, OutObjectsWithSupportedClass);
	};

	auto Rebind = [&](URemoteControlLevelDependantBinding* InvalidBinding)
	{
		const FRemoteControlInitialBindingContext& BindingContext = InvalidBinding->BindingContext;
		TArray<AActor*> ObjectsWithSupportedClass;
		
		if (GetPotentialActors(InvalidBinding, ObjectsWithSupportedClass))
		{
			UE_LOGF(LogRemoteControl, Verbose, "Potential Actor Matches: ");
			for (AActor* Actor : ObjectsWithSupportedClass)
			{
#if WITH_EDITOR
				UE_LOGF(LogRemoteControl, Verbose, "\t%ls (%ls)", *Actor->GetActorLabel(), *Actor->GetName());
#else
				UE_LOGF(LogRemoteControl, Verbose, "\t%ls", *Actor->GetName());
#endif

			}

			if (BindingContext.HasValidSubObjectPath() || BindingContext.HasValidComponentName())
			{
				return FindSubObjectToRebind(InvalidBinding, ObjectsWithSupportedClass);
			}
			else
			{
				if (!RebindUsingActorName(InvalidBinding, ObjectsWithSupportedClass))
				{
					return RebindUsingClass(InvalidBinding, ObjectsWithSupportedClass);
				}
				else
				{
					return true;
				}
			}
		}
		return false;
	};

	auto TryNDisplayRebind = [&](URemoteControlLevelDependantBinding* InvalidBinding)
	{
		UE_LOGF(LogRemoteControl, Verbose, "Binding did not contain a full component path, checking if it's a ndisplay actor.");
		if (UBlueprintGeneratedClass* BPClass = Cast <UBlueprintGeneratedClass>(InvalidBinding->BindingContext.OwnerActorClass.Get()))
		{
			if (BPClass->GetSuperStruct() && BPClass->GetSuperStruct()->GetName() == TEXT("DisplayClusterRootActor"))
			{
				UE_LOGF(LogRemoteControl, Verbose, "Binding was on a ndisplay actor.");
				TArray<AActor*> ObjectsWithSupportedClass;
				if (!GetPotentialActors(InvalidBinding, ObjectsWithSupportedClass))
				{
					UE_LOGF(LogRemoteControl, Verbose, "No potential ndisplay actor to rebind to.");
					return;
				}

				// We manually get the path from the last bound object since it was not saved.
				// We are looking for a path like DisplayClusterConfigurationData_0.DisplayClusterConfigurationCluster_0.Node_1.VP_1
				// We will then combine this with display cluster root actors in the scene to attempt finding a matching object.
				FString LastObjectPath = InvalidBinding->GetLastBoundObjectPath().ToString();
				if (LastObjectPath.IsEmpty())
				{
					UE_LOGF(LogRemoteControl, Verbose, "Could not find the last bound object path on the binding.");
					return;
				}

				const int32 ConfigDataIndex = LastObjectPath.Find(TEXT("DisplayClusterConfigurationData"));

				if (ConfigDataIndex == INDEX_NONE)
				{
					return;
				}

				FString SubObjectSubPath = LastObjectPath.RightChop(ConfigDataIndex);

				for (AActor* Actor : ObjectsWithSupportedClass)
				{
					if (UObject* SubObject = FindObject<UObject>(Actor, *SubObjectSubPath))
					{
						UE_LOGF(LogRemoteControl, Verbose, "Found object %ls that contains subobject %ls", *Actor->GetName(), *SubObjectSubPath);
						TryRebindPair(InvalidBinding, Actor, SubObject);
					}
				}
			}
		}
	};

	UE_LOGF(LogRemoteControl, Verbose, "============================== Rebinding All for preset %ls ===============================", *Preset->GetName());

	for (URemoteControlLevelDependantBinding* InvalidBinding : InvalidBindings)
	{
		UE_LOGF(LogRemoteControl, Verbose, "Rebinding Invalid Binding");
		UE_LOGF(LogRemoteControl, Verbose, "\tOwnerActorName: %ls", *InvalidBinding->BindingContext.OwnerActorName.ToString());
		UE_LOGF(LogRemoteControl, Verbose, "\tOwnerActorClass: %ls", *InvalidBinding->BindingContext.OwnerActorClass.ToString());
		UE_LOGF(LogRemoteControl, Verbose, "\tSupportedClass: %ls", *InvalidBinding->BindingContext.SupportedClass.ToString());
		UE_LOGF(LogRemoteControl, Verbose, "\tComponentName: %ls", *InvalidBinding->BindingContext.ComponentName.ToString());
		UE_LOGF(LogRemoteControl, Verbose, "\tSubObjectPath: %ls", *InvalidBinding->BindingContext.SubObjectPath);
		
		if (!Rebind(InvalidBinding))
		{
			TryNDisplayRebind(InvalidBinding);
		}

		UE_LOGF(LogRemoteControl, Verbose, "===================================================================");
	}

	Preset->RemoveUnusedBindings();
}

TArray<FGuid> FRemoteControlPresetRebindingManager::RebindAllEntitiesUnderSameActor(URemoteControlPreset* Preset, URemoteControlBinding* InitialBinding, AActor* NewActor)
{
	check(Preset && InitialBinding && NewActor);

	TArray<FGuid> ReboundEntityIds;

	URemoteControlLevelDependantBinding* InitialLevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(InitialBinding);
	if (!InitialLevelDependantBinding)
	{
		return ReboundEntityIds;
	}

	TArray<URemoteControlLevelDependantBinding*> BindingsToRebind;

	// Find bindings with the same actor as the original binding
	for (URemoteControlBinding* Binding : Preset->Bindings)
	{
		if (URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(Binding))
		{
			// Todo: Make this more robust by comparing full actor paths instead of names.
			if (LevelDependantBinding->BindingContext.OwnerActorClass == InitialLevelDependantBinding->BindingContext.OwnerActorClass
				&& LevelDependantBinding->BindingContext.OwnerActorName == InitialLevelDependantBinding->BindingContext.OwnerActorName)
			{
				BindingsToRebind.Add(LevelDependantBinding);
			}
		}
	}

	// Gather entities to rebind
	TArray<TSharedPtr<FRemoteControlEntity>> EntitiesToRebind;
	for (const TWeakPtr<FRemoteControlEntity>& WeakEntity : Preset->GetExposedEntities<FRemoteControlEntity>())
	{
		TSharedPtr<FRemoteControlEntity> RCEntity = WeakEntity.Pin();
		if (RCEntity && RCEntity->GetBindings().Num())
		{
			if (BindingsToRebind.Contains(RCEntity->GetBindings()[0].Get()))
			{
				EntitiesToRebind.Add(RCEntity);
			}
		}
	}

	ReboundEntityIds.Reserve(EntitiesToRebind.Num());
	Algo::Transform(EntitiesToRebind, ReboundEntityIds, [](const TSharedPtr<FRemoteControlEntity>& RCEntityPtr) { return RCEntityPtr->GetId();  });

	// Find the new binding target, then rebind.
	for (const TSharedPtr<FRemoteControlEntity>& EntityToRebind : EntitiesToRebind)
	{
		URemoteControlLevelDependantBinding* EntityBinding = Cast<URemoteControlLevelDependantBinding>(EntityToRebind->GetBindings()[0].Get());

		UObject* RebindTarget = NewActor;

		// If the binding is targeting a component rather than an actor
		if (!EntityBinding->BindingContext.ComponentName.IsNone())
		{
			if (UObject* ResolvedComponent = StaticFindObject(UObject::StaticClass(), NewActor, *EntityBinding->Name, EFindObjectFlags::None))
			{
				RebindTarget = ResolvedComponent;
			}
		}

		EntityToRebind->BindObject(RebindTarget);

		if (Preset->GetExposedEntityType(EntityToRebind->GetId()) == FRemoteControlProperty::StaticStruct())
		{
			StaticCastSharedPtr<FRemoteControlProperty>(EntityToRebind)->EnableEditCondition();
		}
	}

	Preset->RemoveUnusedBindings();

	return ReboundEntityIds;
}

TArray<FGuid> FRemoteControlPresetRebindingManager::RebindAllEntitiesUnderSameActor_Legacy(URemoteControlPreset* Preset, const TSharedPtr<FRemoteControlEntity>& Entity, AActor* NewActor)
{
	check(Preset && Entity && NewActor);

	TArray<FGuid> ReboundEntities;

	TMap<FName, TSet<URemoteControlBinding*>> ActorsToBindings;
	ActorsToBindings.Reserve(Preset->Bindings.Num());

	auto GetBindingLastLevel = [](URemoteControlBinding* Binding) -> FSoftObjectPath
	{
		FSoftObjectPath LevelPath;
		if (Binding && Binding->IsA<URemoteControlLevelDependantBinding>())
		{
			LevelPath = CastChecked<URemoteControlLevelDependantBinding>(Binding)->LevelWithLastSuccessfulResolve.ToSoftObjectPath();
		}
		return LevelPath;
	};

	auto GetBindingActorName = [](URemoteControlBinding* Binding) -> FName
	{
		FName ActorName;
		if (Binding && Binding->IsA<URemoteControlLevelDependantBinding>())
		{
			URemoteControlLevelDependantBinding* LDBinding = CastChecked<URemoteControlLevelDependantBinding>(Binding);
			FString Path = LDBinding->BoundObjectMapByPath.FindRef(LDBinding->LevelWithLastSuccessfulResolve.ToSoftObjectPath()).ToString();
			static const TCHAR* PersistentLevel = TEXT("PersistentLevel.");
			const int32 PersistentLevelStringLength = 16;

			int32 Position = Path.Find(PersistentLevel);

			if (Position != INDEX_NONE)
			{
				int32 DotPosition = Path.Find(TEXT("."), ESearchCase::IgnoreCase, ESearchDir::FromStart, Position + PersistentLevelStringLength);
				if (DotPosition != INDEX_NONE)
				{
					ActorName = *Path.Mid(Position + PersistentLevelStringLength, DotPosition - (Position + PersistentLevelStringLength));
				}
			}
		}

		return ActorName;
	};


	FName InitialBindingActorName;
	FSoftObjectPath InitialBindingLastLevel;

	const TArray<TWeakObjectPtr<URemoteControlBinding>>& EntityBindings = Entity->GetBindings();
	if (EntityBindings.Num())
	{
		InitialBindingLastLevel = GetBindingLastLevel(EntityBindings[0].Get());
		InitialBindingActorName = GetBindingActorName(EntityBindings[0].Get());
	}

	TMap<URemoteControlBinding*, FString> BindingToComponentName;

	// Find bindings with the same actor as the original binding
	for (URemoteControlBinding* Binding : Preset->Bindings)
	{
		FSoftObjectPath BindingItLastLevel = GetBindingLastLevel(Binding);
		FName BindingItActorName = GetBindingActorName(Binding);

		// To add a binding to the list of bindings to rebind, make sure it had the same level and actor as the initial binding
		if (BindingItActorName == InitialBindingActorName && BindingItLastLevel == InitialBindingLastLevel)
		{
			BindingToComponentName.Add(Binding, BindingItActorName.ToString());
		}
	}

	// Gather entities to rebind
	TArray<TSharedPtr<FRemoteControlEntity>> EntitiesToRebind;
	for (const TWeakPtr<FRemoteControlEntity>& WeakEntity : Preset->GetExposedEntities<FRemoteControlEntity>())
	{
		TSharedPtr<FRemoteControlEntity> RCEntity = WeakEntity.Pin();
		if (RCEntity && RCEntity->GetBindings().Num())
		{
			if (BindingToComponentName.Contains(RCEntity->GetBindings()[0].Get()))
			{
				EntitiesToRebind.Add(RCEntity);
			}
		}
	}

	// Find the new binding target, then rebind.
	for (const TSharedPtr<FRemoteControlEntity>& EntityToRebind : EntitiesToRebind)
	{
		URemoteControlBinding* EntityBinding = EntityToRebind->GetBindings()[0].Get();
		FString ActorName = BindingToComponentName.FindChecked(EntityBinding);

		UObject* RebindTarget = NewActor;

		// If the binding is targetting a component rather than an actor
		if (ActorName != EntityBinding->Name)
		{
			if (UObject* ResolvedComponent = StaticFindObject(UObject::StaticClass(), NewActor, *EntityBinding->Name, EFindObjectFlags::None))
			{
				RebindTarget = ResolvedComponent;
			}
		}

		EntityToRebind->BindObject(RebindTarget);
		ReboundEntities.Add(EntityToRebind->GetId());

		if (Preset->GetExposedEntityType(EntityToRebind->GetId()) == FRemoteControlProperty::StaticStruct())
		{
			StaticCastSharedPtr<FRemoteControlProperty>(EntityToRebind)->EnableEditCondition();
		}
	}

	Preset->RemoveUnusedBindings();

	return ReboundEntities;
}

TArray<FGuid> FRemoteControlPresetRebindingManager::RebindAllEntitiesUnderSameActor(URemoteControlPreset* Preset, const TSharedPtr<FRemoteControlEntity>& Entity, AActor* NewActor, bool bUseRebindingContext)
{
	check(Preset && Entity && NewActor);
	TArray<FGuid> ReboundEntityIds;

	if (Entity->GetBindings().Num())
	{
		if (bUseRebindingContext)
		{
			ReboundEntityIds = RebindAllEntitiesUnderSameActor(Preset, Entity->GetBindings()[0].Get(), NewActor);
		}
		else
		{
			ReboundEntityIds = RebindAllEntitiesUnderSameActor_Legacy(Preset, Entity, NewActor);
		}
	}

	return ReboundEntityIds;
}

TMap<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>> FRemoteControlPresetRebindingManager::GroupByEntitySupportedOwnerClass(TConstArrayView<TSharedPtr<FRemoteControlEntity>> Entities) const
{
	TMap<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>> GroupedEntities;
	for (const TSharedPtr<FRemoteControlEntity>& RCEntity : Entities)
	{
		GroupedEntities.FindOrAdd(RCEntity->GetSupportedBindingClass()).Add(RCEntity);
	}
	return GroupedEntities;
}

void FRemoteControlPresetRebindingManager::ReinitializeBindings(URemoteControlPreset* Preset) const
{
	if (ensure(Preset))
	{
		for (URemoteControlBinding* Binding : Preset->Bindings)
		{
			if (URemoteControlLevelDependantBinding* LevelDependantBinding = Cast<URemoteControlLevelDependantBinding>(Binding))
			{
				LevelDependantBinding->InitializeForNewLevel();
			}
		}
	}
}

void FRemoteControlPresetRebindingManager::RebindEntitiesForClass_Legacy(UClass* Class, const TArray<TSharedPtr<FRemoteControlEntity>>& UnboundEntities)
{
	// Keep track of matched properties by keeping a record of FieldPaths hash -> Objects.
	// This is to make sure we don't bind the same property to the same object twice.
	TMap<uint32, TSet<UObject*>> MatchedProperties = Context.BoundObjectsBySupportedOwnerClass.FindOrAdd(Class);
	MatchedProperties.Reserve(MatchedProperties.Num() + UnboundEntities.Num());

	if (const TArray<UObject*>* ObjectsToConsider = Context.ObjectsGroupedByRelevantClass.Find(Class))
	{
		for (const TSharedPtr<FRemoteControlEntity>& RCEntity : UnboundEntities)
		{
			if (!RCEntity->IsBound())
			{
				if (UObject* Match = FDefaultRebindingPolicy::FindMatch({RCEntity, *ObjectsToConsider, MatchedProperties}))
				{
					MatchedProperties.FindOrAdd(RCEntity->GetUnderlyingEntityIdentifier()).Add(Match);
					RCEntity->BindObject(Match);
				}
			}
		}
	}
}

void FRemoteControlPresetRebindingManager::RebindEntitiesForActorClass_Legacy(URemoteControlPreset* Preset, UClass* Class, const TArray<TSharedPtr<FRemoteControlEntity>>& UnboundEntities)
{
	TSet<FGuid> BoundProperties;
	TSet<AActor*> BoundActors;

	if (const TArray<UObject*>* ObjectsToConsider = Context.ObjectsGroupedByRelevantClass.Find(Class))
	{
		for (const TSharedPtr<FRemoteControlEntity>& RCEntity : UnboundEntities)
		{
			if (!BoundProperties.Contains(RCEntity->GetId()))
			{
				for (UObject* ObjToConsider : *ObjectsToConsider)
				{
					if (AActor* ActorToConsider = Cast<AActor>(ObjToConsider))
					{
						if (!BoundActors.Contains(ActorToConsider))
						{
							constexpr bool bUseRebindingContext = false;
							BoundProperties.Append(RebindAllEntitiesUnderSameActor(Preset, RCEntity, ActorToConsider, bUseRebindingContext));
							BoundActors.Add(ActorToConsider);
							break;
						}
					}
				}
			}
		}
	}
}

