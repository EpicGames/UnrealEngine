// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RemoteControlPreset.h"

struct FRemoteControlEntity;

struct FFindBindingForEntityArgs
{
	const TSharedPtr<FRemoteControlEntity>& Entity;
	const TArray<UObject*>& PotentialMatches;
	const TMap<uint32, TSet<UObject*>>& ExistingEntityMatches;
};

/**
 * Default rebinding policy: Find first object that supports the entity by using the entity's supported owner class.
 */
struct FDefaultRebindingPolicy
{
	static UObject* FindMatch(const FFindBindingForEntityArgs& Args);
};

/**
 * Name based policy, attempts to find an object with a similar name in the set of available objects.
 * ie. If the old binding was bound to StaticMeshComponent0, the policy will try to bind it to
 * an object with a name that starts with StaticMeshComponent.
 *
 * If no object is found this way, it uses the default rebinding policy.
 */
struct FNameBasedRebindingPolicy
{
	static UObject* FindMatch(const FFindBindingForEntityArgs& Args);
};

/**
 * Handles rebinding RC entities that are invalid in the current level.
 */
class FRemoteControlPresetRebindingManager
{
public:
	struct FRebindingOptions
	{
		/**
		 * Whether to use the entity's binding name as a hint for finding a new object to bind.
		 * It is a slower but more accurate method to rebind entities.
		 */
		bool bUseBindingNameAsHint = false;
	};

	FRemoteControlPresetRebindingManager() = default;

	FRemoteControlPresetRebindingManager(FRebindingOptions InRebindingOptions)
        : RebindingOptions(MoveTemp(InRebindingOptions))
	{
	}

	/**
	 * Attempt rebinding a Preset's entities by first trying to find objects with the same name
	 * in the current map, then by searching for objects that support the underlying entity.
	 * @Note: Set RemoteControl.UseLegacyRebinding to 1 in order to use the legacy algorthm.
	 */
	void Rebind(URemoteControlPreset* Preset);

	/**
	 * Given a RC Entity, rebind all entities with the same owner to a new actor
	 * @return the unique identifiers of the rc entities that were rebound.
	 */
	TArray<FGuid> RebindAllEntitiesUnderSameActor(URemoteControlPreset* Preset, const TSharedPtr<FRemoteControlEntity>& Entity, AActor* NewActor, bool bUseRebindingContext);

private:
	void Rebind_Legacy(URemoteControlPreset* Preset);
	void Rebind_NewAlgo(URemoteControlPreset* Preset);

	/** Group RC Entities by the entity's supported owner class. */
	TMap<UClass*, TArray<TSharedPtr<FRemoteControlEntity>>> GroupByEntitySupportedOwnerClass(TConstArrayView<TSharedPtr<FRemoteControlEntity>> Entities) const;

	/**
	 * Attempt to rebind the existing bindings by trying to find an object of the same name in the new level.
	 * Especially useful for a duplicated level.
	 */
	void ReinitializeBindings(URemoteControlPreset* Preset) const;

	/**  Attempt to rebind all unbound entities. */
	void RebindEntitiesForClass_Legacy(UClass* Class, const TArray<TSharedPtr<FRemoteControlEntity>>& UnboundEntities);

	void RebindEntitiesForActorClass_Legacy(URemoteControlPreset* Preset, UClass* Class, const TArray<TSharedPtr<FRemoteControlEntity>>& UnboundEntities);

	TArray<FGuid> RebindAllEntitiesUnderSameActor(URemoteControlPreset* Preset, URemoteControlBinding* InitialBinding, AActor* NewActor);
	TArray<FGuid> RebindAllEntitiesUnderSameActor_Legacy(URemoteControlPreset* Preset, const TSharedPtr<FRemoteControlEntity>& Entity, AActor* NewActor);
private:
	struct FRebindingContext
	{
		void Reset()
		{
			ObjectsGroupedByRelevantClass.Reset();
			BoundObjectsBySupportedOwnerClass.Reset();
		}
		
		TMap<UClass*, TArray<UObject*>> ObjectsGroupedByRelevantClass;
		TMap<UClass*, TMap<uint32, TSet<UObject*>>> BoundObjectsBySupportedOwnerClass;
	};

private:

	/** Options to customize the rebinding behaviour. */
	FRebindingOptions RebindingOptions;
	
	/** Holds the necessary information to rebind the preset's entities. */
	FRebindingContext Context;
};