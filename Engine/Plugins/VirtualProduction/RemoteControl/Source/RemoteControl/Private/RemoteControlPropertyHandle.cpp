// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPropertyHandle.h"
#include "IRemoteControlModule.h"
#include "RemoteControlField.h"
#include "RemoteControlFieldPath.h"
#include "RemoteControlPreset.h"

#include "Algo/AllOf.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "CborWriter.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Materials/MaterialInterface.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/TextProperty.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlPropertyHandle"

namespace
{
	/** Check if the given property is Array or Set or Map */
	bool IsArrayLike(const FProperty* Property)
	{
		return (Property->IsA<FArrayProperty>()
			|| Property->IsA<FSetProperty>()
			|| Property->IsA<FMapProperty>());
	}

	/** Get inner property of Array or Set or Map */
	FProperty* GetContainerInnerProperty(const FProperty* Property)
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			return ArrayProperty->Inner;
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			return SetProperty->ElementProp;
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			return MapProperty->ValueProp;
		}

		return nullptr;
	}

	/** Add the given property and the given value to the Writer */
	template<typename ValueType>
	void AddPropertyToWriter(FCborWriter& OutWriter, const FString InPropName, const ValueType& InValue)
	{
		OutWriter.WriteValue(InPropName);
		OutWriter.WriteValue(InValue);
	}

	/**
	 * Get the property value for given property handle
	 */
	template<class PropertyType, class ValueType>
	bool GetPropertyValue(const FRemoteControlPropertyHandle& PropertyHandle, ValueType& OutValue)
	{
		FRCFieldPathInfo RCFieldPathInfo(PropertyHandle.GetFieldPath());
		const TSharedPtr<FRemoteControlProperty> RCProperty = PropertyHandle.GetRCProperty();
		const FProperty* Property = PropertyHandle.GetProperty();

		if (!ensure(RCProperty.IsValid()))
		{
			return false;
		}

		if (!ensure(Property))
		{
			return false;
		}

		TArray<UObject*> Objects = RCProperty->GetBoundObjects();
		if (!ensure(Objects.Num() && RCFieldPathInfo.Resolve(Objects[0])))
		{
			return false;
		}

		int32 ArrayIndex = PropertyHandle.GetIndexInArray();
		void* ContainerAddress = RCFieldPathInfo.GetResolvedData().ContainerAddress;
		if (const FProperty* ParentProperty = PropertyHandle.GetParentProperty())
		{
			if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentProperty))
			{
				FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, ContainerAddress);

				if (!ensure(ArrayHelper.IsValidIndex(ArrayIndex)))
				{
					return false;
				}

				const PropertyType* InnerProperty = CastField<PropertyType>(ArrayProperty->Inner);
				if (!ensure(InnerProperty))
				{
					return false;
				}

				OutValue = InnerProperty->GetPropertyValue(ArrayHelper.GetRawPtr(ArrayIndex));
				return true;
			}
			else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ParentProperty))
			{
				FScriptSetHelper_InContainer SetHelper(SetProperty, ContainerAddress);

				if (!ensure(SetHelper.IsValidIndex(ArrayIndex)))
				{
					return false;
				}

				const PropertyType* ElementProp = CastField<PropertyType>(SetProperty->ElementProp);
				if (!ensure(ElementProp))
				{
					return false;
				}

				OutValue = ElementProp->GetPropertyValue(SetHelper.GetElementPtr(ArrayIndex));
				return true;
			}
			else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ParentProperty))
			{
				FScriptMapHelper_InContainer MapHelper(MapProperty, ContainerAddress);

				if (!ensure(MapHelper.IsValidIndex(ArrayIndex)))
				{
					return false;
				}

				const PropertyType* ValueProp = CastField<PropertyType>(MapProperty->ValueProp);
				if (!ensure(ValueProp))
				{
					return false;
				}

				OutValue = ValueProp->GetPropertyValue(MapHelper.GetValuePtr(ArrayIndex));
				return true;
			}
		}

		// Primitive property values and static array values
		const PropertyType* PrimitiveProperty = CastField<PropertyType>(Property);
		if (!ensure(PrimitiveProperty))
		{
			return false;
		}

		OutValue = PrimitiveProperty->GetPropertyValue_InContainer(ContainerAddress, ArrayIndex);
		return true;
	}

	template<typename InValueType>
	void WritePropertyValue(FCborWriter& InCborWriter, const InValueType& InValue)
	{
		InCborWriter.WriteValue(InValue);
	}

	template<>
	void WritePropertyValue<UObject*>(FCborWriter& InCborWriter, UObject* const& InValue)
	{
		InCborWriter.WriteValue(InValue ? InValue->GetPathName() : FString());
	}

	/**
	 * Set property value for given property Handle
	 * Setter following standard SetObjectProperties path from IRemoteControlModule
	 * And follow the replication path
	 */
	template<typename ValueType>
	static bool SetPropertyValue(const FRemoteControlPropertyHandle& PropertyHandle, const ValueType& Value)
	{
		FRCFieldPathInfo RCFieldPathInfo(PropertyHandle.GetFieldPath());
		const TSharedPtr<FRemoteControlProperty> RCProperty = PropertyHandle.GetRCProperty();
		if (!ensure(RCProperty.IsValid()))
		{
			return false;
		}

		FProperty* Property = PropertyHandle.GetProperty();
		const FProperty* ParentProperty = PropertyHandle.GetParentProperty();
		if (!ensure(Property))
		{
			return false;
		}

		// Set Archives
		TArray<uint8> ApiBuffer;
		FMemoryWriter MemoryWriter(ApiBuffer);
		FMemoryReader MemoryReader(ApiBuffer);
		FCborWriter CborWriter(&MemoryWriter);

		// Add value to the buffer
		CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);

		if (ParentProperty && IsArrayLike(ParentProperty))
		{
			CborWriter.WriteValue(ParentProperty->GetName());
		}
		else
		{
			CborWriter.WriteValue(Property->GetName());
		}

		WritePropertyValue(CborWriter, Value);
		CborWriter.WriteContainerEnd();

		// Set a deserializer backend
		FCborStructDeserializerBackend CborStructDeserializerBackend(MemoryReader);

		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Property;
		ObjectRef.Access = PropertyHandle.ShouldGenerateTransaction() ? ERCAccess::WRITE_TRANSACTION_ACCESS : ERCAccess::WRITE_ACCESS;
		ObjectRef.PropertyPathInfo = RCFieldPathInfo;

		bool bSuccess = true;
		for (UObject* Object : RCProperty->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Set object property and follow the replication path if there are any replicators
			bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, CborStructDeserializerBackend, ERCPayloadType::Cbor, ApiBuffer);
		}

		return bSuccess;
	}

	/**
	 * Set property value for the given StructHandle
	 * Setter following standard SetObjectProperties path from IRemoteControlModule
	 * And follow the replication path
	 */
	template<typename InValueType>
	static bool SetStructPropertyValues(const FRemoteControlPropertyHandle& InPropertyHandle, const TMap<FString, InValueType>& InPropNameToValues)
	{
		FRCFieldPathInfo RCFieldPathInfo(InPropertyHandle.GetFieldPath());
		const TSharedPtr<FRemoteControlProperty> RCProperty = InPropertyHandle.GetRCProperty();
		if (!ensure(RCProperty.IsValid()))
		{
			return false;
		}
	
		FProperty* Property = InPropertyHandle.GetProperty();
		if (!ensure(Property))
		{
			return false;
		}
	
		if (!ensure(Property->IsA<FStructProperty>()))
		{
			return false;
		}

		TArray<uint8> ApiBuffer;
		FMemoryWriter MemoryWriter(ApiBuffer);
		FMemoryReader MemoryReader(ApiBuffer);
		FCborWriter CborWriter(&MemoryWriter);

		// START: Struct Writer
		CborWriter.WriteContainerStart(ECborCode::Map, -1);
		CborWriter.WriteValue(Property->GetName());

		// START: Struct properties Writer
		CborWriter.WriteContainerStart(ECborCode::Map, -1);
		for (const TPair<FString, InValueType>& PropToValue : InPropNameToValues)
		{
			AddPropertyToWriter(CborWriter, PropToValue.Key, PropToValue.Value);
		}
		// END: Struct properties Writer
		CborWriter.WriteContainerEnd();

		// END: Struct Writer
		CborWriter.WriteContainerEnd();

		FCborStructDeserializerBackend CborStructDeserializerBackend(MemoryReader);
		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Property;
		ObjectRef.Access = InPropertyHandle.ShouldGenerateTransaction() ? ERCAccess::WRITE_TRANSACTION_ACCESS : ERCAccess::WRITE_ACCESS;
		ObjectRef.PropertyPathInfo = RCFieldPathInfo;

		bool bSuccess = true;
		for (UObject* Object : RCProperty->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Set object property and follow the replication path if there are any replicators
			bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, CborStructDeserializerBackend, ERCPayloadType::Cbor, ApiBuffer);
		}

		return bSuccess;
	}
}

TSharedPtr<IRemoteControlPropertyHandle> IRemoteControlPropertyHandle::GetPropertyHandle(FName PresetName, FGuid PropertyId)
{
	// Resolve preset
	URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(PresetName);
	if (Preset == nullptr)
	{
		return nullptr;
	}

	// Resolve property
	const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin();
	if (!RCProperty.IsValid())
	{
		return nullptr;
	}

	return RCProperty->GetPropertyHandle();
}

TSharedPtr<IRemoteControlPropertyHandle> IRemoteControlPropertyHandle::GetPropertyHandle(FName PresetName, FName PropertyLabel)
{
	// Resolve preset
	URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(PresetName);
	if (Preset == nullptr)
	{
		return nullptr;
	}

	// Resolve property
	const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(Preset->GetExposedEntityId(PropertyLabel)).Pin();
	if (!RCProperty.IsValid())
	{
		return nullptr;
	}

	return RCProperty->GetPropertyHandle();
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandle::GetPropertyHandle(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
{
	TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle;

	if (FRemoteControlPropertyHandleArray::Supports(InProperty, InArrayIndex))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleArray>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleInt::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleInt>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleFloat::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleFloat>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleDouble::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleDouble>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleBool::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleBool>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleByte::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleByte>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleString::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleString>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleName::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleName>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleText::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleText>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleVector::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleVector>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleRotator::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleRotator>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleColor::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleColor>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleLinearColor::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleLinearColor>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleSet::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleSet>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleMap::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleMap>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else if (FRemoteControlPropertyHandleObject::Supports(InProperty))
	{
		PropertyHandle = MakeShared<FRemoteControlPropertyHandleObject>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}
	else
	{
		// Untyped or doesn't support getting the property directly but the property is still valid(probably struct property)
		PropertyHandle = MakeShared<FRemoteControlPropertyHandle>(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex);
	}

	return PropertyHandle;
}

bool FRemoteControlPropertyHandle::IsPropertyTypeOf(FFieldClass* ClassType) const
{
	if (const FProperty* Proeprty = GetProperty())
	{
		return Proeprty->IsA(ClassType);
	}

	return false;
}

void* FRemoteControlPropertyHandle::GetContainerAddress(int32 OwnerIndex) const
{
	TArray<void*> ContainerAddresses = GetContainerAddresses();
	if (ContainerAddresses.IsValidIndex(OwnerIndex))
	{
		return ContainerAddresses[OwnerIndex];
	}

	return nullptr;
}

TArray<void*> FRemoteControlPropertyHandle::GetContainerAddresses() const
{
	TArray<void*> ContainerAddresses;

	const TSharedPtr<FRemoteControlProperty> RCProperty = GetRCProperty();
	if (!ensure(RCProperty.IsValid()))
	{
		return ContainerAddresses;
	}

	FRCFieldPathInfo RCFieldPathInfo(FieldPath);
	TArray<UObject*> Objects = RCProperty->GetBoundObjects();
	ContainerAddresses.Reserve(Objects.Num());

	for (UObject* Owner : Objects)
	{
		RCFieldPathInfo.Resolve(Owner);

		ContainerAddresses.Add(RCFieldPathInfo.GetResolvedData().ContainerAddress);
	}

	return ContainerAddresses;
}

#define IMPLEMENT_PROPERTY_ACCESSOR_STR( ValueType ) \
	bool FRemoteControlPropertyHandle::SetValue( const ValueType& InValue ) { return false; } \
	bool FRemoteControlPropertyHandle::GetValue( ValueType& OutValues ) const { return false; }

IMPLEMENT_PROPERTY_ACCESSOR_STR(FString)
IMPLEMENT_PROPERTY_ACCESSOR_STR(FText)
IMPLEMENT_PROPERTY_ACCESSOR_STR(FName)

#undef IMPLEMENT_PROPERTY_ACCESSOR_STR

#define IMPLEMENT_PROPERTY_ACCESSOR( ValueType ) \
	bool FRemoteControlPropertyHandle::SetValue( ValueType InValue ) { return false; } \
	bool FRemoteControlPropertyHandle::GetValue( ValueType& OutValues ) const { return false; }

IMPLEMENT_PROPERTY_ACCESSOR(bool)
IMPLEMENT_PROPERTY_ACCESSOR(int8)
IMPLEMENT_PROPERTY_ACCESSOR(int16)
IMPLEMENT_PROPERTY_ACCESSOR(int32)
IMPLEMENT_PROPERTY_ACCESSOR(int64)
IMPLEMENT_PROPERTY_ACCESSOR(uint8)
IMPLEMENT_PROPERTY_ACCESSOR(uint16)
IMPLEMENT_PROPERTY_ACCESSOR(uint32)
IMPLEMENT_PROPERTY_ACCESSOR(uint64)
IMPLEMENT_PROPERTY_ACCESSOR(float)
IMPLEMENT_PROPERTY_ACCESSOR(double)
IMPLEMENT_PROPERTY_ACCESSOR(FVector)
IMPLEMENT_PROPERTY_ACCESSOR(FVector2D)
IMPLEMENT_PROPERTY_ACCESSOR(FVector4)
IMPLEMENT_PROPERTY_ACCESSOR(FQuat)
IMPLEMENT_PROPERTY_ACCESSOR(FRotator)
IMPLEMENT_PROPERTY_ACCESSOR(FColor)
IMPLEMENT_PROPERTY_ACCESSOR(FLinearColor)
#undef IMPLEMENT_PROPERTY_ACCESSOR


void* FRemoteControlPropertyHandle::GetValuePtr(void* InContainerAddress) const
{
	const FProperty* Property = GetProperty();
	const FProperty* ParentProperty = GetParentProperty();

	if (!ensure(Property))
	{
		return nullptr;
	}

	if (ParentProperty)
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ParentProperty))
		{
			FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, InContainerAddress);
			return ArrayHelper.IsValidIndex(ArrayIndex) ? ArrayHelper.GetRawPtr(ArrayIndex) : nullptr;
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(ParentProperty))
		{
			FScriptSetHelper_InContainer SetHelper(SetProperty, InContainerAddress);
			if (!ensure(SetHelper.IsValidIndex(ArrayIndex)))
			{
				return nullptr;
			}
			return SetHelper.GetElementPtr(ArrayIndex);
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(ParentProperty))
		{
			FScriptMapHelper_InContainer MapHelper(MapProperty, InContainerAddress);
			if (!ensure(MapHelper.IsValidIndex(ArrayIndex)))
			{
				return nullptr;
			}
			return MapHelper.GetValuePtr(ArrayIndex);
		}
	}

	return Property->ContainerPtrToValuePtr<void>(InContainerAddress, ArrayIndex);
}

FRemoteControlPropertyHandle::FRemoteControlPropertyHandle(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
	: RCPropertyPtr(InRCProperty)
	, PropertyPtr(InProperty)
	, ParentPropertyPtr(InParentProperty)
	, ParentFieldPath(InParentFieldPath)
	, ArrayIndex(InArrayIndex)
	, bGenerateTransaction(false)
{
	// Set a FieldPath
	FString ThisFieldPath = InProperty->GetName();
	if (InParentProperty == nullptr || InParentProperty->ArrayDim != 1)
	{
		if (const UScriptStruct* Struct = Cast<UScriptStruct>(InProperty->GetOwnerUObject()))
		{
			FieldPath = InRCProperty->FieldPathInfo.ToString();
		}
		else
		{
			FieldPath = ThisFieldPath;
		}
	}
	else
	{
		const FString& ParentFullPath = GetParentFieldPath();
		if (IsArrayLike(InParentProperty))
		{
			FieldPath = FString::Printf(TEXT("%s[%d]"), *ParentFullPath, InArrayIndex);
		}
		else if (const FStructProperty* ParentStructProperty = CastField<FStructProperty>(InParentProperty))
		{
			FieldPath = FString::Printf(TEXT("%s.%s"), *ParentFullPath, *ThisFieldPath);
		}
		else
		{
			ensure(false);
		}
	}

	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
		{
			// Add struct child handle
			Children.Add(GetPropertyHandle(InRCProperty, *It, InProperty, FieldPath, InArrayIndex));
		}
	}
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandle::GetChildHandle(FName ChildName, bool bRecurse) const
{
	// Search Children
	for (int32 ChildIndex = 0; ChildIndex < Children.Num(); ChildIndex++)
	{
		const TSharedPtr<IRemoteControlPropertyHandle>& ChildNode = Children[ChildIndex];

		if (ChildNode->GetProperty() && ChildNode->GetProperty()->GetFName() == ChildName)
		{
			return ChildNode;
		}
		else if (bRecurse)
		{
			TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle = ChildNode->GetChildHandle(ChildName, bRecurse);

			if (PropertyHandle.IsValid())
			{
				return PropertyHandle;
			}
		}
	}

	return nullptr;
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandle::GetChildHandle(int32 Index) const
{
	if (Children.IsValidIndex(Index))
	{
		return Children[Index];
	}

	return nullptr;
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandle::GetChildHandleByFieldPath(const FString& InFieldPath)
{
	FProperty* ExposedProperty = GetProperty();
	if (!ensure(ExposedProperty))
	{
		return nullptr;
	}

	TSharedPtr<FRemoteControlProperty> RCProperty = GetRCProperty();
	if (!ensure(RCProperty))
	{
		return nullptr;
	}

	// Return nullptr if there is no FieldPath or it is the same with FieldPath from this handle
	if (InFieldPath.IsEmpty() || FieldPath == *InFieldPath)
	{
		return nullptr;
	}

	// Resolve field path
	FRCFieldPathInfo FieldPathInfo(InFieldPath);
	int32 SegmentsNum = FieldPathInfo.Segments.Num();
	if (!ensure(SegmentsNum > 0))
	{
		return nullptr;
	}

	TArray<UObject*> Objects = RCProperty->GetBoundObjects();
	if (!Objects.Num() || !FieldPathInfo.Resolve(Objects[0]))
	{
		return nullptr;
	}

	const FRCFieldPathSegment& Segment = FieldPathInfo.Segments[SegmentsNum - 1];
	FProperty* SegmentProperty = Segment.ResolvedData.Field;

	// Get Property Handle with container parent
	if (IsArrayLike(SegmentProperty) && Segment.ArrayIndex != INDEX_NONE)
	{
		FString ParentPath = FieldPathInfo.ToString();
		int32 OpenBracketIndex;
		if (!ensure(ParentPath.FindLastChar('[', OpenBracketIndex)))
		{
			return nullptr;
		}
		ParentPath = ParentPath.Left(OpenBracketIndex);

		FProperty* ChildProperty = GetContainerInnerProperty(SegmentProperty);
		if (!ensure(ChildProperty))
		{
			return nullptr;
		}

		return FRemoteControlPropertyHandle::GetPropertyHandle(RCProperty, ChildProperty, SegmentProperty, ParentPath, Segment.ArrayIndex);
	}
	// Get Property Handle with struct parent
	else
	{
		if (!ensure(SegmentsNum > 1))
		{
			return nullptr;
		}

		const FRCFieldPathSegment& ParentSegment = FieldPathInfo.Segments[SegmentsNum - 2];

		const FString ParentPath = FieldPathInfo.ToString(SegmentsNum - 1).LeftChop(1);

		if (FProperty* ParentProperty = IsArrayLike(ParentSegment.ResolvedData.Field) ?
			CastField<FStructProperty>(GetContainerInnerProperty(ParentSegment.ResolvedData.Field)) :
			CastField<FStructProperty>(ParentSegment.ResolvedData.Field))
		{
			return FRemoteControlPropertyHandle::GetPropertyHandle(RCProperty, SegmentProperty, ParentProperty, ParentPath, SegmentProperty->ArrayDim != 1 ? -1 : 0);
		}
	}

	return nullptr;
}

TSharedPtr<FRemoteControlProperty> FRemoteControlPropertyHandle::GetRCProperty() const
{
	return RCPropertyPtr.Pin();
}

#define IMPLEMENT_PROPERTY_VALUE( ClassName ) \
	ClassName::ClassName(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex) \
	: FRemoteControlPropertyHandle(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex )  \
	{}

IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleInt)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleFloat)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleDouble)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleBool)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleByte)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleText)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleString)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleName)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleArray)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleSet)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleMap)
IMPLEMENT_PROPERTY_VALUE(FRemoteControlPropertyHandleObject)

#undef IMPLEMENT_PROPERTY_VALUE

bool FRemoteControlPropertyHandleInt::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA<FInt8Property>()
		|| InProperty->IsA<FInt16Property>()
		|| InProperty->IsA<FIntProperty>()
		|| InProperty->IsA<FInt64Property>()
		|| InProperty->IsA<FUInt16Property>()
		|| InProperty->IsA<FUInt32Property>()
		|| InProperty->IsA<FUInt64Property>();

}

bool FRemoteControlPropertyHandleInt::GetValue(int8& OutValue) const
{
	return GetPropertyValue<FInt8Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(int8 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(int16& OutValue) const
{
	return GetPropertyValue<FInt16Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(int16 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(int32& OutValue) const
{
	return GetPropertyValue<FIntProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(int32 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(int64& OutValue) const
{
	return GetPropertyValue<FInt64Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(int64 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(uint16& OutValue) const
{
	return GetPropertyValue<FUInt16Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(uint16 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(uint32& OutValue) const
{
	return GetPropertyValue<FUInt32Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(uint32 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleInt::GetValue(uint64& OutValue) const
{
	return GetPropertyValue<FUInt64Property>(*this, OutValue);
}

bool FRemoteControlPropertyHandleInt::SetValue(uint64 InValue)
{
	return SetPropertyValue(*this, (int64)InValue);
}

bool FRemoteControlPropertyHandleArray::Supports(const FProperty* InProperty, int32 InArrayIndex )
{
	if (InProperty == nullptr)
	{
		return false;
	}

	// Static array or dynamic array
	return ((InProperty && InProperty->ArrayDim != 1 && InArrayIndex == -1) || CastField<const FArrayProperty>(InProperty));
}

int32 FRemoteControlPropertyHandleArray::GetNumElements() const
{
	const FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return 0;
	}

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		// Container property supports only single container address
		TArray<void*> ContainerAddresses = GetContainerAddresses();
		if (!ensure(ContainerAddresses.Num() == 1))
		{
			return 0;
		}

		FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, ContainerAddresses[0]);
		return ArrayHelper.Num();
	}
	else if (Property->ArrayDim != 1)
	{
		return Property->ArrayDim;
	}

	ensure(false);
	return 0;
}

TSharedPtr<IRemoteControlPropertyHandleArray> FRemoteControlPropertyHandleArray::AsArray()
{
	return SharedThis(this);
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandleArray::GetElement(int32 Index)
{
	const TSharedPtr<FRemoteControlProperty> RCProperty = GetRCProperty();
	if (!ensure(RCProperty.IsValid()))
	{
		return nullptr;
	}

	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return nullptr;
	}

	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		// Container property supports only single container address
		TArray<void*> ContainerAddresses = GetContainerAddresses();
		if (!ensure(ContainerAddresses.Num() == 1))
		{
			return nullptr;
		}

		const FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, ContainerAddresses[0]);
		if (ArrayHelper.IsValidIndex(Index))
		{
			return GetPropertyHandle(RCProperty, ArrayProperty->Inner, Property, FieldPath, Index);
		}

	}
	else if (Property->ArrayDim != 1)
	{
		if (Index <= Property->ArrayDim && Index >= 0)
		{
			return GetPropertyHandle(RCProperty, Property, Property, FieldPath, Index);
		}
	}

	return nullptr;
}

bool FRemoteControlPropertyHandleSet::Supports(const FProperty* InProperty)
{
	return CastField<const FSetProperty>(InProperty) != nullptr;
}

TSharedPtr<IRemoteControlPropertyHandleSet> FRemoteControlPropertyHandleSet::AsSet()
{
	return SharedThis(this);
}

int32 FRemoteControlPropertyHandleSet::GetNumElements()
{
	// Container property supports only single container address
	TArray<void*> ContainerAddresses = GetContainerAddresses();
	if (!ensure(ContainerAddresses.Num() == 1))
	{
		return 0;
	}

	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return 0;
	}

	const FSetProperty* SetProperty = CastField<FSetProperty>(Property);
	if (!ensure(SetProperty))
	{
		return 0;
	}

	FScriptSetHelper_InContainer SetHelper(SetProperty, ContainerAddresses[0]);

	return SetHelper.Num();
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandleSet::FindElement(const void* ElementToFind)
{
	// Container property supports only single container address
	TArray<void*> ContainerAddresses = GetContainerAddresses();
	if (!ensure(ContainerAddresses.Num() == 1))
	{
		return nullptr;
	}

	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return nullptr;
	}


	const FSetProperty* SetProperty = CastField<FSetProperty>(Property);
	if (!ensure(SetProperty))
	{
		return nullptr;
	}
	const FScriptSetHelper_InContainer SetHelper(SetProperty, ContainerAddresses[0]);
	int32 ElementIndex = SetHelper.FindElementIndex(ElementToFind);
	if (SetHelper.IsValidIndex(ElementIndex))
	{
		const TSharedPtr<FRemoteControlProperty> RCProperty = GetRCProperty();
		if (!ensure(RCProperty.IsValid()))
		{
			return nullptr;
		}

		return GetPropertyHandle(RCProperty, SetProperty->ElementProp, Property, FieldPath, ElementIndex);
	}

	return nullptr;
}

bool FRemoteControlPropertyHandleMap::Supports(const FProperty* InProperty)
{
	return CastField<const FMapProperty>(InProperty) != nullptr;
}

TSharedPtr<IRemoteControlPropertyHandleMap> FRemoteControlPropertyHandleMap::AsMap()
{
	return SharedThis(this);
}

int32 FRemoteControlPropertyHandleMap::GetNumElements()
{
	// Container property supports only single container address
	TArray<void*> ContainerAddresses = GetContainerAddresses();
	if (!ensure(ContainerAddresses.Num() == 1))
	{
		return 0;
	}

	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return 0;
	}

	const FMapProperty* MapProperty = CastField<FMapProperty>(Property);
	if (!ensure(MapProperty))
	{
		return 0;
	}

	FScriptMapHelper_InContainer MapHelper(MapProperty, ContainerAddresses[0]);
	return MapHelper.Num();
}

TSharedPtr<IRemoteControlPropertyHandle> FRemoteControlPropertyHandleMap::Find(const void* KeyPtr)
{
	// Container property supports only single container address
	TArray<void*> ContainerAddresses = GetContainerAddresses();
	if (!ensure(ContainerAddresses.Num() == 1))
	{
		return nullptr;
	}

	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return nullptr;
	}

	const TSharedPtr<FRemoteControlProperty> RCProperty = GetRCProperty();
	if (!ensure(RCProperty.IsValid()))
	{
		return nullptr;
	}

	const FMapProperty* MapProperty = CastField<FMapProperty>(Property);
	if (!ensure(MapProperty))
	{
		return nullptr;
	}

	FScriptMapHelper_InContainer MapHelper(MapProperty, ContainerAddresses[0]);
	int32 ElementIndex = MapHelper.FindMapIndexWithKey(KeyPtr);
	if (MapHelper.IsValidIndex(ElementIndex))
	{
		return GetPropertyHandle(RCProperty, MapProperty->ValueProp, Property, FieldPath, ElementIndex);
	}

	return nullptr;
}

bool FRemoteControlPropertyHandleString::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA(FStrProperty::StaticClass());
}

bool FRemoteControlPropertyHandleString::GetValue(FString& OutValue) const
{
	return GetPropertyValue<FStrProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleString::SetValue(const FString& InValue)
{
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleString::SetValue(const TCHAR* InValue)
{
	return SetPropertyValue(*this, FString(InValue));
}

bool FRemoteControlPropertyHandleString::GetValue(FName& OutValue) const
{
	FString OutString;

	if (GetPropertyValue<FStrProperty>(*this, OutString))
	{
		OutValue = *OutString;
		return true;
	}

	return false;
}

bool FRemoteControlPropertyHandleString::SetValue(const FName& InValue)
{
	return SetPropertyValue(*this, InValue.ToString());
}

bool FRemoteControlPropertyHandleName::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA(FNameProperty::StaticClass()) && InProperty->GetFName() != NAME_InitialState;
}

bool FRemoteControlPropertyHandleName::GetValue(FString& OutValue) const
{
	FName OutName;

	if (GetPropertyValue<FNameProperty>(*this, OutName))
	{
		OutValue = OutName.ToString();
		return true;
	}

	return false;
}

bool FRemoteControlPropertyHandleName::SetValue(const FString& InValue)
{
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleName::SetValue(const TCHAR* InValue)
{
	return SetPropertyValue(*this, FString(InValue));
}

bool FRemoteControlPropertyHandleName::GetValue(FName& OutValue) const
{
	 return GetPropertyValue<FNameProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleName::SetValue(const FName& InValue)
{
	return SetPropertyValue(*this, InValue.ToString());
}

bool FRemoteControlPropertyHandleFloat::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA(FFloatProperty::StaticClass());
}

bool FRemoteControlPropertyHandleFloat::GetValue(float& OutValue) const
{
	return GetPropertyValue<FFloatProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleFloat::SetValue(float InValue)
{
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleDouble::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA(FDoubleProperty::StaticClass());
}

bool FRemoteControlPropertyHandleDouble::GetValue(double& OutValue) const
{
	return GetPropertyValue<FDoubleProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleDouble::SetValue(double InValue)
{
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleBool::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA(FBoolProperty::StaticClass());
}

bool FRemoteControlPropertyHandleBool::GetValue(bool& OutValue) const
{
	return GetPropertyValue<FBoolProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleBool::SetValue(bool InValue)
{
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleByte::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	return InProperty->IsA<FByteProperty>() || InProperty->IsA<FEnumProperty>();
}

bool FRemoteControlPropertyHandleByte::GetValue(uint8& OutValue) const
{
	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return false;
	}

	if (Property->IsA<FByteProperty>())
	{
		return GetPropertyValue<FByteProperty>(*this, OutValue);
	}
	else if (Property->IsA<FEnumProperty>())
	{
		void* FirstContainerAddress = GetContainerAddress();
		if (!ensure(FirstContainerAddress))
		{
			return false;
		}

		void* ValueData = GetValuePtr(FirstContainerAddress);
		if (!ensure(ValueData))
		{
			return false;
		}

		const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property);
		if (!ensure(EnumProperty))
		{
			return false;
		}

		OutValue = EnumProperty->GetUnderlyingProperty()->GetUnsignedIntPropertyValue(ValueData);

		return true;
	}

	return false;
}

bool FRemoteControlPropertyHandleByte::SetValue(uint8 InValue)
{
	FProperty* Property = GetProperty();
	if (!ensure(Property))
	{
		return false;
	}

	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
	{
		return SetPropertyValue(*this, EnumProperty->GetEnum()->GetNameStringByValue(int64(InValue)));
	}
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
	{
		if (ByteProperty->IsEnum())
		{
			return SetPropertyValue(*this, ByteProperty->Enum->GetNameStringByValue(int64(InValue)));
		}
		else
		{
			return SetPropertyValue(*this, (int64)InValue);
		}
	}

	return false;
}

bool FRemoteControlPropertyHandleText::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	// Supported if the property is a text property only
	return InProperty->IsA(FTextProperty::StaticClass());
}

bool FRemoteControlPropertyHandleText::GetValue(FText& OutValue) const
{
	return GetPropertyValue<FTextProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleText::SetValue(const FText& InValue)
{	
	FString TextValueString;
	FTextStringHelper::WriteToBuffer(TextValueString, InValue);
	SetPropertyValue(*this, TextValueString);
	return true;
}

bool FRemoteControlPropertyHandleText::SetValue(const FString& InValue)
{
	SetPropertyValue(*this, InValue);
	return true;
}

bool FRemoteControlPropertyHandleText::SetValue(const TCHAR* InValue)
{
	SetPropertyValue(*this, FString(InValue));
	return true;
}

FRemoteControlPropertyHandleVector::FRemoteControlPropertyHandleVector(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
	: FRemoteControlPropertyHandle(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex)
{
	int32 NumChildren = GetNumChildren();

	const bool bRecurse = false;
	// A vector is a struct property that has 3 children.  We get/set the values from the children
	VectorComponents.Add(GetChildHandle("X", bRecurse));

	VectorComponents.Add(GetChildHandle("Y", bRecurse));

	if (NumChildren > 2)
	{
		// at least a 3 component vector
		VectorComponents.Add(GetChildHandle("Z", bRecurse));
	}
	if (NumChildren > 3)
	{
		// a 4 component vector
		VectorComponents.Add(GetChildHandle("W", bRecurse));
	}
}

bool FRemoteControlPropertyHandleVector::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(InProperty);

	bool bSupported = false;
	if (StructProp && StructProp->Struct)
	{
		FName StructName = StructProp->Struct->GetFName();

		bSupported = StructName == NAME_Vector ||
			StructName == NAME_Vector2D ||
			StructName == NAME_Vector4 ||
			StructName == NAME_Quat;
	}

	return bSupported;
}

bool FRemoteControlPropertyHandleVector::GetValue(FVector& OutValue) const
{
	if (!ensure(VectorComponents.Num() != 3 || AreComponentsValid()))
	{
		return false;
	}

	// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
	bool ResX = VectorComponents[0].Pin()->GetValue(OutValue.X);
	bool ResY = VectorComponents[1].Pin()->GetValue(OutValue.Y);
	bool ResZ = VectorComponents[2].Pin()->GetValue(OutValue.Z);

	if (ResX == false || ResY == false || ResZ == false)
	{
		// If reading any value failed the entire value fails
		return false;
	}
	else
	{
		return true;
	}
}

bool FRemoteControlPropertyHandleVector::SetValue(FVector InValue)
{
	if (!ensure(VectorComponents.Num() != 3 || AreComponentsValid()))
	{
		return false;
	}

	// To set the value from the vector we set each child. 
	bool ResX = VectorComponents[0].Pin()->SetValue(InValue.X);
	bool ResY = VectorComponents[1].Pin()->SetValue(InValue.Y);
	bool ResZ = VectorComponents[2].Pin()->SetValue(InValue.Z);

	if (ResX == false || ResY == false || ResZ == false)
	{
		return false;
	}
	
	return true;
}

bool FRemoteControlPropertyHandleVector::GetValue(FVector2D& OutValue) const
{
	if (!ensure(VectorComponents.Num() != 2 || AreComponentsValid()))
	{
		return false;
	}

	// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
	bool ResX = VectorComponents[0].Pin()->GetValue(OutValue.X);
	bool ResY = VectorComponents[1].Pin()->GetValue(OutValue.Y);

	if (ResX == false || ResY == false)
	{
		// If reading any value failed the entire value fails
		return false;
	}
	else
	{
		return true;
	}
}

bool FRemoteControlPropertyHandleVector::SetValue(FVector2D InValue)
{
	if (!ensure(VectorComponents.Num() != 2 || AreComponentsValid()))
	{
		return false;
	}

	// To set the value from the vector we set each child. 
	bool ResX = VectorComponents[0].Pin()->SetValue(InValue.X);
	bool ResY = VectorComponents[1].Pin()->SetValue(InValue.Y);

	if (ResX == false || ResY == false)
	{
		return false;
	}

	return true;
}

bool FRemoteControlPropertyHandleVector::GetValue(FVector4& OutValue) const
{
	if (!ensure(VectorComponents.Num() != 4 || AreComponentsValid()))
	{
		return false;
	}

	// To get the value from the vector we read each child.  If reading a child fails, the value for that component is not set
	bool ResX = VectorComponents[0].Pin()->GetValue(OutValue.X);
	bool ResY = VectorComponents[1].Pin()->GetValue(OutValue.Y);
	bool ResZ = VectorComponents[2].Pin()->GetValue(OutValue.Z);
	bool ResW = VectorComponents[3].Pin()->GetValue(OutValue.W);

	if (ResX == false || ResY == false || ResZ == false || ResW == false)
	{
		// If reading any value failed the entire value fails
		return false;
	}
	else
	{
		return true;
	}
}

bool FRemoteControlPropertyHandleVector::SetValue(FVector4 InValue)
{
	if (!ensure(VectorComponents.Num() != 4 || AreComponentsValid()))
	{
		return false;
	}

	// To set the value from the vector we set each child. 
	bool ResX = VectorComponents[0].Pin()->SetValue(InValue.X);
	bool ResY = VectorComponents[1].Pin()->SetValue(InValue.Y);
	bool ResZ = VectorComponents[2].Pin()->SetValue(InValue.Z);
	bool ResW = VectorComponents[3].Pin()->SetValue(InValue.W);

	if (ResX == false || ResY == false || ResZ == false || ResW == false)
	{
		return false;
	}

	return true;
}

bool FRemoteControlPropertyHandleVector::GetValue(FQuat& OutValue) const
{
	FVector4 VectorProxy;
	bool Res = GetValue(VectorProxy);
	if (Res == true)
	{
		OutValue.X = VectorProxy.X;
		OutValue.Y = VectorProxy.Y;
		OutValue.Z = VectorProxy.Z;
		OutValue.W = VectorProxy.W;
	}

	return Res;
}

bool FRemoteControlPropertyHandleVector::SetValue(FQuat InValue)
{
	FVector4 VectorProxy;
	VectorProxy.X = InValue.X;
	VectorProxy.Y = InValue.Y;
	VectorProxy.Z = InValue.Z;
	VectorProxy.W = InValue.W;

	return SetValue(VectorProxy);
}

bool FRemoteControlPropertyHandleVector::SetX(float InValue)
{
	if (ensure(VectorComponents.IsValidIndex(0) && VectorComponents[0].Pin().IsValid()))
	{
		return VectorComponents[0].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleVector::SetY(float InValue)
{
	if (ensure(VectorComponents.IsValidIndex(1) && VectorComponents[1].Pin().IsValid()))
	{
		return VectorComponents[1].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleVector::SetZ(float InValue)
{
	if (ensure(VectorComponents.IsValidIndex(2) && VectorComponents[2].Pin().IsValid()))
	{
		return VectorComponents[2].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleVector::SetW(float InValue)
{
	if (ensure(VectorComponents.IsValidIndex(3) && VectorComponents[3].Pin().IsValid()))
	{
		return VectorComponents[3].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleVector::AreComponentsValid() const
{
	for (TWeakPtr<IRemoteControlPropertyHandle> VectorComponentPtr : VectorComponents)
	{
		if (!VectorComponentPtr.Pin().IsValid())
		{
			return false;
		}
	}

	return true;
}

FRemoteControlPropertyHandleRotator::FRemoteControlPropertyHandleRotator( const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
	: FRemoteControlPropertyHandle(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex)
{
	const bool bRecurse = false;

	RollValue = GetChildHandle("Roll", bRecurse);
	PitchValue = GetChildHandle("Pitch", bRecurse);
	YawValue = GetChildHandle("Yaw", bRecurse);
}

bool FRemoteControlPropertyHandleRotator::Supports(const FProperty* InProperty)
{
	if (InProperty == nullptr)
	{
		return false;
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(InProperty);
	return StructProp && StructProp->Struct->GetFName() == NAME_Rotator;
}

bool FRemoteControlPropertyHandleRotator::GetValue(FRotator& OutValue) const
{
	if (!ensure(AreComponentsValid()))
	{
		return false;
	}

	// To get the value from the rotator we read each child.  If reading a child fails, the value for that component is not set
	bool ResR = RollValue.Pin()->GetValue(OutValue.Roll);
	bool ResP = PitchValue.Pin()->GetValue(OutValue.Pitch);
	bool ResY = YawValue.Pin()->GetValue(OutValue.Yaw);

	if (ResR == false || ResP == false || ResY == false)
	{
		return false;
	}

	return true;
}

bool FRemoteControlPropertyHandleRotator::SetValue(FRotator InValue)
{
	if (!ensure(AreComponentsValid()))
	{
		return false;
	}

	const FProperty* RollProperty = RollValue.Pin()->GetProperty();
	const FProperty* PitchProperty = PitchValue.Pin()->GetProperty();
	const FProperty* YawProperty = YawValue.Pin()->GetProperty();
	if (!(RollProperty && PitchProperty && YawProperty))
	{
		return false;
	}

	TMap<FString, double> PropertyToValues;
	PropertyToValues.Add(PitchProperty->GetName(), InValue.Pitch);
	PropertyToValues.Add(YawProperty->GetName(), InValue.Yaw);
	PropertyToValues.Add(RollProperty->GetName(), InValue.Roll);

	bool Res = SetStructPropertyValues(*this, MoveTemp(PropertyToValues));
	if (Res == false)
	{
		return false;
	}

	return true;
}

bool FRemoteControlPropertyHandleRotator::SetRoll(float InValue)
{
	if (ensure(RollValue.Pin().IsValid()))
	{
		return RollValue.Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleRotator::SetPitch(float InValue)
{
	if (ensure(PitchValue.Pin().IsValid()))
	{
		return PitchValue.Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleRotator::SetYaw(float InValue)
{
	if (ensure(YawValue.Pin().IsValid()))
	{
		return YawValue.Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleRotator::AreComponentsValid() const
{
	return RollValue.Pin().IsValid() && PitchValue.Pin().IsValid() && YawValue.Pin().IsValid();
}

FRemoteControlPropertyHandleColor::FRemoteControlPropertyHandleColor(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
	: FRemoteControlPropertyHandle(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex)
{
	constexpr bool bRecurse = false;

	ColorComponents.Add(GetChildHandle("R", bRecurse));
	ColorComponents.Add(GetChildHandle("G", bRecurse));
	ColorComponents.Add(GetChildHandle("B", bRecurse));
	ColorComponents.Add(GetChildHandle("A", bRecurse));
}

bool FRemoteControlPropertyHandleColor::Supports(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return false;
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(InProperty);

	bool bSupported = false;
	if (StructProp && StructProp->Struct)
	{
		const FName StructName = StructProp->Struct->GetFName();
		bSupported = StructName == NAME_Color;
	}

	return bSupported;
}

bool FRemoteControlPropertyHandleColor::GetValue(FColor& OutValue) const
{
	if (!ensure(ColorComponents.Num() == 4 && AreComponentsValid()))
	{
		return false;
	}

	const bool ResR = ColorComponents[0].Pin()->GetValue(OutValue.R);
	const bool ResG = ColorComponents[1].Pin()->GetValue(OutValue.G);
	const bool ResB = ColorComponents[2].Pin()->GetValue(OutValue.B);
	const bool ResA = ColorComponents[3].Pin()->GetValue(OutValue.A);

	return ResR && ResG && ResB && ResA;
}

bool FRemoteControlPropertyHandleColor::SetValue(FColor InValue)
{
	if (!ensure(ColorComponents.Num() == 4 && AreComponentsValid()))
	{
		return false;
	}

	const bool ResR = ColorComponents[0].Pin()->SetValue(InValue.R);
	const bool ResG = ColorComponents[1].Pin()->SetValue(InValue.G);
	const bool ResB = ColorComponents[2].Pin()->SetValue(InValue.B);
	const bool ResA = ColorComponents[3].Pin()->SetValue(InValue.A);

	return ResR && ResG && ResB && ResA;
}

bool FRemoteControlPropertyHandleColor::SetR(uint8 InValue)
{
	if (ensure(ColorComponents.IsValidIndex(0) && ColorComponents[0].Pin().IsValid()))
	{
		return ColorComponents[0].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleColor::SetG(uint8 InValue)
{
	if (ensure(ColorComponents.IsValidIndex(1) && ColorComponents[1].Pin().IsValid()))
	{
		return ColorComponents[1].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleColor::SetB(uint8 InValue)
{
	if (ensure(ColorComponents.IsValidIndex(2) && ColorComponents[2].Pin().IsValid()))
	{
		return ColorComponents[2].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleColor::SetA(uint8 InValue)
{
	if (ensure(ColorComponents.IsValidIndex(3) && ColorComponents[3].Pin().IsValid()))
	{
		return ColorComponents[3].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleColor::AreComponentsValid() const
{
	return Algo::AllOf(ColorComponents, [](const TWeakPtr<IRemoteControlPropertyHandle> ColorComponentPtr)
	{
		return ColorComponentPtr.Pin().IsValid();
	});
}

FRemoteControlPropertyHandleLinearColor::FRemoteControlPropertyHandleLinearColor(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex)
	: FRemoteControlPropertyHandle(InRCProperty, InProperty, InParentProperty, InParentFieldPath, InArrayIndex)
{
	constexpr bool bRecurse = false;

	ColorComponents.Add(GetChildHandle("R", bRecurse));
	ColorComponents.Add(GetChildHandle("G", bRecurse));
	ColorComponents.Add(GetChildHandle("B", bRecurse));
	ColorComponents.Add(GetChildHandle("A", bRecurse));
}

bool FRemoteControlPropertyHandleLinearColor::Supports(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return false;
	}

	const FStructProperty* StructProp = CastField<FStructProperty>(InProperty);

	bool bSupported = false;
	if (StructProp && StructProp->Struct)
	{
		const FName StructName = StructProp->Struct->GetFName();
		bSupported = StructName == NAME_LinearColor;
	}

	return bSupported;
}

bool FRemoteControlPropertyHandleLinearColor::GetValue(FLinearColor& OutValue) const
{
	if (!ensure(ColorComponents.Num() == 4 && AreComponentsValid()))
	{
		return false;
	}

	const bool ResR = ColorComponents[0].Pin()->GetValue(OutValue.R);
	const bool ResG = ColorComponents[1].Pin()->GetValue(OutValue.G);
	const bool ResB = ColorComponents[2].Pin()->GetValue(OutValue.B);
	const bool ResA = ColorComponents[3].Pin()->GetValue(OutValue.A);

	return ResR && ResG && ResB && ResA;
}

bool FRemoteControlPropertyHandleLinearColor::SetValue(FLinearColor InValue)
{
	if (!ensure(ColorComponents.Num() == 4 && AreComponentsValid()))
	{
		return false;
	}

	const bool ResR = ColorComponents[0].Pin()->SetValue(InValue.R);
	const bool ResG = ColorComponents[1].Pin()->SetValue(InValue.G);
	const bool ResB = ColorComponents[2].Pin()->SetValue(InValue.B);
	const bool ResA = ColorComponents[3].Pin()->SetValue(InValue.A);

	return ResR && ResG && ResB && ResA;
}

bool FRemoteControlPropertyHandleLinearColor::SetR(float InValue)
{
	if (ensure(ColorComponents.IsValidIndex(0) && ColorComponents[0].Pin().IsValid()))
	{
		return ColorComponents[0].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleLinearColor::SetG(float InValue)
{
	if (ensure(ColorComponents.IsValidIndex(1) && ColorComponents[1].Pin().IsValid()))
	{
		return ColorComponents[1].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleLinearColor::SetB(float InValue)
{
	if (ensure(ColorComponents.IsValidIndex(2) && ColorComponents[2].Pin().IsValid()))
	{
		return ColorComponents[2].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleLinearColor::SetA(float InValue)
{
	if (ensure(ColorComponents.IsValidIndex(3) && ColorComponents[3].Pin().IsValid()))
	{
		return ColorComponents[3].Pin()->SetValue(InValue);
	}

	return false;
}

bool FRemoteControlPropertyHandleLinearColor::AreComponentsValid() const
{
	return Algo::AllOf(ColorComponents, [](const TWeakPtr<IRemoteControlPropertyHandle> ColorComponentPtr)
	{
		return ColorComponentPtr.Pin().IsValid();
	});
}

bool FRemoteControlPropertyHandleObject::SetValue(UObject* InValue)
{
	if (!InValue)
	{
		return false;
	}
	return SetPropertyValue(*this, InValue);
}

bool FRemoteControlPropertyHandleObject::GetValue(UObject*& OutValue) const
{
	return GetPropertyValue<FObjectProperty>(*this, OutValue);
}

bool FRemoteControlPropertyHandleObject::SetValueInArray(UObject* InValue, int32 InIndex)
{
	// Override materials e.g for Static Mesh are still considered array elements, so we need to
	// address setting their value as array elements, even if the exposed property looks like a material
	if (InValue->IsA(UMaterialInterface::StaticClass()))
	{
		return SetMaterialValue(InValue, InIndex);
	}

	return false;
}

bool FRemoteControlPropertyHandleObject::GetValueInArray(UObject*& OutValue, int32 InIndex) const
{
	if (InIndex < 0)
	{
		return false;
	}
	
	if (const TSharedPtr<FRemoteControlProperty>& RemoteControlPropertyPtr = GetRCProperty())
	{
		if (const TSharedPtr<IRemoteControlPropertyHandle>& PropertyHandle = RemoteControlPropertyPtr->GetPropertyHandle())
		{
			if (const TSharedPtr<IRemoteControlPropertyHandleArray>& PropertyHandleArray = PropertyHandle->AsArray())
			{
				if (InIndex < PropertyHandleArray->GetNumElements())
				{
					if (const TSharedPtr<IRemoteControlPropertyHandle>& ElementPropertyHandle = PropertyHandleArray->GetElement(InIndex))
					{
						return ElementPropertyHandle->GetValue(OutValue);
					}
				}
			}
		}
	}

	return false;
}

bool FRemoteControlPropertyHandleObject::SetMaterialValue(UObject* InValue, int32 InIndex) const
{
	if (!InValue || InIndex < 0)
	{
		return false;
	}
	
	if (!InValue->IsA(UMaterialInterface::StaticClass()))
	{
		return false;
	}
	
	bool bSuccess = false;
	
	if (const TSharedPtr<FRemoteControlProperty>& RemoteControlPropertyPtr = GetRCProperty())
	{		
		for (UObject* Object : RemoteControlPropertyPtr->GetBoundObjects())
		{
			if (UMeshComponent* MaterialOwnerComponent = Cast<UMeshComponent>(Object))
			{
				if (MaterialOwnerComponent->GetMaterials().IsValidIndex(InIndex))
				{
					MaterialOwnerComponent->SetMaterial(InIndex, Cast<UMaterialInterface>(InValue));
					bSuccess = true;
				}
			}
		}
	}

	return bSuccess;
}

bool FRemoteControlPropertyHandleObject::Supports(const FProperty* InProperty)
{
	if (!InProperty)
	{
		return false;
	}
	
	if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
