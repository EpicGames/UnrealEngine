// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlPropertyHandle.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"

/**
 * The base implementation of a property handle
 */
class FRemoteControlPropertyHandle : public IRemoteControlPropertyHandle
{
public:
	/**
	 * Gets a property handle for the specified property type
	 *
	 * @param InRCProperty		Remote control property.
	 * @param InProperty		Property being edited.
	 * @param InParentProperty	Parent property being edited.
	 * @param InParentFieldPath	Parent path from UObject to Field
	 * @param InArrayIndex		Array index of the property
	 */
	static TSharedPtr<IRemoteControlPropertyHandle> GetPropertyHandle(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty = nullptr, const FString& InParentFieldPath = FString(), int32 InArrayIndex = 0);

public:
	FRemoteControlPropertyHandle(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty = nullptr, const FString& InParentFieldPath = FString(), int32 InArrayIndex = 0);

	/**
	 * Returns whether or not a property is of a specific subclass of FProperty
	 *
	 * @param ClassType	The class type to check
	 * @param true if the property is a ClassType
	 */
	bool IsPropertyTypeOf(FFieldClass* ClassType) const;

	/**
	 * Gets the value pointer with the appropriate type for the property set
	 *
	 * @param InContainerAddress value container address	
	 * @return The location of the property value
	 */
	void* GetValuePtr(void* InContainerAddress) const;

	/**
	 * Gets the Container address for the property
	 *
	 * @param OwnerIndex index of the Owning object, default 0
	 * @return The container pointer
	 */
	void* GetContainerAddress(int32 OwnerIndex = 0) const;

	/**
	 * Gets properties container pointer array.
	 */
	TArray<void*> GetContainerAddresses() const;

	//~ Begin IRemoteControlPropertyHandle Interface

#define DECLARE_PROPERTY_ACCESSOR_STR( ValueType ) \
	virtual bool SetValue( const ValueType& InValue) override; \
	virtual bool GetValue( ValueType& OutValue ) const override; 

	DECLARE_PROPERTY_ACCESSOR_STR(FString)
	DECLARE_PROPERTY_ACCESSOR_STR(FText)
	DECLARE_PROPERTY_ACCESSOR_STR(FName)

#undef DECLARE_PROPERTY_ACCESSOR_STR

#define DECLARE_PROPERTY_ACCESSOR( ValueType ) \
	virtual bool SetValue( ValueType InValue) override; \
	virtual bool GetValue( ValueType& OutValue ) const override; 

	DECLARE_PROPERTY_ACCESSOR(bool)
	DECLARE_PROPERTY_ACCESSOR(int8)
	DECLARE_PROPERTY_ACCESSOR(int16)
	DECLARE_PROPERTY_ACCESSOR(int32)
	DECLARE_PROPERTY_ACCESSOR(int64)
	DECLARE_PROPERTY_ACCESSOR(uint8)
	DECLARE_PROPERTY_ACCESSOR(uint16)
	DECLARE_PROPERTY_ACCESSOR(uint32)
	DECLARE_PROPERTY_ACCESSOR(uint64)
	DECLARE_PROPERTY_ACCESSOR(float)
	DECLARE_PROPERTY_ACCESSOR(double)

	DECLARE_PROPERTY_ACCESSOR(FVector)
	DECLARE_PROPERTY_ACCESSOR(FVector2D)
	DECLARE_PROPERTY_ACCESSOR(FVector4)
	DECLARE_PROPERTY_ACCESSOR(FQuat)
	DECLARE_PROPERTY_ACCESSOR(FRotator)
	DECLARE_PROPERTY_ACCESSOR(FColor)
	DECLARE_PROPERTY_ACCESSOR(FLinearColor)
#undef DECLARE_PROPERTY_ACCESSOR

	virtual bool SetValue(const TCHAR* InValue) override { return false; }

	virtual bool SetValue(UObject* InValue) override { return false; }
	virtual bool SetValueInArray(UObject* InValue, int32 InIndex) override { return false; }
	virtual bool GetValue(UObject*& OutValue) const override { return false; }
	virtual bool GetValueInArray(UObject*& OutValue, int32 InIndex) const override { return false; }

	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandle(FName ChildName, bool bRecurse = true) const override;
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandle(int32 Index) const override;
	virtual int32 GetNumChildren() const override { return Children.Num(); }
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandleByFieldPath(const FString& InFieldPath) override;

	virtual TSharedPtr<IRemoteControlPropertyHandleArray> AsArray() override { return nullptr; }
	virtual TSharedPtr<IRemoteControlPropertyHandleSet> AsSet() override { return nullptr; }
	virtual TSharedPtr<IRemoteControlPropertyHandleMap> AsMap() override { return nullptr; }

	virtual TSharedPtr<FRemoteControlProperty> GetRCProperty() const override;
	virtual int32 GetIndexInArray() const override { return ArrayIndex; }

	virtual FProperty* GetProperty() const override { return PropertyPtr.Get(); }
	virtual FProperty* GetParentProperty() const override { return ParentPropertyPtr.Get(); }
	virtual const FString& GetFieldPath() const override { return FieldPath; }
	virtual const FString& GetParentFieldPath() const override { return ParentFieldPath; }

	virtual bool ShouldGenerateTransaction() const { return bGenerateTransaction; }
	virtual void SetGenerateTransaction(bool bInGenerateTransaction) { bGenerateTransaction = bInGenerateTransaction; }
	//~ End IRemoteControlPropertyHandle Interface

protected:
	/** Represents a property exposed to remote control. */
	TWeakPtr<FRemoteControlProperty> RCPropertyPtr;

	/** The property being edited */
	const TWeakFieldPtr<FProperty> PropertyPtr;

	/** The parent property being edited */
	const TWeakFieldPtr<FProperty> ParentPropertyPtr;

	/** The path from a UObject to a parent field */
	FString ParentFieldPath;

	/** Array index of the property, in case of the static C Array */
	int32 ArrayIndex;

	/** The path from a UObject to a field */
	FString FieldPath;

	/** Should the property handle generate transactions. */
	bool bGenerateTransaction;

protected:
	/** All struct children */
	TArray<TSharedPtr<IRemoteControlPropertyHandle>> Children;
};

#define DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR( ClassName ) \
	ClassName(const TSharedPtr<FRemoteControlProperty>& InRCProperty, FProperty* InProperty, FProperty* InParentProperty, const FString& InParentFieldPath, int32 InArrayIndex); \

/**
 * The integer implementation of a property handle
 */
class FRemoteControlPropertyHandleInt : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleInt)
	
	/**
	 * Check if int handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(int8& OutValue) const override;
	virtual bool SetValue(int8 InValue) override;

	virtual bool GetValue(int16& OutValue) const override;
	virtual bool SetValue(int16 InValue) override;

	virtual bool GetValue(int32& OutValue) const override;
	virtual bool SetValue(int32 InValue) override;

	virtual bool GetValue(int64& OutValues) const override;
	virtual bool SetValue(int64 InValue) override;

	virtual bool GetValue(uint16& OutValues) const override;
	virtual bool SetValue(uint16 InValue) override;

	virtual bool GetValue(uint32& OutValue) const override;
	virtual bool SetValue(uint32 InValue) override;

	virtual bool GetValue(uint64& OutValue) const override;
	virtual bool SetValue(uint64 InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The float implementation of a property handle
 */
class FRemoteControlPropertyHandleFloat : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleFloat)

	/**
	 * Check if float handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(float& OutValue) const override;
	virtual bool SetValue(float InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The double implementation of a property handle
 */
class FRemoteControlPropertyHandleDouble : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleDouble)

	/**
	 * Check if double handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(double& OutValue) const override;
	virtual bool SetValue(double InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The bool implementation of a property handle
 */
class FRemoteControlPropertyHandleBool : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleBool)
	
	/**
	 * Check if bool handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(bool& OutValue) const override;
	virtual bool SetValue(bool InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The byte implementation of a property handle
 */
class FRemoteControlPropertyHandleByte : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleByte)
	
	/**
	 * Check if byte handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(uint8& OutValue) const override;
	virtual bool SetValue(uint8 InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The string implementation of a property handle
 */
class FRemoteControlPropertyHandleString : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleString)

	/**
	 * Check if string handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FString& OutValue) const override;
	virtual bool SetValue(const FString& InValue) override;
	virtual bool SetValue(const TCHAR* InValue) override;

	virtual bool GetValue(FName& OutValue) const override;
	virtual bool SetValue(const FName& InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The name implementation of a property handle
 */
class FRemoteControlPropertyHandleName : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleName)
	
	/**
	 * Check if name handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FString& OutValue) const override;
	virtual bool SetValue(const FString& InValue) override;
	virtual bool SetValue(const TCHAR* InValue) override;

	virtual bool GetValue(FName& OutValue) const override;
	virtual bool SetValue(const FName& InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The text implementation of a property handle
 */
class FRemoteControlPropertyHandleText : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleText)
	
	/**
	 * Check if text handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FText& OutValue) const override;
	virtual bool SetValue(const FText& InValue) override;

	virtual bool SetValue(const FString& InValue) override;
	virtual bool SetValue(const TCHAR* InValue) override;
	//~ End IRemoteControlPropertyHandle Interface
};

/**
 * The vector implementation of a property handle
 */
class FRemoteControlPropertyHandleVector : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleVector)
	
	/**
	 * Check if vector handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FVector& OutValue) const override;
	virtual bool SetValue(FVector InValue) override;

	virtual bool GetValue(FVector2D& OutValue) const override;
	virtual bool SetValue(FVector2D InValue) override;

	virtual bool GetValue(FVector4& OutValue) const override;
	virtual bool SetValue(FVector4 InValue) override;

	virtual bool GetValue(FQuat& OutValue) const override;
	virtual bool SetValue(FQuat InValue) override;
	//~ End IRemoteControlPropertyHandle Interface

	/** Set X value of the vector */
	bool SetX(float InValue);

	/** Set Y value of the vector */
	bool SetY(float InValue);

	/** Set Z value of the vector */
	bool SetZ(float InValue);

	/** Set W value of the vector */
	bool SetW(float InValue);

	/** Check if all vector components are valid */
	bool AreComponentsValid() const;
	
private:
	/** Array of child struct vector components */
	TArray< TWeakPtr<IRemoteControlPropertyHandle> > VectorComponents;
};

/**
 * The rotator implementation of a property handle
 */
class FRemoteControlPropertyHandleRotator : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleRotator)
	
	/**
	 * Check if rotator handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FRotator& OutValue) const override;
	virtual bool SetValue(FRotator InValue) override;
	//~ End IRemoteControlPropertyHandle Interface

	/** Set roll value of the rotator */
	bool SetRoll(float InValue);

	/** Set pitch value of the rotator */
	bool SetPitch(float InValue);

	/** Set yaw value of the rotator */
	bool SetYaw(float InValue);

	/** Check if all rotator components are valid */
	bool AreComponentsValid() const;

private:
	/** Roll handle of the rotator */
	TWeakPtr<IRemoteControlPropertyHandle> RollValue;

	/** Roll handle of the rotator */
	TWeakPtr<IRemoteControlPropertyHandle> PitchValue;

	/** Roll handle of the rotator */
	TWeakPtr<IRemoteControlPropertyHandle> YawValue;
};

/**
 * The array implementation of a property handle
 */
class FRemoteControlPropertyHandleArray : public FRemoteControlPropertyHandle, public IRemoteControlPropertyHandleArray
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleArray)
	
	/**
	 * Check if array handle support the given property
	 * @param InProperty Property to check
	 * @param InArrayIndex Static array property index
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty, int32 InArrayIndex = -1);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual TSharedPtr<IRemoteControlPropertyHandleArray> AsArray() override;
	//~ End IRemoteControlPropertyHandle Interface

	//~ Begin IRemoteControlPropertyHandleArray Interface
	virtual int32 GetNumElements() const override;
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetElement(int32 Index) override;
	//~ End IRemoteControlPropertyHandleArray Interface
};

/**
 * The set implementation of a property handle
 */
class FRemoteControlPropertyHandleSet : public FRemoteControlPropertyHandle, public IRemoteControlPropertyHandleSet
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleSet)
	
	/**
	 * Check if set handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandleSet Interface
	virtual TSharedPtr<IRemoteControlPropertyHandleSet> AsSet() override;
	//~ End IRemoteControlPropertyHandleSet Interface

	//~ Begin IRemoteControlPropertyHandleSet Interface
	virtual int32 GetNumElements() override;
	virtual TSharedPtr<IRemoteControlPropertyHandle> FindElement(const void* ElementToFind) override;
	//~ Begin IRemoteControlPropertyHandleSet Interface
};

/**
 * The map implementation of a property handle
 */
class FRemoteControlPropertyHandleMap : public FRemoteControlPropertyHandle, public IRemoteControlPropertyHandleMap
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleMap)
	
	/**
	 * Check if map handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandleMap Interface
	virtual TSharedPtr<IRemoteControlPropertyHandleMap> AsMap() override;
	//~ End IRemoteControlPropertyHandleMap Interface

	//~ Begin IRemoteControlPropertyHandleMap Interface
	virtual int32 GetNumElements() override;
	virtual TSharedPtr<IRemoteControlPropertyHandle> Find(const void* KeyPtr) override;
	//~ End IRemoteControlPropertyHandleMap Interface
};

/**
 * The color implementation of a property handle
 */
class FRemoteControlPropertyHandleColor : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleColor)

	/**
	 * Check if color handle support the given property
	 * @param InProperty Property to check
	 * @return true if this handle supports the given property
	 */
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FColor& OutValue) const override;
	virtual bool SetValue(FColor InValue) override;
	//~ End IRemoteControlPropertyHandle Interface

	/** Set R value of the color */
	bool SetR(uint8 InValue);

	/** Set G value of the color */
	bool SetG(uint8 InValue);

	/** Set B value of the color */
	bool SetB(uint8 InValue);

	/** Set A value of the color */
	bool SetA(uint8 InValue);

	/** Check if all color components are valid */
	bool AreComponentsValid() const;
	
private:
	/** Array of child struct color components */
	TArray< TWeakPtr<IRemoteControlPropertyHandle> > ColorComponents;
};

/**
* The linear color implementation of a property handle
*/
class FRemoteControlPropertyHandleLinearColor : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleLinearColor)

	/**
	* Check if linear color handle support the given property
	* @param InProperty Property to check
	* @return true if this handle supports the given property
	*/
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool GetValue(FLinearColor& OutValue) const override;
	virtual bool SetValue(FLinearColor InValue) override;
	//~ End IRemoteControlPropertyHandle Interface

	/** Set R value of the linear color */
	bool SetR(float InValue);

	/** Set G value of the linear color */
	bool SetG(float InValue);

	/** Set B value of the linear color */
	bool SetB(float InValue);

	/** Set A value of the linear color */
	bool SetA(float InValue);

	/** Check if all color components are valid */
	bool AreComponentsValid() const;
	
private:
	/** Array of child struct color components */
	TArray< TWeakPtr<IRemoteControlPropertyHandle> > ColorComponents;
};

/**
* The object implementation of a property handle
*/
class FRemoteControlPropertyHandleObject : public FRemoteControlPropertyHandle
{
public:
	DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR(FRemoteControlPropertyHandleObject)

	/**
	* Check if Object handle supports the given property
	* @param InProperty Property to check
	* @return true if this handle supports the given property
	*/
	static bool Supports(const FProperty* InProperty);

	//~ Begin IRemoteControlPropertyHandle Interface
	virtual bool SetValue(UObject* InValue) override;
	virtual bool GetValue(UObject*& OutValue) const override;

	/** This is currently used for the specific case in which we are setting a material inside the override materials of e.g. a Static Mesh*/
	virtual bool SetValueInArray(UObject* InValue, int32 InIndex) override;
	virtual bool GetValueInArray(UObject*& OutValue, int32 InIndex) const override;
	//~ End IRemoteControlPropertyHandle Interface

	/** Set a Material value using the handled Object */
	bool SetMaterialValue(UObject* InValue, int32 InIndex) const;
};

#undef DECLARE_PROPERTY_ACCESSOR_CONSTRUCTOR
