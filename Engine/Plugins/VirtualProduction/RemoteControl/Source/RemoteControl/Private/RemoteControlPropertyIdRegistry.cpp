// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPropertyIdRegistry.h"

#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "IPropertyIdHandler.h"
#include "IRemoteControlModule.h"
#include "IStructDeserializerBackend.h"
#include "Materials/MaterialInterface.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "StructSerializer.h"
#include "UObject/Field.h"


namespace RCPropIdUtils
{
	bool GetObjectRef(const TSharedPtr<FRemoteControlProperty>& Field, const ERCAccess Access, FRCObjectReference& OutObjectRef)
	{
		if (!Field.IsValid())
		{
			return false;
		}
		if (UObject* FieldBoundObject = Field->GetBoundObject())
		{
			FRCObjectReference ObjectRef;
			FString ErrorText;
			if (IRemoteControlModule::Get().ResolveObjectProperty(Access, FieldBoundObject, Field->FieldPathInfo, ObjectRef, &ErrorText))
			{
				OutObjectRef = ObjectRef;
				return true;
			}
			UE_LOGF(LogRemoteControl, Warning, "Could not resolve object property \"%ls\" in object \"%ls\": %ls", *Field->FieldName.ToString(), *FieldBoundObject->GetPathName(), *ErrorText);
		}
		return false;
	}
}

FRCPropertyIdWrapper::FRCPropertyIdWrapper(const TSharedRef<FRemoteControlProperty>& InRCProperty)
{
	EntityId = InRCProperty->GetId();
	
	PropertyId = InRCProperty->PropertyId;
	if (FProperty* Property = InRCProperty->GetProperty())
	{
		if (UObject* BoundObject = InRCProperty->GetBoundObject())
		{
			UpdateTypes(InRCProperty, BoundObject, Property);
		}
	}
}

const FGuid& FRCPropertyIdWrapper::GetEntityId() const
{
	return EntityId;
}

FName FRCPropertyIdWrapper::GetPropertyId() const
{
	return PropertyId;
}

void FRCPropertyIdWrapper::SetPropertyId(FName InNewPropertyId)
{
	PropertyId = InNewPropertyId;
}

const FName& FRCPropertyIdWrapper::GetSuperType() const
{
	return SuperType;
}

const FName& FRCPropertyIdWrapper::GetSubType() const
{
	return SubType;
}

bool FRCPropertyIdWrapper::IsValid() const
{
	return EntityId.IsValid() && (!SuperType.IsNone() || !SubType.IsNone());
}

bool FRCPropertyIdWrapper::IsValidPropertyId() const
{
	return PropertyId != NAME_None;
}

uint32 GetTypeHash(const FRCPropertyIdWrapper& Wrapper)
{
	return GetTypeHash(Wrapper.EntityId);
}

bool FRCPropertyIdWrapper::operator==(const FGuid& WrappedId) const
{
	if (!IsValid())
	{
		return false;
	}
	return EntityId == WrappedId;
}

bool FRCPropertyIdWrapper::operator==(const FRCPropertyIdWrapper& Other) const
{
	if (IsValid() && Other.IsValid())
	{
		return EntityId == Other.EntityId;
	}
	return false;
}

void FRCPropertyIdWrapper::UpdateTypes(const TSharedRef<FRemoteControlProperty>& InRCProperty, UObject* InOwner, FProperty* InProperty)
{
	SuperType = NAME_None;
	SubType = NAME_None;

	const TSharedPtr<IPropertyIdHandler> PropertyIdHandler = IRemoteControlModule::Get().GetPropertyIdHandlerFor(InProperty);
	if (!PropertyIdHandler.IsValid())
	{
		return;
	}

	SuperType = PropertyIdHandler->GetPropertySuperTypeName(InProperty);
	const EPropertyBagPropertyType PropertyType = PropertyIdHandler->GetPropertyType(InProperty);

	if (PropertyType == EPropertyBagPropertyType::Enum ||
		PropertyType == EPropertyBagPropertyType::Object ||
		PropertyType == EPropertyBagPropertyType::Struct)
	{
		SubType = PropertyIdHandler->GetPropertySubTypeName(InProperty);
	}
}

void URemoteControlPropertyIdRegistry::Initialize()
{
	URemoteControlPreset* SourcePreset = GetSourcePreset();
	if (!SourcePreset)
	{
		return;
	}

	IdentifiedFields.Reset();
	for (TWeakPtr<FRemoteControlEntity> RCEntity : SourcePreset->GetExposedEntities())
	{
		if (const TSharedPtr<FRemoteControlField> RCField = StaticCastSharedPtr<FRemoteControlField>(RCEntity.Pin()))
		{
			AddIdentifiedField(RCField.ToSharedRef());
		}
	}
}

URemoteControlPreset* URemoteControlPropertyIdRegistry::GetSourcePreset() const
{
	return GetTypedOuter<URemoteControlPreset>();
}

#if WITH_EDITOR
void URemoteControlPropertyIdRegistry::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	UObject::PostEditChangeProperty(InPropertyChangedEvent);
	Initialize();
}
#endif

void URemoteControlPropertyIdRegistry::PerformChainReaction(const FRemoteControlPropertyIdArgs& InArgs)
{
	URemoteControlPreset* SourcePreset = GetSourcePreset();
	if (!IsValid(SourcePreset))
	{
		return;
	}

	TSet<FGuid> TargetProperties;

	Algo::TransformIf(IdentifiedFields, TargetProperties,
		[InArgs](const FRCPropertyIdWrapper& Wrapper)
		{
			return Wrapper.IsValid()
			&& Wrapper.GetPropertyId() == InArgs.PropertyId
			&& Wrapper.GetSuperType() == InArgs.SuperType
			&& Wrapper.GetSubType() == InArgs.SubType;
		},
		[](const FRCPropertyIdWrapper& Wrapper)
		{
			return Wrapper.GetEntityId();
		}
		);

	for (const FGuid& TargetProperty : TargetProperties)
	{
		if (TSharedPtr<FRemoteControlProperty> TargetRCProperty = SourcePreset->GetExposedEntity<FRemoteControlProperty>(TargetProperty).Pin())
		{
			const TObjectPtr<URCVirtualPropertySelfContainer>* RealPropContainer = InArgs.RealProperties.Find(TargetRCProperty->GetId());
			if (!RealPropContainer)
			{
				 continue;
			}

			TObjectPtr<URCVirtualPropertySelfContainer> VirtualProperty = InArgs.VirtualProperty;
			if (!VirtualProperty)
			{
				continue;
			}

			// We copy the VirtualProperty value to the RealPropContainer to make sure it is set and updated correctly through the serialization
			if ((*RealPropContainer)->GetValueType() == VirtualProperty->GetValueType())
			{
				// Special case for Color
				if ((*RealPropContainer)->IsColorType() && VirtualProperty->IsLinearColorType())
				{
					FLinearColor LinearColor;
					VirtualProperty->GetValueLinearColor(LinearColor);
					FColor ValueToAssign = LinearColor.ToFColor(true);
					(*RealPropContainer)->SetValueColor(ValueToAssign);
				}
				else
				{
					(*RealPropContainer)->UpdateValueWithProperty(InArgs.VirtualProperty);
				}
			}
			else if ((*RealPropContainer)->GetValueType() == EPropertyBagPropertyType::Float &&
				VirtualProperty->GetValueType() == EPropertyBagPropertyType::Double)
			{
				// Special case for Float
				double Double;
				VirtualProperty->GetValueDouble(Double);
				float ValueToAssign = Double;
				(*RealPropContainer)->SetValueFloat(ValueToAssign);
			}

			// Update the property correctly through the serialization
			FRCObjectReference ObjectRef;
			ObjectRef.Property = TargetRCProperty->GetProperty();
			ObjectRef.Access = ERCAccess::WRITE_ACCESS;
			ObjectRef.PropertyPathInfo = TargetRCProperty->FieldPathInfo.ToString();

			for (UObject* Object : TargetRCProperty->GetBoundObjects())
			{
				if (IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef))
				{
					TArray<uint8> Buffer;
					FMemoryWriter Writer(Buffer);

					FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);

					FStructSerializerPolicies Policies; 
					Policies.MapSerialization = EStructSerializerMapPolicies::Array;

					(*RealPropContainer)->SerializeToBackend(SerializerBackend);

					// Deserialization
					FMemoryReader Reader(Buffer);
					FCborStructDeserializerBackend DeserializerBackend(Reader);

					IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Cbor, Buffer);
				}
			}
		}
	}
}

void URemoteControlPropertyIdRegistry::AddIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify)
{
	if (InFieldToIdentify->FieldType == EExposedFieldType::Property)
	{
		if (const TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedRef<FRemoteControlProperty>(InFieldToIdentify))
		{
			if (RCProperty->IsEditable())
			{
				FRCPropertyIdWrapper Wrapper{ RCProperty.ToSharedRef() };
				if (Wrapper.IsValid())
				{
#if WITH_EDITOR
					Modify();
#endif
					IdentifiedFields.Add(MoveTemp(Wrapper));
				}
			}
		}
	}
}

bool URemoteControlPropertyIdRegistry::IsEmpty() const
{
	return IdentifiedFields.IsEmpty();
}

void URemoteControlPropertyIdRegistry::UpdateIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify)
{
	if (InFieldToIdentify->FieldType == EExposedFieldType::Property)
	{
		const uint32 Hash = GetTypeHash(InFieldToIdentify->GetId());
		if (FRCPropertyIdWrapper* Wrapper = IdentifiedFields.FindByHash(Hash, InFieldToIdentify->GetId()))
		{
			Wrapper->SetPropertyId(InFieldToIdentify->PropertyId);
			OnPropertyIdUpdated().Broadcast();
		}
	}
}

void URemoteControlPropertyIdRegistry::RemoveIdentifiedField(const FGuid& InEntityId)
{
	const uint32 Hash = GetTypeHash(InEntityId);

	if (IdentifiedFields.ContainsByHash(Hash, InEntityId))
	{
#if WITH_EDITOR
		Modify();
#endif
		IdentifiedFields.RemoveByHash(Hash, InEntityId);
	}
}

TSet<FName> URemoteControlPropertyIdRegistry::GetFieldIdsNameList() const
{
	TSet<FName> OutIds;
	for (FRCPropertyIdWrapper Field : IdentifiedFields)
	{
		if (Field.IsValidPropertyId())
		{
			OutIds.Add(Field.GetPropertyId());
		}
	}
	return OutIds;
}

TSet<FName> URemoteControlPropertyIdRegistry::GetFullPropertyIdsNamePossibilitiesList() const
{
	TSet<FName> InitialIdsList = GetFieldIdsNameList();
	TSet<FName> OutIds;

	for (const FName& Id : InitialIdsList)
	{
		OutIds.Append(GetPossiblePropertyIds(Id));
	}

	OutIds.Sort(FNameLexicalLess());
	return OutIds;
}

TSet<FGuid> URemoteControlPropertyIdRegistry::GetEntityIdsForPropertyId(const FName& InPropertyId) const
{
	TSet<FGuid> OutEntityIds;
	if (InPropertyId == NAME_None)
	{
		return OutEntityIds;
	}
	for (FRCPropertyIdWrapper Wrappers : IdentifiedFields)
	{
		if (Wrappers.GetPropertyId() == InPropertyId)
		{
			OutEntityIds.Add(Wrappers.GetEntityId());
		}
	}
	return OutEntityIds;
}

TSet<FGuid> URemoteControlPropertyIdRegistry::GetEntityIdsList()
{
	TSet<FGuid> OutIds;
	for (FRCPropertyIdWrapper Field : IdentifiedFields)
	{
		if (Field.IsValidPropertyId())
		{
			OutIds.Add(Field.GetEntityId());
		}
	}
	return OutIds;
}

bool URemoteControlPropertyIdRegistry::Contains(FName InContainerPropertyId, FName InTargetPropertyId) const
{
	if (InContainerPropertyId == InTargetPropertyId)
	{
		return true;
	}

	const TSet<FName> PossibilitiesSecond = GetPossiblePropertyIds(InContainerPropertyId);
	return PossibilitiesSecond.Contains(InTargetPropertyId);
}

TSet<FName> URemoteControlPropertyIdRegistry::GetPossiblePropertyIds(FName InPropId) const
{
	TSet<FName> OutPossibleIds;

	// Add the given PropId
	OutPossibleIds.Add(InPropId);
	const FString FullPropId = InPropId.ToString();

	// Get all the substring that are before a "."
	FString PossibleId;
	TArray<FString> PossibleSubIds;
	FullPropId.ParseIntoArray(PossibleSubIds, TEXT("."));

	for (const FString& SubId : PossibleSubIds)
	{
		PossibleId.Append(SubId);
		OutPossibleIds.Add(FName(PossibleId));
		PossibleId.AppendChar('.');
	}

	return OutPossibleIds;
}

void URemoteControlPropertyIdRegistry::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	bool bNeedsRehash = false;
	for (FRCPropertyIdWrapper& PropertyId : IdentifiedFields)
	{
		if (const FGuid* FoundNewId = InEntityIdMap.Find(PropertyId.GetEntityId()))
		{
			PropertyId.EntityId = *FoundNewId;
			bNeedsRehash = true;
		}
	}

	if (bNeedsRehash)
	{
		TSet<FRCPropertyIdWrapper> RehashedIdentifiedFields;
		RehashedIdentifiedFields.Reserve(IdentifiedFields.Num());
	
		for (FRCPropertyIdWrapper& Wrapper : IdentifiedFields)
		{
			RehashedIdentifiedFields.Add(FRCPropertyIdWrapper(Wrapper));
		}
	
		IdentifiedFields = MoveTemp(RehashedIdentifiedFields);
	}
}
