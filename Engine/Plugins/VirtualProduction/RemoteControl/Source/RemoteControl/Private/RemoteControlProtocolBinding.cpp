// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolBinding.h"

#include "Algo/MinElement.h"
#include "Algo/Sort.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "CborWriter.h"
#include "Factories/IRemoteControlMaskingFactory.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolEntityInterpolator.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"

namespace UE::RemoteControl::Private
{	
	// Writes a property value to the serialization output.
	template <typename ValueType>
	UE_DEPRECATED(5.5, "Instead refer to RemoteControlProtocolEntityProcessor")
	void WritePropertyValue(FCborWriter& InCborWriter, FProperty* InProperty, const ValueType& Value, bool bWriteName = true)
	{
		if (bWriteName)
		{
			InCborWriter.WriteValue(InProperty->GetName());
		}
		InCborWriter.WriteValue(Value);
	}

	// Specialization for FName that converts to FString
	template <>
	UE_DEPRECATED(5.5, "Instead refer to RemoteControlProtocolEntityProcessor")
	void WritePropertyValue<FName>(FCborWriter& InCborWriter, FProperty* InProperty, const FName& Value, bool bWriteName)
	{
		if (bWriteName)
		{
			InCborWriter.WriteValue(InProperty->GetName());
		}
		InCborWriter.WriteValue(Value.ToString());
	}

	// Specialization for FText that converts to FString
	template <>
	UE_DEPRECATED(5.5, "Instead refer to RemoteControlProtocolEntityProcessor")
	void WritePropertyValue<FText>(FCborWriter& InCborWriter, FProperty* InProperty, const FText& Value, bool bWriteName)
	{
		if (bWriteName)
		{
			InCborWriter.WriteValue(InProperty->GetName());
		}
		InCborWriter.WriteValue(Value.ToString());
	}

	template <typename ProtocolValueType>
	UE_DEPRECATED(5.5, "Instead refer to RemoteControlProtocolEntityProcessor")
	bool WriteProperty(FRemoteControlProtocolEntity* Entity, FProperty* InProperty, FProperty* OuterProperty, TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, ProtocolValueType InProtocolValue, FCborWriter& InCborWriter, int32 InArrayIndex = 0)
	{
		using namespace UE::RemoteControl;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		// Value nested in Array/Set (except single element) or map as array or as root
		const bool bIsInArray = OuterProperty != nullptr
								&& (InProperty->ArrayDim > 1
									|| OuterProperty->GetClass() == FArrayProperty::StaticClass()
									|| OuterProperty->GetClass() == FSetProperty::StaticClass()
									|| OuterProperty->GetClass() == FMapProperty::StaticClass());


		bool bSuccess = false;
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
		{
			bool bBoolValue = false;
			bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, BoolProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, bBoolValue, InArrayIndex);			
			bBoolValue = StaticCast<uint8>(bBoolValue) > 0; // Ensure 0 or 1, can be different if property was packed.
			WritePropertyValue(InCborWriter, InProperty, bBoolValue, !bIsInArray);
		}
		else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
		{
			if (CastField<FFloatProperty>(InProperty))
			{
				float FloatValue = 0.f;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, FloatValue, InArrayIndex);
				WritePropertyValue(InCborWriter, InProperty, FloatValue, !bIsInArray);
			}
			else if (CastField<FDoubleProperty>(InProperty))
			{
				double DoubleValue = 0.0;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, DoubleValue, InArrayIndex);
				WritePropertyValue(InCborWriter, InProperty, DoubleValue, !bIsInArray);
			}
			else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
			{
				if (FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
				{
					uint8 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FIntProperty* IntProperty = CastField<FIntProperty>(InProperty))
				{
					int IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(InProperty))
				{
					uint32 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FInt16Property* Int16Property = CastField<FInt16Property>(InProperty))
				{
					int16 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(InProperty))
				{
					uint16 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FInt64Property* Int64Property = CastField<FInt64Property>(InProperty))
				{
					int64 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(InProperty))
				{
					uint64 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
				else if (FInt8Property* Int8Property = CastField<FInt8Property>(InProperty))
				{
					int8 IntValue = 0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
					WritePropertyValue(InCborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
				}
			}
		}
		else if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			if(!bIsInArray)
			{
				InCborWriter.WriteValue(StructProperty->GetName());	
			}
			
			InCborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);

			bool bStructSuccess = true;
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers;
				RangeMappingBuffers.Reserve(InRangeMappingBuffers.Num());

				FProperty* InnerProperty = *It;
				for (const FRemoteControlProtocolEntity::FRangeMappingData& RangePair : InRangeMappingBuffers)
				{
					const uint8* DataInContainer = StructProperty->ContainerPtrToValuePtr<const uint8>(RangePair.Mapping.GetData(), InArrayIndex);
					const uint8* DataInStruct = InnerProperty->ContainerPtrToValuePtr<const uint8>(DataInContainer);
					RangeMappingBuffers.Emplace(RangePair.Range, DataInStruct, InnerProperty->GetSize(), 1);
				}

				bStructSuccess &= WriteProperty(Entity, InnerProperty, StructProperty, RangeMappingBuffers, InProtocolValue, InCborWriter, InArrayIndex);
			}

			bSuccess = bStructSuccess;
			InCborWriter.WriteContainerEnd();
		}

		else if (FStrProperty* StrProperty = CastField<FStrProperty>(InProperty))
		{
			FString StringValue;
			bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, StrProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, StringValue, InArrayIndex);
			WritePropertyValue(InCborWriter, InProperty, StringValue, !bIsInArray);
		}
		else if (FNameProperty* NameProperty = CastField<FNameProperty>(InProperty))
		{
			FName NameValue;
			bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NameProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, NameValue, InArrayIndex);
			WritePropertyValue(InCborWriter, InProperty, NameValue, !bIsInArray);
		}
		else if (FTextProperty* TextProperty = CastField<FTextProperty>(InProperty))
		{
			FText TextValue;
			bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, TextProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, TextValue, InArrayIndex);
			WritePropertyValue(InCborWriter, InProperty, TextValue, !bIsInArray);
		}

#if !UE_BUILD_SHIPPING && UE_BUILD_DEBUG
		if (!bSuccess)
		{
			if (RemoteControlTypeUtilities::IsSupportedMappingType(InProperty))
			{
				UE_LOGF(LogRemoteControl, Error, "Property type %ls is supported for mapping, but unhandled in EntityInterpolation::WriteProperty", *InProperty->GetClass()->GetName());
			}
		}
#endif

		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		return bSuccess;
	}

	template <typename ProtocolValueType>
	UE_DEPRECATED(5.5, "Instead refer to RemoteControlProtocolEntityProcessor")
	bool ApplyProtocolValueToProperty(FRemoteControlProtocolEntity* Entity, FProperty* InProperty, ProtocolValueType InProtocolValue, TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, FCborWriter& InCborWriter)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		using namespace UE::RemoteControl;

		// Structures
		if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
		{
			UScriptStruct* ScriptStruct = StructProperty->Struct;

			InCborWriter.WriteValue(StructProperty->GetName());
			InCborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);

			bool bStructSuccess = true;
			for (TFieldIterator<FProperty> It(ScriptStruct); It; ++It)
			{
				bStructSuccess &= WriteProperty(Entity, *It, StructProperty, InRangeMappingBuffers, InProtocolValue, InCborWriter);
			}
			
			InCborWriter.WriteContainerEnd();

			return bStructSuccess;
		}

		// @note: temporarily disabled - array of primitives supported, array of structs not
		// Dynamic arrays
		else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			FProperty* InnerProperty = ArrayProperty->Inner;

			InCborWriter.WriteValue(ArrayProperty->GetName());
			InCborWriter.WriteContainerStart(ECborCode::Array, -1/*Indefinite*/);

			bool bArraySuccess = true;

			// Get minimum item count
			const FRemoteControlProtocolEntity::FRangeMappingData* ElementWithSmallestNum = Algo::MinElementBy(InRangeMappingBuffers, [](const FRemoteControlProtocolEntity::FRangeMappingData& RangePair)
			{
				return RangePair.NumElements;
			});

			// No elements in array
			if (ElementWithSmallestNum == nullptr)
			{
				return false;
			}

			for (auto ArrayIndex = 0; ArrayIndex < ElementWithSmallestNum->NumElements; ++ArrayIndex)
			{
				bArraySuccess &= WriteProperty(Entity, InnerProperty, ArrayProperty, InRangeMappingBuffers, InProtocolValue, InCborWriter, ArrayIndex);
			}

			InCborWriter.WriteContainerEnd();

			return bArraySuccess;
		}

		// Maps
		else if (FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			UE_LOGF(LogRemoteControl, Warning, "MapProperty not supported");
			return false;
		}

		// Sets
		else if (FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			UE_LOGF(LogRemoteControl, Warning, "SetProperty not supported");
			return false;
		}

			// Static arrays
		else if (InProperty->ArrayDim > 1)
		{
			UE_LOGF(LogRemoteControl, Warning, "Static arrays not supported");
			return false;
		}

			// All other properties
		else
		{
			return WriteProperty(Entity, InProperty, nullptr, InRangeMappingBuffers, InProtocolValue, InCborWriter);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

// Optionally supplying the MappingId is used by the undo system
FRemoteControlProtocolMapping::FRemoteControlProtocolMapping(FProperty* InProperty, uint8 InRangeValueSize, const FGuid& InMappingId)
	: Id(InMappingId)
{
	if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
	{
		InterpolationMappingPropertyData.AddZeroed(sizeof(bool));
	}
	else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
	{
		InterpolationMappingPropertyData.AddZeroed(NumericProperty->GetElementSize());
	}
	else if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		UScriptStruct* ScriptStruct = StructProperty->Struct;

		InterpolationMappingPropertyData.AddZeroed(ScriptStruct->GetStructureSize());
		ScriptStruct->InitializeStruct(InterpolationMappingPropertyData.GetData());
	}
	else
	{
		InterpolationMappingPropertyData.AddZeroed(1);
	}

	InterpolationRangePropertyData.SetNumZeroed(InRangeValueSize);

	BoundPropertyPath = InProperty;
}

bool FRemoteControlProtocolMapping::operator==(const FRemoteControlProtocolMapping& InProtocolMapping) const
{
	return Id == InProtocolMapping.Id;
}

bool FRemoteControlProtocolMapping::operator==(FGuid InProtocolMappingId) const
{
	return Id == InProtocolMappingId;
}

TSharedPtr<FStructOnScope> FRemoteControlProtocolMapping::GetMappingPropertyAsStructOnScope()
{
	if (FStructProperty* StructProperty = CastField<FStructProperty>(BoundPropertyPath.Get()))
	{
		UScriptStruct* ScriptStruct = StructProperty->Struct;
		check(InterpolationMappingPropertyData.Num() && InterpolationMappingPropertyData.Num() == ScriptStruct->GetStructureSize());

		return MakeShared<FStructOnScope>(ScriptStruct, InterpolationMappingPropertyData.GetData());
	}

	ensure(false);
	return nullptr;
}

bool FRemoteControlProtocolMapping::PropertySizeMatchesData(const TArray<uint8>& InSource, const FName& InPropertyTypeName)
{
	const int32 SourceNum = InSource.Num();
	if(InPropertyTypeName == NAME_ByteProperty
		|| InPropertyTypeName == NAME_BoolProperty)
	{
		return SourceNum == sizeof(uint8);
	}
	else if(InPropertyTypeName == NAME_UInt16Property
		|| InPropertyTypeName == NAME_Int16Property)
	{
		return SourceNum == sizeof(uint16);
	}
	else if(InPropertyTypeName == NAME_UInt32Property
		|| InPropertyTypeName == NAME_Int32Property
		|| InPropertyTypeName == NAME_FloatProperty)
	{
		return SourceNum == sizeof(uint32);
	}
	else if(InPropertyTypeName == NAME_UInt64Property
		|| InPropertyTypeName == NAME_Int64Property
		|| InPropertyTypeName == NAME_DoubleProperty)
	{
		return SourceNum == sizeof(uint64);
	}

	// @note: only the above types are expected
	return false;
}

void FRemoteControlProtocolMapping::RefreshCachedData(const FName& InRangePropertyTypeName)
{
	// Opportunity to write a different representation of the range and mapping properties to be used at runtime
	// (persisted range might be a 1byte uint8, but the property 4byte uint32)
	// (persisted mapping property might be serialized as text)

	// @note: range type seems to be always interpreted from a 4byte value, so just check for that
	if(InRangePropertyTypeName != NAME_None)
	{
		if(!PropertySizeMatchesData(InterpolationRangePropertyData, InRangePropertyTypeName))
		{
			if(InRangePropertyTypeName.ToString().Contains(TEXT("Int")))
			{
				// Only handles UInt32 range types when theres a size mismatch
				if(InRangePropertyTypeName == NAME_UInt32Property)
				{
					uint32 CachedRangeValue = 0;
					if(InterpolationRangePropertyData.Num() == 1)
					{
						uint8* StoredRangeValue = reinterpret_cast<uint8*>(InterpolationRangePropertyData.GetData());
						if(StoredRangeValue != nullptr)
						{
							CachedRangeValue = static_cast<uint32>(*StoredRangeValue);				
						}
					}
					else if(InterpolationRangePropertyData.Num() == 2)
					{
						uint16* StoredRangeValue = reinterpret_cast<uint16*>(InterpolationRangePropertyData.GetData());
						if(StoredRangeValue != nullptr)
						{
							CachedRangeValue = static_cast<uint32>(*StoredRangeValue);
						}
					}
					else if(InterpolationRangePropertyData.Num() == 8)
					{
						uint64* StoredRangeValue = reinterpret_cast<uint64*>(InterpolationRangePropertyData.GetData());
						if(StoredRangeValue != nullptr)
						{
							CachedRangeValue = static_cast<uint32>(*StoredRangeValue);
						}
					}

					InterpolationRangePropertyDataCache.SetNumZeroed(sizeof(uint32));
					*reinterpret_cast<uint32*>(InterpolationRangePropertyDataCache.GetData()) = CachedRangeValue;
				}
			}
		}
		else
		{
			InterpolationRangePropertyDataCache = InterpolationRangePropertyData;
		}
	}

	FProperty* Property = CastField<FProperty>(BoundPropertyPath.Get());
	if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		const RemoteControlPropertyUtilities::FRCPropertyVariant Src{StructProperty, InterpolationMappingPropertyData, InterpolationMappingPropertyElementNum};
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{StructProperty, InterpolationMappingPropertyDataCache};
		RemoteControlPropertyUtilities::Deserialize<FStructProperty>(Src, Dst);
	}
	else if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		const RemoteControlPropertyUtilities::FRCPropertyVariant Src{ArrayProperty, InterpolationMappingPropertyData, InterpolationMappingPropertyElementNum};
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{ArrayProperty, InterpolationMappingPropertyDataCache};
		RemoteControlPropertyUtilities::Deserialize<FArrayProperty>(Src, Dst);
	}
	else
	{
		InterpolationMappingPropertyDataCache = InterpolationMappingPropertyData;
	}
}

void FRemoteControlProtocolEntity::Init(URemoteControlPreset* InOwner, FGuid InPropertyId)
{
	Owner = InOwner;
	PropertyId = MoveTemp(InPropertyId);
}

uint8 FRemoteControlProtocolEntity::GetRangePropertySize() const
{
	if (const EName* PropertyType = GetRangePropertyName().ToEName())
	{
		switch (*PropertyType)
		{
		case NAME_Int8Property:
			return sizeof(int8);

		case NAME_Int16Property:
			return sizeof(int16);

		case NAME_IntProperty:
			return sizeof(int32);

		case NAME_Int64Property:
			return sizeof(int64);

		case NAME_ByteProperty:
			return sizeof(uint8);

		case NAME_UInt16Property:
			return sizeof(uint16);

		case NAME_UInt32Property:
			return sizeof(uint32);

		case NAME_UInt64Property:
			return sizeof(uint64);

		case NAME_FloatProperty:
			return sizeof(float);

		case NAME_DoubleProperty:
			return sizeof(double);

		default:
			break;
		}
	}

	checkNoEntry();
	return 0;
}

const FString& FRemoteControlProtocolEntity::GetRangePropertyMaxValue() const
{
	// returns an empty string by default, so the max value isn't clamped.
	static FString Empty = "";
	return Empty;
}

bool FRemoteControlProtocolEntity::ApplyProtocolValueToProperty(const double InProtocolValue)
{
	// DEPRECATED 5.5

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (Mappings.Num() <= 1)
	{
		UE_LOGF(LogRemoteControl, Warning, "Binding doesn't container any range mappings.");
		return true;
	}

	URemoteControlPreset* Preset = Owner.Get();
	if (!Preset)
	{
		return false;
	}

	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin();
	if (!RemoteControlProperty.IsValid())
	{
		return false;
	}

	if(!RemoteControlProperty->IsBound())
	{
		UE_LOGF(LogRemoteControl, Warning, "Entity isn't bound to any objects.");
		return true;
	}

	if (RemoteControlProperty->GetActiveMasks() == ERCMask::NoMask)
	{
		return true;
	}

	FProperty* Property = RemoteControlProperty->GetProperty();
	if (!Property)
	{
		return false;
	}

	if (!RemoteControlTypeUtilities::IsSupportedMappingType(Property))
	{
		UE_LOGF(LogRemoteControl, Warning, "Property type %ls is unsupported for mapping.", *Property->GetClass()->GetName());
		return true;
	}

	FRCObjectReference ObjectRef;
	ObjectRef.Property = Property;
	ObjectRef.Access = ERCAccess::WRITE_ACCESS;
	
	const ERCModifyOperationFlags ModifyOperationFlags = Preset->GetModifyOperationFlagsForProtocols();
	if (EnumHasAnyFlags(ModifyOperationFlags, ERCModifyOperationFlags::None))
	{
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
	}

	ObjectRef.PropertyPathInfo = RemoteControlProperty->FieldPathInfo.ToString();

	bool bSuccess = true;
	for (UObject* Object : RemoteControlProperty->GetBoundObjects())
	{
		IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);

		TSharedPtr<FRCMaskingOperation> MaskingOperation = MakeShared<FRCMaskingOperation>(ObjectRef.PropertyPathInfo, Object);
		if (OverridenMasks == static_cast<uint8>(ERCMask::NoMask))
		{
			MaskingOperation->Masks = RemoteControlProperty->GetActiveMasks();
		}
		else
		{
			MaskingOperation->Masks = static_cast<ERCMask>(OverridenMasks);
		}

		// Cache the values.
		IRemoteControlModule::Get().PerformMasking(MaskingOperation.ToSharedRef());

		// Set properties after interpolation
		TArray<uint8> InterpolatedBuffer;
		if (GetInterpolatedPropertyBuffer(Property, InProtocolValue, InterpolatedBuffer))
		{
			constexpr ERCModifyOperation ModifyOperation = ERCModifyOperation::EQUAL;

			FMemoryReader MemoryReader(InterpolatedBuffer);
			FCborStructDeserializerBackend CborStructDeserializerBackend(MemoryReader);
			bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, CborStructDeserializerBackend, ERCPayloadType::Cbor, InterpolatedBuffer, ModifyOperation, ModifyOperationFlags);
		
			// Apply the masked the values.
			IRemoteControlModule::Get().PerformMasking(MaskingOperation.ToSharedRef());
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return bSuccess;
}

ERCBindingStatus FRemoteControlProtocolEntity::ToggleBindingStatus()
{
	if (BindingStatus == ERCBindingStatus::Awaiting)
	{
		BindingStatus = ERCBindingStatus::Bound;
	}
	else if (BindingStatus == ERCBindingStatus::Bound || BindingStatus == ERCBindingStatus::Unassigned)
	{
		BindingStatus = ERCBindingStatus::Awaiting;
	}

	return BindingStatus;
}

void FRemoteControlProtocolEntity::ResetDefaultBindingState()
{
	BindingStatus = ERCBindingStatus::Unassigned;
}

void FRemoteControlProtocolEntity::ClearMask(ERCMask InMaskBit)
{
	OverridenMasks &= ~static_cast<uint8>(InMaskBit);
}

void FRemoteControlProtocolEntity::EnableMask(ERCMask InMaskBit)
{
	OverridenMasks |= static_cast<uint8>(InMaskBit);
}

bool FRemoteControlProtocolEntity::HasMask(ERCMask InMaskBit) const
{
	return (OverridenMasks & static_cast<uint8>(InMaskBit)) != static_cast<uint8>(ERCMask::NoMask);
}

#if WITH_EDITOR

const FName FRemoteControlProtocolEntity::GetPropertyName(const FName& ForColumnName)
{
	RegisterProperties();

	if (const FName* PropertyName = ColumnsToProperties.Find(ForColumnName))
	{
		return *PropertyName;
	}

	return NAME_None;
}

#endif // WITH_EDITOR

bool FRemoteControlProtocolEntity::GetInterpolatedPropertyBuffer(FProperty* InProperty, double InProtocolValue, TArray<uint8>& OutBuffer)
{
	// DEPRECATED 5.5

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	using namespace UE::RemoteControl;

	OutBuffer.Empty();

	TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers = GetRangeMappingBuffers();
	bool bSuccess = false;

	// Write interpolated properties to Cbor buffer
	FMemoryWriter MemoryWriter(OutBuffer);
	FCborWriter CborWriter(&MemoryWriter);
	CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);

	// Normalize before apply
	switch (*GetRangePropertyName().ToEName())
	{
	case NAME_Int8Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<int8>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_Int16Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<int16>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_IntProperty:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<int32>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_Int64Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<int64>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_ByteProperty:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<uint8>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_UInt16Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<uint16>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_UInt32Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<uint32>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_UInt64Property:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<uint64>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_FloatProperty:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<float>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	case NAME_DoubleProperty:
		bSuccess = Private::ApplyProtocolValueToProperty(this, InProperty, static_cast<double>(InProtocolValue), RangeMappingBuffers, CborWriter);
		break;
	default:
		checkNoEntry();
	}

	CborWriter.WriteContainerEnd();

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return bSuccess;
}

TArray<FRemoteControlProtocolEntity::FRangeMappingData> FRemoteControlProtocolEntity::GetRangeMappingBuffers()
{
	TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers;
	RangeMappingBuffers.Reserve(Mappings.Num());

	for (FRemoteControlProtocolMapping& Mapping : Mappings)
	{
		if (Mapping.InterpolationMappingPropertyDataCache.Num() == 0
			|| Mapping.InterpolationRangePropertyDataCache.Num() == 0)
		{
			Mapping.RefreshCachedData(GetRangePropertyName());
		}

		RangeMappingBuffers.Emplace(
			Mapping.InterpolationRangePropertyDataCache,
			Mapping.InterpolationMappingPropertyDataCache,
			Mapping.InterpolationMappingPropertyElementNum);
	}

	return RangeMappingBuffers;
}

// Optionally supplying the BindingId is used by the undo system
FRemoteControlProtocolBinding::FRemoteControlProtocolBinding(const FName InProtocolName, const FGuid& InPropertyId, TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InRemoteControlProtocolEntityPtr, const FGuid& InBindingId)
	: Id(InBindingId)
	, ProtocolName(InProtocolName)
	, PropertyId(InPropertyId)
	, RemoteControlProtocolEntityPtr(InRemoteControlProtocolEntityPtr)
{}

bool FRemoteControlProtocolBinding::operator==(const FRemoteControlProtocolBinding& InProtocolBinding) const
{
	return Id == InProtocolBinding.Id;
}

bool FRemoteControlProtocolBinding::operator==(FGuid InProtocolBindingId) const
{
	return Id == InProtocolBindingId;
}

int32 FRemoteControlProtocolBinding::RemoveMapping(const FGuid& InMappingId)
{
	if (FRemoteControlProtocolEntity* ProtocolEntity = GetRemoteControlProtocolEntity())
	{
		return ProtocolEntity->Mappings.RemoveByHash(GetTypeHash(InMappingId), InMappingId);
	}

	return ensure(0);
}

void FRemoteControlProtocolBinding::ClearMappings()
{
	if (FRemoteControlProtocolEntity* ProtocolEntity = GetRemoteControlProtocolEntity())
	{
		ProtocolEntity->Mappings.Empty();
		return;
	}

	ensure(false);
}

void FRemoteControlProtocolBinding::AddMapping(const FRemoteControlProtocolMapping& InMappingsData)
{
	if (FRemoteControlProtocolEntity* ProtocolEntity = GetRemoteControlProtocolEntity())
	{
		ProtocolEntity->Mappings.Add(InMappingsData);
		return;
	}

	ensure(false);
}

void FRemoteControlProtocolBinding::ForEachMapping(FGetProtocolMappingCallback InCallback)
{
	if (FRemoteControlProtocolEntity* ProtocolEntity = GetRemoteControlProtocolEntity())
	{
		for (FRemoteControlProtocolMapping& Mapping : ProtocolEntity->Mappings)
		{
			InCallback(Mapping);
		}
	}
}

bool FRemoteControlProtocolBinding::SetPropertyDataToMapping(const FGuid& InMappingId, const void* InPropertyValuePtr)
{
	if (FRemoteControlProtocolMapping* Mapping = FindMapping(InMappingId))
	{
		FMemory::Memcpy(Mapping->InterpolationMappingPropertyData.GetData(), InPropertyValuePtr, Mapping->InterpolationMappingPropertyData.Num());
		return true;
	}

	return false;
}

FRemoteControlProtocolMapping* FRemoteControlProtocolBinding::FindMapping(const FGuid& InMappingId)
{
	if (FRemoteControlProtocolEntity* ProtocolEntity = GetRemoteControlProtocolEntity())
	{
		return ProtocolEntity->Mappings.FindByHash(GetTypeHash(InMappingId), InMappingId);
	}

	ensure(false);
	return nullptr;
}

TSharedPtr<FStructOnScope> FRemoteControlProtocolBinding::GetStructOnScope() const
{
	return RemoteControlProtocolEntityPtr;
}

void FRemoteControlProtocolBinding::AddStructReferencedObjects(FReferenceCollector& InCollector)
{
	// Check that the shared ptr and the underlying struct on scope are valid
	if (RemoteControlProtocolEntityPtr.IsValid() && RemoteControlProtocolEntityPtr->IsValid())
	{
		// The struct should be guaranteed a script struct (i.e. since it's a TStructOnScope<FRemoteControlProtocolEntity>)
		const UScriptStruct* EntityType = CastChecked<UScriptStruct>(RemoteControlProtocolEntityPtr->GetStruct());
		InCollector.AddPropertyReferencesWithStructARO(EntityType, RemoteControlProtocolEntityPtr->Get());
	}
}

FRemoteControlProtocolEntity* FRemoteControlProtocolBinding::GetRemoteControlProtocolEntity()
{
	if (RemoteControlProtocolEntityPtr.IsValid())
	{
		return RemoteControlProtocolEntityPtr->Get();
	}
	return nullptr;
}

bool FRemoteControlProtocolBinding::Serialize(FArchive& Ar)
{
	if (Ar.IsLoading() || Ar.IsSaving())
	{
		Ar << *this;
	}

	return true;
}

FArchive& operator<<(FArchive& Ar, FRemoteControlProtocolBinding& InProtocolBinding)
{
	UScriptStruct* ScriptStruct = FRemoteControlProtocolBinding::StaticStruct();

	ScriptStruct->SerializeTaggedProperties(Ar, (uint8*)&InProtocolBinding, ScriptStruct, nullptr);

	// Serialize TStructOnScope
	if (Ar.IsLoading())
	{
		InProtocolBinding.RemoteControlProtocolEntityPtr = MakeShared<TStructOnScope<FRemoteControlProtocolEntity>>();
		Ar << *InProtocolBinding.RemoteControlProtocolEntityPtr;
	}
	else if (Ar.IsSaving())
	{
		TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> EntityPtr = InProtocolBinding.RemoteControlProtocolEntityPtr;

		if (FRemoteControlProtocolEntity* ProtocolEntity = InProtocolBinding.GetRemoteControlProtocolEntity())
		{
			Ar << *EntityPtr;
		}
	}

	return Ar;
}

uint32 GetTypeHash(const FRemoteControlProtocolMapping& InProtocolMapping)
{
	return GetTypeHash(InProtocolMapping.Id);
}

uint32 GetTypeHash(const FRemoteControlProtocolBinding& InProtocolBinding)
{
	return GetTypeHash(InProtocolBinding.Id);
}
