// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlModule.h"
#include "RemoteControlProtocolBinding.h"

namespace UE::RemoteControl::ProtocolEntityInterpolator
{
	namespace Internal
	{
		/** 
		 * Converts a map with the range buffer pointers to a map with value pointers. Range keys stay the same.
		 *
		 * For example, when we have an input map TArray<TPair<int32, uint8*>> where uint8* points to the float buffer (4 bytes).
		 * we need to convert it to value buffer where we convert uint8* to value pointer. In this case, is float*.
		 */
		template <typename ValueType, typename PropertyType, typename RangeValueType>
		TArray<FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType>> ContainerPtrMapToValuePtrMap(PropertyType* InProperty, FProperty* Outer, TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, int32 InArrayIndex)
		{
			TArray<FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType>> ValueMap;
			ValueMap.Reserve(InRangeMappingBuffers.Num());

			int32 MappingIndex = 0;
			for (FRemoteControlProtocolEntity::FRangeMappingData& RangePair : InRangeMappingBuffers)
			{
				FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType> TypedPair;
				TypedPair.Range = *reinterpret_cast<RangeValueType*>(RangePair.Range.GetData());

				// In cases when we are dealing with Struct or any containers (Array, Map, Structs) inner elements we need to convert the container pointer to value pointer
				if (Outer)
				{
					if (Outer->GetClass() == FArrayProperty::StaticClass())
					{
						FScriptArrayHelper Helper(CastField<FArrayProperty>(Outer), RangePair.Mapping.GetData());
						TypedPair.Mapping = *reinterpret_cast<ValueType*>(Helper.GetRawPtr(InArrayIndex));
					}
					else
					{
						TypedPair.Mapping = *InProperty->template ContainerPtrToValuePtr<ValueType>(RangePair.Mapping.GetData(), InArrayIndex);
					}
				}
				else
				{
					TArray<uint8> Buffer;
					RemoteControlPropertyUtilities::FRCPropertyVariant Src{ InProperty, RangePair.Mapping };
					RemoteControlPropertyUtilities::FRCPropertyVariant Dst{ InProperty, Buffer };
					RemoteControlPropertyUtilities::Deserialize<PropertyType>(Src, Dst);
					TypedPair.Mapping = *Dst.GetPropertyValue<ValueType>();
				}

				ValueMap.Add(TypedPair);
				MappingIndex++;
			}

			return ValueMap;
		}

		/** Wraps FMath::Lerp, allowing value specific specialization */
		template <class T, class U>
		static T Lerp(const T& A, const T& B, const U& Alpha)
		{
			return FMath::Lerp(A, B, Alpha);
		}

		/** Specialization for bool, toggles at 0.5 Alpha instead of 1.0 */
		template <>
		bool Lerp(const bool& A, const bool& B, const float& Alpha)
		{
			return Alpha >= 0.5f ? B : A;
		}

		/** Specialization for FString, toggles at 0.5 Alpha instead of 1.0 */
		template <>
		FString Lerp(const FString& A, const FString& B, const float& Alpha)
		{
			return Alpha >= 0.5f ? B : A;
		}

		/** Specialization for FName, toggles at 0.5 Alpha instead of 1.0 */
		template <>
		FName Lerp(const FName& A, const FName& B, const float& Alpha)
		{
			return Alpha >= 0.5f ? B : A;
		}

		/** Specialization for FText, toggles at 0.5 Alpha instead of 1.0 */
		template <>
		FText Lerp(const FText& A, const FText& B, const float& Alpha)
		{
			return Alpha >= 0.5f ? B : A;
		}

		/** Interpolates the range of an entity, using the protocol value as alpha */
		template <typename RangeValueType, typename ValueType, typename PropertyType>
		bool InterpolateValue(PropertyType* InProperty, FProperty* InOuter, TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, RangeValueType InProtocolValue, ValueType& OutResultValue, int32 InArrayIndex = 0)
		{
			TArray<FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType>> ValueMap = ContainerPtrMapToValuePtrMap<ValueType, PropertyType, RangeValueType>(InProperty, InOuter, InRangeMappingBuffers, InArrayIndex);

			// sort by input protocol value
			Algo::SortBy(ValueMap, [](const FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType>& Item) -> RangeValueType
				{
					return Item.Range;
				});

			if (ValueMap.IsEmpty())
			{
				return false;
			}

			// clamp to min and max mapped values
			RangeValueType ClampProtocolValue = FMath::Clamp(InProtocolValue, ValueMap[0].Range, ValueMap.Last().Range);

			FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType*, ValueType*> RangeMin{ nullptr, nullptr };
			FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType*, ValueType*> RangeMax{ nullptr, nullptr };

			for (int32 RangeIdx = 0; RangeIdx < ValueMap.Num(); ++RangeIdx)
			{
				FRemoteControlProtocolEntity::TRangeMappingData<RangeValueType, ValueType>& Range = ValueMap[RangeIdx];
				if (ClampProtocolValue > Range.Range || RangeMin.Mapping == nullptr)
				{
					RangeMin.Range = &Range.Range;
					RangeMin.Mapping = &Range.Mapping;
				}
				else if (ClampProtocolValue <= Range.Range)
				{
					RangeMax.Range = &Range.Range;
					RangeMax.Mapping = &Range.Mapping;
					// Max found, no need to continue
					break;
				}
			}

			if (RangeMax.Mapping == nullptr || RangeMin.Mapping == nullptr)
			{
				return ensure(false);
			}
			if (RangeMax.Range == nullptr || RangeMin.Range == nullptr)
			{
				return ensure(false);
			}
			else if (*RangeMax.Range == *RangeMin.Range)
			{
				UE_LOGF(LogRemoteControl, Warning, "Range input values are the same.");
				return true;
			}

			const float Alpha = static_cast<float>(ClampProtocolValue - *RangeMin.Range) / static_cast<float>(*RangeMax.Range - *RangeMin.Range);

			OutResultValue = Lerp(*RangeMin.Mapping, *RangeMax.Mapping, Alpha);

			return true;
		}
	}

	/** Interpolates the range of an entity, using the protocol value as alpha */
	template <typename ValueType, typename PropertyType>
	bool InterpolateValue(FRemoteControlProtocolEntity* InEntity, PropertyType* InProperty, FProperty* InOuter, TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, double InProtocolValue, ValueType& OutResultValue, int32 InArrayIndex = 0)
	{
		// Depending on the range property type
		switch (*InEntity->GetRangePropertyName().ToEName())
		{
		case NAME_DoubleProperty:
			return Internal::InterpolateValue<double>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_FloatProperty:
			return Internal::InterpolateValue<float>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_Int8Property:
			return Internal::InterpolateValue<int8>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_Int16Property:
			return Internal::InterpolateValue<int16>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_IntProperty:
			return Internal::InterpolateValue<int32>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_Int64Property:
			return Internal::InterpolateValue<int64>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_ByteProperty:
			return Internal::InterpolateValue<uint8>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_UInt16Property:
			return Internal::InterpolateValue<uint16>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_UInt32Property:
			return Internal::InterpolateValue<uint32>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		case NAME_UInt64Property:
			return Internal::InterpolateValue<uint64>(InProperty, InOuter, InRangeMappingBuffers, InProtocolValue, OutResultValue, InArrayIndex);
		default:
			checkNoEntry();
		}

		return false;
	}
}
