// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolEntityProcessor.h"

#include "Algo/Find.h"
#include "Algo/MinElement.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "CborWriter.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "RemoteControlCommon.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolEntityInterpolator.h"

namespace UE::RemoteControl::ProtocolEntityProcessor
{
	/** Fundamental types */
	namespace Types
	{
		/** Holds the data that correspond to a single property. */
		struct FRCPropertyData
		{
			/** The preset in which the property resides */
			URemoteControlPreset* Preset = nullptr;

			/** The property */
			TSharedPtr<FRemoteControlProperty> Property;

			/** The entities of this property with their protocol value */
			TMap<FRemoteControlProtocolEntity*, double> EntityToValueMap;
		};
	}

	/** Methods to write cbor values */
	namespace Cbor
	{
		/** Writes a property value to the serialization output */
		template <typename ValueType>
		void WritePropertyValue(FCborWriter& CborWriter, FProperty* InProperty, const ValueType& Value, bool bWriteName = true)
		{
			if (bWriteName)
			{
				CborWriter.WriteValue(InProperty->GetName());
			}

			CborWriter.WriteValue(Value);
		}

		/** Writes an FName property value to the serialization output */
		template <>
		void WritePropertyValue<FName>(FCborWriter& CborWriter, FProperty* InProperty, const FName& Value, bool bWriteName)
		{
			if (bWriteName)
			{
				CborWriter.WriteValue(InProperty->GetName());
			}
			CborWriter.WriteValue(Value.ToString());
		}

		/** Writes an FText property value to the serialization output */
		template <>
		void WritePropertyValue<FText>(FCborWriter& CborWriter, FProperty* InProperty, const FText& Value, bool bWriteName)
		{
			if (bWriteName)
			{
				CborWriter.WriteValue(InProperty->GetName());
			}
			CborWriter.WriteValue(Value.ToString());
		}
	}

	/** Methods to apply masking */
	namespace Masking
	{
		template <typename VectorType>
		bool IsInMask(const FName& PropertyName, ERCMask Mask)
		{
			if constexpr (
				std::is_same_v<VectorType, FVector> ||
				std::is_same_v<VectorType, FIntVector>)
			{
				static const FName X = "X";
				static const FName Y = "Y";
				static const FName Z = "Z";

				return
					(EnumHasAnyFlags(Mask, ERCMask::MaskA) && PropertyName == X) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskB) && PropertyName == Y) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskC) && PropertyName == Z);
			}
			else if constexpr (
				std::is_same_v<VectorType, FVector4> ||
				std::is_same_v<VectorType, FIntVector4>)
			{
				static const FName X = "X";
				static const FName Y = "Y";
				static const FName Z = "Z";
				static const FName W = "W";

				return
					(EnumHasAnyFlags(Mask, ERCMask::MaskA) && PropertyName == X) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskB) && PropertyName == Y) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskC) && PropertyName == Z) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskD) && PropertyName == W);
			}
			else if constexpr (
				std::is_same_v<VectorType, FColor> ||
				std::is_same_v<VectorType, FLinearColor>)
			{
				static const FName R = "R";
				static const FName G = "G";
				static const FName B = "B";
				static const FName A = "A";

				return
					(EnumHasAnyFlags(Mask, ERCMask::MaskA) && PropertyName == R) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskB) && PropertyName == G) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskC) && PropertyName == B) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskD) && PropertyName == A);
			}
			else if constexpr (std::is_same_v<VectorType, FRotator>)
			{
				static const FName Roll = "Roll";
				static const FName Pitch = "Pitch";
				static const FName Yaw = "Yaw";

				return
					(EnumHasAnyFlags(Mask, ERCMask::MaskA) && PropertyName == Roll) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskB) && PropertyName == Pitch) ||
					(EnumHasAnyFlags(Mask, ERCMask::MaskC) && PropertyName == Yaw);
			}
			else
			{
				return false;
			}
		}
	}

	/** Methods to write properties with a Cbor writer */
	namespace PropertyWriter
	{
		/** Writes an interpolated property value using the Cbor writer */
		template <typename ProtocolValueType>
		bool WriteInterpolatedPropertyValue(
			FRemoteControlProtocolEntity* Entity, 
			FProperty* InProperty, 
			FProperty* OuterProperty, 
			TArray<FRemoteControlProtocolEntity::FRangeMappingData>& InRangeMappingBuffers, 
			ProtocolValueType InProtocolValue, 
			FCborWriter& CborWriter, 
			int32 InArrayIndex = 0)
		{
			// Value nested in Array/Set (except single element) or map as array or as root
			const bool bIsInArray = OuterProperty != nullptr
				&& (InProperty->ArrayDim > 1
					|| OuterProperty->GetClass() == FArrayProperty::StaticClass()
					|| OuterProperty->GetClass() == FSetProperty::StaticClass()
					|| OuterProperty->GetClass() == FMapProperty::StaticClass());


			bool bSuccess = false;
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(InProperty))
			{
				bool bBoolValue = false;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, BoolProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, bBoolValue, InArrayIndex);
				bBoolValue = StaticCast<uint8>(bBoolValue) > 0; // Ensure 0 or 1, can be different if property was packed.
				Cbor::WritePropertyValue(CborWriter, InProperty, bBoolValue, !bIsInArray);
			}
			else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
			{
				if (CastField<FFloatProperty>(InProperty))
				{
					float FloatValue = 0.f;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, FloatValue, InArrayIndex);
					Cbor::WritePropertyValue(CborWriter, InProperty, FloatValue, !bIsInArray);
				}
				else if (CastField<FDoubleProperty>(InProperty))
				{
					double DoubleValue = 0.0;
					bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, DoubleValue, InArrayIndex);
					Cbor::WritePropertyValue(CborWriter, InProperty, DoubleValue, !bIsInArray);
				}
				else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
				{
					if (FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
					{
						uint8 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FIntProperty* IntProperty = CastField<FIntProperty>(InProperty))
					{
						int IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(InProperty))
					{
						uint32 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt16Property* Int16Property = CastField<FInt16Property>(InProperty))
					{
						int16 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(InProperty))
					{
						uint16 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt64Property* Int64Property = CastField<FInt64Property>(InProperty))
					{
						int64 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(InProperty))
					{
						uint64 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt8Property* Int8Property = CastField<FInt8Property>(InProperty))
					{
						int8 IntValue = 0;
						bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NumericProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, IntValue, InArrayIndex);
						Cbor::WritePropertyValue(CborWriter, InProperty, static_cast<int64>(IntValue), !bIsInArray);
					}
				}
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
			{
				if (!bIsInArray)
				{
					CborWriter.WriteValue(StructProperty->GetName());
				}

				CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
				{
					for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
					{
						TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers;
						RangeMappingBuffers.Reserve(InRangeMappingBuffers.Num());

						FProperty* InnerProperty = *It;
						for (const FRemoteControlProtocolEntity::FRangeMappingData& RangePair : InRangeMappingBuffers)
						{
							const uint8* DataInContainer = StructProperty->ContainerPtrToValuePtr<const uint8>(RangePair.Mapping.GetData(), InArrayIndex);
							const uint8* DataInStruct = InnerProperty->ContainerPtrToValuePtr<const uint8>(DataInContainer);
							RangeMappingBuffers.Emplace(RangePair.Range, DataInStruct, InnerProperty->GetSize(), 1);
						}

						bSuccess &= WriteInterpolatedPropertyValue(Entity, InnerProperty, StructProperty, RangeMappingBuffers, InProtocolValue, CborWriter, InArrayIndex);
					}
				}
				CborWriter.WriteContainerEnd();
			}

			else if (FStrProperty* StrProperty = CastField<FStrProperty>(InProperty))
			{
				FString StringValue;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, StrProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, StringValue, InArrayIndex);
				Cbor::WritePropertyValue(CborWriter, InProperty, StringValue, !bIsInArray);
			}
			else if (FNameProperty* NameProperty = CastField<FNameProperty>(InProperty))
			{
				FName NameValue;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, NameProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, NameValue, InArrayIndex);
				Cbor::WritePropertyValue(CborWriter, InProperty, NameValue, !bIsInArray);
			}
			else if (FTextProperty* TextProperty = CastField<FTextProperty>(InProperty))
			{
				FText TextValue;
				bSuccess = ProtocolEntityInterpolator::InterpolateValue(Entity, TextProperty, OuterProperty, InRangeMappingBuffers, InProtocolValue, TextValue, InArrayIndex);
				Cbor::WritePropertyValue(CborWriter, InProperty, TextValue, !bIsInArray);
			}

#if !UE_BUILD_SHIPPING && UE_BUILD_DEBUG
			if (!bSuccess)
			{
				if (RemoteControlTypeUtilities::IsSupportedMappingType(InProperty))
				{
					UE_LOGF(LogRemoteControl, Error, "Property type %ls is supported for mapping, but unhandled in ProtocolEntityProcessor::WriteInterpolatedPropertyValue", *InProperty->GetClass()->GetName());
				}
			}
#endif

			return bSuccess;
		}

		/** Writes the property value as is, without using the protocol value and interpolating */
		bool WriteUnchangedPropertyValue(
			const FRCObjectReference& ObjectRef, 
			FProperty* Property, 
			FProperty* OuterProperty, 
			FCborWriter& CborWriter, 
			int32 InArrayIndex = 0)
		{
			if (!Property || !OuterProperty)
			{
				return false;
			}

			// Get the outer in which properties are stored
			void* Outer = [&OuterProperty, &ObjectRef]() -> void*
				{
					if (OuterProperty->GetOwner<UClass>())
					{
						return OuterProperty->ContainerPtrToValuePtr<void>(ObjectRef.Object.Get());
					}
					else
					{
						FProperty* OwnerProperty = OuterProperty->GetOwnerProperty();
						if (OwnerProperty && OwnerProperty != OuterProperty)
						{
							return OuterProperty->ContainerPtrToValuePtr<void>(OwnerProperty);
						}
					}
					
					return nullptr;
				}();
			if (!Outer)
			{
				return false;
			}

			// Value nested in Array/Set (except single element) or map as array or as root
			const bool bIsInArray = OuterProperty != nullptr
				&& (Property->ArrayDim > 1
					|| OuterProperty->GetClass() == FArrayProperty::StaticClass()
					|| OuterProperty->GetClass() == FSetProperty::StaticClass()
					|| OuterProperty->GetClass() == FMapProperty::StaticClass());

			bool bSuccess = false;
			if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
			{
				const bool* ValuePtr = Property->ContainerPtrToValuePtr<bool>(Outer);
				const bool bBoolValue = ValuePtr ? *ValuePtr : false;
				Cbor::WritePropertyValue(CborWriter, Property, bBoolValue, !bIsInArray);
			}
			else if (FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
			{
				if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
				{
					const float* ValuePtr = Property->ContainerPtrToValuePtr<float>(Outer);
					const float FloatValue = ValuePtr ? *ValuePtr : 0.f;
					Cbor::WritePropertyValue(CborWriter, Property, FloatValue, !bIsInArray);
				}
				else if (FDoubleProperty* DoubleProperty = CastField<FDoubleProperty>(Property))
				{
					const double* ValuePtr = Property->ContainerPtrToValuePtr<double>(Outer);
					const double DoubleValue = ValuePtr ? *ValuePtr : 0.0;
					Cbor::WritePropertyValue(CborWriter, Property, DoubleValue, !bIsInArray);
				}
				else if (NumericProperty->IsInteger() && !NumericProperty->IsEnum())
				{
					if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
					{
						const uint8* ValuePtr = Property->ContainerPtrToValuePtr<uint8>(Outer);
						const uint8 ByteValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(ByteValue), !bIsInArray);
					}
					else if (FIntProperty* IntProperty = CastField<FIntProperty>(Property))
					{
						const int32* ValuePtr = Property->ContainerPtrToValuePtr<int32>(Outer);
						const int32 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt32Property* UInt32Property = CastField<FUInt32Property>(Property))
					{
						const uint32* ValuePtr = Property->ContainerPtrToValuePtr<uint32>(Outer);
						const uint32 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt16Property* Int16Property = CastField<FInt16Property>(Property))
					{
						const int16* ValuePtr = Property->ContainerPtrToValuePtr<int16>(Outer);
						const int16 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt16Property* FInt16Property = CastField<FUInt16Property>(Property))
					{
						const uint16* ValuePtr = Property->ContainerPtrToValuePtr<uint16>(Outer);
						const uint16 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt64Property* Int64Property = CastField<FInt64Property>(Property))
					{
						const int64* ValuePtr = Property->ContainerPtrToValuePtr<int64>(Outer);
						const int64 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FUInt64Property* FInt64Property = CastField<FUInt64Property>(Property))
					{
						const uint64* ValuePtr = Property->ContainerPtrToValuePtr<uint64>(Outer);
						const uint64 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
					else if (FInt8Property* Int8Property = CastField<FInt8Property>(Property))
					{
						const int8* ValuePtr = Property->ContainerPtrToValuePtr<int8>(Outer);
						int8 IntValue = ValuePtr ? *ValuePtr : 0;
						Cbor::WritePropertyValue(CborWriter, Property, static_cast<int64>(IntValue), !bIsInArray);
					}
				}
			}
			else if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (!bIsInArray)
				{
					CborWriter.WriteValue(StructProperty->GetName());
				}

				CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
				{
					bool bStructSuccess = true;
					for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
					{
						bSuccess &= WriteUnchangedPropertyValue(ObjectRef, *It, StructProperty, CborWriter, InArrayIndex);
					}
				}
				CborWriter.WriteContainerEnd();
			}

			else if (FStrProperty* StrProperty = CastField<FStrProperty>(Property))
			{
				const FString* ValuePtr = Property->ContainerPtrToValuePtr<FString>(Outer);
				const FString StringValue = ValuePtr ? *ValuePtr : TEXT("");
				Cbor::WritePropertyValue(CborWriter, Property, StringValue, !bIsInArray);
			}
			else if (FNameProperty* NameProperty = CastField<FNameProperty>(Property))
			{
				const FName* ValuePtr = Property->ContainerPtrToValuePtr<FName>(Outer);
				const FName NameValue = ValuePtr ? *ValuePtr : NAME_None;
				Cbor::WritePropertyValue(CborWriter, Property, NameValue, !bIsInArray);
			}
			else if (FTextProperty* TextProperty = CastField<FTextProperty>(Property))
			{
				const FText* ValuePtr = Property->ContainerPtrToValuePtr<FText>(Outer);
				const FText TextValue = ValuePtr ? *ValuePtr : FText();
				Cbor::WritePropertyValue(CborWriter, Property, TextValue, !bIsInArray);
			}

#if !UE_BUILD_SHIPPING && UE_BUILD_DEBUG
			if (!bSuccess)
			{
				if (RemoteControlTypeUtilities::IsSupportedMappingType(Property))
				{
					UE_LOGF(LogRemoteControl, Error, "Property type %ls is supported for mapping, but unhandled in ProtocolEntityProcessor::WriteUnchangedPropertyValue", *Property->GetClass()->GetName());
				}
			}
#endif

			return bSuccess;
		}

		/** Writes a struct property that supports masking, considering the mask */
		template <typename VectorType>
		bool WriteMaskedStructProperty(
			const FRCObjectReference& ObjectRef, 
			FStructProperty* StructProperty, 
			const Types::FRCPropertyData& PropertyData, 
			FCborWriter& CborWriter)
		{
			using namespace UE::RemoteControl;

			if (!StructProperty)
			{
				return false;
			}

			bool bSuccess = true;
			for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
			{
				const FName& PropertyName = (*It)->GetFName();

				// Find an entity that fits the current mask and property name 
				const TTuple<FRemoteControlProtocolEntity*, double>* EntityToValuePtr = Algo::FindByPredicate(PropertyData.EntityToValueMap,
					[&PropertyName, &PropertyData](const TTuple<FRemoteControlProtocolEntity*, double>& EntityToValuePair)
					{
						const ERCMask Mask = EntityToValuePair.Key->GetOverridenMask();
						return Masking::IsInMask<VectorType>(PropertyName, Mask);
					});

				if (EntityToValuePtr)
				{
					// Write the interpolated property value
					TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers = EntityToValuePtr->Key->GetRangeMappingBuffers();
					bSuccess &= WriteInterpolatedPropertyValue(EntityToValuePtr->Key, (*It), StructProperty, RangeMappingBuffers, EntityToValuePtr->Value, CborWriter);
				}
				else
				{
					// There is no property that fits the mask, hence write the unchanged property value
					bSuccess &= WriteUnchangedPropertyValue(ObjectRef, (*It), StructProperty, CborWriter);
				}
			}

			return true;
		}

		/** Writes a single property using the Cbor writer */
		bool WriteProperty(const FRCObjectReference& ObjectRef, FProperty* Property, FProperty* OuterProperty, const Types::FRCPropertyData& PropertyData, FCborWriter& CborWriter, int32 ArrayIndex = 0)
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(OuterProperty);
			const bool bIsMaskedStructProperty = StructProperty && DoesScriptStructSupportMasking(StructProperty->Struct);

			bool bSuccess = true;
			if (bIsMaskedStructProperty)
			{
				if (StructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FVector>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FRotator>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FRotator>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FColor>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FColor>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FLinearColor>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FLinearColor>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FIntVector>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FIntVector>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FVector4>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FVector4>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else if (StructProperty->Struct == TBaseStructure<FIntVector4>::Get())
				{
					bSuccess = WriteMaskedStructProperty<FIntVector4>(ObjectRef, StructProperty, PropertyData, CborWriter);
				}
				else
				{
					ensureMsgf(0, TEXT("Struct type %s should but does not support masking. Cannot process remote control for struct."), *StructProperty->Struct->GetName());
				}
			}
			else
			{
				for (const TTuple<FRemoteControlProtocolEntity*, double>& EntityToValuePair : PropertyData.EntityToValueMap)
				{
					TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers = EntityToValuePair.Key->GetRangeMappingBuffers();

					bSuccess &= WriteInterpolatedPropertyValue(EntityToValuePair.Key, Property, OuterProperty, RangeMappingBuffers, EntityToValuePair.Value, CborWriter, ArrayIndex);
				}
			}

			return bSuccess;
		}

		/** Converts properties that need to be processed into a map of Property Ids and their related property data */
		TMap<FGuid, Types::FRCPropertyData> BuildPropertyToDataMap(const TMap<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, double>& EntityToValueMap)
		{
			TMap<FGuid, Types::FRCPropertyData> PropertyToDataMap;
			for (const TTuple<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, double>& EntityToValuePair : EntityToValueMap)
			{
				if (EntityToValuePair.Key.IsValid() && EntityToValuePair.Key->IsValid())
				{
					FRemoteControlProtocolEntity* Entity = EntityToValuePair.Key->CastChecked<FRemoteControlProtocolEntity>();
					const FGuid& PropertyId = Entity->GetPropertyId();

					Types::FRCPropertyData& PropertyData = PropertyToDataMap.FindOrAdd(Entity->GetPropertyId());
					if (!PropertyData.Preset)
					{
						PropertyData.Preset = Entity->GetOwner().Get();
					}

					// Property may turn invalid if the entity got deleted, but did not unbind from protocols yet
					if (!PropertyData.Preset)
					{
						PropertyToDataMap.Remove(PropertyId);
						continue;
					}

					if (!PropertyData.Property.IsValid())
					{
						PropertyData.Property = PropertyData.Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin();
					}

					// Property may turn invalid if the entity got deleted, but did not unbind from protocols yet
					if (!PropertyData.Property)
					{
						PropertyToDataMap.Remove(PropertyId);
						continue;
					}

					PropertyData.EntityToValueMap.Add(EntityToValuePair.Key->CastChecked<FRemoteControlProtocolEntity>(), EntityToValuePair.Value);
				}
			}

			return PropertyToDataMap;
		}

		/** Serializes a single property into the Cbor buffer */
		bool SerializeProperty(const FRCObjectReference& ObjectRef, const Types::FRCPropertyData& PropertyData, TArray<uint8>& OutCborBuffer)
		{
			using namespace UE::RemoteControl;

			if (PropertyData.EntityToValueMap.IsEmpty())
			{
				return false;
			}

			bool bSuccess = true;

			FMemoryWriter MemoryWriter(OutCborBuffer);
			FCborWriter CborWriter(&MemoryWriter);

			CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
			{
				FProperty* Property = PropertyData.Property->GetProperty();

				// Structs
				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					CborWriter.WriteValue(StructProperty->GetName());
					CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
					{
						bool bStructSuccess = true;
						for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
						{
							bSuccess &= WriteProperty(ObjectRef, *It, StructProperty, PropertyData, CborWriter);
						}
					}
					CborWriter.WriteContainerEnd();
				}

				// @note: temporarily disabled - array of primitives supported, array of structs not
				// Dynamic arrays
				else if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
				{
					CborWriter.WriteValue(ArrayProperty->GetName());
					CborWriter.WriteContainerStart(ECborCode::Array, -1/*Indefinite*/);
					{
						check(!PropertyData.EntityToValueMap.IsEmpty());

						FProperty* InnerProperty = ArrayProperty->Inner;
						TArray<FRemoteControlProtocolEntity::FRangeMappingData> RangeMappingBuffers = PropertyData.EntityToValueMap.begin()->Key->GetRangeMappingBuffers();

						// Get minimum item count
						const FRemoteControlProtocolEntity::FRangeMappingData* ElementWithSmallestNum = Algo::MinElementBy(RangeMappingBuffers, [](const FRemoteControlProtocolEntity::FRangeMappingData& RangePair)
							{
								return RangePair.NumElements;
							});

						// No elements in array
						if (ElementWithSmallestNum)
						{
							for (auto ArrayIndex = 0; ArrayIndex < ElementWithSmallestNum->NumElements; ++ArrayIndex)
							{
								bSuccess &= WriteProperty(ObjectRef, InnerProperty, ArrayProperty, PropertyData, CborWriter, ArrayIndex);
							}
						}
					}
					CborWriter.WriteContainerEnd();
				}

				// Maps
				else if (FMapProperty* MapProperty = CastField<FMapProperty>(Property))
				{
					UE_LOGF(LogRemoteControl, Warning, "MapProperty not supported");
					bSuccess = false;
				}

				// Sets
				else if (FSetProperty* SetProperty = CastField<FSetProperty>(Property))
				{
					UE_LOGF(LogRemoteControl, Warning, "SetProperty not supported");
					bSuccess = false;
				}

				// Static arrays
				else if (Property->ArrayDim > 1)
				{
					UE_LOGF(LogRemoteControl, Warning, "Static arrays not supported");
					bSuccess = false;
				}

				// All other properties
				else
				{
					bSuccess = WriteProperty(ObjectRef, Property, nullptr, PropertyData, CborWriter);
				}
			}
			CborWriter.WriteContainerEnd();

			return bSuccess;
		}

		/** Compares and swaps the protocol value of each entity in property data, returns true if all are equal. */
		bool CompareSwapProtocolValuesForProperty(const Types::FRCPropertyData& PropertyData)
		{
			bool bEqual = true;
			for (const TTuple<FRemoteControlProtocolEntity*, double>& EntityToValuePair : PropertyData.EntityToValueMap)
			{
				check(EntityToValuePair.Key);
				if (!FMath::IsNearlyEqual(EntityToValuePair.Key->ProtocolValue, EntityToValuePair.Value))
				{
					bEqual = false;
					EntityToValuePair.Key->ProtocolValue = EntityToValuePair.Value;
				}
			}

			return bEqual;
		}
	}
	
	bool DoesScriptStructSupportMasking(const UScriptStruct* InStruct)
	{
		if (InStruct)
		{
			// When adding new types here, also add them to logic in ProcessProperties
			return
				InStruct == TBaseStructure<FVector>::Get() ||
				InStruct == TBaseStructure<FRotator>::Get() ||
				InStruct == TBaseStructure<FColor>::Get() ||
				InStruct == TBaseStructure<FLinearColor>::Get() ||
				InStruct == TBaseStructure<FIntVector>::Get() ||
				InStruct == TBaseStructure<FVector4>::Get() ||
				InStruct == TBaseStructure<FIntVector4>::Get();
		}

		return false;
	}

	bool DoesPropertySupportMasking(const FProperty* Property)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		return StructProperty ? DoesScriptStructSupportMasking(StructProperty->Struct) : false;
	}

	void ProcessEntities(const TMap<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, double>& EntityToValueMap)
	{
		using namespace PropertyWriter;
		using namespace Types;

		const TMap<FGuid, FRCPropertyData> PropertyToDataMap = BuildPropertyToDataMap(EntityToValueMap);
		
		if (PropertyToDataMap.IsEmpty())
		{
			return;
		}

		// Process each property
		TArray<AActor*> ChangedActors;
		for (const TTuple<FGuid, FRCPropertyData>& PropertyToDataPair : PropertyToDataMap)
		{
			const URemoteControlPreset* Preset = PropertyToDataPair.Value.Preset;
			const TSharedPtr<FRemoteControlProperty>& RemoteControlProperty = PropertyToDataPair.Value.Property;
			if (!PropertyToDataPair.Value.Property->IsBound())
			{
				UE_LOGF(LogRemoteControl, Warning, "Entity isn't bound to any objects.");
				continue;
			}

			FProperty* Property = RemoteControlProperty->GetProperty();
			if (!Property)
			{
				continue;
			}

			if (!RemoteControlTypeUtilities::IsSupportedMappingType(Property))
			{
				UE_LOGF(LogRemoteControl, Warning, "Property type %ls is unsupported for mapping.", *Property->GetClass()->GetName());
				continue;
			}

			if (CompareSwapProtocolValuesForProperty(PropertyToDataPair.Value))
			{
				continue;
			}

			FRCObjectReference ObjectRef;
			ObjectRef.Property = Property;
			ObjectRef.Access = ERCAccess::WRITE_ACCESS;
			ObjectRef.PropertyPathInfo = RemoteControlProperty->FieldPathInfo.ToString();

			const ERCModifyOperationFlags ModifyOperationFlags = Preset->GetModifyOperationFlagsForProtocols();
			if (EnumHasAnyFlags(ModifyOperationFlags, ERCModifyOperationFlags::None))
			{
				ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
			}

			bool bSuccess = true;
			for (UObject* BoundObject : RemoteControlProperty->GetBoundObjects())
			{
				// Resolve the property for the currently bound bject
				if (!IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, BoundObject, ObjectRef.PropertyPathInfo, ObjectRef))
				{
					continue;
				}

				// Remember actors about to be changed
				if (AActor* Actor = Cast<AActor>(BoundObject))
				{
					ChangedActors.AddUnique(Actor);
				}
				else if (AActor* ParentActor = BoundObject->GetTypedOuter<AActor>())
				{
					ChangedActors.AddUnique(ParentActor);
				}

				TArray<uint8> CborBuffer;
				if (SerializeProperty(ObjectRef, PropertyToDataPair.Value, CborBuffer))
				{
					constexpr ERCModifyOperation ModifyOperation = ERCModifyOperation::EQUAL;

					FMemoryReader MemoryReader(CborBuffer);
					FCborStructDeserializerBackend CborStructDeserializerBackend(MemoryReader);
					bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, CborStructDeserializerBackend, ERCPayloadType::Cbor, CborBuffer, ModifyOperation, ModifyOperationFlags);
				}
			}
		}

		// Update changed actors
		for (AActor* Actor : ChangedActors)
		{
			Actor->UpdateComponentTransforms();
			Actor->MarkComponentsRenderStateDirty();
		}
	}
}
