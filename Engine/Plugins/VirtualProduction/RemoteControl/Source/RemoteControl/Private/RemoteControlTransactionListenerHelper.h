// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "IRemoteControlModule.h"
#include "RemoteControlTransactionListenerManager.h"
/**
 * Helper class which create a Listener and Register to the Manager
 * @tparam CallbackTypeArgs Callback Args type
 */
template <typename  ...CallbackTypeArgs>
class FRCTransactionListenerHelper
{
	FRCTransactionListenerHelper() = delete;
	
public:
	
	/**
	 * Create and Register a Listener to the ListenerManager
	 * @param InType When the broadcast should be called (Undo or Redo), see ERCTransactionType
	 * @param PresetGuid Preset Guid to check whether or not is Transient
	 * @param InDelegateToBroadcast Multicast delegate to broadcast
	 * @param InOrderedArgs Ordered arguments that should be passed to the MulticastDelegate
	 */
	FRCTransactionListenerHelper(ERCTransaction::Type InType, const FGuid& PresetGuid, TMulticastDelegate<void(CallbackTypeArgs...)>& InDelegateToBroadcast, CallbackTypeArgs... InOrderedArgs)
	{
#if WITH_EDITOR
		if (!IRemoteControlModule::Get().IsPresetTransient(PresetGuid))
		{
			FRCTransactionListenerManager<CallbackTypeArgs...>::CreateListenerAndRegister(InType, InDelegateToBroadcast, Forward<CallbackTypeArgs>(InOrderedArgs)...);
		}
#endif // WITH_EDITOR
	}
};