// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#if WITH_EDITOR
#include "Editor.h"
#include "Editor/TransBuffer.h"
#endif // WITH_EDITOR
#include "Misc/ITransaction.h"

/**
 * Enum that helps to distinguish between Undo and Redo
 */
namespace ERCTransaction
{
	enum Type
	{
		None,
		Undo,
		Redo
	};
}

/**
 * Collection of FRCTransactionListener which holds them and executes their Callback on the respective UTransBuffer Callback.
 * @tparam CallbackArgs Callback Args type
 */
template <typename... CallbackArgs>
class FRCTransactionListenerManager
{
	/**
 	* Helper class that stores a callback and its Args to then call it later on during Transaction Undo/Redo.
 	* Assign the Callback and the Args (if needed) before calling RegisterToCollection.
 	* @tparam ListenerCallbackArgs Args types to pass to the callback
 	*/
	template <typename... ListenerCallbackArgs>
	class FRCTransactionListener
	{	
		FRCTransactionListener() = delete;

	public:

		/**
		* Actual listener which holds the broadcast to make at Undo or Redo
	 	* @param InType When the broadcast should be called (Undo or Redo)
	 	* @param InDelegateToBroadcast Multicast delegate to broadcast
	 	* @param InArgs Ordered arguments that should be passed to the MulticastDelegate
		*/
		FRCTransactionListener(ERCTransaction::Type InType, TMulticastDelegate<void(ListenerCallbackArgs...)>& InDelegateToBroadcast, ListenerCallbackArgs&&... InArgs) : Type(InType)
		{
			OnUndoRedoDelegate.AddLambda([&InDelegateToBroadcast, InArgs...]()
			{
				InDelegateToBroadcast.Broadcast(InArgs...);
			});
		}

		/**
		 * Getter for the holder of the callback called during Undo or Redo
		 * @return The UndoRedo Multicast Delegate
		 */
		const TMulticastDelegate<void()>& OnUndoRedo()
		{
			return OnUndoRedoDelegate;
		}

	public:
		/**
 		 * Type of the transaction
 		 */
		ERCTransaction::Type Type;

	private:
		/**
 		 * Holds the callback to be executed during Undo or Redo
 		 */
		TMulticastDelegate<void()> OnUndoRedoDelegate;
	};

public:

	/**
 	 * Create a listener with the parameter passed and Register it into the Manager to be called during Undo/Redo callbacks
 	 */
	static void CreateListenerAndRegister(ERCTransaction::Type InType, TMulticastDelegate<void(CallbackArgs...)>& InDelegateToBroadcast, CallbackArgs&&... InOrderedArgs)
	{
		FRCTransactionListener<CallbackArgs...> Listener = FRCTransactionListener<CallbackArgs...>(InType, InDelegateToBroadcast, Forward<CallbackArgs>(InOrderedArgs)...);
		Get().Register(Listener);
	}
	
private:
	
	/**
	 * Return the only Instance for CallbackType and CallbackArgs of this class.
	 * @return Singleton of the class.
	 */
	static FRCTransactionListenerManager<CallbackArgs...>& Get()
	{
		if (!Instance.IsValid())
		{
			Instance = TUniquePtr<FRCTransactionListenerManager<CallbackArgs...>>(new FRCTransactionListenerManager<CallbackArgs...>());
		}
		return *(Instance.Get());
	}

public:

	/**
 	 * UnSubscribe to TransactionBuffer callback
 	 */
	~FRCTransactionListenerManager()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);

			TransBuffer->OnTransactionStateChanged().RemoveAll(this);
			TransBuffer->OnUndo().RemoveAll(this);
			TransBuffer->OnRedo().RemoveAll(this);
			TransBuffer->OnUndoBufferChanged().RemoveAll(this);
		}
#endif
	}
	
private:
	
	/**
	 * Add the Listener to the PendingListener.
	 * @param Listener Listener to add to the PendingListeners
	 */
	void Register(FRCTransactionListener<CallbackArgs...>& Listener)
	{
		PendingListeners.Add(Listener);
	}
	
	/**
	 * Private constructor used to bind to TransactionBuffer callback
	 */
	FRCTransactionListenerManager()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);

			TransBuffer->OnTransactionStateChanged().AddRaw(this, &FRCTransactionListenerManager::OnTransactionChanged);
			TransBuffer->OnUndo().AddRaw(this, &FRCTransactionListenerManager::OnUndo);
			TransBuffer->OnRedo().AddRaw(this, &FRCTransactionListenerManager::OnRedo);
			TransBuffer->OnUndoBufferChanged().AddRaw(this, &FRCTransactionListenerManager::OnUndoBufferChanged);
		}
#endif
	}

	/**
	 * Execute all the Listeners that are binded to the current TransactionId and have the same TransactionType.
	 * @param InListeners Listener binded to the current TransactionId.
	 * @param InTransactionType Current Transaction type (Undo or Redo).
	 */
	void ExecuteAllListeners(TArray<FRCTransactionListener<CallbackArgs...>>& InListeners, ERCTransaction::Type InTransactionType)
	{
		for (FRCTransactionListener<CallbackArgs...>& InListener : InListeners)
		{
			if (InListener.Type == InTransactionType)
			{
				InListener.OnUndoRedo().Broadcast();
			}
		}
	}

	/**
	 * Gets executed when an Undo is attempted.
	 * @param InTransactionContext Transaction information.
	 * @param bSucceeded True if the Undo succeeded otherwise false.
	 */
	void OnUndo(const FTransactionContext& InTransactionContext, bool bSucceeded)
	{
		if (!bSucceeded)
		{
			return;
		}
		if (TArray<FRCTransactionListener<CallbackArgs...>>* FoundListeners = Listeners.Find(
			InTransactionContext.TransactionId))
		{
			ExecuteAllListeners(*FoundListeners, ERCTransaction::Undo);
		}
	}

	/**
	 * Gets executed when a Redo is attempted.
	 * @param InTransactionContext Transaction information.
	 * @param bSucceeded True if the Redo succeeded otherwise false.
	 */
	void OnRedo(const FTransactionContext& InTransactionContext, bool bSucceeded)
	{
		if (!bSucceeded)
		{
			return;
		}
		if (TArray<FRCTransactionListener<CallbackArgs...>>* FoundListeners = Listeners.Find(
			InTransactionContext.TransactionId))
		{
			ExecuteAllListeners(*FoundListeners, ERCTransaction::Redo);
		}
	}

	/**
 	 * Gets executed when the undo buffer change.
 	 */
	void OnUndoBufferChanged()
	{
		DeleteUnusedCallbacks();
	}

	/**
	 * Delete from the Listeners all the Callbacks which are not binded to any Transaction inside the TransactionBuffer by their TransactionID.
	 */
	void DeleteUnusedCallbacks()
	{
#if WITH_EDITOR
		if (GEditor)
		{
			UTransBuffer* TransBuffer = CastChecked<UTransBuffer>(GEditor->Trans);
			TArray<FGuid> Keys;
			Listeners.GetKeys(Keys);
			for (FGuid Key : Keys)
			{
				if (!TransBuffer->UndoBuffer.ContainsByPredicate([Key](TSharedPtr<FTransaction> InTransaction)
				{
					return InTransaction->GetId() == Key;
				}))
				{
					Listeners.Remove(Key);
				}
			}
		}
		if (PendingListeners.Num() == 0 && Listeners.Num() == 0)
		{
			Instance.Reset();
		}
#endif
	}

	/**
	 * Gets executed when a transaction state changes.
	 * @param InTransactionContext Transaction information.
	 * @param InTransactionState Current state of the Transaction, see ETransactionStateEventType to know all types.
	 */
	void OnTransactionChanged(const FTransactionContext& InTransactionContext, ETransactionStateEventType InTransactionState)
	{
		if (InTransactionState == ETransactionStateEventType::TransactionStarted)
		{
			PendingListeners.Reset();
		}
		else if (InTransactionState == ETransactionStateEventType::TransactionFinalized
			&& !Listeners.Contains(InTransactionContext.TransactionId))
		{
			if (!PendingListeners.IsEmpty())
			{
				Listeners.FindOrAdd(InTransactionContext.TransactionId).Append(PendingListeners);
				PendingListeners.Empty();
			}
			DeleteUnusedCallbacks();
		}
	}

private:
	static TUniquePtr<FRCTransactionListenerManager<CallbackArgs...>> Instance;
	TArray<FRCTransactionListener<CallbackArgs...>> PendingListeners;
	TMap<FGuid, TArray<FRCTransactionListener<CallbackArgs...>>> Listeners;
};

template <typename... CallbackArgs>
TUniquePtr<FRCTransactionListenerManager<CallbackArgs...>> FRCTransactionListenerManager<CallbackArgs...>::Instance(nullptr);
