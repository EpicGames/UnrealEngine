// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyMapStructDeserializerBackendWrapper.h"
#include "UObject/UnrealType.h"

bool FPropertyMapStructDeserializerBackendWrapper::ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	if (InnerBackend.ReadProperty(Property, Outer, Data, ArrayIndex))
	{
		FReadPropertyData ReadPropertyData;
		ReadPropertyData.Property = Property;
		ReadPropertyData.Data = Property->ContainerPtrToValuePtr<void>(Data, ArrayIndex);

		ReadProperties.Add(ReadPropertyData);
		
		return true;
	}

	return false;
}