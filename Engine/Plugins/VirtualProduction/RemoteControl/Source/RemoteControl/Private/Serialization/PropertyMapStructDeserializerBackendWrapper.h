// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructDeserializerBackend.h"

/**
 * Wraps an existing IStructDeserializerBackend and passes calls through to it, but tracks which properties
 * were successfully read in the process of deserialization.
 */
class FPropertyMapStructDeserializerBackendWrapper
	: public IStructDeserializerBackend
{
public:
	struct FReadPropertyData
	{
		FProperty* Property;
		void* Data;
	};

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InnerBackend The deserializer to wrap. This will actually handle deserializing the struct.
	 */
	FPropertyMapStructDeserializerBackendWrapper(IStructDeserializerBackend& InnerBackend)
		: InnerBackend(InnerBackend)
	{ }

	// IStructDeserializerBackend interface

	virtual const FString& GetCurrentPropertyName() const override { return InnerBackend.GetCurrentPropertyName(); }
	virtual FString GetDebugString() const override { return InnerBackend.GetDebugString(); }
	virtual const FString& GetLastErrorMessage() const override { return InnerBackend.GetLastErrorMessage(); }
	virtual bool GetNextToken(EStructDeserializerBackendTokens& OutToken) override { return InnerBackend.GetNextToken(OutToken); }
	virtual void SkipArray() override { InnerBackend.SkipArray(); }
	virtual void SkipStructure() override { InnerBackend.SkipStructure(); }
	virtual bool ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) override;

	/** Get the array of properties that have been read so far during deserialization. */
	const TArray<FReadPropertyData>& GetReadProperties() const { return ReadProperties; }


private:
	IStructDeserializerBackend& InnerBackend;
	TArray<FReadPropertyData> ReadProperties;
};
