// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDeltaAPITestData.h"

const FLinearColor FRemoteControlDeltaAPITestStruct::ColorValueDefault = FLinearColor(0.5f, 0.1f, 0.3f, 0.8f);

const int32 URemoteControlDeltaAPITestObject::Int32ValueDefault = 523;
const float URemoteControlDeltaAPITestObject::FloatValueDefault = 0.3f;
const FVector URemoteControlDeltaAPITestObject::VectorValueDefault = FVector(112.f, -40.f, 37.f);
const FLinearColor URemoteControlDeltaAPITestObject::ColorValueDefault = FLinearColor(0.2f, 0.6f, 0.9f, 0.5f);