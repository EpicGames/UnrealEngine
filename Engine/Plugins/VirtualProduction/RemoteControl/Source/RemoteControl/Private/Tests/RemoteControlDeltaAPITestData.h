// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "RemoteControlDeltaAPITestData.generated.h"

USTRUCT()
struct FRemoteControlDeltaAPITestStruct
{
	GENERATED_BODY()

public:
	static const FLinearColor ColorValueDefault;

	UPROPERTY(EditAnywhere, Category = "RC")
	FLinearColor ColorValue = ColorValueDefault;
};

UCLASS()
class URemoteControlDeltaAPITestObject : public UObject
{
	GENERATED_BODY()

public:
	static const int32 Int32ValueDefault;
	static const float FloatValueDefault;
	static const FVector VectorValueDefault;
	static const FLinearColor ColorValueDefault;
	
	static const FName GetInt32WithSetterValuePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32WithSetterValue);
	}

	static const FName GetFloatWithSetterValuePropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, FloatWithSetterValue);
	}

	UFUNCTION(BlueprintGetter)
	int32 GetInt32WithSetterValue() { return Int32WithSetterValue; }

	UFUNCTION(BlueprintSetter)
	void SetInt32WithSetterValue(const int32 NewValue) { Int32WithSetterValue = NewValue; }
	
	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = Int32ValueDefault;

	UPROPERTY(EditAnywhere, Category = "RC")
	float FloatValue = FloatValueDefault;

	UPROPERTY(EditAnywhere, Category = "RC")
	FVector VectorValue = VectorValueDefault;

	UPROPERTY(EditAnywhere, Category = "RC")
	FLinearColor ColorValue = ColorValueDefault;

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlDeltaAPITestStruct StructValue;

private:
	UPROPERTY(EditAnywhere, Category = "RC", BlueprintGetter = GetInt32WithSetterValue, BlueprintSetter = SetInt32WithSetterValue)
	int32 Int32WithSetterValue;

	/** Uses Setter - not BlueprintSetter */
	UPROPERTY(EditAnywhere, Setter, Category = "RC")
	float FloatWithSetterValue = FloatValueDefault;

public:
	float GetFloatWithSetterValue() { return FloatWithSetterValue; }
	void SetFloatWithSetterValue(const float& InValue);
};

inline void URemoteControlDeltaAPITestObject::SetFloatWithSetterValue(const float& InValue)
{
	FloatWithSetterValue = InValue;
}
