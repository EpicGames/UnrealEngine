// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Engine.h"
#include "Misc/AutomationTest.h"

#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "Dom/JsonObject.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonSerializer.h"
#include "StructSerializer.h"

#include "IRemoteControlModule.h"
#include "RemoteControlDeltaAPITestData.h"

BEGIN_DEFINE_SPEC(FRemoteControlDeltaAPISpec, "Plugins.RemoteControl.DeltaAPI", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

URemoteControlDeltaAPITestObject* TestObject;

TArray<uint8> JsonBuffer;
FMemoryReader Reader = FMemoryReader(JsonBuffer);
FMemoryWriter Writer = FMemoryWriter(JsonBuffer);
FJsonStructSerializerBackend SerializerBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
FJsonStructDeserializerBackend DeserializerBackend = FJsonStructDeserializerBackend(Reader);

/** Fill out JsonBuffer with data from a string containing JSON data. */
void SerializeBufferFromJsonString(const FString& JsonString);

/** Fill out JsonBuffer with data for the given property using the data stored in TestObject. */
void SerializeElementAndReset(FProperty* Property);

/** Reset all values in TestObject to their defaults. */
void ResetTestObject();

/**
 * Apply an operation to a property by calling IRemoteControlModule::SetObjectProperties.
 * 
 * @param Property The property to modify on TestObject.
 * @param Operation The operation to apply to the property.
 * @param bIncludeInterceptPayload If true, JsonBuffer will be passed as the InPayload argument of SetObjectProperties. Otherwise, an empty array will be passed.
 */
void ApplyOperation(FProperty* Property, ERCModifyOperation Operation, bool bIncludeInterceptPayload = true);

END_DEFINE_SPEC(FRemoteControlDeltaAPISpec)

void FRemoteControlDeltaAPISpec::SerializeBufferFromJsonString(const FString& JsonString)
{
	JsonBuffer.SetNumUninitialized(JsonString.GetAllocatedSize());
	FMemory::Memcpy(JsonBuffer.GetData(), JsonString.GetCharArray().GetData(), JsonString.GetAllocatedSize());
}

void FRemoteControlDeltaAPISpec::SerializeElementAndReset(FProperty* Property)
{
	FStructSerializer::SerializeElement(TestObject, Property, INDEX_NONE, SerializerBackend, FStructSerializerPolicies());

	// We use the test object to temporarily store the value we want to serialize, so reset it afterward since we'll also use it to test the change
	ResetTestObject();
}

void FRemoteControlDeltaAPISpec::ResetTestObject()
{
	TestObject->Int32Value = URemoteControlDeltaAPITestObject::Int32ValueDefault;
	TestObject->FloatValue = URemoteControlDeltaAPITestObject::FloatValueDefault;
	TestObject->VectorValue = URemoteControlDeltaAPITestObject::VectorValueDefault;
	TestObject->ColorValue = URemoteControlDeltaAPITestObject::ColorValueDefault;
	TestObject->SetInt32WithSetterValue(URemoteControlDeltaAPITestObject::Int32ValueDefault);
	TestObject->SetFloatWithSetterValue(URemoteControlDeltaAPITestObject::FloatValueDefault);

	TestObject->StructValue.ColorValue = FRemoteControlDeltaAPITestStruct::ColorValueDefault;
}

void FRemoteControlDeltaAPISpec::ApplyOperation(FProperty* Property, ERCModifyOperation Operation, bool bIncludeInterceptPayload)
{
	const FRCFieldPathInfo FieldPath(Property);
	FRCObjectReference ObjectRef;
	IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::WRITE_ACCESS, TestObject, FieldPath, ObjectRef);
	IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Json, bIncludeInterceptPayload ? JsonBuffer : TArray<uint8>(), Operation);
}

void FRemoteControlDeltaAPISpec::Define()
{
	BeforeEach([this]
	{
		 TestObject = NewObject<URemoteControlDeltaAPITestObject>();

		 JsonBuffer.Empty();

		 Writer.Reset();
		 Writer.Seek(0);

		 Reader.Reset();
		 Reader.Seek(0);

		 SerializerBackend = FJsonStructSerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		 DeserializerBackend = FJsonStructDeserializerBackend(Reader);
	});

	Describe("Add", [this]
	{
		It("should add integers", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32Value));

			const int32 AddedValue = 42;
			TestObject->Int32Value = AddedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->Int32Value, URemoteControlDeltaAPITestObject::Int32ValueDefault + AddedValue);
		});

		It("should add floats", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, FloatValue));

			const float AddedValue = 0.6f;
			TestObject->FloatValue = AddedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->FloatValue, URemoteControlDeltaAPITestObject::FloatValueDefault + AddedValue);
		});

		It("should add vectors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			const FVector AddedValue = FVector(-20.f, 0.7f, 101.f);
			TestObject->VectorValue = AddedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->VectorValue, URemoteControlDeltaAPITestObject::VectorValueDefault + AddedValue);
		});

		It("should add colors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, ColorValue));

			const FLinearColor AddedValue = FLinearColor(0.4f, 0.1f, 0.2f, 0.8f);
			TestObject->ColorValue = AddedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->ColorValue, URemoteControlDeltaAPITestObject::ColorValueDefault + AddedValue);
		});

		It("should add a partial struct's elements only to the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			SerializeBufferFromJsonString("{ \"VectorValue\": { \"Y\": 12.3 } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("X value", TestObject->VectorValue.X, URemoteControlDeltaAPITestObject::VectorValueDefault.X);
			TestEqual("Combined Y value", TestObject->VectorValue.Y, URemoteControlDeltaAPITestObject::VectorValueDefault.Y + 12.3f);
			TestEqual("Z value", TestObject->VectorValue.Z, URemoteControlDeltaAPITestObject::VectorValueDefault.Z);
		});

		It("should add property with a blueprint getter/setter", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 AddedValue = 12;
			TestObject->SetInt32WithSetterValue(AddedValue);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault + AddedValue);
		});

		It("should add property with a getter/setter", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetFloatWithSetterValuePropertyName());

			const float AddedValue = 4.0f;
			TestObject->SetFloatWithSetterValue(AddedValue);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->GetFloatWithSetterValue(), URemoteControlDeltaAPITestObject::FloatValueDefault + AddedValue);
		});

		It("should add property with a blueprint getter/setter and no intercept payload", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 AddedValue = 12;
			TestObject->SetInt32WithSetterValue(AddedValue);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD, false);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault + AddedValue);
		});

		It("should add nested struct values", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			const FLinearColor AddedValue = FLinearColor(0.2f, 0.5f, 0.1f, 0.3f);
			TestObject->StructValue.ColorValue = AddedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("Combined value", TestObject->StructValue.ColorValue, FRemoteControlDeltaAPITestStruct::ColorValueDefault + AddedValue);
		});

		It("should add a nested partial struct's elements only to the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			SerializeBufferFromJsonString("{ \"StructValue\": { \"ColorValue\": { \"G\": 0.2 } } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::ADD);

			TestEqual("R value", TestObject->StructValue.ColorValue.R, FRemoteControlDeltaAPITestStruct::ColorValueDefault.R);
			TestEqual("Combined G value", TestObject->StructValue.ColorValue.G, FRemoteControlDeltaAPITestStruct::ColorValueDefault.G + 0.2f);
			TestEqual("B value", TestObject->StructValue.ColorValue.B, FRemoteControlDeltaAPITestStruct::ColorValueDefault.B);
			TestEqual("A value", TestObject->StructValue.ColorValue.A, FRemoteControlDeltaAPITestStruct::ColorValueDefault.A);
		});
	});

	Describe("Subtract", [this]
	{
		It("should subtract integers", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32Value));

			const int32 SubtractedValue = 42;
			TestObject->Int32Value = SubtractedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->Int32Value, URemoteControlDeltaAPITestObject::Int32ValueDefault - SubtractedValue);
		});

		It("should subtract floats", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, FloatValue));

			const float SubtractedValue = 0.6f;
			TestObject->FloatValue = SubtractedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->FloatValue, URemoteControlDeltaAPITestObject::FloatValueDefault - SubtractedValue);
		});

		It("should subtract vectors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			const FVector SubtractedValue = FVector(-20.f, 0.7f, 101.f);
			TestObject->VectorValue = SubtractedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->VectorValue, URemoteControlDeltaAPITestObject::VectorValueDefault - SubtractedValue);
		});

		It("should subtract colors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, ColorValue));

			const FLinearColor SubtractedValue = FLinearColor(0.4f, 0.1f, 0.2f, 0.8f);
			TestObject->ColorValue = SubtractedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->ColorValue, URemoteControlDeltaAPITestObject::ColorValueDefault - SubtractedValue);
		});

		It("should subtract a partial struct's elements only from the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			SerializeBufferFromJsonString("{ \"VectorValue\": { \"Z\": 6.7 } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("X value", TestObject->VectorValue.X, URemoteControlDeltaAPITestObject::VectorValueDefault.X);
			TestEqual("Y value", TestObject->VectorValue.Y, URemoteControlDeltaAPITestObject::VectorValueDefault.Y);
			TestEqual("Combined Z value", TestObject->VectorValue.Z, URemoteControlDeltaAPITestObject::VectorValueDefault.Z - 6.7f);
		});

		It("should subtract property with a blueprint getter/setter", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 SubtractedValue = 27;
			TestObject->SetInt32WithSetterValue(SubtractedValue);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault - SubtractedValue);
		});

		It("should subtract property with a blueprint getter/setter and no intercept payload", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 SubtractedValue = 27;
			TestObject->SetInt32WithSetterValue(SubtractedValue);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT, false);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault - SubtractedValue);
		});

		It("should subtract nested struct values", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			const FLinearColor SubtractedValue = FLinearColor(0.2f, 0.5f, 0.1f, 0.3f);
			TestObject->StructValue.ColorValue = SubtractedValue;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("Combined value", TestObject->StructValue.ColorValue, FRemoteControlDeltaAPITestStruct::ColorValueDefault - SubtractedValue);
		});

		It("should subtract a nested partial struct's elements only to the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			SerializeBufferFromJsonString("{ \"StructValue\": { \"ColorValue\": { \"B\": 0.7 } } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::SUBTRACT);

			TestEqual("R value", TestObject->StructValue.ColorValue.R, FRemoteControlDeltaAPITestStruct::ColorValueDefault.R);
			TestEqual("G value", TestObject->StructValue.ColorValue.G, FRemoteControlDeltaAPITestStruct::ColorValueDefault.G);
			TestEqual("Combined B value", TestObject->StructValue.ColorValue.B, FRemoteControlDeltaAPITestStruct::ColorValueDefault.B - 0.7f);
			TestEqual("A value", TestObject->StructValue.ColorValue.A, FRemoteControlDeltaAPITestStruct::ColorValueDefault.A);
		});
	});

	Describe("Multiply", [this]
	{
		It("should multiply integers", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32Value));

			const int32 Multiplier = 2;
			TestObject->Int32Value = Multiplier;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->Int32Value, URemoteControlDeltaAPITestObject::Int32ValueDefault * Multiplier);
		});

		It("should multiply floats", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, FloatValue));

			const float Multiplier = 3.f;
			TestObject->FloatValue = Multiplier;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->FloatValue, URemoteControlDeltaAPITestObject::FloatValueDefault * Multiplier);
		});

		It("should multiply vectors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			const FVector Multiplier = FVector(-0.5f, 0.2f, 3.f);
			TestObject->VectorValue = Multiplier;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->VectorValue, URemoteControlDeltaAPITestObject::VectorValueDefault * Multiplier);
		});

		It("should multiply colors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, ColorValue));

			const FLinearColor Multiplier = FLinearColor(2.f, 0.8f, 1.6f, 1.3f);
			TestObject->ColorValue = Multiplier;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->ColorValue, URemoteControlDeltaAPITestObject::ColorValueDefault * Multiplier);
		});

		It("should multiply a partial struct's elements only for the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			SerializeBufferFromJsonString("{ \"VectorValue\": { \"X\": 3.2 } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined X value", TestObject->VectorValue.X, URemoteControlDeltaAPITestObject::VectorValueDefault.X * 3.2f);
			TestEqual("Y value", TestObject->VectorValue.Y, URemoteControlDeltaAPITestObject::VectorValueDefault.Y);
			TestEqual("Z value", TestObject->VectorValue.Z, URemoteControlDeltaAPITestObject::VectorValueDefault.Z);
		});

		It("should multiply property with a blueprint getter/setter", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 Multiplier = 6;
			TestObject->SetInt32WithSetterValue(Multiplier);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault * Multiplier);
		});

		It("should multiply property with a blueprint getter/setter and no intercept payload", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 Multiplier = 6;
			TestObject->SetInt32WithSetterValue(Multiplier);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY, false);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault * Multiplier);
		});

		It("should multiply nested struct values", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			const FLinearColor Multiplier = FLinearColor(2.f, 0.8f, 1.6f, 1.3f);
			TestObject->StructValue.ColorValue = Multiplier;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined value", TestObject->StructValue.ColorValue, FRemoteControlDeltaAPITestStruct::ColorValueDefault * Multiplier);
		});

		It("should multiply a nested partial struct's elements only to the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			SerializeBufferFromJsonString("{ \"StructValue\": { \"ColorValue\": { \"R\": 2.5 } } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::MULTIPLY);

			TestEqual("Combined r value", TestObject->StructValue.ColorValue.R, FRemoteControlDeltaAPITestStruct::ColorValueDefault.R * 2.5f);
			TestEqual("G value", TestObject->StructValue.ColorValue.G, FRemoteControlDeltaAPITestStruct::ColorValueDefault.G);
			TestEqual("B value", TestObject->StructValue.ColorValue.B, FRemoteControlDeltaAPITestStruct::ColorValueDefault.B);
			TestEqual("A value", TestObject->StructValue.ColorValue.A, FRemoteControlDeltaAPITestStruct::ColorValueDefault.A);
		});
	});

	Describe("Divide", [this]
	{
		It("should divide integers", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32Value));

			const int32 Divisor = 2;
			TestObject->Int32Value = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->Int32Value, URemoteControlDeltaAPITestObject::Int32ValueDefault / Divisor);
		});

		It("should ignore divisions by zero", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, Int32Value));

			const int32 Divisor = 0;
			TestObject->Int32Value = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Unaffected value", TestObject->Int32Value, URemoteControlDeltaAPITestObject::Int32ValueDefault);
		});

		It("should divide floats", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, FloatValue));

			const float Divisor = 3.f;
			TestObject->FloatValue = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->FloatValue, URemoteControlDeltaAPITestObject::FloatValueDefault / Divisor);
		});

		It("should divide vectors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			const FVector Divisor = FVector(-0.5f, 0.2f, 3.f);
			TestObject->VectorValue = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->VectorValue, URemoteControlDeltaAPITestObject::VectorValueDefault / Divisor);
		});

		It("should divide colors", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, ColorValue));

			const FLinearColor Divisor = FLinearColor(2.f, 0.8f, 1.6f, 1.3f);
			TestObject->ColorValue = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->ColorValue, URemoteControlDeltaAPITestObject::ColorValueDefault / Divisor);
		});

		It("should divide a partial struct's elements only for the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, VectorValue));

			SerializeBufferFromJsonString("{ \"VectorValue\": { \"Y\": 2.2 } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("X value", TestObject->VectorValue.X, URemoteControlDeltaAPITestObject::VectorValueDefault.X);
			TestEqual("Combined Y value", TestObject->VectorValue.Y, URemoteControlDeltaAPITestObject::VectorValueDefault.Y / 2.2f);
			TestEqual("Z value", TestObject->VectorValue.Z, URemoteControlDeltaAPITestObject::VectorValueDefault.Z);
		});

		It("should multiply property with a blueprint getter/setter", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 Divisor = 4;
			TestObject->SetInt32WithSetterValue(Divisor);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault / Divisor);
		});

		It("should multiply property with a blueprint getter/setter and no intercept payload", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), URemoteControlDeltaAPITestObject::GetInt32WithSetterValuePropertyName());

			const int32 Divisor = 4;
			TestObject->SetInt32WithSetterValue(Divisor);

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE, false);

			TestEqual("Combined value", TestObject->GetInt32WithSetterValue(), URemoteControlDeltaAPITestObject::Int32ValueDefault / Divisor);
		});

		It("should divide nested struct values", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			const FLinearColor Divisor = FLinearColor(2.f, 0.8f, 1.6f, 1.3f);
			TestObject->StructValue.ColorValue = Divisor;

			SerializeElementAndReset(ValueProperty);
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("Combined value", TestObject->StructValue.ColorValue, FRemoteControlDeltaAPITestStruct::ColorValueDefault / Divisor);
		});

		It("should divide a nested partial struct's elements only to the provided field", [this]
		{
			FProperty* ValueProperty = FindFProperty<FProperty>(URemoteControlDeltaAPITestObject::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlDeltaAPITestObject, StructValue));

			SerializeBufferFromJsonString("{ \"StructValue\": { \"ColorValue\": { \"A\": 1.7 } } }");
			ApplyOperation(ValueProperty, ERCModifyOperation::DIVIDE);

			TestEqual("R value", TestObject->StructValue.ColorValue.R, FRemoteControlDeltaAPITestStruct::ColorValueDefault.R);
			TestEqual("G value", TestObject->StructValue.ColorValue.G, FRemoteControlDeltaAPITestStruct::ColorValueDefault.G);
			TestEqual("B value", TestObject->StructValue.ColorValue.B, FRemoteControlDeltaAPITestStruct::ColorValueDefault.B);
			TestEqual("Combined A value", TestObject->StructValue.ColorValue.A, FRemoteControlDeltaAPITestStruct::ColorValueDefault.A / 1.7f);
		});
	});
}
