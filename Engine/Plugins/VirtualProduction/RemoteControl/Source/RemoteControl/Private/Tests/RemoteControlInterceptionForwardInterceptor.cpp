// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlInterceptionForwardInterceptor.h"

#include "Features/IModularFeatures.h"

namespace UE::RemoteControl::Test::Private
{
	bool ForEachInterceptionProcessor(TFunctionRef<void(IRemoteControlInterceptionFeatureProcessor*)> InFunction)
	{
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		const FName ProcessorFeatureName = IRemoteControlInterceptionFeatureProcessor::GetName();
		const int32 NumProcessors = ModularFeatures.GetModularFeatureImplementationCount(ProcessorFeatureName);

		for (int32 ProcessorIdx = 0; ProcessorIdx < NumProcessors; ++ProcessorIdx)
		{
			IRemoteControlInterceptionFeatureProcessor* const Processor = static_cast<IRemoteControlInterceptionFeatureProcessor*>(
				ModularFeatures.GetModularFeatureImplementation(ProcessorFeatureName, ProcessorIdx));

			if (Processor)
			{
				InFunction(Processor);
			}
		}
		
		return NumProcessors > 0;
	}
}

ERCIResponse FRemoteControlInterceptionForwardInterceptor::SetObjectProperties(FRCIPropertiesMetadata& InObjectProperties)
{
	using namespace UE::RemoteControl::Test::Private;
	const bool bForwarded = ForEachInterceptionProcessor([&InObjectProperties](IRemoteControlInterceptionFeatureProcessor* InProcessor)
	{
		InProcessor->SetObjectProperties(InObjectProperties);
	});
	return bForwarded ? ERCIResponse::Intercept : ERCIResponse::Apply;
}

ERCIResponse FRemoteControlInterceptionForwardInterceptor::ResetObjectProperties(FRCIObjectMetadata& InObject)
{
	using namespace UE::RemoteControl::Test::Private;
	const bool bForwarded = ForEachInterceptionProcessor([&InObject](IRemoteControlInterceptionFeatureProcessor* InProcessor)
	{
		InProcessor->ResetObjectProperties(InObject);
	});
	return bForwarded ? ERCIResponse::Intercept : ERCIResponse::Apply;
}

ERCIResponse FRemoteControlInterceptionForwardInterceptor::InvokeCall(FRCIFunctionMetadata& InFunction)
{
	using namespace UE::RemoteControl::Test::Private;
	const bool bForwarded = ForEachInterceptionProcessor([&InFunction](IRemoteControlInterceptionFeatureProcessor* InProcessor)
	{
		InProcessor->InvokeCall(InFunction);
	});
	return bForwarded ? ERCIResponse::Intercept : ERCIResponse::Apply;
}

ERCIResponse FRemoteControlInterceptionForwardInterceptor::SetPresetController(FRCIControllerMetadata& InController)
{
	using namespace UE::RemoteControl::Test::Private;
	const bool bForwarded = ForEachInterceptionProcessor([&InController](IRemoteControlInterceptionFeatureProcessor* InProcessor)
	{
		InProcessor->SetPresetController(InController);
	});
	return bForwarded ? ERCIResponse::Intercept : ERCIResponse::Apply;
}
