// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlInterceptionProcessor.h"

/**
 * Interceptor implementation that forwards data directly to the processor.
 */
class FRemoteControlInterceptionForwardInterceptor : public IRemoteControlInterceptionFeatureInterceptor
{
public:
	FRemoteControlInterceptionForwardInterceptor() = default;
	virtual ~FRemoteControlInterceptionForwardInterceptor() override = default;

protected:
	//~ Begin IRemoteControlInterceptionCommands
	virtual ERCIResponse SetObjectProperties(FRCIPropertiesMetadata& InObjectProperties) override;
	virtual ERCIResponse ResetObjectProperties(FRCIObjectMetadata& InObject) override;
	virtual ERCIResponse InvokeCall(FRCIFunctionMetadata& InFunction) override;
	virtual ERCIResponse SetPresetController(FRCIControllerMetadata& InController) override;
	//~ End IRemoteControlInterceptionCommands
};