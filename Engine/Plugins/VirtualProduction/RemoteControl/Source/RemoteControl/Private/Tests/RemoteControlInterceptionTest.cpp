// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "UObject/StrongObjectPtr.h"

// Test related headers
#include "Backends/CborStructSerializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"
#include "CborWriter.h"
#include "Features/IModularFeatures.h"
#include "RemoteControlInterceptionProcessor.h"
#include "RemoteControlInterceptionTestData.h"
#include "RemoteControlPreset.h"
#include "StructSerializer.h"

// Interception related headers
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "IRemoteControlModule.h"
#include "RemoteControlInterceptionForwardInterceptor.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

struct FRemoteControlInterceptionTest
{
	FRemoteControlInterceptionTest() = delete;

	FRemoteControlInterceptionTest(FAutomationTestBase* InTest)
		: Test(InTest)
	{
		// Create Preset and Test object
		Preset = TStrongObjectPtr<URemoteControlPreset>(NewObject<URemoteControlPreset>());
		RemoteControlInterceptionTestObject = TStrongObjectPtr<URemoteControlInterceptionTestObject>{ NewObject<URemoteControlInterceptionTestObject>() };

		// Expose property
		CustomStructProperty = FindFProperty<FProperty>(URemoteControlInterceptionTestObject::StaticClass(),  GET_MEMBER_NAME_CHECKED(URemoteControlInterceptionTestObject,  CustomStruct));
		Int32ValueProperty   = FindFProperty<FProperty>(FRemoteControlInterceptionTestStruct::StaticStruct(), GET_MEMBER_NAME_CHECKED(FRemoteControlInterceptionTestStruct, Int32Value));
		const FString Int32ValuePropertyFullPath = FString::Printf(TEXT("%s.%s"), *CustomStructProperty->GetName(), *Int32ValueProperty->GetName());
		Int32ValuePropertyRCProp = Preset->ExposeProperty(RemoteControlInterceptionTestObject.Get(), FRCFieldPathInfo{ Int32ValuePropertyFullPath }).Pin();

		// Expose property with setter
		static const FName ValueWithSetterName(TEXT("ValueWithSetter"));
		ValueWithSetterProperty = FindFProperty<FProperty>(URemoteControlInterceptionTestObject::StaticClass(), ValueWithSetterName);
		check(ValueWithSetterProperty);
		ValueWithSetterRCProp = Preset->ExposeProperty(RemoteControlInterceptionTestObject.Get(), FRCFieldPathInfo{ ValueWithSetterProperty->GetName() }).Pin();
		
		// Expose function
		UClass* TestObjectClass = RemoteControlInterceptionTestObject->StaticClass();
		UFunction* TestFunctionClass = TestObjectClass->FindFunctionByName("TestFunction");
		TestFunctionRC = Preset->ExposeFunction(RemoteControlInterceptionTestObject.Get(), TestFunctionClass).Pin();
		
		// Reset tested value
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = FRemoteControlInterceptionTestStruct::Int32ValueDefault;

		// Instantiate the interceptor and processor features
		FeatureInterceptor = MakeUnique<FRemoteControlInterceptionForwardInterceptor>();
		FeatureProcessor   = MakeUnique<FRemoteControlInterceptionProcessor>();

		// Register the features
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.RegisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), FeatureInterceptor.Get());
		ModularFeatures.RegisterModularFeature(IRemoteControlInterceptionFeatureProcessor::GetName(),   FeatureProcessor.Get());
	}

	~FRemoteControlInterceptionTest()
	{
		// Unregister the test features
		IModularFeatures& ModularFeatures = IModularFeatures::Get();
		ModularFeatures.UnregisterModularFeature(IRemoteControlInterceptionFeatureInterceptor::GetName(), FeatureInterceptor.Get());
		ModularFeatures.UnregisterModularFeature(IRemoteControlInterceptionFeatureProcessor::GetName(),   FeatureProcessor.Get());
	}

	void TestCPPApi()
	{
		// Set testing Archives
		TArray<uint8> ApiBuffer;
		FMemoryWriter MemoryWriter(ApiBuffer);
		FMemoryReader MemoryReader(ApiBuffer);
		FCborWriter CborWriter(&MemoryWriter);

		// Simulate CPP api interception buffer
		CborWriter.WriteContainerStart(ECborCode::Map, -1/*Indefinite*/);
		CborWriter.WriteValue(Int32ValuePropertyRCProp->GetProperty()->GetName());
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = Int32ValueTest; // Set test value before serialization
		CborWriter.WriteValue((int64)RemoteControlInterceptionTestObject->CustomStruct.Int32Value);
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = FRemoteControlInterceptionTestStruct::Int32ValueDefault; // Set default value after serialization
		CborWriter.WriteContainerEnd();

		// Set a deserializer backend
		FCborStructDeserializerBackend CborStructDeserializerBackend(MemoryReader);

		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Int32ValuePropertyRCProp->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		ObjectRef.PropertyPathInfo = Int32ValuePropertyRCProp->FieldPathInfo.ToString();

		for (UObject* Object : Int32ValuePropertyRCProp->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Archive should be intercepted inside SetObjectProperties
			IRemoteControlModule::Get().SetObjectProperties(ObjectRef, CborStructDeserializerBackend, ERCPayloadType::Cbor, ApiBuffer);
		}
	}

	void TestCbor()
	{
		// Set testing Archives
		TArray<uint8> CborBuffer;
		FMemoryReader Reader(CborBuffer);
		FMemoryWriter Writer(CborBuffer);

		// Set serializers
		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FCborStructDeserializerBackend DeserializerBackend(Reader);

		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = Int32ValueTest; // Set test value before serialization
		FStructSerializer::SerializeElement(&RemoteControlInterceptionTestObject->CustomStruct, Int32ValueProperty, INDEX_NONE, SerializerBackend, FStructSerializerPolicies());
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = FRemoteControlInterceptionTestStruct::Int32ValueDefault; // Set default value after serialization

		// Set object references
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Int32ValuePropertyRCProp->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		ObjectRef.PropertyPathInfo = Int32ValuePropertyRCProp->FieldPathInfo.ToString();

		for (UObject* Object : Int32ValuePropertyRCProp->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Archive should be intercepted inside SetObjectProperties
			IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Cbor, CborBuffer);
		}
	}

	void TestJson()
	{
		// Set testing Archives
		TArray<uint8> JsonBuffer;
		FMemoryReader Reader(JsonBuffer);
		FMemoryWriter Writer(JsonBuffer);

		// Set serializers
		FJsonStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);

		// Serialize with test value and change the value back to default
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = Int32ValueTest; // Set test value before serialization
		FStructSerializer::SerializeElement(&RemoteControlInterceptionTestObject->CustomStruct, Int32ValueProperty, INDEX_NONE, SerializerBackend, FStructSerializerPolicies());
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = FRemoteControlInterceptionTestStruct::Int32ValueDefault; // Set default value after serialization

		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Int32ValuePropertyRCProp->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		ObjectRef.PropertyPathInfo = Int32ValuePropertyRCProp->FieldPathInfo.ToString();

		for (UObject* Object : Int32ValuePropertyRCProp->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Archive should be intercepted inside SetObjectProperties
			IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Json, JsonBuffer);
		}
	}

	void TestValueWithSetter_Json()
	{
		// Set testing Archives
		TArray<uint8> JsonBuffer;
		FMemoryReader Reader(JsonBuffer);
		FMemoryWriter Writer(JsonBuffer);

		// Set serializers
		FJsonStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);

		// Serialize with test value and change the value back to default
		const FString TestValue = TEXT("ValueWithSetter_TestValue");
		const FString DefaultValue = RemoteControlInterceptionTestObject->GetValueWithSetter();
		RemoteControlInterceptionTestObject->SetValueWithSetter(TestValue); // Set test value before serialization
		FStructSerializer::SerializeElement(RemoteControlInterceptionTestObject.Get(), ValueWithSetterProperty, INDEX_NONE, SerializerBackend, FStructSerializerPolicies());
		RemoteControlInterceptionTestObject->SetValueWithSetter(DefaultValue); // Set default value after serialization

		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = ValueWithSetterRCProp->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		ObjectRef.PropertyPathInfo = ValueWithSetterRCProp->FieldPathInfo.ToString();

		for (UObject* Object : ValueWithSetterRCProp->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Json, JsonBuffer);
		}

		Test->TestEqual(TEXT("Property value with setter different than expected"), RemoteControlInterceptionTestObject->GetValueWithSetter(), TestValue);
	}
	
	void ResetObjectProperties()
	{
		// Set reset value before reset object property
		RemoteControlInterceptionTestObject->CustomStruct.Int32Value = Int32ValueTest;

		// Set object reference
		FRCObjectReference ObjectRef;
		ObjectRef.Property = Int32ValuePropertyRCProp->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		ObjectRef.PropertyPathInfo = Int32ValuePropertyRCProp->FieldPathInfo.ToString();

		for (UObject* Object : Int32ValuePropertyRCProp->GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef);
			// Reset property should be intercepted inside SetObjectProperties
			IRemoteControlModule::Get().ResetObjectProperties(ObjectRef, true);
		}

		Test->TestNotEqual(TEXT("The property value should be applied only after interception"), Int32ValueTest, RemoteControlInterceptionTestObject->CustomStruct.Int32Value);
	}

	void InvokeCall()
	{
		if (!TestFunctionRC.IsValid())
		{
			Test->AddError(TEXT("TestFunctionRC not valid"));
			return;
		}

		constexpr auto InStructName = TEXT("InStruct");
		constexpr auto InTestFactorName = TEXT("InTestFactor");
		const FName Int32ValueStructName = GET_MEMBER_NAME_CHECKED(FRemoteControlInterceptionFunctionParamStruct, Int32Value);
		const FName IntArrayStructName = GET_MEMBER_NAME_CHECKED(FRemoteControlInterceptionFunctionParamStruct, IntArray);
		const FName IntStringStructName = GET_MEMBER_NAME_CHECKED(FRemoteControlInterceptionFunctionParamStruct, IntString);

		// Define test struct
		FRemoteControlInterceptionFunctionParamStruct FunctionParamStruct;
		FunctionParamStruct.Int32Value = 14;
		FunctionParamStruct.IntArray.Add(1);
		FunctionParamStruct.IntArray.Add(2);
		FunctionParamStruct.IntArray.Add(3);
		FunctionParamStruct.IntString = FString::FromInt(FunctionParamStruct.Int32Value);
		constexpr int32 TestFactorInt = 3;

		// Copy test struct into the object struct
		RemoteControlInterceptionTestObject->FunctionParamStruct = FunctionParamStruct;
		
		// Set a input function argument values
		FStructOnScope FunctionArgs{ TestFunctionRC->GetFunction() };
		for (TFieldIterator<FProperty> It(TestFunctionRC->GetFunction()); It; ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				const FProperty* Property = *It;
				void* const ValueAddress = Property->ContainerPtrToValuePtr<void>(FunctionArgs.GetStructMemory());
				const FName PropertyName = Property->NamePrivate;

				// Set struct input values
				if (PropertyName == InStructName)
				{
					if (const FStructProperty* StructProperty =  CastField<FStructProperty>(Property))
					{
						StructProperty->InitializeValue_InContainer(FunctionArgs.GetStructMemory());

						for (TFieldIterator<FProperty> StructIt(StructProperty->Struct); StructIt; ++StructIt)
						{
							const FProperty* InnerStructProperty = *StructIt;
							const FName StructPropertyName = InnerStructProperty->NamePrivate;
							void* InnerStructValueAddress = InnerStructProperty->ContainerPtrToValuePtr<void>(ValueAddress, 0);

							// Set test integer value
							if (StructPropertyName == Int32ValueStructName)
							{
								if (const FNumericProperty* StructNumericProperty = CastField<FNumericProperty>(InnerStructProperty))
								{
									StructNumericProperty->CopyCompleteValue(InnerStructValueAddress, &FunctionParamStruct.Int32Value);
								}
							}

							// Set test array values
							else if (StructPropertyName == IntArrayStructName)
							{
								if (const FArrayProperty* StructArrayProperty = CastField<FArrayProperty>(InnerStructProperty))
								{
									if (const FNumericProperty* ArrayNumericProperty = CastField<FNumericProperty>(StructArrayProperty->Inner))
									{
										FScriptArrayHelper ScriptArrayHelper(StructArrayProperty, StructArrayProperty->ContainerPtrToValuePtr<void>(ValueAddress));

										for (const int32 IntValue : FunctionParamStruct.IntArray)
										{
											const int32 Index = ScriptArrayHelper.AddValue();
											void* ArrayElementValuePtr = ScriptArrayHelper.GetRawPtr(Index);
											ArrayNumericProperty->CopyCompleteValue(ArrayElementValuePtr, &IntValue);
										}	
									}
								}
							}
							else if (StructPropertyName == IntStringStructName)
							{
								if (const FStrProperty* StructStringProperty = CastField<FStrProperty>(InnerStructProperty))
								{
									StructStringProperty->SetPropertyValue(InnerStructValueAddress, FunctionParamStruct.IntString);
								}
							}
						}
					}
				}
				// Set integer input value
				else if (PropertyName == InTestFactorName)
				{
					if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
					{
						NumericProperty->CopyCompleteValue(ValueAddress, &TestFactorInt);
					}
				}
			}
		}

		// Set testing Archives
		TArray<uint8> JsonBuffer;
		FMemoryWriter Writer(JsonBuffer);

		// Set serializers
		FJsonStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FStructSerializer::Serialize(FunctionArgs.GetStructMemory(), *const_cast<UStruct*>(FunctionArgs.GetStruct()), SerializerBackend);

		FRCCallReference CallRef;
		CallRef.Object = RemoteControlInterceptionTestObject.Get();
		CallRef.Function = TestFunctionRC->GetFunction();
		
		FRCCall Call;
		Call.CallRef = MoveTemp(CallRef);
		Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
		Call.ParamStruct = FStructOnScope(FunctionArgs.GetStruct(), FunctionArgs.GetStructMemory());

		const bool bSuccess = IRemoteControlModule::Get().InvokeCall(Call, ERCPayloadType::Json, JsonBuffer);
		Test->TestTrue(TEXT("InvokeCall should return true"), bSuccess);

		Test->TestEqual(TEXT("The property value should be applied after interception"), FunctionParamStruct.Int32Value * TestFactorInt, RemoteControlInterceptionTestObject->FunctionParamStruct.Int32Value);
		Test->TestEqual(TEXT("The property value should be applied after interception"), FunctionParamStruct.IntArray[2] * TestFactorInt, RemoteControlInterceptionTestObject->FunctionParamStruct.IntArray[2]);
	}

private:
	FAutomationTestBase* Test;
	TStrongObjectPtr<URemoteControlPreset> Preset;
	TStrongObjectPtr<URemoteControlInterceptionTestObject> RemoteControlInterceptionTestObject;
	TSharedPtr<FRemoteControlProperty> Int32ValuePropertyRCProp;
	TSharedPtr<FRemoteControlFunction> TestFunctionRC;
	FProperty* CustomStructProperty = nullptr;
	FProperty* Int32ValueProperty = nullptr;
	const int32 Int32ValueTest = 34780;

	TSharedPtr<FRemoteControlProperty> ValueWithSetterRCProp;
	FProperty* ValueWithSetterProperty = nullptr;

	TUniquePtr<IRemoteControlInterceptionFeatureInterceptor> FeatureInterceptor;
	TUniquePtr<IRemoteControlInterceptionFeatureProcessor>   FeatureProcessor;
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRemoteControlPresetInterceptionTest, "Plugins.RemoteControl.Interception", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FRemoteControlPresetInterceptionTest::RunTest(const FString& Parameters)
{
	FRemoteControlInterceptionTest RemoteControlInterceptionTest(this);

	RemoteControlInterceptionTest.TestCPPApi();
	RemoteControlInterceptionTest.TestCbor();
	RemoteControlInterceptionTest.TestJson();
	RemoteControlInterceptionTest.ResetObjectProperties();
	RemoteControlInterceptionTest.InvokeCall();
	RemoteControlInterceptionTest.TestValueWithSetter_Json();
	
	return true;
}
