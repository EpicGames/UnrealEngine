// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RemoteControlInterceptionTestData.generated.h"


USTRUCT()
struct FRemoteControlInterceptionTestStruct
{
	GENERATED_BODY()

	static const int32 Int32ValueDefault = -1;

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = Int32ValueDefault;
};

USTRUCT(BlueprintType)
struct FRemoteControlInterceptionFunctionParamStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = 0;

	UPROPERTY(EditAnywhere, Category = "RC")
	TArray<int32> IntArray;

	UPROPERTY(EditAnywhere, Category = "RC")
	FString IntString;
};

UCLASS()
class URemoteControlInterceptionTestObject : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlInterceptionTestStruct CustomStruct;

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlInterceptionFunctionParamStruct FunctionParamStruct;

	UFUNCTION(BlueprintCallable, Category = "RC")
	FRemoteControlInterceptionFunctionParamStruct TestFunction(const FRemoteControlInterceptionFunctionParamStruct& InStruct, int32 InTestFactor)
	{
		FRemoteControlInterceptionFunctionParamStruct Copy = InStruct;

		Copy.Int32Value *= InTestFactor;

		for (int32& IntValue : Copy.IntArray)
		{
			IntValue *= InTestFactor;
		}

		int32 StringInt = 0;
		LexFromString(StringInt, *Copy.IntString);

		StringInt *= InTestFactor;
		
		Copy.IntString = FString::FromInt(StringInt);

		FunctionParamStruct = Copy;

		return Copy;
	}
	
	void SetValueWithSetter(const FString& InString)
	{
		ValueWithSetter = InString;
	}

	const FString& GetValueWithSetter() const
	{
		return ValueWithSetter;
	}

private:
	// Testing private property with a setter.
	UPROPERTY(EditAnywhere, Setter = "SetValueWithSetter", Category = "RC")
	FString ValueWithSetter;
};

