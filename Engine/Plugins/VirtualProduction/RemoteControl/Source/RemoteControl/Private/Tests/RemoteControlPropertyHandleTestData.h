// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Components/StaticMeshComponent.h"
#include "RemoteControlPropertyHandleTestData.generated.h"

UENUM()
enum class ERemoteControlEnumClass : uint8
{
	E_One,
	E_Two,
	E_Three
};

UENUM()
namespace ERemoteControlEnum
{
	enum Type : int
	{
		E_One,
		E_Two,
		E_Three
	};
}

USTRUCT()
struct FRemoteControlTestStructInnerSimple
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = 0;
};

USTRUCT()
struct FRemoteControlTestStructInner
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	int8 Int8Value = 36;

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlTestStructInnerSimple InnerSimple;

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = -791;

	FGuid Id = FGuid::NewGuid();

	friend bool operator==(const FRemoteControlTestStructInner& A, const FRemoteControlTestStructInner& B)
	{
		return A.Id == B.Id;
	}

	friend uint32 GetTypeHash(const FRemoteControlTestStructInner& Inner)
	{
		return GetTypeHash(Inner.Id);
	}
};

USTRUCT()
struct FRemoteControlTestStructOuter
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	int8 Int8Value = 12;

	UPROPERTY(EditAnywhere, Category = "RC")
	TSet<FRemoteControlTestStructInner> StructInnerSet;

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = -55;

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlTestStructInner RemoteControlTestStructInner;
};

UCLASS()
class URemoteControlAPITestObject : public UObject
{
public:
	GENERATED_BODY()

	URemoteControlAPITestObject()
	{
		for (int8 i = 0; i < 3; i++)
		{
			CStyleIntArray[i] = i+1;
			IntArray.Add(i+1);
			IntSet.Add(i+1);
			IntMap.Add(i, i+1);
		}

		StringColorMap.Add(TEXT("mykey"), FColor{1,2,3,4});
		StaticMeshComponent = CreateDefaultSubobject<UStaticMeshComponent>("StaticMeshComponentStructTest");
	}

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 CStyleIntArray[3];

	UPROPERTY(EditAnywhere, Category = "RC")
	TArray<int32> IntArray;

	UPROPERTY(EditAnywhere, Category = "RC")
	TArray<FRemoteControlTestStructOuter> StructOuterArray;

	UPROPERTY(EditAnywhere, Category = "RC")
	TSet<int32> IntSet;

	UPROPERTY(EditAnywhere, Category = "RC")
	TMap<int32, int32> IntMap;

	UPROPERTY(EditAnywhere, Category = "RC")
	TMap<int32, FRemoteControlTestStructOuter> StructOuterMap;

	UPROPERTY(EditAnywhere, Category = "RC")
	TMap<FString, FColor> StringColorMap;

	UPROPERTY(EditAnywhere, Category = "RC")
	TArray<FVector> ArrayOfVectors;

	UPROPERTY(EditAnywhere, Category = "RC")
	TObjectPtr<UStaticMeshComponent> StaticMeshComponent;

	UPROPERTY(EditAnywhere, Category = "RC")
	int8 Int8Value = 5;

	UPROPERTY(EditAnywhere, Category = "RC")
	int16 Int16Value = 53;

	UPROPERTY(EditAnywhere, Category = "RC")
	int32 Int32Value = 533;

	UPROPERTY(EditAnywhere, Category = "RC")
	float FloatValue = 2.3f;

	UPROPERTY(EditAnywhere, Category = "RC")
	double DoubleValue = 963.2f;;

	UPROPERTY(EditAnywhere, Category = "RC")
	FRemoteControlTestStructOuter RemoteControlTestStructOuter;

	UPROPERTY(EditAnywhere, Category = "RC")
	FString StringValue;

	UPROPERTY(EditAnywhere, Category = "RC")
	FName NameValue;

	UPROPERTY(EditAnywhere, Category = "RC")
	FText TextValue;

	UPROPERTY(EditAnywhere, Category = "RC")
	bool bValue = true;

	UPROPERTY(EditAnywhere, Category = "RC")
	uint8 ByteValue = 4;

	UPROPERTY(EditAnywhere, Category = "RC")
	TEnumAsByte<ERemoteControlEnum::Type> RemoteControlEnumByteValue = ERemoteControlEnum::E_Two;

	UPROPERTY(EditAnywhere, Category = "RC")
	ERemoteControlEnumClass RemoteControlEnumValue = ERemoteControlEnumClass::E_One;

	UPROPERTY(EditAnywhere, Category = "RC")
	FVector VectorValue = {12.2f, 4.33f, 9.1f};

	UPROPERTY(EditAnywhere, Category = "RC")
	FRotator RotatorValue = {45.1f, 96.2f, 184.5f};

	UPROPERTY(EditAnywhere, Category = "RC")
	FColor ColorValue = {40, 150, 200, 50};

	UPROPERTY(EditAnywhere, Category = "RC")
	FLinearColor LinearColorValue = {0.7f, 0.8f, 0.22f, 0.652f};
};
