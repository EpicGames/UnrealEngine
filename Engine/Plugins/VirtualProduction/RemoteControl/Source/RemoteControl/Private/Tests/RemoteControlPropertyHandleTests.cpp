// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"
#include "UObject/StrongObjectPtr.h"

#include "Components/StaticMeshComponent.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlPropertyHandle.h"
#include "RemoteControlPropertyHandleTestData.h"
#include "RemoteControlPreset.h"

#define PROP_NAME(Class, Name) GET_MEMBER_NAME_CHECKED(Class, Name)
#define GET_TEST_PROP(PropName) URemoteControlAPITestObject::StaticClass()->FindPropertyByName(PROP_NAME(URemoteControlAPITestObject, PropName))

namespace RemoteControlAPIIntegrationTest
{
	void TestIntPropertyHandle(FAutomationTestBase& Test)
	{
		// Get testing FProperties
		const FProperty* Int8ValueProperty = GET_TEST_PROP(Int8Value);
		const FProperty* Int16ValueProperty = GET_TEST_PROP(Int16Value);
		const FProperty* Int32ValueProperty = GET_TEST_PROP(Int32Value);

		// Create preset and test UObject
		const TCHAR* PresetName = TEXT("IntPresetName");
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		// Set test object int values
		constexpr int8 DefaultInt8Value = 29;
		constexpr int16 DefaultInt16Value = 1487;
		constexpr int32 DefaultInt32Value = 439062;
		TestObject->Int8Value = DefaultInt8Value;
		TestObject->Int16Value = DefaultInt16Value;
		TestObject->Int32Value = DefaultInt32Value;

		// Expose properties, done in UI, this is for testing only
		const FName RCInt8ValueLabel = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ Int8ValueProperty->GetName() }).Pin()->GetLabel();
		const FName RCInt16ValueLabel = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ Int16ValueProperty->GetName() }).Pin()->GetLabel();
		const FName RCInt32ValueLabel = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ Int32ValueProperty->GetName() }).Pin()->GetLabel();

		// Test the property handles
		{
			IRemoteControlModule::Get().RegisterEmbeddedPreset(Preset.Get());
			// 1. Find the Remote Control preset
			URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(PresetName);
			if (!ResolvedPreset)
			{
				Test.AddError(TEXT("ResolvedPreset not valid"));
				return;
			}

			// 2. Find exposed properties by label or ID
			const TSharedPtr<FRemoteControlProperty> RCInt8Value = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(ResolvedPreset->GetExposedEntityId(RCInt8ValueLabel)).Pin();
			const TSharedPtr<FRemoteControlProperty> RCInt16Value = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(ResolvedPreset->GetExposedEntityId(RCInt16ValueLabel)).Pin();
			const TSharedPtr<FRemoteControlProperty> RCInt32Value = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(ResolvedPreset->GetExposedEntityId(RCInt32ValueLabel)).Pin();
			if (!RCInt8Value.IsValid())
			{
				Test.AddError(TEXT("RCInt8Value not valid"));
				return;
			}
			if (!RCInt16Value.IsValid())
			{
				Test.AddError(TEXT("RCInt16Value not valid"));
				return;
			}
			if (!RCInt32Value.IsValid())
			{
				Test.AddError(TEXT("RCInt32Value not valid"));
				return;
			}

			// 3. Get property handles
			const TSharedPtr<IRemoteControlPropertyHandle> Int8PropertyHandle = RCInt8Value->GetPropertyHandle();
			const TSharedPtr<IRemoteControlPropertyHandle> Int16PropertyHandle = RCInt16Value->GetPropertyHandle();
			const TSharedPtr<IRemoteControlPropertyHandle> Int32PropertyHandle = RCInt32Value->GetPropertyHandle();
			if (!Int8PropertyHandle.IsValid())
			{
				Test.AddError(TEXT("Int8PropertyHandle not valid"));
				return;
			}
			if (!Int16PropertyHandle.IsValid())
			{
				Test.AddError(TEXT("Int16PropertyHandle not valid"));
				return;
			}
			if (!Int32PropertyHandle.IsValid())
			{
				Test.AddError(TEXT("Int32PropertyHandle not valid"));
				return;
			}

			// 4. Get remote control handles values
			int8 GetInt8Value = 0;
			int16 GetInt16Value = 0;
			int32 GetInt32Value = 0;
			Int8PropertyHandle->GetValue(GetInt8Value);
			Int16PropertyHandle->GetValue(GetInt16Value);
			Int32PropertyHandle->GetValue(GetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int8Value, GetInt8Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int16Value, GetInt16Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int32Value, GetInt32Value);

			// 5. Set remote control handles values
			constexpr int8 SetInt8Value = 18;
			constexpr int16 SetInt16Value = -1369;
			constexpr int32 SetInt32Value = -6870989;
			Int8PropertyHandle->SetValue(SetInt8Value);
			Int16PropertyHandle->SetValue(SetInt16Value);
			Int32PropertyHandle->SetValue(SetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int8Value, SetInt8Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int16Value, SetInt16Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->Int32Value, SetInt32Value);
		}
		IRemoteControlModule::Get().UnregisterEmbeddedPreset(PresetName);
	}

	void TestCreatePropertyHandleStruct(FAutomationTestBase& Test)
	{
		// Create preset and uobject
		const TCHAR* PresetName = TEXT("StructPresetName");
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		// Expose property
		const FProperty* RemoteControlTestStructOuter = GET_TEST_PROP(RemoteControlTestStructOuter);
		const FGuid StructOuterPropertyId = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ RemoteControlTestStructOuter->GetName() }).Pin()->GetId();

		// Test the api property handles
		{
			IRemoteControlModule::Get().RegisterEmbeddedPreset(Preset.Get());
			// 1. Find the Remote Control preset
			URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(PresetName);
			if (!ResolvedPreset)
			{
				Test.AddError(TEXT("ResolvedPreset not valid"));
				return;
			}

			// 2. Find exposed properties by label or ID
			const TSharedPtr<FRemoteControlProperty> StructProp = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(StructOuterPropertyId).Pin();

			// 3. Get API property handles
			TSharedPtr<IRemoteControlPropertyHandle> StructOuterPropertyHandle = StructProp->GetPropertyHandle();
			if (!StructOuterPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("StructOuterPropertyHandle not valid"));
				return;
			}
			TSharedPtr<IRemoteControlPropertyHandle> RemoteControlTestStructInnerPropertyHandle = StructOuterPropertyHandle->GetChildHandle(3);
			if (!RemoteControlTestStructInnerPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("RemoteControlTestStructInnerPropertyHandle not valid"));
				return;
			}

			// 4. Check num Children
			int32 NumChildren = StructOuterPropertyHandle->GetNumChildren();
			Test.TestEqual(TEXT("Num children should be equal 4"), NumChildren, 4);
			NumChildren = RemoteControlTestStructInnerPropertyHandle->GetNumChildren();
			Test.TestEqual(TEXT("Num children should be equal 3"), NumChildren, 3);

			// 5. Get/Set value to the outer struct RemoteControlAPITestObject/RemoteControlTestStructOuter
			{
				// RemoteControlAPITestObject/RemoteControlTestStructOuter/TestInt8Value
				TSharedPtr<IRemoteControlPropertyHandle> OuterTestInt8ValuePropertyHandle = StructOuterPropertyHandle->GetChildHandle(0);
				Test.TestTrue(TEXT("The PropertyHandle must be valid."), OuterTestInt8ValuePropertyHandle.IsValid());

				// 5.1 Get outer struct value
				{
					const int8 TestInt8Value = 41;
					TestObject->RemoteControlTestStructOuter.Int8Value = TestInt8Value;

					int8 ComparedTestInt8Value = 0;
					OuterTestInt8ValuePropertyHandle->GetValue(ComparedTestInt8Value);
					Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->RemoteControlTestStructOuter.Int8Value, ComparedTestInt8Value);
				}

				// 5.2 Set outer struct value
				{
					const int8 TestInt8Value = -54;
					OuterTestInt8ValuePropertyHandle->SetValue(TestInt8Value);
					Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->RemoteControlTestStructOuter.Int8Value, TestInt8Value);
				}
			}

			// 6. Get/Set value to the inner struct RemoteControlAPITestObject/RemoteControlTestStructOuter/RemoteControlTestStructInner
			{
				// RemoteControlAPITestObject/RemoteControlTestStructOuter/RemoteControlTestStructInner/TestInt8Value
				TSharedPtr<IRemoteControlPropertyHandle> InnerTestInt8ValuePropertyHandle = RemoteControlTestStructInnerPropertyHandle->GetChildHandle(0);
				Test.TestTrue(TEXT("The PropertyHandle must be valid."), InnerTestInt8ValuePropertyHandle.IsValid());

				// 6.1 Get inner struct value
				{
					const int8 TestInt8Value = 51;
					TestObject->RemoteControlTestStructOuter.RemoteControlTestStructInner.Int8Value = TestInt8Value;

					int8 ComparedTestInt8Value = 0;
					InnerTestInt8ValuePropertyHandle->GetValue(ComparedTestInt8Value);
					Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->RemoteControlTestStructOuter.RemoteControlTestStructInner.Int8Value, ComparedTestInt8Value);
				}

				// 6.2 Set inner struct value
				{
					const int8 TestInt8Value = -54;
					InnerTestInt8ValuePropertyHandle->SetValue(TestInt8Value);
					Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->RemoteControlTestStructOuter.RemoteControlTestStructInner.Int8Value, TestInt8Value);
				}
			}
		}
		IRemoteControlModule::Get().UnregisterEmbeddedPreset(PresetName);
	}

	void TestCreatePropertyHandleArray(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* CStyleIntArray = GET_TEST_PROP(CStyleIntArray);
		const FProperty* IntArray = GET_TEST_PROP(IntArray);
		const FProperty* StructOuterArray = GET_TEST_PROP(StructOuterArray);

		// Expose properties
		const TSharedPtr<FRemoteControlProperty> CStyleIntArrayProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ CStyleIntArray->GetName() }).Pin();
		const TSharedPtr<FRemoteControlProperty> IntArrayProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ IntArray->GetName() }).Pin();
		const TSharedPtr<FRemoteControlProperty> StructOuterArrayProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ StructOuterArray->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), CStyleIntArrayProp.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), IntArrayProp.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), StructOuterArrayProp.Get());

		// Test C style array
		{
			const int32 CStyleIntArrayValue_0 = 174073;
			TestObject->CStyleIntArray[0] = CStyleIntArrayValue_0;

			// Get API property C style array handle
			const TSharedPtr<IRemoteControlPropertyHandle> CStyleIntArrayPropertyHandle = CStyleIntArrayProp->GetPropertyHandle();
			Test.TestNotNull(TEXT("The exposed property must be valid."), CStyleIntArrayPropertyHandle.Get());

			const TSharedPtr<IRemoteControlPropertyHandleArray> CStyleIntArrayPropertyHandleArray = CStyleIntArrayPropertyHandle->AsArray();
			Test.TestNotNull(TEXT("The exposed property must be valid."), CStyleIntArrayPropertyHandleArray.Get());

			const TSharedPtr<IRemoteControlPropertyHandle> CStyleIntArrayPropertyHandle_0 = CStyleIntArrayPropertyHandleArray->GetElement(0);
			Test.TestNotNull(TEXT("The exposed property must be valid."), CStyleIntArrayPropertyHandle_0.Get());

			// Get value api call
			int32 CStyleIntArrayValueTestGet_0 = 0;
			CStyleIntArrayPropertyHandle_0->GetValue(CStyleIntArrayValueTestGet_0);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->CStyleIntArray[0], CStyleIntArrayValueTestGet_0);

			// Set value api call
			const int32 CStyleIntArrayValueTestSet_0 = -98747;
			CStyleIntArrayPropertyHandle_0->SetValue(CStyleIntArrayValueTestSet_0);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->CStyleIntArray[0], CStyleIntArrayValueTestSet_0);
		}

		// Test dynamic Array
		{
			const int32 IntArrayValue_0 = 74073;
			TestObject->IntArray.Add(IntArrayValue_0);

			// Get API property TArray style array handle
			const TSharedPtr<IRemoteControlPropertyHandle> IntArrayPropertyHandle = IntArrayProp->GetPropertyHandle();
			Test.TestNotNull(TEXT("The exposed property must be valid."), IntArrayPropertyHandle.Get());

			const TSharedPtr<IRemoteControlPropertyHandleArray> IntArrayPropertyHandleArray = IntArrayPropertyHandle->AsArray();
			Test.TestNotNull(TEXT("The exposed property must be valid."), IntArrayPropertyHandleArray.Get());

			const int32 NumElements = IntArrayPropertyHandleArray->GetNumElements();
			Test.TestEqual(TEXT("Num elements should be the same"), NumElements, TestObject->IntArray.Num());

			const TSharedPtr<IRemoteControlPropertyHandle> IntArrayPropertyHandle_0 = IntArrayPropertyHandleArray->GetElement(TestObject->IntArray.Num() - 1);
			Test.TestNotNull(TEXT("The exposed property must be valid."), IntArrayPropertyHandle_0.Get());

			// Get value api call
			int32 IntArrayValueTestGet_0 = 0;
			IntArrayPropertyHandle_0->GetValue(IntArrayValueTestGet_0);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->IntArray[TestObject->IntArray.Num() - 1], IntArrayValueTestGet_0);

			// Set value api call
			const int32 IntArrayValueTestSet_0 = -98747;
			IntArrayPropertyHandle_0->SetValue(IntArrayValueTestSet_0);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->IntArray[TestObject->IntArray.Num() - 1], IntArrayValueTestSet_0);
		}

		// Test dynamic Array of structs
		{
			int32 Int32Value = 74073;
			FRemoteControlTestStructOuter StructOuter;
			StructOuter.Int32Value = Int32Value;
			TestObject->StructOuterArray.Add(StructOuter);

			// Get API property TArray style array handle
			const TSharedPtr<IRemoteControlPropertyHandle> StructOuterArrayPropertyHandle = StructOuterArrayProp->GetPropertyHandle();
			Test.TestNotNull(TEXT("The exposed property must be valid."), StructOuterArrayPropertyHandle.Get());

			const TSharedPtr<IRemoteControlPropertyHandleArray> StructOuterArrayPropertyHandleArray = StructOuterArrayPropertyHandle->AsArray();
			Test.TestNotNull(TEXT("The exposed property must be valid."), StructOuterArrayPropertyHandleArray.Get());

			const int32 NumElements = StructOuterArrayPropertyHandleArray->GetNumElements();
			Test.TestEqual(TEXT("Num elements should be the same"), NumElements, TestObject->StructOuterArray.Num());

			const TSharedPtr<IRemoteControlPropertyHandle> StructOuterPropertyHandle_0 = StructOuterArrayPropertyHandleArray->GetElement(TestObject->StructOuterArray.Num() - 1);
			Test.TestNotNull(TEXT("The exposed property must be valid."), StructOuterPropertyHandle_0.Get());

			const TSharedPtr<IRemoteControlPropertyHandle> Int32PropertyHandle = StructOuterPropertyHandle_0->GetChildHandle(2);
			Test.TestNotNull(TEXT("The exposed property must be valid."), Int32PropertyHandle.Get());

			// Get value api call
			int32 GetInt32Value = 0;
			Int32PropertyHandle->GetValue(GetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->StructOuterArray[TestObject->StructOuterArray.Num() - 1].Int32Value, GetInt32Value);

			// Set value api call
			const int32 SetInt32Value = -98747;
			Int32PropertyHandle->SetValue(SetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->StructOuterArray[TestObject->StructOuterArray.Num() - 1].Int32Value, SetInt32Value);
		}
	}

	void TestCreatePropertyHandleSet(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		// Find element, that is should be dynamic in API, now that is impssible after CreatePropertyHandles call
		int32 TestIntSetValueAndIndex = 1234;
		TestObject->IntSet.Add(TestIntSetValueAndIndex);

		const FProperty* IntSetProperty = GET_TEST_PROP(IntSet);

		// Expose properties
		const TSharedPtr<FRemoteControlProperty> RCIntSetProperty = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ IntSetProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCIntSetProperty.Get());

		// Get API property Sets
		const TSharedPtr<IRemoteControlPropertyHandle> SetPropertyHandle = RCIntSetProperty->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), SetPropertyHandle.Get());

		const TSharedPtr<IRemoteControlPropertyHandleSet> SetPropertyHandleAsSet = SetPropertyHandle->AsSet();
		Test.TestNotNull(TEXT("The exposed property must be valid."), SetPropertyHandleAsSet.Get());

		const int32 NumElements = SetPropertyHandleAsSet->GetNumElements();
		Test.TestEqual(TEXT("Num set value should be the same"), TestObject->IntSet.Num(), NumElements);

		const TSharedPtr<IRemoteControlPropertyHandle> FoundPropertyHandleFromSet = SetPropertyHandleAsSet->FindElement(&TestIntSetValueAndIndex);
		Test.TestNotNull(TEXT("The exposed property must be valid."), FoundPropertyHandleFromSet.Get());

		int32 GetIntSetValueAndIndex = 0;
		FoundPropertyHandleFromSet->GetValue(GetIntSetValueAndIndex);
		Test.TestEqual(TEXT("Api value should be the same as a object set value"), TestIntSetValueAndIndex, GetIntSetValueAndIndex);

		const int32 SetIntSetValueAndIndex = -1234567;
		FoundPropertyHandleFromSet->SetValue(SetIntSetValueAndIndex);
		Test.TestEqual(TEXT("Api value should be the same as a object set value"), SetIntSetValueAndIndex, *TestObject->IntSet.FindByHash(GetTypeHash(SetIntSetValueAndIndex), SetIntSetValueAndIndex));
	}

	void TestCreatePropertyHandleMap(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		// Find element, that is should be dynamic in API, now that is impssible after CreatePropertyHandles call
		const int32 TestIntMapValueAndIndex = 1234;
		TestObject->IntMap.Add(TestIntMapValueAndIndex, TestIntMapValueAndIndex);

		const FProperty* IntMapProperty = GET_TEST_PROP(IntMap);

		// Expose properties
		const TSharedPtr<FRemoteControlProperty> RCIntMapProperty = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ IntMapProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCIntMapProperty.Get());

		// Get API property Sets
		const TSharedPtr<IRemoteControlPropertyHandle> MapPropertyHandle = RCIntMapProperty->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), MapPropertyHandle.Get());

		const TSharedPtr<IRemoteControlPropertyHandleMap> MapPropertyHandleAsMap = MapPropertyHandle->AsMap();
		Test.TestNotNull(TEXT("The exposed property must be valid."), MapPropertyHandleAsMap.Get());

		const int32 NumElements = MapPropertyHandleAsMap->GetNumElements();
		Test.TestEqual(TEXT("Num set value should be the same"), TestObject->IntMap.Num(), NumElements);

		const TSharedPtr<IRemoteControlPropertyHandle> FoundPropertyHandleFromMap = MapPropertyHandleAsMap->Find(&TestIntMapValueAndIndex);
		Test.TestNotNull(TEXT("The exposed property must be valid."), FoundPropertyHandleFromMap.Get());

		int32 GetIntMapValueAndIndex = 0;
		FoundPropertyHandleFromMap->GetValue(GetIntMapValueAndIndex);
		Test.TestEqual(TEXT("Api value should be the same as a object set value"), TestIntMapValueAndIndex, GetIntMapValueAndIndex);

		const int32 SetIntMapValueAndIndex = -1234567;
		FoundPropertyHandleFromMap->SetValue(SetIntMapValueAndIndex);
		Test.TestEqual(TEXT("Api value should be the same as a object set value"), SetIntMapValueAndIndex, *TestObject->IntMap.Find(TestIntMapValueAndIndex));
	}

	void TestStringProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };
		
		const FProperty* StringProperty = GET_TEST_PROP(StringValue);

		const TSharedPtr<FRemoteControlProperty> RCStringProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ StringProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCStringProp.Get());
		
		// Test FString property
		const FString StringPropertyTestValue = TEXT("StringPropertyTestValue");
		const TCHAR* CharArrayValue = TEXT("CharArrayValue");
		const FString StringPropertyDefaultValue = TEXT("StringPropertyDefaultValue");
		TestObject->StringValue = StringPropertyDefaultValue;

		// Get API property String property handle
		const TSharedPtr<IRemoteControlPropertyHandle> StringHandle = RCStringProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), StringHandle.Get());

		FString StringValueFromAPI = TEXT("");
		StringHandle->GetValue(StringValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), StringValueFromAPI, TestObject->StringValue);

		StringHandle->SetValue(StringPropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), StringPropertyTestValue, TestObject->StringValue);

		StringHandle->SetValue(CharArrayValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), FString(CharArrayValue), TestObject->StringValue);
	}

	void TestNameProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* NameProperty = GET_TEST_PROP(NameValue);

		const TSharedPtr<FRemoteControlProperty> RCNameProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ NameProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCNameProp.Get());

		// Test FName property
		const FName NamePropertyTestValue = TEXT("NamePropertyTestValue");
		const FName NamePropertyDefaultValue = TEXT("NamePropertyDefaultValue");
		const TCHAR* CharArrayValue = TEXT("CharArrayValue");
		TestObject->NameValue = NamePropertyDefaultValue;

		// Get API property Name property handle
		const TSharedPtr<IRemoteControlPropertyHandle> NameHandle = RCNameProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), NameHandle.Get());

		FName NameValueFromAPI = TEXT("");
		NameHandle->GetValue(NameValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), NameValueFromAPI, TestObject->NameValue);

		NameHandle->SetValue(NamePropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), NamePropertyTestValue, TestObject->NameValue);

		NameHandle->SetValue(CharArrayValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), FName(CharArrayValue), TestObject->NameValue);
	}

	void TestFloatProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* FloatProperty = GET_TEST_PROP(FloatValue);

		const TSharedPtr<FRemoteControlProperty> RCFloatProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ FloatProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCFloatProp.Get());

		// Test float property
		const float FloatPropertyTestValue = 14.f;
		const float FloatPropertyDefaultValue = -1.f;
		TestObject->FloatValue = FloatPropertyDefaultValue;

		// Get API property float property handle
		const TSharedPtr<IRemoteControlPropertyHandle> FloatHandle = RCFloatProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), FloatHandle.Get());

		float FloatValueFromAPI = 0.f;
		FloatHandle->GetValue(FloatValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), FloatValueFromAPI, TestObject->FloatValue);

		FloatHandle->SetValue(FloatPropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), FloatPropertyTestValue, TestObject->FloatValue);
	}

	void TestDoubleProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* DoubleProperty = GET_TEST_PROP(DoubleValue);

		const TSharedPtr<FRemoteControlProperty> RCDoubleProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ DoubleProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCDoubleProp.Get());

		// Test Double property
		const double DoublePropertyTestValue = 151.14;
		const double DoublePropertyDefaultValue = -3.0;
		TestObject->DoubleValue = DoublePropertyDefaultValue;

		// Get API property Double property handle
		const TSharedPtr<IRemoteControlPropertyHandle> DoubleHandle = RCDoubleProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), DoubleHandle.Get());

		double DoubleValueFromAPI = 0.0;
		DoubleHandle->GetValue(DoubleValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), DoubleValueFromAPI, TestObject->DoubleValue);

		DoubleHandle->SetValue(DoublePropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), DoublePropertyTestValue, TestObject->DoubleValue);
	}

	void TestBoolProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		const FName PresetName = TEXT("BoolPresetName");
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* BoolProperty = GET_TEST_PROP(bValue);

		const TSharedPtr<FRemoteControlProperty> RCBoolProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ BoolProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCBoolProp.Get());

		// Test bool property
		const bool BoolPropertyTestValue = false;
		const bool BoolPropertyDefaultValue = true;
		TestObject->bValue = BoolPropertyDefaultValue;

		// Get API property bool property handle
		TSharedPtr<IRemoteControlPropertyHandle> BoolHandle = RCBoolProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), BoolHandle.Get());

		bool bValueFromAPI = false;
		BoolHandle->GetValue(bValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), bValueFromAPI, TestObject->bValue);

		BoolHandle->SetValue(BoolPropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), BoolPropertyTestValue, TestObject->bValue);
	}

	void TestByteProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		const FName PresetName = TEXT("BytePresetName");
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* ByteProperty = GET_TEST_PROP(ByteValue);
		const FProperty* EnumByteProperty = GET_TEST_PROP(RemoteControlEnumByteValue);
		const FProperty* EnumProperty = GET_TEST_PROP(RemoteControlEnumValue);

		const TSharedPtr<FRemoteControlProperty> RCByteProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ ByteProperty->GetName() }).Pin();
		const TSharedPtr<FRemoteControlProperty> RCEnumByteProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ EnumByteProperty->GetName() }).Pin();
		const TSharedPtr<FRemoteControlProperty> RCEnumProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ EnumProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCByteProp.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCEnumByteProp.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCEnumProp.Get());

		// Test byte property
		const uint8 BytePropertyTestValue = 134;
		const uint8 BytePropertyDefaultValue = 0;
		TestObject->ByteValue = BytePropertyDefaultValue;

		// Get API property byte property handle
		const TSharedPtr<IRemoteControlPropertyHandle> ByteHandle = RCByteProp->GetPropertyHandle();
		const TSharedPtr<IRemoteControlPropertyHandle> EnumByteHandle = RCEnumByteProp->GetPropertyHandle();
		const TSharedPtr<IRemoteControlPropertyHandle> EnumHandle = RCEnumProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), ByteHandle.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), EnumByteHandle.Get());
		Test.TestNotNull(TEXT("The exposed property must be valid."), EnumHandle.Get());

		uint8 ByteValueFromAPI = 0;
		ByteHandle->GetValue(ByteValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), ByteValueFromAPI, TestObject->ByteValue);

		ByteHandle->SetValue(BytePropertyTestValue);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), BytePropertyTestValue, TestObject->ByteValue);

		ERemoteControlEnum::Type EnumBytePropertyTestValue = ERemoteControlEnum::E_Three;
		ERemoteControlEnum::Type EnumBytePropertyDefaultValue = ERemoteControlEnum::E_Two;
		TestObject->RemoteControlEnumByteValue = EnumBytePropertyDefaultValue;

		uint8 EnumByteValueFromAPI = 0;
		EnumByteHandle->GetValue(EnumByteValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), static_cast<ERemoteControlEnum::Type>(EnumByteValueFromAPI), TestObject->RemoteControlEnumByteValue);

		EnumByteHandle->SetValue(static_cast<uint8>(EnumBytePropertyTestValue));
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), EnumBytePropertyTestValue, TestObject->RemoteControlEnumByteValue);

		const ERemoteControlEnumClass EnumPropertyTestValue = ERemoteControlEnumClass::E_Three;
		const ERemoteControlEnumClass EnumPropertyDefaultValue = ERemoteControlEnumClass::E_Two;
		TestObject->RemoteControlEnumValue = EnumPropertyDefaultValue;

		uint8 EnumValueFromAPI = 0;
		EnumHandle->GetValue(EnumValueFromAPI);
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), static_cast<ERemoteControlEnumClass>(EnumValueFromAPI), TestObject->RemoteControlEnumValue);

		EnumHandle->SetValue(static_cast<uint8>(EnumPropertyTestValue));
		Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), EnumPropertyTestValue, TestObject->RemoteControlEnumValue);
	}

	void TestTextProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* TextProperty = GET_TEST_PROP(TextValue);

		const TSharedPtr<FRemoteControlProperty> RCTextProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ TextProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCTextProp.Get());

		// Test FText property
		const FText TextPropertyTestValue = FText::FromString(TEXT("TextPropertyTestValue"));
		const TCHAR* CharArrayValue = TEXT("CharArrayValue");
		const FString StringValue = TEXT("StringValue");
		const FText TextPropertyDefaultValue = FText::FromString(TEXT("TextPropertyDefaultValue"));
		TestObject->TextValue = TextPropertyDefaultValue;

		// Get API property Text property handle
		const TSharedPtr<IRemoteControlPropertyHandle> TextHandle = RCTextProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), TextHandle.Get());

		FText TextValueFromAPI;
		TextHandle->GetValue(TextValueFromAPI);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), TextValueFromAPI.EqualTo(TestObject->TextValue));

		TextHandle->SetValue(TextPropertyTestValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), TextPropertyTestValue.EqualTo(TestObject->TextValue));

		TextHandle->SetValue(CharArrayValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), FText::FromString(CharArrayValue).EqualTo(TestObject->TextValue));

		TextHandle->SetValue(StringValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), FText::FromString(StringValue).EqualTo(TestObject->TextValue));
	}

	void TestVectorProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* VectorProperty = GET_TEST_PROP(VectorValue);

		TSharedPtr<FRemoteControlProperty> RCVectorProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ VectorProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCVectorProp.Get());

		// Test FText property
		const FVector VectorPropertyTestValue(-1, -2, -3);
		const FVector VectorPropertyDefaultValue(4, 5, 6);
		TestObject->VectorValue = VectorPropertyDefaultValue;

		// Get API property Text property handle
		const TSharedPtr<IRemoteControlPropertyHandle> VectorHandle = RCVectorProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), VectorHandle.Get());

		FVector VectorValueFromAPI;
		VectorHandle->GetValue(VectorValueFromAPI);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), VectorValueFromAPI.Equals(TestObject->VectorValue));

		VectorHandle->SetValue(VectorPropertyTestValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), VectorPropertyTestValue.Equals(TestObject->VectorValue));
	}

	void TestRotatorProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* RotatorProperty = GET_TEST_PROP(RotatorValue);

		const TSharedPtr<FRemoteControlProperty> RCRotatorProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ RotatorProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCRotatorProp.Get());

		// Test FRotator property
		const FRotator RotatorPropertyTestValue(-1, -2, -3);
		const FRotator RotatorPropertyDefaultValue(4, 5, 6);
		TestObject->RotatorValue = RotatorPropertyDefaultValue;

		// Get API property Text property handle
		const TSharedPtr<IRemoteControlPropertyHandle> RotatorHandle = RCRotatorProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RotatorHandle.Get());

		FRotator RotatorValueFromAPI;
		RotatorHandle->GetValue(RotatorValueFromAPI);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), RotatorValueFromAPI.Equals(TestObject->RotatorValue));

		RotatorHandle->SetValue(RotatorPropertyTestValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), RotatorPropertyTestValue.Equals(TestObject->RotatorValue));
	}

	void TestColorProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* ColorProperty = GET_TEST_PROP(ColorValue);

		const TSharedPtr<FRemoteControlProperty> RCColorProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ ColorProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCColorProp.Get());

		// Test FColor property
		const FColor ColorPropertyTestValue(8, 140, 20);
		const FColor ColorPropertyDefaultValue(4, 5, 6);
		TestObject->ColorValue = ColorPropertyDefaultValue;

		// Get API property Text property handle
		const TSharedPtr<IRemoteControlPropertyHandle> ColorHandle = RCColorProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), ColorHandle.Get());

		FColor ColorValueFromAPI;
		ColorHandle->GetValue(ColorValueFromAPI);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), ColorValueFromAPI == TestObject->ColorValue);

		ColorHandle->SetValue(ColorPropertyTestValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), ColorPropertyTestValue == TestObject->ColorValue);
	}

	void TestLinearColorProperty(FAutomationTestBase& Test)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		const FProperty* LinearColorProperty = GET_TEST_PROP(LinearColorValue);

		const TSharedPtr<FRemoteControlProperty> RCLinearColorProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ LinearColorProperty->GetName() }).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCLinearColorProp.Get());

		// Test FLinearColor property
		const FLinearColor LinearColorPropertyTestValue(0.84f, 0.12f, 0.2f);
		const FLinearColor LinearColorPropertyDefaultValue(0.42f, 0.5f, 0.6f);
		TestObject->LinearColorValue = LinearColorPropertyDefaultValue;

		// Get API property Text property handle
		const TSharedPtr<IRemoteControlPropertyHandle> LinearColorHandle = RCLinearColorProp->GetPropertyHandle();
		Test.TestNotNull(TEXT("The exposed property must be valid."), LinearColorHandle.Get());

		FLinearColor LinearColorValueFromAPI;
		LinearColorHandle->GetValue(LinearColorValueFromAPI);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), LinearColorValueFromAPI == TestObject->LinearColorValue);

		LinearColorHandle->SetValue(LinearColorPropertyTestValue);
		Test.TestTrue(TEXT("Value in UObject should be the same as a value from Property Handle."), LinearColorPropertyTestValue == TestObject->LinearColorValue);
	}

	void TestComplexPath(FAutomationTestBase& Test)
	{
		// Create preset and uobject
		const TCHAR* PresetName = TEXT("TestComplexPath");
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };

		// Set a test value
		constexpr int32 Int32ValueTest = 14923;
		constexpr int32 StructOuterMapIndex = 468;

		FRemoteControlTestStructInner RemoteControlTestStructInner;
		RemoteControlTestStructInner.InnerSimple.Int32Value = Int32ValueTest;

		FRemoteControlTestStructOuter RemoteControlTestStructOuter;
		RemoteControlTestStructOuter.StructInnerSet.Add(RemoteControlTestStructInner);

		TestObject->StructOuterMap.Add(StructOuterMapIndex, RemoteControlTestStructOuter);

		// Expose property
		const FProperty* StructOuterMapProp = GET_TEST_PROP(StructOuterMap);
		const FGuid StructOuterMapPropId = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ StructOuterMapProp->GetName() }).Pin()->GetId();

		// Test complex path handles
		{
			// 1. Find the Remote Control preset
			IRemoteControlModule::Get().RegisterEmbeddedPreset(Preset.Get());
			URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(PresetName);
			if (!ResolvedPreset)
			{
				Test.AddError(TEXT("ResolvedPreset not valid"));
				return;
			}

			// 2. Find exposed properties by label or ID
			const TSharedPtr<FRemoteControlProperty> StructOuterMapProperty = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(StructOuterMapPropId).Pin();
			if (!StructOuterMapProperty.IsValid())
			{
				Test.AddError(TEXT("StructOuterMapProperty not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> StructOuterMapPropertyHandle = StructOuterMapProperty->GetPropertyHandle();
			if (!StructOuterMapPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("StructOuterMapPropertyHandle not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandleMap> StructOuterMapPropertyHandleAsMap = StructOuterMapPropertyHandle->AsMap();
			if (!StructOuterMapPropertyHandleAsMap.IsValid())
			{
				Test.AddError(TEXT("StructOuterMapPropertyHandleAsMap not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> StructOuterPropertyHandle = StructOuterMapPropertyHandleAsMap->Find(&StructOuterMapIndex);
			if (!StructOuterPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("StructOuterPropertyHandle not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> StructInnerSetPropertyHandle = StructOuterPropertyHandle->GetChildHandle(1);
			if (!StructInnerSetPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("StructInnerSetPropertyHandle not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandleSet> StructInnerSetPropertyHandleAsSet = StructInnerSetPropertyHandle->AsSet();
			if (!StructInnerSetPropertyHandleAsSet.IsValid())
			{
				Test.AddError(TEXT("StructInnerSetPropertyHandleAsSet not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> StructInnerPropertyHandle = StructInnerSetPropertyHandleAsSet->FindElement(&RemoteControlTestStructInner);
			if (!StructInnerPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("StructInnerropertyHandle not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> StructInnerSimpleHandle = StructInnerPropertyHandle->GetChildHandle(1);
			if (!StructInnerSimpleHandle.IsValid())
			{
				Test.AddError(TEXT("StructInnerSimpleHandle not valid"));
				return;
			}

			const TSharedPtr<IRemoteControlPropertyHandle> Int32ValueHandle =  StructInnerSimpleHandle->GetChildHandle(0);
			if (!Int32ValueHandle.IsValid())
			{
				Test.AddError(TEXT("Int32ValueHandle not valid"));
				return;
			}

			int32 GetInt32Value = 0;
			Int32ValueHandle->GetValue(GetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), RemoteControlTestStructInner.InnerSimple.Int32Value, GetInt32Value);

			int32 SetInt32Value = 0;
			Int32ValueHandle->SetValue(SetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), 
				TestObject
					->StructOuterMap.Find(StructOuterMapIndex)
					->StructInnerSet.FindByHash(GetTypeHash(RemoteControlTestStructInner), RemoteControlTestStructInner)
					->InnerSimple.Int32Value,
				SetInt32Value);
		}
		IRemoteControlModule::Get().UnregisterEmbeddedPreset(PresetName);
	}

	void TestGetPropertyHandleByFieldPath(FAutomationTestBase& Test)
	{
		// Create preset and uobject
		const TCHAR* PresetName = TEXT("TestComplexPath");

		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		check(Preset);
		
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };
		check(TestObject);

		IRemoteControlModule::Get().RegisterEmbeddedPreset(Preset.Get());

		// Set a test value
		constexpr int32 Int32ValueTest = 14923;

		FRemoteControlTestStructInner RemoteControlTestStructInner;
		RemoteControlTestStructInner.InnerSimple.Int32Value = Int32ValueTest;

		FRemoteControlTestStructOuter RemoteControlTestStructOuter;
		RemoteControlTestStructOuter.StructInnerSet.Add(RemoteControlTestStructInner);

		TestObject->StructOuterMap.Add(1, RemoteControlTestStructOuter);
		TestObject->StructOuterMap.Add(2, RemoteControlTestStructOuter);

		// Expose property
		const FProperty* StructOuterMapProp = GET_TEST_PROP(StructOuterMap);
		const FProperty* ArrayOfVectorsProp = GET_TEST_PROP(ArrayOfVectors);
		const FProperty* IntMapProp = GET_TEST_PROP(IntMap);
		const FGuid StructOuterMapPropId = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ StructOuterMapProp->GetName() }).Pin()->GetId();
		const FName ArrayOfVectorsPropLabel = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ ArrayOfVectorsProp->GetName() }).Pin()->GetLabel();
		const FName IntMapLabel = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ IntMapProp->GetName() }).Pin()->GetLabel();

		const TSharedPtr<IRemoteControlPropertyHandle> StructOuterMapPropertyHandle = IRemoteControlPropertyHandle::GetPropertyHandle(PresetName, StructOuterMapPropId);
		if(!Test.TestValid(TEXT("Property handle is valid"), StructOuterMapPropertyHandle))
		{
			return;
		}

		Test.TestFalse(TEXT("Child property handle should nullptr if the path same as a parent path"), StructOuterMapPropertyHandle->GetChildHandleByFieldPath(TEXT("StructOuterMap")).IsValid());
		Test.TestFalse(TEXT("Child property handle should nullptr if the field path is wrong"), StructOuterMapPropertyHandle->GetChildHandleByFieldPath(TEXT("StructOuterMap_Wrong_Path")).IsValid());

		const TSharedPtr<IRemoteControlPropertyHandleMap> StructOuterMapPropertyHandleAsMap = StructOuterMapPropertyHandle->AsMap();
		if(!Test.TestValid(TEXT("StructOuterMapPropertyHandleAsMap is valid"), StructOuterMapPropertyHandleAsMap))
		{
			return;
		}

		const TSharedPtr<IRemoteControlPropertyHandle> StructOuterPropertyHandle_0 = StructOuterMapPropertyHandle->GetChildHandleByFieldPath(TEXT("StructOuterMap[1]"));
		Test.TestValid(TEXT("Property handle is valid"), StructOuterPropertyHandle_0);
		Test.TestNotNull(TEXT("Property handle should be valid"), CastField<FMapProperty>(StructOuterPropertyHandle_0->GetParentProperty()));
		Test.TestTrue(TEXT("Index should be 1"), StructOuterPropertyHandle_0->GetIndexInArray() == 1);

		const TSharedPtr<IRemoteControlPropertyHandle> IntPropertyHandle = StructOuterMapPropertyHandle->GetChildHandleByFieldPath(TEXT("StructOuterMap[1].StructInnerSet[0].InnerSimple.Int32Value"));
		Test.TestValid(TEXT("Property handle is valid"), IntPropertyHandle);
		
		{
			int32 GetInt32Value = 0;
			IntPropertyHandle->GetValue(GetInt32Value);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), RemoteControlTestStructInner.InnerSimple.Int32Value, GetInt32Value);

		}

		// Test Vectors
		const FVector TestVector_0(1.f, 2.f, 3.f);
		const FVector TestVector_1(1.f, 2.f, 3.f);
		TestObject->ArrayOfVectors.Add(TestVector_0);
		TestObject->ArrayOfVectors.Add(TestVector_1);
		const TSharedPtr<IRemoteControlPropertyHandle> ArrayOfVectorsPropertyHandle = IRemoteControlPropertyHandle::GetPropertyHandle(PresetName, ArrayOfVectorsPropLabel);
		const TSharedPtr<IRemoteControlPropertyHandle> ArrayOfVectorsPropertyHandle_O_X = ArrayOfVectorsPropertyHandle->GetChildHandleByFieldPath(TEXT("ArrayOfVectors[0].X"));
		const TSharedPtr<IRemoteControlPropertyHandle> ArrayOfVectorsPropertyHandle_1_X = ArrayOfVectorsPropertyHandle->GetChildHandleByFieldPath(TEXT("ArrayOfVectors[1].X"));
		const TSharedPtr<IRemoteControlPropertyHandle> ArrayOfVectorsPropertyHandle_O = ArrayOfVectorsPropertyHandle->GetChildHandleByFieldPath(TEXT("ArrayOfVectors[0]"));
		const TSharedPtr<IRemoteControlPropertyHandle> IntMap_PropertyHandle = IRemoteControlPropertyHandle::GetPropertyHandle(PresetName, IntMapLabel);
		const TSharedPtr<IRemoteControlPropertyHandle> IntMap_PropertyHandle_O = IntMap_PropertyHandle->GetChildHandleByFieldPath(TEXT("IntMap[1]"));
		Test.TestTrue(TEXT("Property handle is valid"), ArrayOfVectorsPropertyHandle.IsValid());
		Test.TestTrue(TEXT("Property handle is valid"), ArrayOfVectorsPropertyHandle_O_X.IsValid());
		Test.TestTrue(TEXT("Property handle is valid"), ArrayOfVectorsPropertyHandle_1_X.IsValid());
		Test.TestTrue(TEXT("Property handle is valid"), ArrayOfVectorsPropertyHandle_O.IsValid());
		Test.TestTrue(TEXT("Property handle is valid"), IntMap_PropertyHandle.IsValid());
		Test.TestTrue(TEXT("Property handle is valid"), IntMap_PropertyHandle_O.IsValid());

		{
			FVector TestGetVector(0.f);
			ArrayOfVectorsPropertyHandle_O->GetValue(TestGetVector);
			Test.TestTrue(TEXT("Property handle is valid"), TestGetVector.Equals(TestObject->ArrayOfVectors[0]));
		}

		{
			FVector::FReal GetFloatValue = 0;
			ArrayOfVectorsPropertyHandle_O_X->GetValue(GetFloatValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestVector_0.X, GetFloatValue);

			FVector::FReal SetFloatValue = 0;
			ArrayOfVectorsPropertyHandle_O_X->SetValue(SetFloatValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->ArrayOfVectors[0].X, SetFloatValue);
		}
		{
			FVector::FReal GetFloatValue = 0;
			ArrayOfVectorsPropertyHandle_1_X->GetValue(GetFloatValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestVector_1.X, GetFloatValue);

			FVector::FReal SetFloatValue = 0;
			ArrayOfVectorsPropertyHandle_1_X->SetValue(SetFloatValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->ArrayOfVectors[1].X, SetFloatValue);
		}

		{
			int32 GetIntValue = 0;
			IntMap_PropertyHandle_O->GetValue(GetIntValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), *TestObject->IntMap.Find(1), GetIntValue);

			int32 SetIntValue = -2456;
			IntMap_PropertyHandle_O->SetValue(SetIntValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), *TestObject->IntMap.Find(1), SetIntValue);
		}
		IRemoteControlModule::Get().UnregisterEmbeddedPreset(PresetName);
	}

	void TestStructInObjectProperty(FAutomationTestBase& Test)
	{
		// Create preset and uobject
		const TCHAR* PresetName = TEXT("TestComplexPath");

		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>(GetTransientPackage(), PresetName) };
		check(Preset);
		
		TStrongObjectPtr<URemoteControlAPITestObject> TestObject{ NewObject<URemoteControlAPITestObject>() };
		check(TestObject);

		IRemoteControlModule::Get().RegisterEmbeddedPreset(Preset.Get());

		// Set a test value
		TestObject->StaticMeshComponent->BodyInstance.LinearDamping = 5.0;
		
		const FProperty* LinearProperty = FBodyInstance::StaticStruct()->FindPropertyByName(PROP_NAME(FBodyInstance, LinearDamping));
		if(!LinearProperty)
		{
			Test.AddError(TEXT("LinearProperty was not valid"));
			return;
		}

		FRemoteControlPresetExposeArgs ExposeArgs;
		ExposeArgs.Label = "LinearDamping";
		TWeakPtr<FRemoteControlProperty> LinearDampingRC = Preset->ExposeProperty(TestObject->StaticMeshComponent, FRCFieldPathInfo{ "BodyInstance." + LinearProperty->GetName() }, ExposeArgs);
		
		{
			// 1. Find the Remote Control preset
			URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(PresetName);
			if (!ResolvedPreset)
			{
				Test.AddError(TEXT("ResolvedPreset not valid"));
				return;
			}

			// 2. Find exposed properties by label or ID
			const TSharedPtr<FRemoteControlProperty> RCLinearDampingProperty = ResolvedPreset->GetExposedEntity<FRemoteControlProperty>(ResolvedPreset->GetExposedEntityId(*LinearProperty->GetName())).Pin();
			if (!RCLinearDampingProperty.IsValid())
			{
				Test.AddError(TEXT("RCLinearDampingProperty not valid"));
				return;
			}

			// 3. Get property handle
			const TSharedPtr<IRemoteControlPropertyHandle> RCLinearDampingPropertyHandle = RCLinearDampingProperty->GetPropertyHandle();
			if (!RCLinearDampingPropertyHandle.IsValid())
			{
				Test.AddError(TEXT("RCLinearDampingPropertyHandle not valid"));
				return;
			}

			// 4. Get remote control handles values
			float LinearDampingValue = 0.0;
			RCLinearDampingPropertyHandle->GetValue(LinearDampingValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->StaticMeshComponent->BodyInstance.LinearDamping, LinearDampingValue);

			// 5. Set remote control handles values
			constexpr float SetLinearDampingValue = 100.0;
			RCLinearDampingPropertyHandle->SetValue(SetLinearDampingValue);
			Test.TestEqual(TEXT("Value in UObject should be the same as a value from Property Handle."), TestObject->StaticMeshComponent->BodyInstance.LinearDamping, SetLinearDampingValue);
		}

		IRemoteControlModule::Get().UnregisterEmbeddedPreset(Preset.Get());
	}
}

DEFINE_SPEC(FRemoteControlAPIIntegrationTest, "Plugins.RemoteControl.PropertyHandle", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
void FRemoteControlAPIIntegrationTest::Define()
{
	// Test expose whole container
	It("IntPropertyHandle", [this]() {
		RemoteControlAPIIntegrationTest::TestIntPropertyHandle(*this);
	});
	It("CreatePropertyHandleStruct", [this]() {
		RemoteControlAPIIntegrationTest::TestCreatePropertyHandleStruct(*this);
	});
	It("CreatePropertyHandleArray", [this]() {
		RemoteControlAPIIntegrationTest::TestCreatePropertyHandleArray(*this);
	});
	It("CreatePropertyHandleSet", [this]() {
		RemoteControlAPIIntegrationTest::TestCreatePropertyHandleSet(*this);
	});
	It("CreatePropertyHandleMap", [this]() {
		RemoteControlAPIIntegrationTest::TestCreatePropertyHandleMap(*this);
	});
	It("FloatProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestFloatProperty(*this);
	});
	It("DoubleProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestDoubleProperty(*this);
	});
	It("StringProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestStringProperty(*this);
	});
	It("NameProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestNameProperty(*this);
	});
	It("BoolProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestBoolProperty(*this);
	});
	It("ByteProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestByteProperty(*this);
	});
	It("TextProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestTextProperty(*this);
	});
	It("VectorProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestVectorProperty(*this);
	});
	It("RotatorProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestRotatorProperty(*this);
	});
	It("ColorProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestColorProperty(*this);
	});
	It("LinearColorProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestLinearColorProperty(*this);
	});
	It("ComplexPath", [this]() {
		RemoteControlAPIIntegrationTest::TestComplexPath(*this);
	});
	It("GetPropertyHandleByFieldPath", [this]() {
		RemoteControlAPIIntegrationTest::TestGetPropertyHandleByFieldPath(*this);
	});
	It("StructInObjectProperty", [this]() {
		RemoteControlAPIIntegrationTest::TestStructInObjectProperty(*this);
	});
}

#undef GET_TEST_PROP
#undef PROP_NAME
