// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "RemoteControlTestData.generated.h"

USTRUCT(BlueprintType)
struct FRemoteControlTestStruct
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "RC")
	bool bSomeBool = true;
	
	UPROPERTY(EditAnywhere, Category = "RC")
	uint32 SomeUInt32 = 45;

	UPROPERTY(EditAnywhere, Category = "RC")
	float SomeFloat = 49.0f;

	UPROPERTY(EditAnywhere, Category = "RC")
	FVector SomeVector = {0.2f, 0.3f, 0.6f};

	UPROPERTY(EditAnywhere, Category = "RC")
	FRotator SomeRotator = FRotator::ZeroRotator;

	UPROPERTY(EditAnywhere, Category = "RC", meta = (ClampMin = 20, ClampMax = 145))
	int32 SomeClampedInt = 5;

	UPROPERTY(EditAnywhere, Category = "RC", meta = (ClampMin = 0.2f, ClampMax = 0.92f))
	float SomeClampedFloat = 0.25f;
};

USTRUCT()
struct FRemoteControlTestInnerStruct
{
	GENERATED_BODY()

	FRemoteControlTestInnerStruct()
	{
		FloatArray.Add(10.0f);
		FloatArray.Add(104.01f);
		FloatArray.Add(-9.04f);

		// VectorArray.Add({0.5f, 0.6f, 15.05f});
		// VectorArray.Add({-45.5f, 55.0f, 98.1f});
		ArrayOfVectors.Add({1.0f, 1.0f, 1.0f});
	}

	FRemoteControlTestInnerStruct(uint8 Index)
		: Color(FColor(Index, Index, Index, Index))
	{
	}

	UPROPERTY()
	FColor Color = FColor(1,2,3,4 );

	UPROPERTY()
	TArray<float> FloatArray;

	UPROPERTY()
	TArray<FVector> ArrayOfVectors;
};

UCLASS()
class URemoteControlTestObject : public UObject
{
public:
	GENERATED_BODY()

	URemoteControlTestObject()
	{
		for (int8 i = 0; i < 3; i++)
		{
			CStyleIntArray[i] = i+1;
			IntArray.Add(i+1);
			FloatArray.Add((float)i+1);
			IntSet.Add(i+1);
			IntMap.Add(i, i+1);
			IntInnerStructMap.Add((int32)i, FRemoteControlTestInnerStruct((uint8)i));
		}

		StringColorMap.Add(TEXT("mykey"), FColor{1,2,3,4});
	}

	UPROPERTY()
	int32 CStyleIntArray[3];

	UPROPERTY()
	TArray<int32> IntArray;

	UPROPERTY()
	TArray<float> FloatArray;

	UPROPERTY()
	TSet<int32> IntSet;

	UPROPERTY()
	TMap<int32, int32> IntMap;

	UPROPERTY()
	TMap<int32, FRemoteControlTestInnerStruct> IntInnerStructMap;

	UPROPERTY()
	TMap<FString, FColor> StringColorMap;
};
