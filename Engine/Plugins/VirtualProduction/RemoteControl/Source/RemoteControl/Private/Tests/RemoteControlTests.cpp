// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "RemoteControlPreset.h"
#include "RemoteControlTestData.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "UObject/StrongObjectPtr.h"

#define PROP_NAME(Class, Name) GET_MEMBER_NAME_CHECKED(Class, Name)
#define GET_TEST_PROP(PropName) URemoteControlTestObject::StaticClass()->FindPropertyByName(PROP_NAME(URemoteControlTestObject, PropName))

namespace RemoteControlTest 
{
	void ValidateExposePropertyTest(FAutomationTestBase& Test, URemoteControlPreset* Preset, URemoteControlTestObject* TestObject, FProperty* Property, FRemoteControlProperty RCProp)
	{
		Test.TestTrue(TEXT("IsExposed returns true."), Preset->IsExposed(RCProp.GetId()));
		TSharedPtr<FRemoteControlProperty> FetchedProperty = Preset->GetExposedEntity<FRemoteControlProperty>(RCProp.GetId()).Pin();
		Test.TestTrue(TEXT("Preset::GetProperty returns a valid property."), FetchedProperty.IsValid());
		if (!FetchedProperty)
		{
			return;
		}

		if (!((*FetchedProperty) == RCProp))
		{
			Test.AddError(TEXT("Preset::GetProperty returns the same property."));
		}

		Test.TestTrue(Property->GetName() + TEXT(" must resolve correctly."), RCProp.FieldPathInfo.Resolve(TestObject));
		FRCFieldResolvedData ResolvedData = RCProp.FieldPathInfo.GetResolvedData();
		Test.TestTrue(TEXT("Resolved data is valid"), ResolvedData.ContainerAddress && ResolvedData.Field && ResolvedData.Struct);

		Test.TestTrue(TEXT("Resolved property must be valid"), !!RCProp.GetProperty());
		Test.TestTrue(TEXT("Resolved property's owner objects must be valid"), RCProp.GetBoundObjects().Num() > 0);

		Test.TestEqual(TEXT("Resolved property must be equal to the original property."), Property, RCProp.GetProperty());

		if (RCProp.GetBoundObjects().Num())
		{
			Test.TestEqual(TEXT("Resolved property's owner objects must be valid"), TestObject, Cast<URemoteControlTestObject>(RCProp.GetBoundObjects()[0]));
		}
	}

	void TestExpose(FAutomationTestBase& Test, FProperty* Property)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlTestObject> TestObject{ NewObject<URemoteControlTestObject>() };

		// Execute test
		TSharedPtr<FRemoteControlProperty> RCProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{Property->GetName()}).Pin();
		Test.TestNotNull(TEXT("The exposed property must be valid."), RCProp.Get());
		
		if (RCProp)
		{
			// Validate result
			ValidateExposePropertyTest(Test, Preset.Get(), TestObject.Get(), Property, *RCProp);
		}
	}

	void TestExposeContainerElement(FAutomationTestBase& Test, FProperty* Property, FString PropertyPath, bool bCleanDuplicates = false)
	{
		// Setup test data
		TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
		TStrongObjectPtr<URemoteControlTestObject> TestObject{ NewObject<URemoteControlTestObject>() };

		// Execute test
		if (TSharedPtr<FRemoteControlProperty> RCProp = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{ PropertyPath, bCleanDuplicates }).Pin())
		{
			// Validate result
			ValidateExposePropertyTest(Test, Preset.Get(), TestObject.Get(), Property, *RCProp);	
		}
		else
		{
			Test.AddError(TEXT("Could not expose property"));
		}
	}
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRemoteControlPresetIntegrationTest, "Plugins.RemoteControl.Expose", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FRemoteControlPresetIntegrationTest::RunTest(const FString& Parameters)
{
	// Test expose whole container
	RemoteControlTest::TestExpose(*this, GET_TEST_PROP(CStyleIntArray));
	RemoteControlTest::TestExpose(*this, GET_TEST_PROP(IntArray));
	RemoteControlTest::TestExpose(*this, GET_TEST_PROP(IntSet));
	RemoteControlTest::TestExpose(*this, GET_TEST_PROP(IntMap));

	// Test expose container element
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(CStyleIntArray), GET_TEST_PROP(CStyleIntArray)->GetName() + TEXT("[0]"));
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntArray),		GET_TEST_PROP(IntArray)->GetName() + TEXT("[0]"));
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntSet),			GET_TEST_PROP(IntSet)->GetName() + TEXT("[0]"));
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntMap),			GET_TEST_PROP(IntMap)->GetName() + TEXT("[0]"));

	// Test expose container element while skipping duplicates
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(CStyleIntArray),  FString::Printf(TEXT("%s.%s[0]"),		 *GET_TEST_PROP(CStyleIntArray)->GetName(), *GET_TEST_PROP(CStyleIntArray)->GetName()), true);
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntArray),		 FString::Printf(TEXT("%s.%s[0]"),		 *GET_TEST_PROP(IntArray)->GetName(), *GET_TEST_PROP(IntArray)->GetName()), true);
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntSet),			 FString::Printf(TEXT("%s.%s[0]"),		 *GET_TEST_PROP(IntSet)->GetName(), *GET_TEST_PROP(IntSet)->GetName()), true);
	RemoteControlTest::TestExposeContainerElement(*this, GET_TEST_PROP(IntMap),			 FString::Printf(TEXT("%s.%s_Value[0]"), *GET_TEST_PROP(IntMap)->GetName(), *GET_TEST_PROP(IntMap)->GetName()), true);

	{
		FStructSerializerPolicies Policies;
		Policies.MapSerialization = EStructSerializerMapPolicies::Array;

		FStructDeserializerPolicies DeserializerPolicies;
		DeserializerPolicies.MissingFields = EStructDeserializerErrorPolicies::Warning;
		DeserializerPolicies.MapPolicies = EStructDeserializerMapPolicies::Array;
		
		TArray<uint8> Buffer;
		FMemoryReader Reader(Buffer);
		FMemoryWriter Writer(Buffer);

		FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
		FCborStructDeserializerBackend DeserializerBackend(Reader);

		FRemoteControlTestInnerStruct TestStruct;
		const FName Member = GET_MEMBER_NAME_CHECKED(FRemoteControlTestInnerStruct, ArrayOfVectors);
		FProperty* Property = FindFProperty<FProperty>(FRemoteControlTestInnerStruct::StaticStruct(), Member);
		FStructSerializer::SerializeElement(&TestStruct, Property, INDEX_NONE, SerializerBackend, Policies);

		auto o = Buffer;
	}
	
	FProperty* RProperty = TBaseStructure<FColor>::Get()->FindPropertyByName(TEXT("R"));
	// Test exposing map with array indexing
	RemoteControlTest::TestExposeContainerElement(*this, RProperty, FString::Printf(TEXT("%s.%s_Value[0].Color.R"), *GET_TEST_PROP(IntInnerStructMap)->GetName(), *GET_TEST_PROP(IntInnerStructMap)->GetName()), true);
	// Test exposing map with key indexing
	RemoteControlTest::TestExposeContainerElement(*this, RProperty,	FString::Printf(TEXT("%s[\"mykey\"].R"), *GET_TEST_PROP(StringColorMap)->GetName()));
	return true;
}

#undef GET_TEST_PROP
#undef PROP_NAME
