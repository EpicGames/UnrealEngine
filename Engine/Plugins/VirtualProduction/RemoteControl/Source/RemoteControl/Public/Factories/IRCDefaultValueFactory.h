// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Holds the arguments required by the Reset Factories.
 */
struct FRCResetToDefaultArgs
{
	/** Holds a reference to the actual property. */
	FProperty* Property;

	/** Holds the property path of the actual property. */
	FString Path;

	/** Holds the index of the property being modified incase it is embeded in an array. */
	int32 ArrayIndex;

	/** Determines whether should we notify the transaction manager or not. */
	bool bCreateTransaction;

public:

	FRCResetToDefaultArgs()
		: Property(nullptr)
		, Path(TEXT(""))
		, ArrayIndex(0)
		, bCreateTransaction(false)
	{}
	
	FRCResetToDefaultArgs(const FRCResetToDefaultArgs&);
	FRCResetToDefaultArgs& operator=(const FRCResetToDefaultArgs&);
};

/**
 * Factory which resets the FRemoteControlProperty to its default values.
 */
class IRCDefaultValueFactory : public TSharedFromThis<IRCDefaultValueFactory>
{
public:

	/** Virtual destructor */
	virtual ~IRCDefaultValueFactory() {}

	/**
	 * Returns true when the given object can be reset to its default value, false otherwise.
	 * @param InObject Reference to the exposed object.
	 * @param InArgs Arguments to be passed.
	 */
	virtual bool CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const = 0;

	/**
	 * Performs actual data reset on the given remote object.
	 * @param InObject Reference to the exposed object.
	 * @param InArgs Arguments to be passed.
	 */
	virtual void ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs) = 0;
	
	/**
	 * Whether the factory support exposed entity.
	 * @param InObjectClass Reference to the exposed object class.
	 * @return true if the exposed object is supported by given factory
	 */
	virtual bool SupportsClass(const UClass* InObjectClass) const = 0;

	/**
	 * Whether the factory support property.
	 * @param InProperty Reference to the exposed property.
	 * @return true if the exposed property is supported by given factory
	 */
	virtual bool SupportsProperty(const FProperty* InProperty) const = 0;
};
