// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IRemoteControlModule.h"
#include "RCModifyOperationFlags.h"

/**
 * Reference to a UObject or one of its properties for the purpose of masking.
 */
struct UE_DEPRECATED(5.5, "FRCMaskingOperation is deprecated, masking is now applied where appropriate automatically.") FRCMaskingOperation;
struct FRCMaskingOperation
{
	FRCMaskingOperation()
		: ModifyOperationFlags(ERCModifyOperationFlags::None)
	{}

	explicit FRCMaskingOperation(FRCFieldPathInfo InPathInfo, UObject* InObject, const ERCModifyOperationFlags InModifyOperationFlag = ERCModifyOperationFlags::None)
		: OperationId(FGuid::NewGuid())
		, ObjectRef(ERCAccess::NO_ACCESS, InObject, InPathInfo)
		, ModifyOperationFlags(InModifyOperationFlag)
	{
		check(InObject);
	}

	explicit FRCMaskingOperation(const FRCObjectReference& InObjectRef, const ERCModifyOperationFlags InModifyOperationFlag = ERCModifyOperationFlags::None)
		: OperationId(FGuid::NewGuid())
		, ObjectRef(InObjectRef)
		, ModifyOperationFlags(InModifyOperationFlag)
	{
	}

	bool HasMask(ERCMask InMaskBit) const
	{
		return (Masks & InMaskBit) != ERCMask::NoMask;
	}

	bool IsValid() const
	{
		return OperationId.IsValid() && ObjectRef.IsValid();
	}

	friend bool operator==(const FRCMaskingOperation& LHS, const FRCMaskingOperation& RHS)
	{
		return LHS.OperationId == RHS.OperationId && LHS.ObjectRef == RHS.ObjectRef;
	}

	friend uint32 GetTypeHash(const FRCMaskingOperation& MaskingOperation)
	{
		return HashCombine(GetTypeHash(MaskingOperation.OperationId), GetTypeHash(MaskingOperation.ObjectRef));
	}

public:
	/** Unique identifier of the operation being performed. */
	FGuid OperationId;

	/** Masks to be applied. */
	ERCMask Masks = RC_AllMasks;

	/** Holds Object reference. */
	FRCObjectReference ObjectRef;

	/** Holds the state of this RC property before applying any masking. */
	FVector4 PreMaskingCache = FVector4::Zero();

	/** Modify operation flags used when masking */
	const ERCModifyOperationFlags ModifyOperationFlags;
	
	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRCMaskingOperation(const FRCMaskingOperation&) = default;
	FRCMaskingOperation(FRCMaskingOperation&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

/**
 * Factory which is responsible for masking support for FRemoteControlProperty.
 */
class UE_DEPRECATED(5.5, "IRemoteControlMaskingFactory is deprecated, masking is now applied where appropriate automatically.") IRemoteControlMaskingFactory;
class IRemoteControlMaskingFactory : public TSharedFromThis<IRemoteControlMaskingFactory>
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	/** Virtual destructor */
	virtual ~IRemoteControlMaskingFactory(){}

	/**
	 * Applies masked values to the given struct property.
	 * 
	 * @param InMaskingOperation			Shared reference of the masking operation to perform.
	 * @param bIsInteractive				If bWithPropertyChangedEvents, defined if the property changed events are interactive or not.
	 * @param ModifyOperationFlags			(optional) Flags that specify how the property is modified when the value is applied.
	 */
	virtual void ApplyMaskedValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation, bool bIsInteractive) {};

	/**
	 * Caches premasking values from the given struct property.
	 * @param InMaskingOperation Shared reference of the masking operation to perform.
	 */
	virtual void CacheRawValues(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) = 0;

	/**
	 * Whether the factory support exposed entity.
	 * @param ScriptStruct Static struct of exposed property.
	 * @return true if the script struct is supported by given factory
	 */
	virtual bool SupportsExposedEntity(UScriptStruct* ScriptStruct) const = 0;

	// Workaround for clang deprecation warnings for any deprecated members in implicitly-defined special member functions
	IRemoteControlMaskingFactory() = default;
	IRemoteControlMaskingFactory(const IRemoteControlMaskingFactory&) = default;
	IRemoteControlMaskingFactory(IRemoteControlMaskingFactory&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};
