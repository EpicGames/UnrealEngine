// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealType.h"
#include "StructUtils/PropertyBag.h"

class FProperty;
class UObject;

class IPropertyIdHandler
{
public:
	virtual ~IPropertyIdHandler() = default;

	/** Whether this PropertyId Handler handles the given Property */
	virtual bool IsPropertySupported(const FProperty* InProperty) const = 0;

	/** Get the PropertyBag type of the given property */
	virtual EPropertyBagPropertyType GetPropertyType(const FProperty* InProperty) const = 0;

	/**
	 * Get the Property super type FName\n
	 * ex: Object/Struct/Enum or the primitive type if it is a primitive property (float/bool/etc...)
	 */
	virtual FName GetPropertySuperTypeName(const FProperty* InProperty) const = 0;

	/**
	 * Get the Property sub type FName\n
	 * ex: StaticMesh/Material or the primitive type if it is a primitive property (float/bool/etc...)
	 */
	virtual FName GetPropertySubTypeName(const FProperty* InProperty) const = 0;

	/** Get the Property type Object */
	virtual UObject* GetPropertyTypeObject(const FProperty* InProperty) const = 0;

	/** Get the property inside the container if not inside any container return the property passed */
	const FProperty* GetPropertyInsideContainer(const FProperty* InProperty) const
	{
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			return ArrayProperty->Inner;
		}
		if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			return MapProperty->ValueProp;
		}
		if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			return SetProperty->ElementProp;
		}
		return InProperty;
	}
};
