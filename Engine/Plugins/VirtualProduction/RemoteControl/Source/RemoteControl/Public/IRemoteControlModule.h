// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RCModifyOperationFlags.h"
#include "RemoteControlField.h"
#include "RemoteControlFieldPath.h"
#include "UObject/StructOnScope.h"
#include "UObject/WeakFieldPtr.h"

#include "IRemoteControlModule.generated.h"

REMOTECONTROL_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteControl, Log, All);

class IPropertyIdHandler;
class IRemoteControlMaskingFactory;
class IStructDeserializerBackend;
class IStructSerializerBackend;
class URemoteControlPreset;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
struct FExposedProperty;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
struct FRCMaskingOperation;
struct FRCResetToDefaultArgs;
struct FRemoteControlProperty;
class IRemoteControlPropertyFactory;

/**
 * Delegate called to initialize an exposed entity metadata entry registered with the RegisterDefaultEntityMetadata method.
 * Use RemoteControlPreset::GetExposedEntity to retrieve the entity that will contain the metadata.
 * The delegate return value will be put into metadata map for the metadata key that was used when registering the default metadata entry.
 */
DECLARE_DELEGATE_RetVal_TwoParams(FString /*Value*/, FEntityMetadataInitializer, URemoteControlPreset* /*Preset*/, const FGuid& /*EntityId*/);

/**
 * Delegate called after a property has been modified through SetObjectProperties..
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostPropertyModifiedRemotely, const FRCObjectReference& /*ObjectRef*/);

/**
 * Deserialize payload type for interception purposes
 */
enum class ERCPayloadType : uint8
{
	Cbor,
	Json
};

/**
 * Reference to a function in a UObject
 */
struct FRCCallReference
{
	FRCCallReference()
		: Object(nullptr)
		, Function(nullptr)
		, PropertyWithSetter(nullptr)
	{}

	bool IsValid() const
	{
		/** The object and either the function or property must be valid */
		return Object.IsValid() && (Function.IsValid() || PropertyWithSetter.IsValid());
	}

	TWeakObjectPtr<UObject> Object; 
	TWeakObjectPtr<UFunction> Function;
	TWeakFieldPtr<FProperty> PropertyWithSetter;
	
	friend uint32 GetTypeHash(const FRCCallReference& CallRef)
	{
		if(!CallRef.IsValid())
		{
			return 0;
		}
		
		return HashCombine(GetTypeHash(CallRef.Object),
			CallRef.PropertyWithSetter.IsValid()
			? GetTypeHash(CallRef.PropertyWithSetter)
			: GetTypeHash(CallRef.Function));
	}
};

UENUM()
enum class ERCTransactionMode : uint8
{
	NONE = 0,
	AUTOMATIC,
	MANUAL,
};

/**
 * Object to hold a UObject remote call
 */
struct FRCCall
{
	bool IsValid() const
	{
		return CallRef.IsValid() && (ParamStruct.IsValid() || !ParamData.IsEmpty());
	}

	FRCCallReference CallRef;
	// Payload for UFunctiom
	FStructOnScope ParamStruct;
	// Payload for native function
	TArray<uint8> ParamData;
	ERCTransactionMode TransactionMode;
};

/**
 * Object to hold information necessary to resolve a preset property/function.
 */
struct FResolvePresetFieldArgs
{
	FString PresetName;
	FString FieldLabel;
};

/**
 * Arguments for resolving a Controller on a preset.
 */
struct FResolvePresetControllerArgs
{
	FString PresetName;
	FString ControllerName;
};

/**
 * Requested access mode to a remote property
 */
UENUM()
enum class ERCAccess : uint8
{
	NO_ACCESS,
	READ_ACCESS,
	WRITE_ACCESS,
	WRITE_TRANSACTION_ACCESS,
	WRITE_MANUAL_TRANSACTION_ACCESS,
};

/**
 * Type of operation to perform when setting a remote property's value
 */
UENUM()
enum class ERCModifyOperation : uint8
{
	EQUAL,
	ADD,
	SUBTRACT,
	MULTIPLY,
	DIVIDE
};

/**
 * Type of compression applied to WebSocket traffic
 */
UENUM()
enum class ERCWebSocketCompressionMode : uint8
{
	NONE,
	ZLIB
};

/**
 * Reference to a UObject or one of its properties
 */
struct FRCObjectReference
{
	/** Callback type for post set object properties */
	using FPostSetObjectPropertyCallback = TFunction<bool(UObject* /*InObject*/, const FRCFieldPathInfo& /*InPathInfo*/, bool /*bInSuccess*/)>;
	
	FRCObjectReference() = default;

	FRCObjectReference(ERCAccess InAccessType, UObject* InObject)
		: Access(InAccessType)
		, Object(InObject)
	{
		check(InObject);
		ContainerType = InObject->GetClass();
		ContainerAdress = static_cast<void*>(InObject);
	}

	FRCObjectReference(ERCAccess InAccessType, UObject* InObject, FRCFieldPathInfo InPathInfo)
		: Access(InAccessType)
		, Object(InObject)
		, PropertyPathInfo(MoveTemp(InPathInfo))
	{
		check(InObject);
		PropertyPathInfo.Resolve(InObject);
		Property = PropertyPathInfo.GetResolvedData().Field;
		ContainerAdress = PropertyPathInfo.GetResolvedData().ContainerAddress;
		ContainerType = PropertyPathInfo.GetResolvedData().Struct;
		PropertyPathInfo = MoveTemp(PropertyPathInfo);
	}

	bool IsValid() const
	{
		return Object.IsValid() && ContainerType.IsValid() && ContainerAdress != nullptr;
	}

	friend bool operator==(const FRCObjectReference& LHS, const FRCObjectReference& RHS)
	{
		return LHS.Object == RHS.Object && LHS.Property == RHS.Property && LHS.ContainerAdress == RHS.ContainerAdress;
	}

	friend uint32 GetTypeHash(const FRCObjectReference& ObjectReference)
	{
		return HashCombine(GetTypeHash(ObjectReference.Object), ObjectReference.PropertyPathInfo.PathHash);
	}

	/** Type of access on this object (read, write) */
	ERCAccess Access = ERCAccess::NO_ACCESS;

	/** UObject owning the target property */
	TWeakObjectPtr<UObject> Object;

	/** Actual property that is being referenced */
	TWeakFieldPtr<FProperty> Property;

	/** Address of the container of the property for serialization purposes in case of a nested property */
	void* ContainerAdress = nullptr;

	/** Type of the container where the property resides */
	TWeakObjectPtr<UStruct> ContainerType;

	/** Path to the property under the Object */
	FRCFieldPathInfo PropertyPathInfo;
};

/**
 * Interface for the remote control module.
 */
class IRemoteControlModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IRemoteControlModule& Get()
	{
		static const FName ModuleName = "RemoteControl";
		return FModuleManager::LoadModuleChecked<IRemoteControlModule>(ModuleName);
	}

	/** Delegate triggered when a preset has been registered */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPresetRegistered, FName /*PresetName*/);
	UE_DEPRECATED(4.27, "OnPresetUnregistered is deprecated.")
	virtual FOnPresetRegistered& OnPresetRegistered() = 0;

	/** Delegate triggered when a preset has been unregistered */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPresetUnregistered, FName /*PresetName*/);
	UE_DEPRECATED(4.27, "OnPresetUnregistered is deprecated.")
	virtual FOnPresetUnregistered& OnPresetUnregistered() = 0;

	/** Delegate triggered when an error occurs, which aren't necessarily written to UE LOG */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnError, const FString& /*Message*/, ELogVerbosity::Type /*Verbosity*/);
	virtual FOnError& OnError() = 0;

	/** Broadcast an error to the OnError delegate */
	static void BroadcastError(const FString& Message, ELogVerbosity::Type Verbosity = ELogVerbosity::Error)
	{
		Get().OnError().Broadcast(Message, Verbosity);
	}

	/**
	 * Register the preset with the module, enabling using the preset remotely using its name.
	 * @return whether registration was successful.
	 */
	UE_DEPRECATED(4.27, "RegisterPreset is deprecated.")
	virtual bool RegisterPreset(FName Name, URemoteControlPreset* Preset) = 0;

	/** Unregister a preset */
	UE_DEPRECATED(4.27, "UnregisterPreset is deprecated.")
	virtual void UnregisterPreset(FName Name) = 0;

	/**
	 * Registers a hosted (non-asset based) preset with an attached name.
	 * Embedded presets should only be registered while they are active.
	 * @return whether registration was successful.
	 */
	virtual bool RegisterEmbeddedPreset(URemoteControlPreset* Preset, bool bReplaceExisting = false) = 0;

	/**
	 * Unregisters a hosted (non-asset based) preset with the given name.
	 */
	virtual void UnregisterEmbeddedPreset(FName Name) = 0;
	virtual void UnregisterEmbeddedPreset(URemoteControlPreset* Preset) = 0;

	/**
	 * Returns true when the given object can be reset to its default value, false otherwise.
	 * @param InObject Reference to the exposed object.
	 * @param InArgs Arguments to be passed.
	 */
	virtual bool CanResetToDefaultValue(UObject* InObject, const FRCResetToDefaultArgs& InArgs) const = 0;

	/**
	 * Returns true when the given object and property has custom default value, false otherwise.
	 * @param InObject Reference to the exposed object.
	 * @param InProperty Reference to the exposed property.
	 */
	virtual bool HasDefaultValueCustomization(const UObject* InObject, const FProperty* InProperty) const = 0;

	/**
	 * Performs actual data reset on the given object.
	 * @param InObject Reference to the exposed object.
	 * @param InArgs Arguments to be passed.
	 */
	virtual void ResetToDefaultValue(UObject* InObject, FRCResetToDefaultArgs& InArgs) = 0;

	/**
	 * Performs the given masking operation.
	 * @param InMaskingOperation	Masking operation to be performed.
	 * @param ModifyOperationFlags	(optional) Flags that specify how the property is modified when the value is applied.
	 */
	UE_DEPRECATED(5.5, "PerformMasking is deprecated, masking is now applied where appropriate automatically.")
	virtual void PerformMasking(const TSharedRef<FRCMaskingOperation>& InMaskingOperation) = 0;

	/**
	 * Register a masking factory to handle that masks the supported properties.
	 */
	UE_DEPRECATED(5.5, "RegisterMaskingFactoryForType is deprecated, masking is now applied where appropriate automatically.")
	virtual void RegisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType, const TSharedPtr<IRemoteControlMaskingFactory>& InMaskingFactory) = 0;

	/**
	 * Unregister a previously registered masking factory.
	 */
	UE_DEPRECATED(5.5, "UnregisterMaskingFactoryForType is deprecated, masking is now applied where appropriate automatically.")
	virtual void UnregisterMaskingFactoryForType(UScriptStruct* RemoteControlPropertyType) = 0;

	/**
	 * Returns true if the given property can be masked, false otherwise.
	 */
	virtual bool SupportsMasking(const UScriptStruct* InStruct) const = 0;

	/**
	 * Returns true if the given property can be masked, false otherwise.
	 */
	virtual bool SupportsMasking(const FProperty* InProperty) const = 0;

	/**
	 * Resolve a RemoteCall Object and Function.
	 * This will look for object and function to resolve. 
	 * It will only successfully resolve function that are blueprint callable. 
	 * The blueprint function name can be used.
	 * @param ObjectPath	The object path to the UObject we want to resolve
	 * @param FunctionName	The function name of the function we want to resolve
	 * @param OutCallRef	The RemoteCallReference in which the object and function will be resolved into
	 * @param OutErrorText	Optional pointer to an error text in case of resolving error.
	 * @return true if the resolving was successful
	 */
	virtual bool ResolveCall(const FString& ObjectPath, const FString& FunctionName, FRCCallReference& OutCallRef, FString* OutErrorText = nullptr) = 0;

	/**
	 * Invoke a Remote Call
	 * This is a thin wrapper around UObject::ProcessEvent
	 * This expects that the caller has already validated the call as it will assert otherwise.
	 * @param InCall the remote call structure to call.
	 * @param InPayloadType the payload type archive.
	 * @param InInterceptPayload the payload reference archive for the interception.
	 * @return true if the call was allowed and done.
	 */
	virtual bool InvokeCall(FRCCall& InCall, ERCPayloadType InPayloadType = ERCPayloadType::Json, const TArray<uint8>& InInterceptPayload = TArray<uint8>()) = 0;

	/**
	 * Resolve a remote object reference to a property
	 * @param AccessType the requested access to the object, (i.e. read or write)
	 * @param ObjectPath the object path to resolve
	 * @param PropertyName the property to resolve, if any (specifying no property will return back the whole object when getting/setting it)
	 * @param OutObjectRef the object reference to resolve into
	 * @param OutErrorText an optional error string pointer to write errors into.
	 * @return true if resolving the object and its property succeeded or just the object if no property was specified.
	 */
	virtual bool ResolveObject(ERCAccess AccessType, const FString& ObjectPath, const FString& PropertyName, FRCObjectReference& OutObjectRef, FString* OutErrorText = nullptr) = 0;

	/**
	 * Resolve a remote object reference to a property
	 * @param AccessType the requested access to the object, (i.e. read or write)
	 * @param Object the object to resolve the property on
	 * @param PropertyPath the path to or the name of the property to resolve. Specifying an empty path will return back the whole object when getting/setting it)
	 * @param OutObjectRef the object reference to resolve into
	 * @param OutErrorText an optional error string pointer to write errors into.
	 * @return true if resolving the object and its property succeeded or just the object if no property was specified.
	 */
	virtual bool ResolveObjectProperty(ERCAccess AccessType, UObject* Object, FRCFieldPathInfo PropertyPath, FRCObjectReference& OutObjectRef, FString* OutErrorText = nullptr) = 0;

	/**
	 * Serialize the Object Reference into the specified backend.
	 * @param ObjectAccess the object reference to serialize, it should be a read access reference.
	 * @param Backend the struct serializer backend to use to serialize the object properties.
	 * @return true if the serialization succeeded
	 */
	virtual bool GetObjectProperties(const FRCObjectReference& ObjectAccess, IStructSerializerBackend& Backend) = 0;

	/**
	 * Deserialize the Object Reference from the specified backend.
	 * @param ObjectAccess the object reference to deserialize into, it should be a write access reference. if the object is WRITE_TRANSACTION_ACCESS, the setting will be wrapped in a transaction.
	 * @param Backend the struct deserializer backend to use to deserialize the object properties.
	 * @param InPayloadType the payload type archive.
	 * @param InInterceptPayload the payload reference archive for the interception.
	 * @param Operation the type of operation to perform when setting the value.
	 * @param ModifyOperationFlags (optional) Flags that specify how the property is modified when the value is applied.
	 * @return true if the deserialization succeeded
	 */
	virtual bool SetObjectProperties(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType = ERCPayloadType::Json, const TArray<uint8>& InInterceptPayload = TArray<uint8>(), ERCModifyOperation Operation = ERCModifyOperation::EQUAL, const ERCModifyOperationFlags ModifyOperationFlags = ERCModifyOperationFlags::None) = 0;

	/**
	 * Reset the property or the object the Object Reference is pointing to
	 * @param ObjectAccess the object reference to reset, it should be a write access reference
	 * @param bAllowIntercept interception flag, if that is set to true it should follow the interception path
	 * @return true if the reset succeeded.
	 */
	virtual bool ResetObjectProperties(const FRCObjectReference& ObjectAccess, const bool bAllowIntercept = false) = 0;

	/**
	 * Deserialize the Object Reference from the specified backend and append it to an array property.
	 * @param ObjectAccess the object reference to deserialize into, it should be a write access reference. if the object is WRITE_TRANSACTION_ACCESS, the setting will be wrapped in a transaction.
	 * @param Backend the struct deserializer backend to use to deserialize the object properties.
	 * @param InPayloadType the payload type archive.
	 * @param InInterceptPayload the payload reference archive for the interception.
	 * @return true if the operation succeeded
	 */
	virtual bool AppendToObjectArrayProperty(const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType = ERCPayloadType::Json, const TArray<uint8>& InInterceptPayload = TArray<uint8>()) = 0;

	/**
	 * Deserialize the Object Reference from the specified backend and insert it at the given position in an array property.
	 * @param Index The index into the array to insert at.
	 * @param ObjectAccess the object reference to deserialize into, it should be a write access reference. if the object is WRITE_TRANSACTION_ACCESS, the setting will be wrapped in a transaction.
	 * @param Backend the struct deserializer backend to use to deserialize the object properties.
	 * @param InPayloadType the payload type archive.
	 * @param InInterceptPayload the payload reference archive for the interception.
	 * @return true if the operation succeeded
	 */
	virtual bool InsertToObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess, IStructDeserializerBackend& Backend, ERCPayloadType InPayloadType = ERCPayloadType::Json, const TArray<uint8>& InInterceptPayload = TArray<uint8>()) = 0;

	/**
	 * Remove an item from the referenced array.
	 * @param Index The index into the array to remove.
	 * @param ObjectAccess The reference to the array. If the object is WRITE_TRANSACTION_ACCESS, the operation will be wrapped in a transaction.
	 * @return true if the operation succeeded
	 */
	virtual bool RemoveFromObjectArrayProperty(int32 Index, const FRCObjectReference& ObjectAccess) = 0;

	/**
	* Set a controller's value on a Remote Control Preset
	* @param PresetName - The Remote Control Preset asset's name
	* @param ControllerName - The name of the controller being manipulated
	* @param Backend - the struct deserializer backend to use to deserialize the object properties.
	* @param InPayload - the JSON payoad
	* @param bAllowIntercept - Whether the call needs to be replicated to interceptors or invoked directly
	*	@return true if the operation succeeded
	*/
	virtual bool SetPresetController(const FName PresetName, const FName ControllerName, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept) = 0;

	/**
	 * Set a controller's value on a Remote Control Preset
	 * @param PresetName - The Remote Control Preset asset's name
	 * @param Controller - The controller object being manipulated
	 * @param Backend - the struct deserializer backend to use to deserialize the object properties.
	 * @param InPayload - the JSON payoad
	* @param bAllowIntercept - Whether the call needs to be replicated to interceptors or invoked directly
	 * @return true if the operation succeeded
	 */
	virtual bool SetPresetController(const FName PresetName, class URCVirtualPropertyBase* Controller, IStructDeserializerBackend& Backend, const TArray<uint8>& InPayload, const bool bAllowIntercept) = 0;

	/**
	 * Resolve the underlying function from a preset.
	 * @return the underlying function and objects that the property is exposed on.
	 */
	UE_DEPRECATED(4.27, "This function is deprecated, please resolve directly on the preset.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TOptional<struct FExposedFunction> ResolvePresetFunction(const FResolvePresetFieldArgs& Args) const = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * Resolve the underlying property from a preset.
	 * @return the underlying property and objects that the property is exposed on.
	 */
	UE_DEPRECATED(4.27, "This function is deprecated, please resolve directly on the preset.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual TOptional<struct FExposedProperty> ResolvePresetProperty(const FResolvePresetFieldArgs& Args) const = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
	/**
	 * Get a preset using its name.
	 * @arg PresetName name of the preset to resolve.
	 * @return the preset if found.
	 */
	virtual URemoteControlPreset* ResolvePreset(FName PresetName) const = 0;

	/**
     * Get a preset using its id.
     * @arg PresetId id of the preset to resolve.
     * @return the preset if found.
     */
	virtual URemoteControlPreset* ResolvePreset(const FGuid& PresetId) const = 0;

	/**
	 * Create a transient preset.
	 * Make sure to call DestroyTransientPreset when done with the preset or it will stay in memory.
	 * @return the new preset, or nullptr if it couldn't be created.
	 */
	virtual URemoteControlPreset* CreateTransientPreset() = 0;

	/**
	 * Destroy a transient preset using its name.
	 * @arg PresetName name of the preset to destroy.
	 * @return true if a transient preset with that name existed and was destroyed.
	 */
	virtual bool DestroyTransientPreset(FName PresetName) = 0;

	/**
	 * Destroy a transient preset using its id.
     * @arg PresetId id of the preset to destroy.
	 * @return true if a transient preset with that name existed and was destroyed.
	 */
	virtual bool DestroyTransientPreset(const FGuid& PresetId) = 0;

	/**
	 * Check whether a preset is transient using its name.
	 * @arg PresetName name of the preset to check.
	 * @return true if the preset exists and is transient.
	 */
	virtual bool IsPresetTransient(FName PresetName) const = 0;

	/**
	 * Check whether a preset is transient using its id.
     * @arg PresetId id of the preset to check.
	 * @return true if the preset exists and is transient.
	 */
	virtual bool IsPresetTransient(const FGuid& PresetId) const = 0;

	/**
	 * Get all the presets currently registered with the module.
	 */
	virtual void GetPresets(TArray<TSoftObjectPtr<URemoteControlPreset>>& OutPresets) const = 0;

	/**
	 * Get all the preset asset currently registered with the module.
	 * @arg bIncludeTransient Whether to include transient presets.
	 */
	virtual void GetPresetAssets(TArray<FAssetData>& OutPresetAssets, bool bIncludeTransient = true) const = 0;

	/**
	 * Gets all hosted presets currently registered with the module.
	 */
	virtual void GetEmbeddedPresets(TArray<TWeakObjectPtr<URemoteControlPreset>>& OutEmbeddedPresets) const = 0;

	/**
	 * Get the map of registered default entity metadata initializers. 
	 */
	virtual const TMap<FName, FEntityMetadataInitializer>& GetDefaultMetadataInitializers() const = 0;
	
	/**
	 * Register a default entity metadata which will show up in an entity's detail panel.
	 * The initializer will be called upon an entity being exposed or when a preset is loaded in
	 * order to update all existing entities that don't have that metadata key.
	 * @param MetadataKey The desired metadata key.
	 * @param MetadataInitializer The delegate to call to handle initializing the metadata.
	 */
	virtual bool RegisterDefaultEntityMetadata(FName MetadataKey, FEntityMetadataInitializer MetadataInitializer) = 0;

	/**
	 * Unregister a default entity metadata.
	 * @param MetadataKey The metadata entry to unregister.
	 */
	virtual void UnregisterDefaultEntityMetadata(FName MetadataKey) = 0;

	/**
	 * Returns whether the property can be modified through SetObjectProperties when running without an editor.
	 */
	UE_DEPRECATED(5.4, "This function is deprecated, please use PropertySupportsRawModification.")
	virtual bool PropertySupportsRawModificationWithoutEditor(FProperty* Property, UClass* OwnerClass = nullptr) const = 0;

	/**
	 * Returns whether the property can be modified through SetObjectProperties or read through ResolveObjectProperties when running with or without an editor.
	 * @param InProperty Property to check
	 * @param InObject Object that owns the property
	 * @param bInWithEditor True if it should check if the property is supported in Editor, false to check for Packaged
	 * @param OutError Will contain the error in case it is not supported
	 * @return True if the property is supported both for read and write, false otherwise
	 */
	virtual bool PropertySupportsRawModification(FProperty* InProperty, const UObject* InObject, const bool bInWithEditor, FString* OutError = nullptr) const = 0;

	/**
	 * Register factory 
	 * @param InFactoryName the factory unique name
	 * @param InFactory Factory instance
	 */
	virtual void RegisterEntityFactory( const FName InFactoryName, const TSharedRef<IRemoteControlPropertyFactory>& InFactory) = 0;

	/**
	 * Remove factory by name
	 * @param InFactoryName the factory unique name
	 */
	virtual void UnregisterEntityFactory( const FName InFactoryName ) = 0;

	/**
	 * Start a manual editor transaction originating from a remote request.
	 * @param InDescription A description of the transaction.
	 * @param TypeHash The type hash of the object or call to which the transaction applies, or 0 if not applicable (e.g. multiple objects being modified).
	 * @return The ID of the new transaction if one was created, or an invalid ID otherwise.
	 */
	virtual FGuid BeginManualEditorTransaction(const FText& InDescription, uint32 TypeHash) = 0;

	/**
	 * End a manual editor transaction originating from a remote request.
	 * @param TransactionId The ID of the transaction. If this doesn't match the current active transaction, the transaction won't be ended.
	 * @return INDEX_NONE if there was no transaction to end; otherwise, the number of remaining actions in the transaction (i.e. 1 means this was the last action and the transaction will end).
	 */
	virtual int32 EndManualEditorTransaction(const FGuid& TransactionId) = 0;

	/** Get map of the factories which is responsible for the Remote Control property creation */
	virtual const TMap<FName, TSharedPtr<IRemoteControlPropertyFactory>>& GetEntityFactories() const = 0;

	/**
	 *  @return whether an object can be accessed remotely.
	 *  @note Check Remote Control project settings to configure.
	 */
	virtual bool CanBeAccessedRemotely(UObject* InObject) const = 0;

	template<class InPropertyIdPropertyHandlerType, typename... InArgTypes
		UE_REQUIRES(std::is_base_of_v<IPropertyIdHandler, InPropertyIdPropertyHandlerType>)>
	TSharedRef<InPropertyIdPropertyHandlerType> RegisterPropertyIdPropertyHandler(InArgTypes&&... InArgs)
	{
		TSharedRef<InPropertyIdPropertyHandlerType> KeyPropertyHandler = MakeShared<InPropertyIdPropertyHandlerType>(Forward<InArgTypes>(InArgs)...);
		this->RegisterPropertyIdPropertyHandlerImpl(KeyPropertyHandler);
		return KeyPropertyHandler;
	}

	virtual TSharedPtr<IPropertyIdHandler> GetPropertyIdHandlerFor(FProperty* InProperty) = 0;

protected:
	virtual void RegisterPropertyIdPropertyHandlerImpl(const TSharedRef<IPropertyIdHandler>& InKeyPropertyHandler) = 0;
};
