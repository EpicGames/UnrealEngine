// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

struct FRemoteControlProperty;
class IRemoteControlPropertyHandleArray;
class IRemoteControlPropertyHandleMap;
class IRemoteControlPropertyHandleSet;

/**
* A handle to a property which is used to read and write the value
*/
class IRemoteControlPropertyHandle : public TSharedFromThis<IRemoteControlPropertyHandle>
{
public:

	/**
	 * Get the property handle based on field path
	 * @param PresetName Preset Name
	 * @param PropertyId Unique property Id
	 * @param FieldPath full path to the property field. 
	 * @note 
	 * 
	 * @return IRemoteControlPropertyHandle pointer or nullptr
	 */
	REMOTECONTROL_API static TSharedPtr<IRemoteControlPropertyHandle> GetPropertyHandle(FName PresetName, FGuid PropertyId);

	/**
	 * Get the property handle based on field path
	 * @param PresetName Preset Name
	 * @param PropertyLabel Exposed property label
	 * @param FieldPath full path to the property field
	 *
	 * @return IRemoteControlPropertyHandle pointer or nullptr
	 */
	REMOTECONTROL_API static TSharedPtr<IRemoteControlPropertyHandle> GetPropertyHandle(FName PresetName, FName PropertyLabel);

public: 
	virtual ~IRemoteControlPropertyHandle() = default;

	/**
	 * Gets the typed value of a property.
	 * If the property does not support the value type false is returned
	 *
	 * @param OutValue	The value that will be set if successful
	 * @return true if the operation supported and value read successfully 
	 * or false if that is not supported by property handle or it can't be read
	 */
	virtual bool GetValue(float& OutValue) const = 0;
	virtual bool GetValue(double& OutValue) const = 0;
	virtual bool GetValue(bool& OutValue) const = 0;
	virtual bool GetValue(int8& OutValue) const = 0;
	virtual bool GetValue(int16& OutValue) const = 0;
	virtual bool GetValue(int32& OutValue) const = 0;
	virtual bool GetValue(int64& OutValue) const = 0;
	virtual bool GetValue(uint8& OutValue) const = 0;
	virtual bool GetValue(uint16& OutValue) const = 0;
	virtual bool GetValue(uint32& OutValue) const = 0;
	virtual bool GetValue(uint64& OutValue) const = 0;
	virtual bool GetValue(FString& OutValue) const = 0;
	virtual bool GetValue(FText& OutValue) const = 0;
	virtual bool GetValue(FName& OutValue) const = 0;
	virtual bool GetValue(FVector& OutValue) const = 0;
	virtual bool GetValue(FVector2D& OutValue) const = 0;
	virtual bool GetValue(FVector4& OutValue) const = 0;
	virtual bool GetValue(FQuat& OutValue) const = 0;
	virtual bool GetValue(FRotator& OutValue) const = 0;
	virtual bool GetValue(FColor& OutValue) const = 0;
	virtual bool GetValue(FLinearColor& OutValue) const = 0;
	virtual bool GetValue(UObject*& OutValue) const = 0;
	virtual bool GetValueInArray(UObject*& OutValue, int32 InIndex) const = 0;
	
	/**
	 * Sets the typed value of a property.
	 * If the property does not support the value type false is returned
	 *
	 * @param InValue The value to set
	 * @return true if the operation supported and value set successfully
	 * or false if that is not supported by property handle or it can't be set
	 */
	virtual bool SetValue(float InValue) = 0;
	virtual bool SetValue(double InValue) = 0;
	virtual bool SetValue(bool InValue) = 0;
	virtual bool SetValue(int8 InValue) = 0;
	virtual bool SetValue(int16 InValue) = 0;
	virtual bool SetValue(int32 InValue) = 0;
	virtual bool SetValue(int64 InValue) = 0;
	virtual bool SetValue(uint8 InValue) = 0;
	virtual bool SetValue(uint16 InValue) = 0;
	virtual bool SetValue(uint32 InValue) = 0;
	virtual bool SetValue(uint64 InValue) = 0;
	virtual bool SetValue(const FString& InValue) = 0;
	virtual bool SetValue(const FText& InValue) = 0;
	virtual bool SetValue(const FName& InValue) = 0;
	virtual bool SetValue(FVector InValue) = 0;
	virtual bool SetValue(FVector2D InValue) = 0;
	virtual bool SetValue(FVector4 InValue) = 0;
	virtual bool SetValue(FQuat InValue) = 0;
	virtual bool SetValue(FRotator InValue) = 0;
	virtual bool SetValue(FColor InValue) = 0;
	virtual bool SetValue(FLinearColor InValue) = 0;
	virtual bool SetValue(const TCHAR* InValue) = 0;
	virtual bool SetValue(UObject* InValue) = 0;
	virtual bool SetValueInArray(UObject* InValue, int32 InIndex) = 0;

	/**
	 * Gets a child handle of this handle.  Useful for accessing properties in structs.
	 * Array elements cannot be accessed in this way
	 *
	 * @param ChildName The name of the child
	 * @param bRecurse	Whether or not to recurse into children of children and so on. If false will only search all immediate children
	 * @return The property handle for the child if it exists
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandle(FName ChildName, bool bRecurse = true) const = 0;

	/**
	 * Gets a child handle of this handle.  Useful for accessing properties in structs.
	 *
	 * @param The index of the child
	 * @return The property handle for the child if it exists
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandle(int32 Index) const = 0;

	/**
	 * @return The number of children the property handle has
	 */
	virtual int32 GetNumChildren() const = 0;

	/**
	 * Gets handle by full field path
	 * For example the property could be ArrayOfVectors and the ArrayOfVectors[0].Y path needed for access of Y property from index 0
	 *
	 * @param InFieldPath the path from a UObject to a parent field
	 * @return The property handle if it exists by given path
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetChildHandleByFieldPath(const FString& InFieldPath) = 0;

	/**
	 * Returns this handle as an array if possible
	 *
	 * @return the handle as an array if it is an array (static or dynamic)
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandleArray> AsArray() = 0;

	/**
	 * @return This handle as a set if possible
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandleSet> AsSet() = 0;

	/**
	 * @return This handle as a map if possible
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandleMap> AsMap() = 0;

	/**
	 * @return the property exposed to remote control.
	 */
	virtual TSharedPtr<FRemoteControlProperty> GetRCProperty() const = 0;

	/**
	 * @return The index of this element in an array;
	 */
	virtual int32 GetIndexInArray() const = 0;

	/**
	 * Gets the property being edited
	 */
	virtual FProperty* GetProperty() const = 0;

	/**
	 * Gets the parent property being edited
	 */
	virtual FProperty* GetParentProperty() const = 0;

	/**
	 * Gets the path from a UObject to a field
	 */
	virtual const FString& GetFieldPath() const = 0;

	/**
	 * Gets the path from a UObject to a parent field
	 */
	virtual const FString& GetParentFieldPath() const = 0;

	/**
	 * Should the property handle generate transactions.
	 */
	virtual bool ShouldGenerateTransaction() const = 0;

	/**
	 * Set transaction generation for this property handle.
	 */
	virtual void SetGenerateTransaction(bool bInGenerateTransaction) = 0;
};

/**
 * A handle to an array property which allows you to manipulate the array
 */
class IRemoteControlPropertyHandleArray
{
public:
	virtual ~IRemoteControlPropertyHandleArray() = default;

	/**
	 * @return The number of elements in the array
	 */
	virtual int32 GetNumElements() const = 0;

	/**
	 * @return a handle to the element at the specified index
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> GetElement(int32 Index) = 0;
};

/**
 * A handle to a property which allows you to manipulate a Set
 */
class IRemoteControlPropertyHandleSet
{
public:
	virtual ~IRemoteControlPropertyHandleSet() = default;

	/**
	 * @return The number of elements in the set
	 */
	virtual int32 GetNumElements() = 0;

	/**
	 * @return a handle to the element at the specified set value pointer
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> FindElement(const void* ElementToFind) = 0;
};

/**
 * A handle to a property which allows you to manipulate a Map
 */
class IRemoteControlPropertyHandleMap
{
public:
	virtual ~IRemoteControlPropertyHandleMap() = default;

	/**
	 * @return The number of elements in the map
	 */
	virtual int32 GetNumElements() = 0;

	/**
	 * @return a handle to the element at the specified key value pointer
	 */
	virtual TSharedPtr<IRemoteControlPropertyHandle> Find(const void* KeyPtr) = 0;
};

