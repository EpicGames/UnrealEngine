// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "GenericPlatform/GenericPlatform.h"
#include "Misc/EnumClassFlags.h"

#include "RCModifyOperationFlags.generated.h"

UENUM()
enum class ERCModifyOperationFlags : uint8
{
	/** Properties are modified with Transactions and Property Changed events */
	None = 0x00,

	/** Properties are modified without Property Change events and implicitly without transactions */
	SkipPropertyChangeEvents = 0x01,

	/** Properties are modified without Transactions */
	SkipTransactions = 0x02
};

ENUM_CLASS_FLAGS(ERCModifyOperationFlags)
