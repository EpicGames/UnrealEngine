// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureActionInstance.h"
#include "RemoteControlFieldPath.h"
#include "StructUtils/PropertyBag.h"
#include "RCSignature.generated.h"

enum class EPropertyBagPropertyType : uint8;
class URemoteControlPreset;

/** Representation class of a Field (e.g. Property) in an Object */
USTRUCT()
struct FRCSignatureField
{
	GENERATED_BODY()

	FRCSignatureField() = default;

	REMOTECONTROL_API static FRCSignatureField CreateField(const FRCFieldPathInfo& InFieldPathInfo, const UObject* InOwnerObject, const FProperty* InProperty);

	void PostLoad();

	bool operator==(const FRCSignatureField& InOtherField) const
	{
		return FieldPath == InOtherField.FieldPath
			&& ObjectRelativePath == InOtherField.ObjectRelativePath
			&& SupportedClass == InOtherField.SupportedClass;
	}

	/** Path Info for this Field */
	UPROPERTY()
	FRCFieldPathInfo FieldPath;

	/** Optional: Relative path from an owner (e.g. Actor) to the object owning the property (e.g. an Actor Component) */
	UPROPERTY()
	FString ObjectRelativePath;

	/** Object class holding the property */
	UPROPERTY()
	FSoftClassPath SupportedClass;

	/** Optional: Property Description of the field (if property) */
	UPROPERTY()
	FPropertyBagPropertyDesc PropertyDesc;

	/** Container holding the action instances for the field */
	UPROPERTY()
	TArray<FRCSignatureActionInstance> Actions;

	/** Whether to consider this field when applying a Signature */
	UPROPERTY()
	bool bEnabled = true;
};

USTRUCT()
struct FRCSignature
{
	GENERATED_BODY()

	bool operator==(const FGuid& InSignatureId) const
	{
		return Id == InSignatureId;
	}

	void PostLoad();

	/**
	 * Adds the given fields to this signature and ensuring no repeated field is present
	 * @return the number of new fields added
	 */
	REMOTECONTROL_API int32 AddFields(TConstArrayView<FRCSignatureField> InFields);

	/**
	 * Applies this Signature to the given Objects by exposing all this Signature's fields to the given preset
	 * @param InPreset the preset where these properties will be exposed to
	 * @param InObjects the objects whose properties or properties of its subojects to expose
	 * @return the number of properties affected in total
	 */
	REMOTECONTROL_API int32 ApplySignature(URemoteControlPreset* InPreset, TConstArrayView<TWeakObjectPtr<UObject>> InObjects) const;

	/** User facing friendly name. Used as the Label when exposing */
	UPROPERTY()
	FText DisplayName;

	/** Unique Id identifying this Signature */
	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	FGuid Id;

	/** The fields owned by this Signature */
	UPROPERTY()
	TArray<FRCSignatureField> Fields;

	/** Whether this Signature can be applied */
	UPROPERTY()
	bool bEnabled = true;
};