// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectPtr.h"
#include "RCSignatureAction.generated.h"

class URemoteControlPreset;
struct FRCSignature;
struct FRCSignatureField;
struct FRemoteControlProperty;

#if WITH_EDITOR
class FEditPropertyChain;
struct FPropertyChangedEvent;
#endif

/** The context for a Signature Action to execute */
struct FRCSignatureActionContext
{
	/** The preset where the Signature is being applied */
	TObjectPtr<URemoteControlPreset> Preset = nullptr;

	/** The object that the Signature is applying */
	TObjectPtr<UObject> Object = nullptr;

	/** The exposed property from the Signature */
	TSharedPtr<FRemoteControlProperty> Property;
};

#if WITH_EDITOR
/**
 * Editor information on the Icon of a given Action
 * Names are explicitly used over FSlateIcon to avoid a dependency on Slate Core API
 */
struct FRCSignatureActionIcon
{
	/** Name of the style set the icon can be found in */
	FName StyleSetName;

	/** Name of the style for the icon */
	FName StyleName;

	/** Name of the style for the overlay icon (if any) */
	FName OverlayStyleName;

	/** Color of the Base Icon */
	FLinearColor BaseColor = FLinearColor::White;

	/** Optional Color of the Overlay Icon (if any). Uses the Icon Color if not set */
	TOptional<FLinearColor> OverlayColor;
};
#endif

/** Base class of Actions that execute when Applying a Signature Field */
USTRUCT(meta=(Hidden))
struct FRCSignatureAction
{
	GENERATED_BODY()

	virtual ~FRCSignatureAction() = default;

	/**
	 * Called when the Signature Action is first added to the action list or loaded
	 * @param InField the owner of the action list
	 */
	virtual void Initialize(const FRCSignatureField& InField)
	{
	}

	/**
	 * Determines whether this Action can execute under a given field
	 * @param InField the field that will contain this action if supported
	 * @return true if the action is supported
	 */
	virtual bool IsSupported(const FRCSignatureField& InField) const
	{
		return true;
	}

	/**
	 * Executes the Action logic
	 * @param InContext context of the action (preset, exposed property and uobject being used)
	 * @return true if the action did anything to the preset
	 */
	virtual bool Execute(const FRCSignatureActionContext& InContext) const
	{
		return false;
	}

#if WITH_EDITOR
	/** Called whenever there's a change affecting the action in the Details Panel */
	virtual void PostEditChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
	{
	}

	/** Retrieves the icon to use to represent this Signature Action */
	virtual FRCSignatureActionIcon GetIcon() const
	{
		return FRCSignatureActionIcon();
	}
#endif
};
