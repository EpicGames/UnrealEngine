// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/InstancedStruct.h"
#include "RCSignatureActionInstance.generated.h"

class FStructOnScope;
struct FRCSignatureAction;
struct FRCSignatureActionContext;
struct FRCSignatureField;

#if WITH_EDITOR
class FEditPropertyChain;
struct FPropertyChangedEvent;
#endif

/** Struct containing an Action Instance and handling its Execution */
USTRUCT()
struct FRCSignatureActionInstance
{
	GENERATED_BODY()

	FRCSignatureActionInstance() = default;

	REMOTECONTROL_API explicit FRCSignatureActionInstance(const UScriptStruct* InScriptStruct, const FRCSignatureField& InFieldOwner);

	REMOTECONTROL_API const FRCSignatureAction* GetAction() const;

	REMOTECONTROL_API TSharedRef<FStructOnScope> MakeStructOnScope();

	void PostLoad(const FRCSignatureField& InFieldOwner);

	bool Execute(const FRCSignatureActionContext& InContext) const;

#if WITH_EDITOR
	REMOTECONTROL_API void PostEditChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged);
#endif

private:
	UPROPERTY()
	TInstancedStruct<FRCSignatureAction> ActionInstance;
};
