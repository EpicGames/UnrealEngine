// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignature.h"
#include "UObject/Object.h"
#include "RCSignatureRegistry.generated.h"

UCLASS(MinimalAPI)
class URCSignatureRegistry : public UObject
{
	GENERATED_BODY()

public:
	TConstArrayView<FRCSignature> GetSignatures() const
	{
		return Signatures;
	}

	/**
	 * Find an existing Signature matching the given id
	 * @param InSignatureId the id of the signature to find
	 * @return const pointer to the signature if found, nullptr otherwise 
	 */
	REMOTECONTROL_API const FRCSignature* FindSignature(const FGuid& InSignatureId) const;

	/**
	 * Find an existing Signature matching the given id
	 * @param InSignatureId the id of the signature to find
	 * @return pointer to the signature if found, nullptr otherwise 
	 */
	REMOTECONTROL_API FRCSignature* FindSignatureMutable(const FGuid& InSignatureId);

	/** Adds a new Signature to this registry */
	REMOTECONTROL_API FRCSignature& AddSignature();

	/**
	 * Removes the Signature with the matching Signature Id
	 * @param InSignatureId the id of the signature to remove
	 * @return the number of signatures removed
	 */
	REMOTECONTROL_API int32 RemoveSignature(const FGuid& InSignatureId);

	/** Removes all the Signature in this registry */
	REMOTECONTROL_API void EmptySignatures();

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

private:
	/** Holds all the Signatures in this Registry */
	UPROPERTY()
	TArray<FRCSignature> Signatures;
};
