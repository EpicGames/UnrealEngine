// Copyright Epic Games, Inc. All Rights Reserved.
 
#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/Map.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Object.h"
 
#include "RCVirtualProperty.generated.h"

class FStructOnScope;
class IStructSerializerBackend;
class IStructDeserializerBackend;
class URemoteControlPreset;
class URCVirtualPropertyContainerBase;

/**
 * Base class for dynamic virtual properties
 * Remote Control Virtual Properties using Property Bag and FInstancedPropertyBag to serialize FProperties values and UStruct 
 */
UCLASS(BlueprintType)
class REMOTECONTROL_API URCVirtualPropertyBase : public UObject
{
	GENERATED_BODY()

	friend struct FRCVirtualPropertyCastHelpers;
	friend class URCVirtualPropertySelfContainer;

protected:
	/** Return pointer to memory const container */
	virtual const uint8* GetContainerPtr() const;

	/** Return pointer to memory container */
	virtual uint8* GetContainerPtr();

	/** Return pointer to const value */
	virtual const uint8* GetValuePtr() const;

	/** Return pointer to value */
	virtual uint8* GetValuePtr();

	/** Return Pointer to Bag Property Description */
	virtual const FPropertyBagPropertyDesc* GetBagPropertyDesc() const;

	/** Pointer to Instanced property which is holds  bag of properties */
	virtual const FInstancedPropertyBag* GetPropertyBagInstance() const;

	/** Notifies pre change for Virtual Property value*/
	virtual void OnPreChangePropertyValue() {}

	/** Notifies post change for Virtual Property value*/
	virtual void OnModifyPropertyValue() {}

public:
	/** Initialization routine. Called after the parent container has setup data for this property */
	virtual void Init() {}

	virtual void InitPropertyBagPropertyDescription(FPropertyBagPropertyDesc& InOutPropertyDesc) {}

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) {}

	/** Returns const FProperty for this RC virtual property */
	virtual const FProperty* GetProperty() const;

	/** Returns FProperty for this RC virtual property */
	virtual FProperty* GetProperty();
	
	/** Return property bag property type */
	EPropertyBagPropertyType GetValueType() const;

	/**
	 * Get the meta-data matching the given key.
	 * NOTE: This meta-data is available at runtime and is different from the property bag property's meta-data
	 * @return the value mapped to the key
	 */
	FString GetMetadataValue(FName Key) const;

	/**
	 * Adds/sets the meta-data with the given key and value.
	 * NOTE: This meta-data is available at runtime and is different from the property bag property's meta-data
	 */
	void SetMetadataValue(FName Key, FString Data);

	/**
	 * Removes the meta-data matching the given key.
	 * NOTE: This meta-data is available at runtime and is different from the property bag property's meta-data
	 */
	void RemoveMetadataValue(FName Key);

	/** Return pointer to object that defines the Enum, Struct, or Class. */
	const UObject* GetValueTypeObjectWeakPtr() const;

	/**
	 * Serialize Virtual Property to given Backend
	 *
	 * @param OutBackend Struct Serialize Backend 
	 */
	void SerializeToBackend(IStructSerializerBackend& OutBackend);

	/**
	 * Deserialize Virtual Property from a given Backend
	 *
	 * @param InBackend  - Deserializer containing a payload with value data
	 */
	bool DeserializeFromBackend(IStructDeserializerBackend& InBackend);

	/** Whether this Virtual Property is a numeric type, i.e. an Integral or Floating point type */
	bool IsNumericType() const;

	/** Whether this Virtual Property represents an FVector */
	bool IsVectorType() const;

	/** Whether this Virtual Property represents an FVector2D */
	bool IsVector2DType() const;

	/** Whether this Virtual Property represents an FColor */
	bool IsColorType() const;

	/** Whether this Virtual Property represents an FColor */
	bool IsLinearColorType() const;

	/** Whether this Virtual Property represents an FRotator */
	bool IsRotatorType() const;

	/** Compare this virtual property value with given property value */
	bool IsValueEqual(URCVirtualPropertyBase* InVirtualProperty) const;

	/** Operator > comparison of self with a given virtual property */
	bool IsValueGreaterThan(URCVirtualPropertyBase* InVirtualProperty) const;

	/** Operator >= comparison of self with a given virtual property */
	bool IsValueGreaterThanOrEqualTo(URCVirtualPropertyBase* InVirtualProperty) const;

	/** Operator < comparison of self with a given virtual property */
	bool IsValueLesserThan(URCVirtualPropertyBase* InVirtualProperty) const;

	/** Operator <= comparison of self with a given virtual property */
	bool IsValueLesserThanOrEqualTo(URCVirtualPropertyBase* InVirtualProperty) const;

	/** Copy this virtual property's data onto a given FProperty
	* 
	* @param InTargetProperty - The property onto which our value is to be copied
	* @param InTargetValuePtr - The memory location for the target property
	* 
	* @return true if copied successfully
	*/
	bool CopyCompleteValue(const FProperty* InTargetProperty, uint8* InTargetValuePtr);

	/** Copy this virtual property's data onto a given FProperty
	* 
	* @param InTargetProperty - The property onto which our value is to be copied
	* @param InTargetValuePtr - The memory location for the target property
	* @param bPassByteEnumPropertyComparison - Go to the copy if the Property and the Target property are Enum and Byte
	* 
	* @return true if copied successfully
	*/
	bool CopyCompleteValue(const FProperty* InTargetProperty, uint8* InTargetValuePtr, bool bPassByteEnumPropertyComparison);

	/** Copy this virtual property's data onto a given Virtual Property
	*
	* @param InVirtualProperty - The virtual property onto which our value is to be copied
	* 
	* @return true if copied successfully
	*/
	bool CopyCompleteValue(URCVirtualPropertyBase* InVirtualProperty);
	
	/** Get Bool value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueBool(bool& OutBoolValue) const;

	/** Get Int8 value from Virtual Property */
	UFUNCTION()
	bool GetValueInt8(int8& OutInt8) const;

	/** Get Byte value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueByte(uint8& OutByte) const;

	/** Get raw Enum value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueEnum(uint8& OutByte) const;

	/** Get Int16 value from Virtual Property */
	UFUNCTION()
	bool GetValueInt16(int16& OutInt16) const;

	/** Get Uint16 value from Virtual Property */
	UFUNCTION()
	bool GetValueUint16(uint16& OutUInt16) const;

	/** Get Int32 value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueInt32(int32& OutInt32) const;

	/** Get Uint32 value from Virtual Property */
	UFUNCTION()
	bool GetValueUInt32(uint32& OutUInt32) const;

	/** Get Int64 value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueInt64(int64& OuyInt64) const;

	/** Get Uint64 value from Virtual Property */
	UFUNCTION()
	bool GetValueUint64(uint64& OuyUInt64) const;

	/** Get Float value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueFloat(float& OutFloat) const;

	/** Get Double value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueDouble(double& OutDouble) const;

	/** Get String value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueString(FString& OutStringValue) const;

	/** Get Name value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueName(FName& OutNameValue) const;

	/** Get Text value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueText(FText& OutTextValue) const;

	/** Get Numeric value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueNumericInteger(int64& OutInt64Value) const;

	/** Get Vector value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueVector(FVector& OutVector) const;

	/** Get Vector2D value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueVector2D(FVector2D& OutVector2D) const;

	/** Get Rotator value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueRotator(FRotator& OutRotator) const;

	/** Get Color value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueColor(FColor& OutColor) const;

	/** Get LinearColor value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool GetValueLinearColor(FLinearColor& OutLinearColor) const;

	/** Get Object value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	UObject* GetValueObject() const;

	/** Infers correct type internally, fetches value from memory and returns the value as a string 
	* that can be immediately used for dispaly (without needing to create a generic readonly property widget)
	*/
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	FString GetDisplayValueAsString() const;

	/** Set Bool value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueBool(const bool InBoolValue);

	/** Set Int8 value from Virtual Property */
	UFUNCTION()
	bool SetValueInt8(const int8 InInt8);

	/** Set Byte value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueByte(const uint8 InByte);

	/** Set Int16 value from Virtual Property */
	UFUNCTION()
	bool SetValueInt16(const int16 InInt16);

	/** Set Uint16 value from Virtual Property */
	UFUNCTION()
	bool SetValueUint16(const uint16 InUInt16);

	/** Set Int32 value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueInt32(const int32 InInt32);

	/** Set Uint32 value from Virtual Property */
	UFUNCTION()
	bool SetValueUInt32(const uint32 InUInt32);

	/** Set Int64 value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueInt64(const int64 InInt64);

	/** Set Uint64 value from Virtual Property */
	UFUNCTION()
	bool SetValueUint64(const uint64 InUInt64);

	/** Set Float value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueFloat(const float InFloat);

	/** Set Double value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueDouble(const double InDouble);

	/** Set String value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueString(const FString& InStringValue);

	/** Set Name value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueName(const FName& InNameValue);

	/** Set Text value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueText(const FText& InTextValue);

	/** Set Numeric value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueNumericInteger(const int64 InInt64Value);

	/** Set Vector value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueVector(const FVector& InVector);

	/** Set Vector2D value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueVector2D(const FVector2D& InVector2D);

	/** Set Rotator value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueRotator(const FRotator& InRotator);

	/** Set Color value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueColor(const FColor& InColor);

	/** Set LinearColor value from Virtual Property */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Behaviour")
	bool SetValueLinearColor(const FLinearColor& InLinearColor);

	/** Get FProperty Name */
	UFUNCTION(BlueprintPure, Category = "Remote Control Behaviour")
	FName GetPropertyName() const;

	/** Fetches a user-friendly representation of the Virtual Property's type name (i.e. Controller type), given a Property Bag type and value object (for structures) */
	static FName GetVirtualPropertyTypeDisplayName(const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject);

	/** Updates the display index of the virtual property (transaction aware) */
	void SetDisplayIndex(const int32 InDisplayIndex);

	virtual void UpdateEntityUsage(const URemoteControlPreset* InPreset, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const {}

public:
	/** Unique property name */
	UPROPERTY()
	FName PropertyName;

	/** Property Id */
	UPROPERTY()
	FGuid Id;

	/** Property Field Id */
	UPROPERTY()
	FName FieldId;

	/** Pointer to Remote Control Preset */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> PresetWeakPtr;

	/** Unique user friendly name of the controller, used as identifier in some API. */
	UPROPERTY()
	FName DisplayName;

	/** Controller description */
	UPROPERTY()
	FText Description;

	/** User configurable Display Index for this Virtual Property (as Logic Controller) when represented as a row in the RC Logic Controllers list  */
	UPROPERTY()
	int32 DisplayIndex;

	UPROPERTY()
	TMap<FName, FString> Metadata;
};

/**
 * Remote Control Virtual Property which is stores in container with many properties
 * Where Property Bag has more then one Property 
 */
UCLASS()
class REMOTECONTROL_API URCVirtualPropertyInContainer : public URCVirtualPropertyBase
{
	GENERATED_BODY()

protected:
	//~ Begin URCVirtualPropertyBase interface
	virtual const FInstancedPropertyBag* GetPropertyBagInstance() const override;
	//~ End URCVirtualPropertyBase interface

public:
	/** Pointer to container where stores Virtual Properties */
	UPROPERTY()
	TWeakObjectPtr<URCVirtualPropertyContainerBase> ContainerWeakPtr;
};

/**
 * Remote Control Virtual Property which is stores in self defined Property Bag
 * In this case SelfContainer holds only single property in the Property Bag
 */
UCLASS()
class REMOTECONTROL_API URCVirtualPropertySelfContainer : public URCVirtualPropertyBase
{
	GENERATED_BODY()

public:
	/**
	 * Adds a new property to the bag. If property with this name already in bag the function just return
	 *
	 * @param InPropertyName				Name of the property
	 * @param InValueType					Property Type
	 * @param InValueTypeObject				Property Type object if exists
	 * 
	 */
	void AddProperty(const FName InPropertyName, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject = nullptr);

	/**
	 * Duplicates property from give Property. If property with this name already in bag the function just return
	 *
	 * @param InPropertyName				Name of the property
	 * @param InSourceProperty				Source FProperty for duplication
	 *
	 * @return true if property duplicated successfully 
	 */
	bool DuplicateProperty(const FName& InPropertyName, const FProperty* InSourceProperty);

	/**
	 * Duplicates property from give Property and copy the value. If property with this name already in bag the function just return
	 *
	 * @param InPropertyName				Name of the property
	 * @param InSourceProperty				Source FProperty for duplication
	 * @param InSourceContainerPtr			Pointer to source container
	 *
	 * @return true if property duplicated and value copied successfully 
	 */
	bool DuplicatePropertyWithCopy(const FName& InPropertyName, const FProperty* InSourceProperty, const uint8* InSourceContainerPtr);

	/**
	 * Duplicates property from given Virtual Property. If property with this name already in bag the function just return
	 *
	 * @param InVirtualProperty				Virtual Property to duplicate from
	 *
	 * @return true if property duplicated and value copied successfully 
	 */
	bool DuplicatePropertyWithCopy(URCVirtualPropertyBase* InVirtualProperty);

	/**
 	 * Update property bag value using the property and the container passed.
 	 *
 	 * @param InProperty Property used to update the bag property value
 	 * @param InPropertyContainer Container to take the value using the property
 	 *
 	 * @return true if SetValue of the bag returned Success otherwise false
 	 */
	bool UpdateValueWithProperty(const FProperty* InProperty, const void* InPropertyContainer);

	/**
  	* Update property bag value using URCVirtualPropertyBase passed.
  	*
  	* @param InVirtualProperty Virtual Property to update from
  	*
  	* @return true if value copied successfully
  	*/
	bool UpdateValueWithProperty(const URCVirtualPropertyBase* InVirtualProperty);

	/** Resets the property bag instance to empty and remove Virtual Property data */
	void Reset();

	/** Creates new Struct on Scope for this Property Bag UStruct and Memory */
	TSharedPtr<FStructOnScope> CreateStructOnScope();

protected:
	//~ Begin URCVirtualPropertyBase interface
	virtual const FInstancedPropertyBag* GetPropertyBagInstance() const override;
	//~ End URCVirtualPropertyBase interface

private:
	/** Instanced property bag for store a bag of properties. */
	UPROPERTY()
	FInstancedPropertyBag Bag;
};
