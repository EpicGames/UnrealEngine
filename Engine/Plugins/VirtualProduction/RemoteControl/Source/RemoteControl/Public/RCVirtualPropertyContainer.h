// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Internationalization/Text.h"
#include "Misc/Guid.h"
#include "StructUtils/PropertyBag.h"

#include "RCVirtualPropertyContainer.generated.h"

class URCVirtualPropertyInContainer;
class URCVirtualPropertyBase;
class URemoteControlPreset;
class FStructOnScope;

/** 
 * Describes a category into which properties can be placed.
 */
USTRUCT()
struct FRCVirtualPropertyCategory
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid Id = FGuid();
	
	UPROPERTY()
	FText Name = FText::GetEmpty();

	UPROPERTY()
	int32 DisplayIndex = INDEX_NONE;
};

/**
 * Container for more then one Virtual Property
 */
UCLASS(BlueprintType)
class REMOTECONTROL_API URCVirtualPropertyContainerBase : public UObject
{
	GENERATED_BODY()

	friend class URCVirtualPropertyInContainer;
	
public:
	/**
	 * Add property to this container.
	 * That will add property to Property Bag and Create Remote Control Virtual Property
	 *
	 * @param InPropertyName				Property Name to add
	 * @param InPropertyClass				Class of the Virtual Property 
	 * @param InValueType					Property Type
	 * @param InValueTypeObject				Property Type object if exists
	 * @param MetaData                      Property Metadata used by Slate widgets (eg: Delta, LinearDeltaSensitivity, etc)
	 *
	 * @return Virtual Property Object
	 */
	virtual URCVirtualPropertyInContainer* AddProperty(const FName& InPropertyName, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject = nullptr, TArray<FPropertyBagPropertyDescMetaData> MetaData = TArray<FPropertyBagPropertyDescMetaData>());

	/**
	 * Duplicates a property from given FProperty.
	 * That will add property to Property Bag and Create Remote Control Virtual Property
	 *
	 * @param InPropertyName				Name of the property
	 * @param InSourceProperty				Source FProperty for duplication
	 * @param InPropertyClass				Class of the Virtual Property 
	 *
	 * @return Virtual Property Object
	 */
	virtual URCVirtualPropertyInContainer* DuplicateProperty(const FName& InPropertyName, const FProperty* InSourceProperty, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass);

	/**
	 * Duplicates a property from a given FProperty and copies the property value
	 * That will add property to Property Bag and Create Remote Control Virtual Property
	 *
	 * @param InPropertyName				Name of the property
	 * @param InSourceProperty				Source FProperty for duplication
	 * @param InPropertyClass				Class of the Virtual Property
	 * @param InSourceContainerPtr			Pointer to source container
	 *
	 * @return Virtual Property Object
	 */
	virtual URCVirtualPropertyInContainer* DuplicatePropertyWithCopy(const FName& InPropertyName, const FProperty* InSourceProperty, const uint8* InSourceContainerPtr, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass);

	/**
	 * Given a virtual property this funciton duplicates it (via DuplicateObject)
	 * and syncs the internal Controller Container and PropertyBag containers as well.
	 *
	 * @param InVirtualProperty				Virtual Property to be duplicated
	 *
	 * @return Virtual Property Object
	 */
	virtual URCVirtualPropertyInContainer* DuplicateVirtualProperty(URCVirtualPropertyInContainer* InVirtualProperty);

	/**
	 * Returns whether this container contains the given property.
	 */
	bool HasVirtualProperty(const URCVirtualPropertyBase* InVirtualProperty) const;

	/**
	 * Removes a property from the container by name if it exists.
	 */
	virtual bool RemoveProperty(const FName& InPropertyName);

	/** Resets the property bag instance to empty and remove Virtual Properties */
	virtual void Reset();

	/**
	 * Returns virtual property by specified name.
	 */ 
	virtual URCVirtualPropertyBase* GetVirtualProperty(const FName InPropertyName) const;

	/**
	 * Returns virtual property by unique Id.
	 */
	URCVirtualPropertyBase* GetVirtualProperty(const FGuid& InId) const;

	/**
	 * Returns virtual property by user-friendly display name (Controller Name)
	 */
	virtual URCVirtualPropertyBase* GetVirtualPropertyByDisplayName(const FName InDisplayName) const;

	/**
	 * Returns the first found virtual property matching the specified Field Id
	 */
	virtual URCVirtualPropertyBase* GetVirtualPropertyByFieldId(const FName InFieldId) const;
	
	/**
	 * Returns the first virtual property matching the specified Field Id and ValueType
	 */
	virtual URCVirtualPropertyBase* GetVirtualPropertyByFieldId(const FName InFieldId, const EPropertyBagPropertyType InType) const;

	/**
	 * Returns all virtual properties with the specified Field Id
	 */
	virtual TArray<URCVirtualPropertyBase*> GetVirtualPropertiesByFieldId(const FName InFieldId) const;

	/**
	 * Returns number of virtual properties.
	 */ 
	int32 GetNumVirtualProperties() const;

	/**
	 * Returns the Property Bag Struct of this Container
	 */
	const UPropertyBag* GetPropertyBagStruct() const;

	/**
	 * Creates new Struct View for this Property Bag UStruct and Memory
	 */
	FStructView GetPropertyBagMutableValue();

	/**
	 * Creates new Struct on Scope for this Property Bag UStruct and Memory
	 */
	TSharedPtr<FStructOnScope> CreateStructOnScope();

	/**
	 * Rename the controller with the given InGuid with the InNewName passed as candidate and return the new name
	 */
	FName SetControllerDisplayName(FGuid InGuid, FName InNewName);

	/**
	 * Generates unique name for the property for specified property container
	 */
	static FName GenerateUniquePropertyName(const FName& InPropertyName, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject, const URCVirtualPropertyContainerBase* InContainer);

	/**
	* Generates unique name for the property for specified property container
	*/
	static FName GenerateUniquePropertyName(const FName& InPropertyName, const URCVirtualPropertyContainerBase* InContainer);

	/**
	 * Generates unique display name for the controllers
	 */
	static FName GenerateUniqueDisplayName(const FName& InPropertyName, const URCVirtualPropertyContainerBase* InContainer);

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap);

	/** Cache this preset's controllers labels. */
	void CacheControllersLabels();

	/** Fix controllers labels for older presets. */
	void FixAndCacheControllersLabels();

#if WITH_EDITOR
	/** Called after applying a transaction to the object. Used to broadcast Undo related container changes to UI. */
	virtual void PostEditUndo();

	/** Delegate when the value is being scrubbed in UI*/
	virtual void OnPreChangePropertyValue(const FPropertyChangedEvent& PropertyChangedEvent) {}

	/** Delegate when object changed */
	virtual void OnModifyPropertyValue(const FPropertyChangedEvent& PropertyChangedEvent);
#endif

	/** Returns the delegate that notifies changes to the virtual property container */
	DECLARE_MULTICAST_DELEGATE(FOnVirtualPropertyContainerModified);
	FOnVirtualPropertyContainerModified& OnVirtualPropertyContainerModified() { return OnVirtualPropertyContainerModifiedDelegate; }

private:
	void AddVirtualProperty(URCVirtualPropertyBase* InVirtualProperty);

protected:
	/** Holds bag of properties. */
	UPROPERTY()
	FInstancedPropertyBag Bag;

public:
	/** Set of the virtual properties */
	UPROPERTY()
	TSet<TObjectPtr<URCVirtualPropertyBase>> VirtualProperties;
	
	/** Pointer to Remote Control Preset */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> PresetWeakPtr;

	/** Delegate that notifies changes to the virtual property container*/
	FOnVirtualPropertyContainerModified OnVirtualPropertyContainerModifiedDelegate;

	UPROPERTY()
	TArray<FRCVirtualPropertyCategory> Categories;

private:
	/** Map of Controller Name to GUID. */
	UPROPERTY(Transient)
	TMap<FName, FGuid> ControllerLabelToIdCache;
};


