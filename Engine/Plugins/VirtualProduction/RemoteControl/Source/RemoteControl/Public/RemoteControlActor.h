// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "RemoteControlEntity.h"

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "RemoteControlBinding.h"
#include "RemoteControlPreset.h"
#include "UObject/SoftObjectPath.h"

#include "RemoteControlActor.generated.h"

class URemoteControlPreset;

/**
 * Represents an actor exposed in the panel.
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlActor : public FRemoteControlEntity
{
	GENERATED_BODY()

	FRemoteControlActor() = default;

	FRemoteControlActor(URemoteControlPreset* InOwner, FName InLabel, const TArray<URemoteControlBinding*>& InBindings)
		: FRemoteControlEntity(InOwner, InLabel, InBindings)
	{
		if (InBindings.Num())
		{
			Path = InBindings[0]->Resolve();
		}
	}
	
	//~	Begin RemoteControlEntityInterface 
	virtual void BindObject(UObject* InObjectToBind) override
	{
		if (InObjectToBind && InObjectToBind->IsA<AActor>())
		{
			FRemoteControlEntity::BindObject(InObjectToBind);
		}
	}

	virtual uint32 GetUnderlyingEntityIdentifier() const override
	{
		return GetTypeHash(Path);
	}
	
    virtual UClass* GetSupportedBindingClass() const override { return AActor::StaticClass(); }
	//~ End RemoteControlEntityInterface
	
	AActor* GetActor() const
	{
		if (Bindings.Num() && Bindings[0].IsValid())
		{
			return Cast<AActor>(Bindings[0]->Resolve());
		}
		return nullptr;
	}

	void SetActor(AActor* InActor)
	{
		BindObject(InActor);
	}

public:
	/**
	 * Path to the exposed object.
	 */
	UPROPERTY(BlueprintReadOnly, Category = "RemoteControlEntity")
	FSoftObjectPath Path;
};