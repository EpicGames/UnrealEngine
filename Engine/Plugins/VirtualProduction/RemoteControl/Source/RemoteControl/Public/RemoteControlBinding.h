// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"

#include "Components/ActorComponent.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "RemoteControlBinding.generated.h"

namespace UE::RemoteControlBinding
{
	/**
	 * @return whether an object is a valid rebinding target.
	 */
	bool REMOTECONTROL_API IsValidObjectForRebinding(UObject* InObject, UWorld* PresetWorld);
	/**
	 * @return whether an actor is a valid rebinding target.
	 */
	bool REMOTECONTROL_API IsValidActorForRebinding(AActor* InActor, UWorld* PresetWorld);
	/**
	 * @return whether a subobject is a valid rebinding target.
	 */
	bool REMOTECONTROL_API IsValidSubObjectForRebinding(UObject* InComponent, UWorld* PresetWorld);
}

/**
 * Acts as a bridge between an exposed property/function and an object to act on.
 */
UCLASS(Abstract, BlueprintType)
class REMOTECONTROL_API URemoteControlBinding : public UObject
{
public:
	GENERATED_BODY()

	/**
	 * Set the object this binding should represent.
	 */
	virtual void SetBoundObject(const TSoftObjectPtr<UObject>& InObject) PURE_VIRTUAL(URemoteControlBinding::SetBoundObject,);

	/**
	 * Unset the underlying object this binding currently represents.
	 */
	virtual void UnbindObject(const TSoftObjectPtr<UObject>& InBoundObject) PURE_VIRTUAL(URemoteControlBinding::UnbindObject,);

	/**
	 * Resolve the bound object for the current map.
	 * @Note Will return the PIE object when in a PIE session.
	 */
	virtual UObject* Resolve() const PURE_VIRTUAL(URemoteControlBinding::Resolve, return nullptr;);

	/**
	 * Return the Entity Object in the given world
	 * @param InWorld World where to retrieve the object
	 * @return The object in the given world or nullptr if not found
	 */
	virtual UObject* ResolveForWorld(const UWorld* InWorld) const PURE_VIRTUAL(URemoteControlBinding::ResolveForWorld, return nullptr;);

	/**
	 * Whether this binding represents a valid object.
	 */
	virtual bool IsValid() const PURE_VIRTUAL(URemoteControlBinding::IsValid, return false;);

	/**
	 * Check if the object is bound.
	 */
	virtual bool IsBound(const TSoftObjectPtr<UObject>& Object) const PURE_VIRTUAL(URemoteControlBinding::IsBound, return false;);

	/**
	 * Remove objects that were deleted from the binding.
	 * @return true if at least one object was removed from this binding's objects.
	 */
	virtual bool PruneDeletedObjects() PURE_VIRTUAL(URemoteControlBinding::PruneDeletedObjects, return false;);

	/**
	 *  Get the last object this binding was pointing to.
	 */
	FSoftObjectPath GetLastBoundObjectPath() const;

public:
	/**
	 * The name of this binding. Defaults to the bound object's name.
	 */
	UPROPERTY(EditAnywhere, Category=Default)
	FString Name;

protected:
	/**
	 * The path to the last object that was bound.
	 */
	UPROPERTY()
	mutable FSoftObjectPath LastBoundObjectPath;
};

UCLASS(BlueprintType)
class REMOTECONTROL_API URemoteControlLevelIndependantBinding : public URemoteControlBinding
{
public:
	GENERATED_BODY()

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin URemoteControlBinding Interface
	virtual void SetBoundObject(const TSoftObjectPtr<UObject>& InObject) override;
	virtual void UnbindObject(const TSoftObjectPtr<UObject>& InBoundObject) override;
	virtual UObject* Resolve() const override;
	virtual UObject* ResolveForWorld(const UWorld* InWorld) const override;
	virtual bool IsValid() const override;
	virtual bool IsBound(const TSoftObjectPtr<UObject>& Object) const override;
	virtual bool PruneDeletedObjects() override;
	//~ End URemoteControlBinding interface


private:
	/**
	 * Holds the bound object.
	 */
	UPROPERTY()
	TSoftObjectPtr<UObject> BoundObject;
};

USTRUCT()
struct FRemoteControlInitialBindingContext
{
	GENERATED_BODY()

	/*
	 * The class that's supported by this binding.
	 */
	UPROPERTY()
	TSoftClassPtr<UObject> SupportedClass;

	/**
	 *  Name of the component if any.
	 */
	UPROPERTY()
	FName ComponentName;

	/**
	 * Path to the subobject if any.
	 */
	UPROPERTY()
	FString SubObjectPath;

	/**
	 * The class of the actor that's targeted by this binding or the class of the actor that owns the component that is targeted.
	 */
	UPROPERTY()
	TSoftClassPtr<AActor> OwnerActorClass;

	/**
	 * Name of the initial actor that was targetted by this binding.
	 */
	UPROPERTY()
	FName OwnerActorName;

	bool IsEmpty() const
	{
		return SupportedClass.GetUniqueID().ToString().IsEmpty()
			&& ComponentName.IsNone() 
			&& OwnerActorClass.GetUniqueID().ToString().IsEmpty()
			&& OwnerActorName.IsNone();
	}

	friend bool operator==(const FRemoteControlInitialBindingContext& LHS, const FRemoteControlInitialBindingContext& RHS)
	{
		return LHS.SupportedClass == RHS.SupportedClass
			&& LHS.ComponentName == RHS.ComponentName
			&& LHS.OwnerActorClass == RHS.OwnerActorClass
			&& LHS.OwnerActorName == RHS.OwnerActorName
			&& LHS.SubObjectPath == RHS.SubObjectPath;
	}

	friend bool operator!=(const FRemoteControlInitialBindingContext& LHS, const FRemoteControlInitialBindingContext& RHS)
	{
		return !(LHS == RHS);
	}

	bool HasValidComponentName() const
	{
		return !ComponentName.IsNone();
	}

	bool HasValidSubObjectPath() const
	{
		return !SubObjectPath.IsEmpty();
	}
};

UCLASS(BlueprintType)
class REMOTECONTROL_API URemoteControlLevelDependantBinding : public URemoteControlBinding
{
public:
	GENERATED_BODY()

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface
	
	//~ Begin URemoteControlBinding Interface
	virtual void SetBoundObject(const TSoftObjectPtr<UObject>& BoundObject) override;
	virtual void UnbindObject(const TSoftObjectPtr<UObject>& BoundObject) override;
	virtual UObject* Resolve() const override;
	virtual UObject* ResolveForWorld(const UWorld* InWorld) const override;
	virtual bool IsValid() const override;
	virtual bool IsBound(const TSoftObjectPtr<UObject>& Object) const override;
	virtual bool PruneDeletedObjects() override;
	//~ Begin URemoteControlBinding Interface

	/**
	 *	Set the bound object by specifying the level it resides in.
	 *	@Note Useful is you want to set the bound object without loading the level/object.
	 */
	void SetBoundObject(const TSoftObjectPtr<ULevel>& Level, const TSoftObjectPtr<UObject>& BoundObject);

	/**
	 * Set the bound object and reinitialize the context used for rebinding purposes.
	 */
	void SetBoundObject_OverrideContext(const TSoftObjectPtr<UObject>& InObject);

	/**
	 * Initialize this binding to be used with the new current level.
	 * Copies the binding from the last successful resolve in case the level was duplicated.
	 */
	void InitializeForNewLevel();

	/**
	 * Get the suppported class of the bound object or of the owner of the bound object in the case of a bound component.
	 */
	UClass* GetSupportedOwnerClass() const;

	/** Get the current world associated with the preset or the editor world as a fallback. */
	UWorld* GetCurrentWorld(bool bAllowPIE = false) const;

private:
	/** Attempt resolving the binding for the current world. */
	TSoftObjectPtr<UObject> ResolveForCurrentWorld(bool bAllowPIE = false) const;

	/** Update this binding's context with the object passed as argument. */
	void UpdateBindingContext(UObject* InObject) const;


private:
#if WITH_EDITORONLY_DATA
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "BoundObjectMap is deprecated, please use BoundObjectMapByPath instead."))
	TMap<TSoftObjectPtr<ULevel>, TSoftObjectPtr<UObject>> BoundObjectMap_DEPRECATED;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "SubLevelSelectionMap is deprecated, please use SubLevelSelectionMapByPath instead."))
	mutable TMap<TSoftObjectPtr<UWorld>, TSoftObjectPtr<ULevel>> SubLevelSelectionMap_DEPRECATED;
#endif

	/**
	 *	The map bound objects with their level as key.
	 */
	UPROPERTY()
	TMap<FSoftObjectPath, TSoftObjectPtr<UObject>> BoundObjectMapByPath;

	/**
	 * Keeps track of which sublevel was last used when binding in a particular world.
	 * Used in the case where a binding points to objects that end up in the same world but in different sublevels,
	 * this ensures that we know which object was last
	 */
	UPROPERTY()
	mutable TMap<FSoftObjectPath, TSoftObjectPtr<ULevel>> SubLevelSelectionMapByPath;

	/**
	 * Caches the last level that had a successful resolve.
	 * Used to decide which level to use when reinitializing this binding in a new level.
	 */
	UPROPERTY()
	mutable TSoftObjectPtr<ULevel> LevelWithLastSuccessfulResolve;

	UPROPERTY()
	mutable FRemoteControlInitialBindingContext BindingContext;

	friend class FRemoteControlPresetRebindingManager;
	friend class URemoteControlPreset;
};
