// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

#include "RemoteControlEntity.generated.h"

class URemoteControlBinding;
class URemoteControlPreset;

/**
 * Base class for exposed objects, properties, functions etc...
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlEntity
{
	GENERATED_BODY()

	FRemoteControlEntity() = default;

	virtual ~FRemoteControlEntity(){}

	/** Get the type of this remote control entity. */
	const UScriptStruct* GetStruct() const;

	/**
	 * Change this entity's label.
	 * @param NewLabel the desired label.
	 * @return The assigned label.
	 * @note The returned label can be different from the desired one if the label was already tkaen in the preset.
	 */
	FName Rename(FName NewLabel);

	/**
	 * Get the label of this entity.
	 */
	FName GetLabel() const { return Label; }

	/**
	 * Get the id of this entity.
	 */
	FGuid GetId() const { return Id; }

	/**
	 * Get the preset that owns this entity.
	 */
	URemoteControlPreset* GetOwner() { return Owner.Get(); }

	/**
	 * Get all resolved bindings under this entity.
	 */
	TArray<UObject*> GetBoundObjects() const;

	/** 
	 * Resolve the first binding for this entity. 
	 */
	UObject* GetBoundObject() const;

	/**
	 * Return the Entity Object in the given world
	 * @param InWorld World where to retrieve the object
	 * @return The object in the given world or nullptr if not found
	 */
	UObject* GetBoundObjectForWorld(const UWorld* InWorld) const;

	/**
	 * Get the unresolved bindings of this exposed entity.
	 */
	const TArray<TWeakObjectPtr<URemoteControlBinding>>& GetBindings() const;

	/**
	 * Get the entity's metadata.
	 */
	const TMap<FName, FString>& GetMetadata() const;

	/**
	 * Remove a metadata entry.
	 */
	void RemoveMetadataEntry(FName Key);

	/**
	 * Find or add a value in the entity's metadata map.
	 */
	void SetMetadataValue(FName Key, FString Value);

	/**
	 * Change the object this exposed entity is currently pointing to.
	 * @Note You should call CanBindObject before calling this method.
	 */
	virtual void BindObject(UObject* InObjectToBind);

	/**
	 * Return whether this entity supports binding to a given object.
	 */
	virtual bool CanBindObject(const UObject* InObjectToBind) const { return true; }

	/**
	 * Returns whether this entity can be resolved using its bound objects.
	 */
	virtual bool IsBound() const;

	/**
	 * Get an identifier for the underlying entity.
	 * This is used to figure out if two remote control entities have the same underlying field/actor. 
	 */
	virtual uint32 GetUnderlyingEntityIdentifier() const
	{
		ensureAlwaysMsgf(false, TEXT("GetUnderlyingEntityIdentifier must be implemented in derived structs."));
		return 0;
	}

	/**
	 *  Get the class of the object that can hold this field.
	 */
	virtual UClass* GetSupportedBindingClass() const { return UObject::StaticClass(); }

	/**
	 * Get the last valid binding path for this entity.
	 */
	FSoftObjectPath GetLastBindingPath() const;

	bool operator==(const FRemoteControlEntity& InEntity) const;
	bool operator==(FGuid InEntityId) const;
	friend uint32 REMOTECONTROL_API GetTypeHash(const FRemoteControlEntity& InEntity);

protected:
	/**
	 * User specified metadata for this entity.
	 */
	UPROPERTY()
	TMap<FName, FString> UserMetadata;

	/**
	 * The bound objects that are exposed or that hold the exposed entity.
	 */
	UPROPERTY()
	TArray<TWeakObjectPtr<URemoteControlBinding>> Bindings;

protected:
	FRemoteControlEntity(URemoteControlPreset* InPreset, FName InLabel, const TArray<URemoteControlBinding*>& InBindings);
	
	/** The preset that owns this entity. */
	UPROPERTY(BlueprintReadOnly, Category = "RemoteControlEntity")
	TWeakObjectPtr<URemoteControlPreset> Owner;
	
protected:
	/**
	 * This exposed entity's alias.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "RemoteControlEntity")
	FName Label;

	/**
	 * Unique identifier for this entity
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "RemoteControlEntity")
	FGuid Id;

	/**
	 * Delegate called when the label of this entity changes or if one of its binding is updated.
	 */
	DECLARE_DELEGATE_OneParam(FOnEntityModified, const FGuid&);
	FOnEntityModified OnEntityModifiedDelegate;
	
	friend class URemoteControlExposeRegistry;
	friend class URemoteControlPreset;
};