// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class URemoteControlPreset;
struct FRCFieldPathInfo;
struct FRemoteControlPresetExposeArgs;
struct FRemoteControlProperty;

/**
 * Factory which is creates the instances of FRemoteControlProperty
 */
class IRemoteControlPropertyFactory : public TSharedFromThis<IRemoteControlPropertyFactory>
{
public:
	/** Virtual destructor */
	virtual ~IRemoteControlPropertyFactory(){}

	/**
	 * Expose a property from given preset
	 * @param Preset the remote control preset object.
	 * @param Object the object that holds the property.
	 * @param FieldPath The name/path to the property.
	 * @param Args Optional arguments used to expose the property.
	 * @return The exposed property.
	 */
	virtual TSharedPtr<FRemoteControlProperty> CreateRemoteControlProperty(URemoteControlPreset* Preset, UObject* Object, FRCFieldPathInfo FieldPath, FRemoteControlPresetExposeArgs Args) = 0;

	/**
	 * Post set property action
	 * @param Object the object that holds the property.
	 * @param bInSuccess Whether the property was set.
	 */
	virtual void PostSetObjectProperties(UObject* Object, bool bInSuccess) const {}

	/**
	 * Whether the factory support exposed object
	 * @param Class class to expose
	 * @return true if the object is supported by given factory
	 */
	virtual bool SupportExposedClass(UClass* Class) const = 0;
};

