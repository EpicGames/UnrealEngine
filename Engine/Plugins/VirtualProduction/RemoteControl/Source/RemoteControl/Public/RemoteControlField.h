// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "RemoteControlCommon.h"
#include "RemoteControlEntity.h"
#include "RemoteControlFieldPath.h"
#include "RemoteControlProtocolBinding.h"
#include "RemoteControlField.generated.h"

class IRemoteControlPropertyHandle;
struct FPropertyChangedEvent;

/**
 * The type of the exposed field.
 */
UENUM()
enum class EExposedFieldType : uint8
{
	Invalid,
	Property,
	Function
};

/**
 * Represents a property or function that has been exposed to remote control.
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlField : public FRemoteControlEntity
{
	GENERATED_BODY()

	FRemoteControlField() = default;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FRemoteControlField(const FRemoteControlField&) = default;
	FRemoteControlField(FRemoteControlField&&) = default;
	FRemoteControlField& operator=(const FRemoteControlField&) = default;
	FRemoteControlField& operator=(FRemoteControlField&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Resolve the field's owners using the section's top level objects.
	 * @param SectionObjects The top level objects of the section.
	 * @return The list of UObjects that own the exposed field.
	 */
	UE_DEPRECATED(4.27, "Please use GetBoundObjects.")
	TArray<UObject*> ResolveFieldOwners(const TArray<UObject*>& SectionObjects) const;

	//~ Begin FRemoteControlEntity interface
	virtual void BindObject(UObject* InObjectToBind) override;
	virtual bool CanBindObject(const UObject* InObjectToBind) const override;
	//~ End FRemoteControlEntity interface

	/** Disables the given mask. */
	UE_DEPRECATED(5.6, "ClearMask is deprecated. Use per-protocol-binding masking instead.")
	virtual void ClearMask(ERCMask InMaskBit);

	/** Enables the given mask. */
	UE_DEPRECATED(5.6, "EnableMask is deprecated. Use per-protocol-binding masking instead.")
	virtual void EnableMask(ERCMask InMaskBit);

	/** Returns true if the given mask is enabled, false otherwise. */
	UE_DEPRECATED(5.6, "HasMask is deprecated. Use per-protocol-binding masking instead.")
	virtual bool HasMask(ERCMask InMaskBit) const;

	/** Returns true if this field supports masking. */
	UE_DEPRECATED(5.6, "SupportsMasking is deprecated. Use per-protocol-binding masking instead.")
	virtual bool SupportsMasking() const { return false; }

	/** Retrieves the enabled masks. */
	UE_DEPRECATED(5.6, "GetActiveMasks is deprecated. Use per-protocol-binding masking instead.")
	ERCMask GetActiveMasks() const;

	/**	Returns whether the field is only available in editor. */
	bool IsEditorOnly() const { return bIsEditorOnly; }

public:
	/**
	 * The field's type.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "RemoteControlEntity")
	EExposedFieldType FieldType = EExposedFieldType::Invalid;

	/**
	 * The exposed field's name.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "RemoteControlEntity")
	FName FieldName;

	/**
	 * The exposed field's identifier.
	 */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "RemoteControlEntity")
	FName PropertyId;

	/**
	 * Path information pointing to this field
	 */
	UPROPERTY()
	FRCFieldPathInfo FieldPathInfo;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FString> ComponentHierarchy_DEPRECATED;
#endif

	/**
	 * Stores the bound protocols for this exposed field
	 * It could store any types of the implemented protocols such as DMX, OSC, MIDI, etc
	 * The map holds protocol bindings stores the protocol mapping and protocol-specific mapping
	 */
	UPROPERTY()
	TSet<FRemoteControlProtocolBinding> ProtocolBindings;
	
protected:
	/**
	 * The class of the object that can have this field.
	 */
	UPROPERTY()
	FSoftClassPath OwnerClass;

	/** Whether the field is only available in editor. */
	UPROPERTY()
	bool bIsEditorOnly = false;

	/** Holds the actively enabled masks. */
	UE_DEPRECATED(5.6, "ActiveMasks is deprecated. Use per-protocol-binding masking instead.")
	UPROPERTY()
	uint8 ActiveMasks_DEPRECATED = (uint8)RC_AllMasks;

protected:
	FRemoteControlField(URemoteControlPreset* InPreset, EExposedFieldType InType, FName InLabel, FRCFieldPathInfo InFieldPathInfo, const TArray<URemoteControlBinding*> InBindings);
	void PostSerialize(const FArchive& Ar);
	
private:
#if WITH_EDITORONLY_DATA
	/**
	 * Resolve the field's owners using the section's top level objects and the deprecated component hierarchy.
	 * @param SectionObjects The top level objects of the section.
	 * @return The list of UObjects that own the exposed field.
	 */
	TArray<UObject*> ResolveFieldOwnersUsingComponentHierarchy(const TArray<UObject*>& SectionObjects) const;
#endif
};

/**
 * Represents a property exposed to remote control.
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlProperty : public FRemoteControlField
{
public:
	GENERATED_BODY()

	FRemoteControlProperty() = default;

	UE_DEPRECATED(4.27, "This constructor is deprecated. Use the other constructor.")
	FRemoteControlProperty(FName InLabel, FRCFieldPathInfo FieldPathInfo, TArray<FString> InComponentHierarchy);

	FRemoteControlProperty(URemoteControlPreset* InPreset, FName InLabel, FRCFieldPathInfo InFieldPathInfo, const TArray<URemoteControlBinding*>& InBindings);

	//~ Begin FRemoteControlEntity interface
	virtual uint32 GetUnderlyingEntityIdentifier() const override;
	virtual UClass* GetSupportedBindingClass() const override;
	virtual bool IsBound() const override;
	//~ End FRemoteControlEntity interface
	
	//~ Begin FRemoteControlField interface
	UE_DEPRECATED(5.6, "SupportsMasking is deprecated. Use per-protocol-binding masking instead.")
	virtual bool SupportsMasking() const override;
	//~ End FRemoteControlField interface

	/**
	 * Check whether given property path is bound to the property 
	 * @param InPropertyPath Given Property Path
	 * return true if the property bound to given path
	 */
	virtual bool CheckIsBoundToPropertyPath(const FString& InPropertyPath) const;

	/**
	 * Check whether given property string is bound to the property
	 * @param InPathString Given Property String, this is the path without duplicates
	 * return true if the property bound to given path
	 */
	virtual bool CheckIsBoundToString(const FString& InPath) const;

	/**
	 * Check whether any of the given objects are bound to this property
	 * @param InObjects Objects to check
	 * return true if at least one object is bound to the property
	 */
	virtual bool ContainsBoundObjects(TArray<UObject*> InObjects) const;

	/**
	 * Called on Post Load on Owner UObject.
	 * Useful to initialize specific functionality of the inherit property classes
	 */
	virtual void PostLoad() {}

	/**
	 * Should be called when property changed on bound object
	 * @param InObject Edited object
	 * @param InEvent  Change Event
	 */
	virtual void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent) {}

	/**
	 * Get the underlying property.
	 * @return The exposed property or nullptr if it couldn't be resolved.
	 * @note This field's binding must be valid to get the property.
	 */
	FProperty* GetProperty() const;

	/**
 	 * Get the container address.
 	 * @return The container address of the value exposed by this field or nullptr if it couldn't be resolved.
 	 * @note This field's binding must be valid to get the container address.
 	 */
	void* GetFieldContainerAddress() const;

	/**
	 * Get the property handle with ability set and get property value directly.
	 * @return The property handle for exposed property.
	 */
	TSharedPtr<IRemoteControlPropertyHandle> GetPropertyHandle() const;

	/**
	 * Enable the edit condition for the underlying property on the owner objects.
	 */
	void EnableEditCondition();
	
	/** Returns whether the property is editable in a packaged build. */
	bool IsEditableInPackaged(FString* OutError = nullptr) const;

	/** Returns whether the property is editable in the Editor. */
	bool IsEditableInEditor(FString* OutError = nullptr) const;

	/** Returns whether the property is editable, will check for editor or packaged automatically */
	bool IsEditable(FString* OutError = nullptr) const;

	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
	
public:
	/** Key for the metadata's Min entry. */
	static FName MetadataKey_Min;
	/** Key for the metadata's Max entry. */
	static FName MetadataKey_Max;
	
private:
	/** Assign the default metadata for this exposed property. (ie. Min, Max...) */
	void InitializeMetadata();

private:
	/** Whether the property is blueprint read only. */
	UPROPERTY()
	bool bIsEditableInPackaged = false;

#if WITH_EDITOR

	/** Cached edit condition path used to enable the exposed property's edit condition. */
	FRCFieldPathInfo CachedEditConditionPath;

#endif // WITH_EDITOR
};

/**
 * Represents a function exposed to remote control.
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlFunction : public FRemoteControlField
{
	GENERATED_BODY()

	FRemoteControlFunction() = default;

	UE_DEPRECATED(4.27, "This constructor is deprecated. Use the other constructor.")
	FRemoteControlFunction(FName InLabel, FRCFieldPathInfo FieldPathInfo, UFunction* InFunction);

	FRemoteControlFunction(URemoteControlPreset* InPreset, FName InLabel, FRCFieldPathInfo InFieldPathInfo, UFunction* InFunction, const TArray<URemoteControlBinding*>& InBindings);

	//~ Begin FRemoteControlEntity interface
	virtual uint32 GetUnderlyingEntityIdentifier() const override;
	virtual UClass* GetSupportedBindingClass() const override;
	virtual bool IsBound() const override;
	//~ End FRemoteControlEntity interface

	/** Returns whether the function is callable in a packaged build. */
	bool IsCallableInPackaged() const { return bIsCallableInPackaged; }

	/** Returns the underlying exposed function. */
	UFunction* GetFunction() const;

#if WITH_EDITOR
	/**
	 * Recreates the function arguments but tries to preserve old values when possible.
	 * Useful for updating function arguments after a blueprint recompile.
	 */
	void RegenerateArguments();
#endif

	friend FArchive& operator<<(FArchive& Ar, FRemoteControlFunction& RCFunction);
	bool Serialize(FArchive& Ar);
	void PostSerialize(const FArchive& Ar);
	
public:
	/**
	 * The exposed function.
	 */
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	mutable TObjectPtr<UFunction> Function_DEPRECATED = nullptr;
#endif

	/**
	 * The function arguments.
	 */
	TSharedPtr<class FStructOnScope> FunctionArguments;

private:
	/** Parse function metadata to get the function's default parameters */
	void AssignDefaultFunctionArguments();

#if WITH_EDITOR
	/** Hash function arguments using their type and size. */
	static uint32 HashFunctionArguments(UFunction* InFunction);
#endif

private:
	/** Whether the function is callable in a packaged build. */
	UPROPERTY()
	bool bIsCallableInPackaged = false;
	
	/** The exposed function. */
	UPROPERTY()
	FSoftObjectPath FunctionPath;

	/** Cached resolved underlying function used to avoid doing a findobject while serializing. */
	mutable TWeakObjectPtr<UFunction> CachedFunction;

#if WITH_EDITORONLY_DATA
	/** Hash of the underlying function arguments used to check if it has changed after a recompile. */
	uint32 CachedFunctionArgsHash = 0;
#endif
};

template<> struct TStructOpsTypeTraits<FRemoteControlFunction> : public TStructOpsTypeTraitsBase2<FRemoteControlFunction>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
};

template<> struct TStructOpsTypeTraits<FRemoteControlProperty> : public TStructOpsTypeTraitsBase2<FRemoteControlProperty>
{
	enum
	{
		WithSerializer = true,
		WithPostSerialize = true
	};
};
