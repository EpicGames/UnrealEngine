// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/ObjectMacros.h"
#include "CoreTypes.h"
#include "RemoteControlFieldMetadata.generated.h"


USTRUCT()
struct REMOTECONTROL_API FRCFieldMetadata
{
	GENERATED_BODY()

	FRCFieldMetadata() = default;
	virtual ~FRCFieldMetadata() {}
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_byte : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	uint8 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint8 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint8 DefaultValue = 0;
};


USTRUCT()
struct REMOTECONTROL_API FRCMetadata_uint16 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	uint16 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint16 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint16 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_uint32 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	uint32 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint32 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint32 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_uint64 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	uint64 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint64 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	uint64 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_int8 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	int8 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int8 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int8 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_int16 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	int16 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int16 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int16 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_int32 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	int32 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int32 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int32 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_int64 : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	int64 Min = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int64 Max = 0;

	UPROPERTY(EditAnywhere, Category=Default)
	int64 DefaultValue = 0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_float : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	float Min = 0.0f;

	UPROPERTY(EditAnywhere, Category=Default)
	float Max = 0.0f;

	UPROPERTY(EditAnywhere, Category=Default)
	float DefaultValue = 0.0f;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_double : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	double Min = 0.0;

	UPROPERTY(EditAnywhere, Category=Default)
	double Max = 0.0;

	UPROPERTY(EditAnywhere, Category=Default)
	double DefaultValue = 0.0;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_FString : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	FString DefaultValue;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_FName : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	FName DefaultValue;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_UObject : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	TSoftObjectPtr<class UObject> DefaultValue;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_UClass : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	TSoftClassPtr<UClass> DefaultValue;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_UScriptStruct : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	TSoftObjectPtr<class UScriptStruct> DefaultValue;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_bool : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	bool DefaultValue = false;
};

USTRUCT()
struct REMOTECONTROL_API FRCMetadata_FVector : public FRCFieldMetadata
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category=Default)
	FVector DefaultValue = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category=Default)
	FVector MinimumValue = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, Category=Default)
	FVector MaximumValue = FVector::ZeroVector;
};

/*
CASTCLASS_UScriptStruct = 0x0000000000000010,
CASTCLASS_UClass = 0x0000000000000020,
CASTCLASS_UEnum = 0x0000000000000004,
CASTCLASS_UStruct = 0x0000000000000008,
CASTCLASS_UScriptStruct = 0x0000000000000010,

CASTCLASS_FBoolProperty = 0x0000000000020000,
CASTCLASS_FUInt16Property = 0x0000000000040000,
CASTCLASS_UFunction = 0x0000000000080000,
CASTCLASS_FStructProperty = 0x0000000000100000,
CASTCLASS_FArrayProperty = 0x0000000000200000,
CASTCLASS_FInt64Property = 0x0000000000400000,
CASTCLASS_FDelegateProperty = 0x0000000000800000,
CASTCLASS_FNumericProperty = 0x0000000001000000,
CASTCLASS_FMulticastDelegateProperty = 0x0000000002000000,
CASTCLASS_FObjectPropertyBase = 0x0000000004000000,
CASTCLASS_FWeakObjectProperty = 0x0000000008000000,
CASTCLASS_FLazyObjectProperty = 0x0000000010000000,
CASTCLASS_FSoftObjectProperty = 0x0000000020000000,
CASTCLASS_FTextProperty = 0x0000000040000000,
CASTCLASS_FInt16Property = 0x0000000080000000,
CASTCLASS_FDoubleProperty = 0x0000000100000000,
CASTCLASS_FSoftClassProperty = 0x0000000200000000,
CASTCLASS_UPackage = 0x0000000400000000,
CASTCLASS_ULevel = 0x0000000800000000,
CASTCLASS_AActor = 0x0000001000000000,
CASTCLASS_APlayerController = 0x0000002000000000,
CASTCLASS_APawn = 0x0000004000000000,
CASTCLASS_USceneComponent = 0x0000008000000000,
CASTCLASS_UPrimitiveComponent = 0x0000010000000000,
CASTCLASS_USkinnedMeshComponent = 0x0000020000000000,
CASTCLASS_USkeletalMeshComponent = 0x0000040000000000,
CASTCLASS_UBlueprint = 0x0000080000000000,
CASTCLASS_UDelegateFunction = 0x0000100000000000,
CASTCLASS_UStaticMeshComponent = 0x0000200000000000,
CASTCLASS_FMapProperty = 0x0000400000000000,
CASTCLASS_FSetProperty = 0x0000800000000000,
CASTCLASS_FEnumProperty = 0x0001000000000000,
CASTCLASS_USparseDelegateFunction = 0x0002000000000000,
CASTCLASS_FMulticastInlineDelegateProperty = 0x0004000000000000,
CASTCLASS_FMulticastSparseDelegateProperty = 0x0008000000000000,
CASTCLASS_FFieldPathProperty = 0x0010000000000000,
*/


#undef DECLARE_NUMERIC_METADATA