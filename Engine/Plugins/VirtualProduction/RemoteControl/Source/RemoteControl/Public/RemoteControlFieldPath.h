// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#if WITH_EDITOR
#include "PropertyPath.h"
#endif

#include "RemoteControlFieldPath.generated.h"


/** Small container for the resolved data of a remote control field segment */
struct REMOTECONTROL_API FRCFieldResolvedData
{
	/** Type of that segment owner */
	UStruct* Struct = nullptr;

	/** Container address of this segment */
	void* ContainerAddress = nullptr;

	/** Resolved field for this segment */
	FProperty* Field = nullptr;

	/** Index resolved from the key of the segment. */
	int32 MapIndex = INDEX_NONE;

	/** Returns validity of the resolved data */
	bool IsValid() const
	{
		return Field != nullptr
			&& ContainerAddress != nullptr
			&& Struct != nullptr; 
	}
};

/** RemoteControl Path segment holding a property layer */
USTRUCT()
struct REMOTECONTROL_API FRCFieldPathSegment
{
	GENERATED_BODY()

public:
	FRCFieldPathSegment() = default;

	/** Builds a segment from a name. */
	FRCFieldPathSegment(FStringView SegmentName);

	/** Returns true if a Field was found for a given owner */
	bool IsResolved() const;

	/**
	 * Converts this segment to a string
	 * FieldName, FieldName[Index]
	 * If bDuplicateContainer is asked, format will be different if its indexed
	 * FieldName.FieldName[Index]  -> This is to bridge for PathToProperty
	 */
	FString ToString(bool bDuplicateContainer = false) const;


private:

	/** Reset resolved pointers */
	void ClearResolvedData();

public:

	/** Name of the segment */
	UPROPERTY()
	FName Name;

	/** Container index if any. */
	UPROPERTY()
	int32 ArrayIndex = INDEX_NONE;

	/**
	 * Value property name, in case a map is being indexed.
	 * ie. The path Var.Var_Value[0] will populate ValuePropertyName with "Var"
	 * This is needed because sometimes the value's property name will differ from the
	 * name before.
	 * For example a map property named Test_1 will generate the following path: Test_1.Test_Value[0]
	 */
	UPROPERTY()
	FString ValuePropertyName;

	/**
	 * Holds the key in case of a path containing an indexed map.
	 * ie. Field path MapProp["mykey"] will fill MapKey with "mykey" 
	 */
	UPROPERTY()
	FString MapKey;

	/** Resolved Data of the segment */
	FRCFieldResolvedData ResolvedData;
};

/**
 * Holds a path from a UObject to a field.
 * Has facilities to resolve for a given owner.
 *
 * Example Usage Create a path to relative location's x value, then resolve it on an static mesh component.
 * FRCFieldPathInfo Path("RelativeLocation.X"));
 * bool bResolved = Path.Resolve(MyStaticMeshComponent);
 * if (bResolved)
 * {
 *   FRCFieldResolvedData Data = Path.GetResolvedData();
 *   // Data.ContainerAddress corresponds to &MyStaticMeshComponent.RelativeLocation
 *   // Data.Field corresponds to FFloatProperty (X) 
 *   // Data.Struct corresponds to FVector
 * }
 * 
 * Other example paths:
 * "MyStructProperty.NestedArrayProperty[3]"
 * "RelativeLocation"
 * "RelativeLocation.X"
 * 
 * Supports array/set/map indexing.
 * @Note Only String keys are currently supported for map key indexing.
 * ie. MyStructProperty.MyStringToVectorMap["MyKey"].X
 * Be aware that MyStringToVectorMap[2] will not correspond to the key 2 in the map, but to the index 2 of the map.
 * This behaviour is intended to match PropertyHandle's GeneratePathToProperty method.
 */
USTRUCT()
struct REMOTECONTROL_API FRCFieldPathInfo
{
	GENERATED_BODY()

public:
	FRCFieldPathInfo() = default;

	/**
	 * Builds a path info from a string of format with '.' delimiters
	 * Optionally can reduce duplicates when dealing with containers
	 * If true -> Struct.ArrayName.ArrayName[2].Member will collapse to Struct.ArrayName[2].Member
	 * This is when being used with PathToProperty
	 */
	FRCFieldPathInfo(const FString& PathInfo, bool bSkipDuplicates = false);

	/**
	 * Builds a path info from a property.
	 */
	FRCFieldPathInfo(FProperty* Property);

	bool operator == (const FRCFieldPathInfo& Other) const
	{
		return PathHash == Other.PathHash;
	}

	bool operator != (const FRCFieldPathInfo& Other) const
	{
		return PathHash != Other.PathHash;
	}

public:
	/** Go through each segment and finds the property associated + container address for a given UObject owner */
	bool Resolve(UObject* Owner);

	/** Returns true if last segment was resolved */
	bool IsResolved() const;

	/** Returns true if the hash of the string corresponds to the string we were built with */
	bool IsEqual(FStringView OtherPath) const;

	/** Returns true if hash of both PathInfo matches */
	bool IsEqual(const FRCFieldPathInfo& OtherPath) const;

	/**
	 * Converts this PathInfo to a string
	 * Walks the full path by default
	 * If EndSegment is not none, will stop at the desired segment
	 */
	FString ToString(int32 EndSegment = INDEX_NONE) const;

	/**
	 * Converts this PathInfo to a string of PathToProperty format
	 * Struct.ArrayName.ArrayName[Index]
	 * If EndSegment is not none, will stop at the desired segment
	 */
	FString ToPathPropertyString(int32 EndSegment = INDEX_NONE) const;

	/** Returns the number of segment in this path */
	int32 GetSegmentCount() const { return Segments.Num(); }

	/** Gets a segment from this path */
	const FRCFieldPathSegment& GetFieldSegment(int32 Index) const;

	/**
	 * Returns the resolved data of the last segment
	 * If last segment is not resolved, data won't be valid
	 */
	FRCFieldResolvedData GetResolvedData() const;

	/** Returns last segment's name */
	FName GetFieldName() const;

	/** Builds a property change event from all the segments */
	FPropertyChangedEvent ToPropertyChangedEvent(EPropertyChangeType::Type InChangeType = EPropertyChangeType::Unspecified) const;

	/** Builds an EditPropertyChain from the segments */
	void ToEditPropertyChain(FEditPropertyChain& OutPropertyChain) const;

#if WITH_EDITOR
	/** Converts this RCPath to a property path. */
	TSharedRef<FPropertyPath> ToPropertyPath() const;
#endif


private:
	/**
	 * Initialize from a string of format with '.' delimiters
	 * Optionally can reduce duplicates when dealing with containers
	 * If true -> Struct.ArrayName.ArrayName[2].Member will collapse to Struct.ArrayName[2].Member
	 * This is when being used with PathToProperty
	 */
	void Initialize(const FString& PathInfo, bool bCleanDuplicates);
	
	/** Recursively resolves all segment until the final one */
	bool ResolveInternalRecursive(UStruct* OwnerType, void* ContainerAddress, int32 SegmentIndex);

public:

	/** List of segments to point to a given field */
	UPROPERTY()
	TArray<FRCFieldPathSegment> Segments;

	/** Hash created from the string we were built from to quickly compare to paths */
	UPROPERTY()
	uint32 PathHash = 0;
};
