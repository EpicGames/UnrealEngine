// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"
#include "RemoteControlActor.h"
#include "RemoteControlPreset.h"
#include "RemoteControlFunctionLibrary.generated.h"

class URemoteControlPreset;
class AActor;

USTRUCT(Blueprintable)
struct FRemoteControlOptionalExposeArgs
{
	GENERATED_BODY()
	
	/**
	 * The display name of the exposed entity in the panel.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RemoteControlPreset")
	FString DisplayName;

	/**
	 * The name of the group to expose the entity in.
	 * If it does not exist, a group with that name will be created.
	 */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "RemoteControlPreset")
	FString GroupName;
};

/** Base data for a color represented as its position on a color wheel. */
USTRUCT(Blueprintable)
struct FColorWheelColorBase
{
	GENERATED_BODY()

	/**
	* The position on the unit circle. Magnitude should be in range [0, 1].
	*/
	UPROPERTY()
	FVector2D Position = FVector2D::ZeroVector;

	/**
	 * The color's value component (as in HSV).
	 */
	UPROPERTY()
	double Value = 0.0;
};

/** A color as represented by a position on a color wheel. */
USTRUCT(Blueprintable)
struct FColorWheelColor : public FColorWheelColorBase
{
	GENERATED_BODY()

	/**
	 * The color's alpha component.
	 */
	UPROPERTY()
	double Alpha = 0.0;
};

/** A color value as represented by a position on a color grading wheel. */
USTRUCT(Blueprintable)
struct FColorGradingWheelColor : public FColorWheelColorBase
{
	GENERATED_BODY()

	/**
	 * The color's luminance component, which is separate from the HSV color itself.
	 */
	UPROPERTY()
	double Luminance = 0.0;
};

UCLASS()
class URemoteControlFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()
	
public:
	/**
	 * Expose a property in a remote control preset.
	 * @param Preset the preset to expose the property in.
	 * @param SourceObject the object that contains the property to expose.
	 * @param Property the name or path of the property to expose.
	 * @param Args optional arguments.
	 * @return Whether the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlPreset")
	static bool ExposeProperty(URemoteControlPreset* Preset, UObject* SourceObject, const FString& Property, FRemoteControlOptionalExposeArgs Args);

	/**
	 * Expose a function in a remote control preset.
	 * @param Preset the preset to expose the property in.
	 * @param SourceObject the object that contains the property to expose.
	 * @param Function the name of the function to expose.
	 * @param Args optional arguments.
	 * @return Whether the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlPreset")
	static bool ExposeFunction(URemoteControlPreset* Preset, UObject* SourceObject, const FString& Function, FRemoteControlOptionalExposeArgs Args);

	/**
	 * Expose an actor in a remote control preset.
	 * @param Preset the preset to expose the property in.
	 * @param Actor the actor to expose.
	 * @param Args optional arguments.
	 * @return Whether the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlPreset")
	static bool ExposeActor(URemoteControlPreset* Preset, AActor* Actor, FRemoteControlOptionalExposeArgs Args);

	/**
	 * Add/subtract from the value of an FLinearColor property using a delta value based on color wheel coordinates.
	 * @param TargetObject the object that contains the property to modify.
	 * @param PropertyName the name of the property to modify.
	 * @param DeltaValue the amount to change the color by.
	 * @param ReferenceColor if the color's current position on the wheel is ambiguous as calculated from RGB values (e.g. black), use this reference color's position instead.
	 * @param bIsInteractive if true, this is treated as an interactive change. If false, it will be treated as the final value set change.
	 * @return Whether the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Color")
	static bool ApplyColorWheelDelta(UObject* TargetObject, const FString& PropertyName, const FColorWheelColor& DeltaValue, const FColorWheelColor& ReferenceColor, bool bIsInteractive);

	/**
	 * Add/subtract from the value of an FVector4 property interpreted as RGBV using a delta value based on color wheel coordinates.
	 * @param TargetObject the object that contains the property to modify.
	 * @param PropertyName the name of the property to modify.
	 * @param DeltaValue the amount to change the color by.
	 * @param ReferenceColor if the color's current position on the wheel is ambiguous as calculated from RGB values (e.g. black), use this reference color's position instead.
	 * @param bIsInteractive if true, this is treated as an interactive change. If false, it will be treated as the final value set change.
	 * @return Whether the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Remote Control Color")
	static bool ApplyColorGradingWheelDelta(UObject* TargetObject, const FString& PropertyName, const FColorGradingWheelColor& DeltaValue, const FColorGradingWheelColor& ReferenceColor, bool bIsInteractive, float MinValue = 0.f, float MaxValue = 2.f);
};