// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlField.h"
#include "RemoteControlEntityFactory.h"

#include "Materials/MaterialLayersFunctions.h"

#include "RemoteControlInstanceMaterial.generated.h"

class UDEditorParameterValue;
class URemoteControlBinding;
class URemoteControlPreset;

/**
 * Represents a material instance property that has been exposed to remote control.
 */
USTRUCT(BlueprintType)
struct REMOTECONTROL_API FRemoteControlInstanceMaterial : public FRemoteControlProperty
{
	GENERATED_BODY()

public:
	FRemoteControlInstanceMaterial() = default;

	// Material Instance property available only in editor
#if WITH_EDITOR
	FRemoteControlInstanceMaterial(URemoteControlPreset* InPreset, FName InLabel,
	                               UDEditorParameterValue* InEditorParameterValue,
	                               const FRCFieldPathInfo& InOriginalFieldPathInfo, UObject* InInstance,
	                               const FRCFieldPathInfo& InFieldPath,
	                               const TArray<URemoteControlBinding*>& InBindings);
#endif

	//~ Begin FRemoteControlProperty interface
#if WITH_EDITOR
	virtual void OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent) override;
#endif
	virtual bool CheckIsBoundToPropertyPath(const FString& InPath) const override;
	virtual bool ContainsBoundObjects(TArray<UObject*> InObjects) const override;
	virtual void PostLoad() override;
	//~ End FRemoteControlProperty interface

public:
	/** Stores original binding class */
	UPROPERTY()
	TObjectPtr<UClass> OriginalClass = nullptr;

	/** Stores original material parameter info */
	UPROPERTY()
	FMaterialParameterInfo ParameterInfo;

	/** Store original property path */
	UPROPERTY()
	FRCFieldPathInfo OriginalFieldPathInfo;

	/** Store path to Material Instance */
	UPROPERTY()
	FSoftObjectPath InstancePath;
};

class FRemoteControlInstanceMaterialFactory : public IRemoteControlPropertyFactory
{
public:
	static TSharedRef<IRemoteControlPropertyFactory> MakeInstance();

	//~ Begin IRemoteControlPropertyFactory interface
	virtual TSharedPtr<FRemoteControlProperty> CreateRemoteControlProperty(URemoteControlPreset* Preset, UObject* Object, FRCFieldPathInfo FieldPath, FRemoteControlPresetExposeArgs Args) override;
	virtual void PostSetObjectProperties(UObject* Object, bool bInSuccess) const override;
	virtual bool SupportExposedClass(UClass* Class) const override;
	//~ End IRemoteControlPropertyFactory interface
};

