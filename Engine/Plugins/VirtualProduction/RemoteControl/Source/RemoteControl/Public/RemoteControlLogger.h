// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class IMessageLogListing;
class SWidget;

/**
 * Log verbosity level
 */
enum class EVerbosityLevel
{
	Log,
	Warning,
	Error
};

/**
 * Remote control logger.
 * Generated Log UI and collect logs from APIs and protocols
 */
class REMOTECONTROL_API FRemoteControlLogger : public TSharedFromThis<FRemoteControlLogger>
{
public:
	/** Get singleton logger instance */
	static FRemoteControlLogger& Get()
	{
		static FRemoteControlLogger* Instance = nullptr;
		if (Instance == nullptr)
		{
			Instance = new FRemoteControlLogger;
		}
		return *Instance;
	}

	/** Callback for logging, it should return FText to log */
	using FLogCallback = TFunctionRef<FText(void)>;
	
	FRemoteControlLogger();
	virtual ~FRemoteControlLogger() = default;

	/**
	 * Log the message
	 *
	 * @param InputType Type of the log, protocol, api, webapi
	 * @param InLogTextCallback	Log callback which should return the Text to log
	 * @param Verbosity			Log level
	 */
	void Log(const FName& InputType, FLogCallback InLogTextCallback, EVerbosityLevel Verbosity = EVerbosityLevel::Log);

	/** 
	 * Enable/Disable the log for Remote Control
	 * @param bEnable enable flag
	 */
	void EnableLog(const bool bEnable);

	/** Return the current state of the log */
	bool IsEnabled() const { return bIsEnabled; }

	/** Removes all messages from log */
	void ClearLog() const;

#if WITH_EDITOR
	/** Log listening interface */
	TSharedPtr<IMessageLogListing> GetMessageLogListing() const { return MessageLogListing; }
#endif

private:

#if WITH_EDITOR
	/** Log listening interface */
	TSharedPtr<IMessageLogListing> MessageLogListing;
#endif

	/** Is the logger enabled */
	bool bIsEnabled = false;
};
