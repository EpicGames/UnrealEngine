// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlObjectVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FRemoteControlObjectVersion::GUID(0x83E2C5C2, 0x364D11EB, 0x90A93CDF, 0xFB2CA371);

// Register the custom version with core
FCustomVersionRegistration GRegisterRemoteControlCustomVersion(FRemoteControlObjectVersion::GUID, FRemoteControlObjectVersion::LatestVersion, TEXT("RemoteControlObjectVer"));