// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"

// Custom serialization version for changes to remote control objects
struct FRemoteControlObjectVersion
{
	enum Type
	{
		// Roughly corresponds to 4.26
		BeforeCustomVersionWasAdded = 0,

		// Properties can now be directly exposed on components
		RemovedComponentChain,

		// RCProperties and RCFunctions now inherit from FRemoteControlEntity
		ConvertRCFieldsToRCEntities,

		// Converted FRemoteControlTarget to URemoteControlBinding, put RCFields in the expose registry.
		ConvertTargetsToBindings,

		// ExposedEntities can now be rebound.
		AddedRebindingFunctionality,

		// Added property/function flags to RemoteControlField
		AddedFieldFlags,
		
		// Instead of the global 'Protocol Generates Transactions' setting, each preset can now specify its modify operation flags
		AddedPerPresetModifyOperationFlags,

		// Change FRCEntityWrapper::ExposedEntities' Set to an Array to retain ordering across sessions
		ChangedExposedEntitiesSetToAnOrderedTArray,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	const static FGuid GUID;

private:
	FRemoteControlObjectVersion() {}
};
