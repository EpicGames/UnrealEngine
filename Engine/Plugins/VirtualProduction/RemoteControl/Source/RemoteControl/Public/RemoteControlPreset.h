// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Algo/Transform.h"
#include "CoreTypes.h"
#include "RCModifyOperationFlags.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "RemoteControlPropertyIdRegistry.h"
#include "Templates/PimplPtr.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/SoftObjectPtr.h"

#include "RemoteControlPreset.generated.h"

class AActor;
class IStructSerializerBackend;
class IStructDeserializerBackend;
enum class EPackageReloadPhase : uint8;
enum class EPropertyBagPropertyType : uint8;
struct FPropertyChangedEvent;
struct FRCFieldPathInfo;
struct FRemoteControlActor;
struct FRemoteControlPresetLayout;
class FRemoteControlPresetRebindingManager;
class FTransactionObjectEvent;
class UBlueprint;
class URCVirtualPropertyBase;
class URCVirtualPropertyContainerBase;
class URCVirtualPropertyInContainer;
class URCVirtualPropertySelfContainer;
class URemoteControlExposeRegistry;
class URCSignatureRegistry;
class URemoteControlBinding;
class URemoteControlPreset;

DECLARE_MULTICAST_DELEGATE(FOnVirtualPropertyContainerModified);

/** Arguments used to expose an entity (Actor, property, function, etc.) */
struct REMOTECONTROL_API FRemoteControlPresetExposeArgs
{
	FRemoteControlPresetExposeArgs();
	FRemoteControlPresetExposeArgs(FString Label, FGuid GroupId, bool bEnableEditCondition = true);

	/** (Optional) The label to use for the new exposed entity. */
	FString Label;
	/** (Optional) The group in which to put the field. */
	FGuid GroupId;
	/** Whether to automatically enable the edit condition for the exposed property. */
	bool bEnableEditCondition;
};

/** Arguments used to expose an entity (Actor, property, function, etc.) */
struct REMOTECONTROL_API FRemoteControlPropertyIdArgs
{
	FRemoteControlPropertyIdArgs() = default;

	bool IsValid() const
	{
		return VirtualProperty && (SourceObject || SourceClass)
			&& (!SuperType.IsNone() || !SubType.IsNone());
	}

public:
	/** PropertyId */
	FName PropertyId;

	/** SuperType of the Property */
	FName SuperType;

	/** SubType of the Property */
	FName SubType;

	/** (Optional) The source object to use for special cases. */
	TObjectPtr<UObject> SourceObject;

	/** (Optional) The class of the source object to use for special cases. */
	UClass* SourceClass;

	/** (Optional) The virtual property to use for all the cases. */
	TObjectPtr<URCVirtualPropertySelfContainer> VirtualProperty;

	/** Map with the Guid of the RCField and as its value the real PropContainer of the real property */
	TMap<FGuid, TObjectPtr<URCVirtualPropertySelfContainer>> RealProperties;
};

/**
 * Data cached for every exposed field.
 */
USTRUCT()
struct FRCCachedFieldData
{
	GENERATED_BODY()

	/** The group the field is in. */
	UPROPERTY()
	FGuid LayoutGroupId;

	/** The target that owns this field. */
	UPROPERTY()
	FName OwnerObjectAlias;
};

/**
 * Holds an exposed property and owner objects.
 */
struct UE_DEPRECATED(4.27, "FExposedProperty is deprecated. Please use FRemoteControlProperty::GetBoundObjects to access an exposed property's owner objects.") FExposedProperty;
struct REMOTECONTROL_API FExposedProperty
{
	bool IsValid() const
	{
		return !!Property;
	}

	FProperty* Property;
	TArray<UObject*> OwnerObjects;
};

/**
 * Holds an exposed function, its default parameters and owner objects.
 */
struct UE_DEPRECATED(4.27, "FExposedFunction is deprecated. Please use FRemoteControlFunction::GetBoundObjects to access an exposed function's owner objects.") FExposedFunction;
struct REMOTECONTROL_API FExposedFunction
{
	bool IsValid() const 
	{
		return Function && DefaultParameters;
	}

	UFunction* Function;
	TSharedPtr<class FStructOnScope> DefaultParameters;
	TArray<UObject*> OwnerObjects;
};

/**
 * Temporarily prevent Remote Control Presets from renewing it's guids during duplicate operation.
 */
struct REMOTECONTROL_API FRCPresetGuidRenewGuard
{
	FRCPresetGuidRenewGuard();
	~FRCPresetGuidRenewGuard();

	static bool IsAllowingPresetGuidRenewal();

private:
	bool bPreviousValue;
};

/**
 * Represents a group of field and offers operations to operate on the fields inside of that group.
 */
USTRUCT()
struct REMOTECONTROL_API FRemoteControlPresetGroup
{
	GENERATED_BODY()

	FRemoteControlPresetGroup()
		:TagColor(ForceInit)
	{}

	FRemoteControlPresetGroup(FName InName, FGuid InId)
		: Name(InName)
		, Id(MoveTemp(InId))
		, TagColor(MakeSimilarSaturatedColor())
	{}

	/** Get the fields under this group. */
	const TArray<FGuid>& GetFields() const;

	/** Get the fields under this group (Non-const)*/
	TArray<FGuid>& AccessFields();

	friend bool operator==(const FRemoteControlPresetGroup& LHS, const FRemoteControlPresetGroup& RHS)
	{
		return LHS.Id == RHS.Id;
	}

private:

	/**
	* Makes a random but similar saturated color.
	*/
	static FLinearColor MakeSimilarSaturatedColor(float InSaturationLevel = 0.5f)
	{
		const uint8 Hue = (uint8)(FMath::FRand() * 255.f);
		const uint8 Saturation = (uint8)(255.f * InSaturationLevel);
		return FLinearColor::MakeFromHSV8(Hue, Saturation, 255);
	}
 
public:
	/** Name of this group. */
	UPROPERTY()
	FName Name;

	/** This group's ID. */
	UPROPERTY()
	FGuid Id;

	/* Color Tag for this group. */
	UPROPERTY()
	FLinearColor TagColor;

private:
	/** The list of exposed fields under this group. */
	UPROPERTY()
	TArray<FGuid> Fields;
};

/** Layout that holds groups of fields. */
USTRUCT()
struct REMOTECONTROL_API FRemoteControlPresetLayout
{
	GENERATED_BODY()

	/** Arguments for swapping fields across groups.  */
	struct FFieldSwapArgs
	{
		FGuid OriginGroupId;
		FGuid TargetGroupId;
		TArray<FGuid> DraggedFieldsIds;
		FGuid TargetFieldId;
	};

	FRemoteControlPresetLayout() = default;
	FRemoteControlPresetLayout(URemoteControlPreset* OwnerPreset);

	/** Get or create the default group. */
	FRemoteControlPresetGroup& GetDefaultGroup();
	/** Return the DefaultGroupOrder */
	TArray<FGuid>& GetDefaultGroupOrder() { return DefaultGroupOrder; }
	/** Set the All group order */
	void SetDefaultGroupOrder(const TArray<FGuid>& InNewDefaultGroupOrder) { DefaultGroupOrder = InNewDefaultGroupOrder; }
	/** Returns true when the given group id is a default one. */
	bool IsDefaultGroup(FGuid GroupId) const;
	
	/** Returns the tag color of the given group id. */
	FLinearColor GetTagColor(FGuid GroupId);

	/**
	 * Get a group by searching by ID.
	 * @param GroupId the id to use.
	 * @return A pointer to the found group or nullptr.
	 */
	FRemoteControlPresetGroup* GetGroup(FGuid GroupId); 

	/**
	 *
	 * Get a group by searching by name.
	 * @param GroupName the name to use.
	 * @return A pointer to the found group or nullptr.
	 */
	FRemoteControlPresetGroup* GetGroupByName(FName GroupName);

	/** Create a group by giving it a name and ID */
	UE_DEPRECATED(4.27, "This function was deprecated, use the overload that doesn't accept a group id.")
	FRemoteControlPresetGroup& CreateGroup(FName GroupName, FGuid GroupId);

	/** Create a group in the layout with a given name. */
	FRemoteControlPresetGroup& CreateGroup(FName GroupName = NAME_None);

	/** Find the group that holds the specified field. */
	/**
	 * Search for a group that contains a certain field.
	 * @param FieldId the field to search a group for.
	 * @return A pointer to the found group or nullptr.
	 */
	FRemoteControlPresetGroup* FindGroupFromField(FGuid FieldId);

	/** Swap two groups. */
	void SwapGroups(FGuid OriginGroupId, FGuid TargetGroupId);

	/** Swap fields across groups or in the same one. */
	void SwapFields(const FFieldSwapArgs& InFieldSwapArgs);

	/** Swap fields across groups or in the same one for the default group. */
	void SwapFieldsDefaultGroup(const FFieldSwapArgs& FieldSwapArgs, const FGuid InFieldRealGroup, TArray<FGuid> InEntities);

	/** Delete a group from the layout. */
	void DeleteGroup(FGuid GroupId);

	/** Rename a group in the layout. */
	void RenameGroup(FGuid GroupId, FName NewGroupName);

	/** Get this layout's groups. */
	const TArray<FRemoteControlPresetGroup>& GetGroups() const;

	/** 
	 * Non-Const getter for this layout's groups. 
	 * @Note Use carefully, as adding/removing groups should be done using their respective methods.
	 */
	TArray<FRemoteControlPresetGroup>& AccessGroups();

	/** Append a field to the group's field list. */
	void AddField(FGuid GroupId, FGuid FieldId);

	/** Insert a field in the group. */
	void InsertFieldAt(FGuid GroupId, FGuid FieldId, int32 Index);

	/** Remove a field using the field's name. */
	void RemoveField(FGuid GroupId, FGuid FieldId);

	/** Remove a field at a provided index. */
	void RemoveFieldAt(FGuid GroupId, int32 Index);

	/** Get the preset that owns this layout. */
	URemoteControlPreset* GetOwner();

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap);
	
	// Layout operation delegates
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGroupAdded, const FRemoteControlPresetGroup& /*NewGroup*/);
	FOnGroupAdded& OnGroupAdded() { return OnGroupAddedDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGroupDeleted, FRemoteControlPresetGroup/*DeletedGroup*/);
	FOnGroupDeleted& OnGroupDeleted() { return OnGroupDeletedDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnGroupOrderChanged, const TArray<FGuid>& /*GroupIds*/);
	FOnGroupOrderChanged& OnGroupOrderChanged() { return OnGroupOrderChangedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGroupRenamed, const FGuid& /*GroupId*/, FName /*NewName*/);
	FOnGroupRenamed& OnGroupRenamed() { return OnGroupRenamedDelegate; }

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnFieldAdded, const FGuid& /*GroupId*/, const FGuid& /*FieldId*/, int32 /*FieldPosition*/);
	FOnFieldAdded& OnFieldAdded() { return OnFieldAddedDelegate; }

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnFieldDeleted, const FGuid& /*GroupId*/, const FGuid& /*FieldId*/, int32 /*FieldPosition*/);
	FOnFieldDeleted& OnFieldDeleted() { return OnFieldDeletedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnFieldOrderChanged, const FGuid& /*GroupId*/, const TArray<FGuid>& /*FieldIds*/);
	FOnFieldOrderChanged& OnFieldOrderChanged() { return OnFieldOrderChangedDelegate; }

private:
	/** Create a group by providing a name and ID. */
	FRemoteControlPresetGroup& CreateGroupInternal(FName GroupName, FGuid GroupId);

	/** Keep the ALL group order since its not saved in the normal workflow */
	UPROPERTY()
	TArray<FGuid> DefaultGroupOrder;

	/** The list of groups under this layout. */
	UPROPERTY()
	TArray<FRemoteControlPresetGroup> Groups;

	/** The preset that owns this layout. */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> Owner = nullptr;

	// Layout operation delegates
	FOnGroupAdded OnGroupAddedDelegate;
	FOnGroupDeleted OnGroupDeletedDelegate;
	FOnGroupOrderChanged OnGroupOrderChangedDelegate;
	FOnGroupRenamed OnGroupRenamedDelegate;
	FOnFieldAdded OnFieldAddedDelegate;
	FOnFieldDeleted OnFieldDeletedDelegate;
	FOnFieldOrderChanged OnFieldOrderChangedDelegate;
};

/**
 * Holds exposed functions and properties.
 */
UCLASS(BlueprintType, EditInlineNew)
class REMOTECONTROL_API URemoteControlPreset : public UObject
{
public:
	GENERATED_BODY()

	using UObject::GetWorld;
	
	/** Callback for post remote control preset load, called by URemoteControlPreset::PostLoad function */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostLoadRemoteControlPreset, URemoteControlPreset* /* InPreset */);
	static FOnPostLoadRemoteControlPreset OnPostLoadRemoteControlPreset;

	/** Callback for post init properties of remote control preset, called by URemoteControlPreset::PostInitProperties function */
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPostInitPropertiesRemoteControlPreset, URemoteControlPreset* /* InPreset */);
	static FOnPostInitPropertiesRemoteControlPreset OnPostInitPropertiesRemoteControlPreset;
	
	URemoteControlPreset();

	//~ Begin UObject interface
	virtual void PostInitProperties() override;
	virtual void PostLoad() override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void BeginDestroy() override;
	
#if WITH_EDITOR
	void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent);
#endif /*WITH_EDITOR*/
	//~ End UObject interface

	/**
	 * Get this preset's unique ID.
	 */
	const FGuid& GetPresetId() const { return PresetId; }

	/**
	 * Returns the FName that represents this asset or hosted preset.
	 */
	FName GetPresetName() const;

	/**
	 * Expose an actor on this preset.
	 * @param Actor the actor to expose.
	 * @param Args The arguments used to expose the actor.
	 * @return The exposed actor.
	 */
	TWeakPtr<FRemoteControlActor> ExposeActor(AActor* Actor, FRemoteControlPresetExposeArgs Args = FRemoteControlPresetExposeArgs());

	/**
	 * Gets a given object's property exposed property guid in this preset
	 * @param InOuterObject the object to check
	 * @param InFieldPath the path to the property
	 * @return The exposed property if found, nullptr otherwise
	 */
	TSharedPtr<FRemoteControlProperty> FindExposedProperty(UObject* InOuterObject, const FRCFieldPathInfo& InFieldPath) const;

	/**
	 * Expose a property on this preset.
	 * @param Object the object that holds the property.
	 * @param FieldPath The name/path to the property.
	 * @param Args Optional arguments used to expose the property.
	 * @return The exposed property.
	 */
	TWeakPtr<FRemoteControlProperty> ExposeProperty(UObject* Object, FRCFieldPathInfo FieldPath, FRemoteControlPresetExposeArgs Args = FRemoteControlPresetExposeArgs());
	
	/**
	 * Expose a function on this preset.
	 * @param Object the object that holds the property.
	 * @param Function The function to expose.
	 * @param Args Optional arguments used to expose the property.
	 * @return The exposed function.
	 */
	TWeakPtr<FRemoteControlFunction> ExposeFunction(UObject* Object, UFunction* Function, FRemoteControlPresetExposeArgs Args = FRemoteControlPresetExposeArgs());

	/**
	 * Get the exposed entities of a certain type.
	 */
	template <typename ExposableEntityType = FRemoteControlEntity>
	TArray<TWeakPtr<const ExposableEntityType>> GetExposedEntities() const
	{
		static_assert(TIsDerivedFrom<ExposableEntityType, FRemoteControlEntity>::Value, "ExposableEntityType must derive from FRemoteControlEntity.");
		TArray<TWeakPtr<const ExposableEntityType>> ReturnedEntities;
		TArray<TSharedPtr<const FRemoteControlEntity>> Entities = GetEntities(ExposableEntityType::StaticStruct());
		Algo::Transform(Entities, ReturnedEntities,
			[](const TSharedPtr<const FRemoteControlEntity>& Entity)
			{
				return StaticCastSharedPtr<const ExposableEntityType>(Entity);
			});
		return ReturnedEntities;
	}

	/**
	 * Get the exposed entities of a certain type.
	 */
	template <typename ExposableEntityType = FRemoteControlEntity>
	TArray<TWeakPtr<ExposableEntityType>> GetExposedEntities()
	{
		static_assert(TIsDerivedFrom<ExposableEntityType, FRemoteControlEntity>::Value, "ExposableEntityType must derive from FRemoteControlEntity.");

		TArray<TWeakPtr<ExposableEntityType>> ReturnedEntities;
		TArray<TSharedPtr<FRemoteControlEntity>> Entities = GetEntities(ExposableEntityType::StaticStruct());
		Algo::Transform(Entities, ReturnedEntities,
			[](const TSharedPtr<FRemoteControlEntity>& Entity)
			{
				return StaticCastSharedPtr<ExposableEntityType>(Entity);
			});
		return ReturnedEntities;
	}

	TArray<TWeakPtr<FRemoteControlEntity>> GetExposedEntities(UScriptStruct* EntityType)
	{
		TArray<TWeakPtr<FRemoteControlEntity>> ReturnedEntities;
		const TArray<TSharedPtr<FRemoteControlEntity>> Entities = GetEntities(EntityType);
		ReturnedEntities.Append(Entities);
		return ReturnedEntities;
	}

	/**
	 * Get a copy of an exposed entity on the preset.
	 * @param ExposedEntityId The id of the entity to get.
	 * @note ExposableEntityType must derive from FRemoteControlEntity.
	 */
	template <typename ExposableEntityType = FRemoteControlEntity>
	TWeakPtr<const ExposableEntityType> GetExposedEntity(const FGuid& ExposedEntityId) const
	{
		static_assert(TIsDerivedFrom<ExposableEntityType, FRemoteControlEntity>::Value, "ExposableEntityType must derive from FRemoteControlEntity.");
		return StaticCastSharedPtr<const ExposableEntityType>(FindEntityById(ExposedEntityId, ExposableEntityType::StaticStruct()));
	}

	/**
	 * Get a pointer to an exposed entity on the preset.
	 * @param ExposedEntityId The id of the entity to get.
	 * @note ExposableEntityType must derive from FRemoteControlEntity.
	 */
	template <typename ExposableEntityType = FRemoteControlEntity>
	TWeakPtr<ExposableEntityType> GetExposedEntity(const FGuid& ExposedEntityId)
	{
		static_assert(TIsDerivedFrom<ExposableEntityType, FRemoteControlEntity>::Value, "ExposableEntityType must derive from FRemoteControlEntity.");
		return StaticCastSharedPtr<ExposableEntityType>(FindEntityById(ExposedEntityId, ExposableEntityType::StaticStruct()));
	}

	/** Get the type of an exposed entity by querying with its id. (ie. FRemoteControlActor) */
	const UScriptStruct* GetExposedEntityType(const FGuid& ExposedEntityId) const;
	
	/** Get all types of exposed entities currently exposed. (ie. FRemoteControlActor) */
	const TSet<TObjectPtr<UScriptStruct>>& GetExposedEntityTypes() const;

	/** Returns true when Exposed Entities is populated. */
	const bool HasEntities() const;

	/** Returns whether an entity is exposed on the preset. */
	bool IsExposed(const FGuid& ExposedEntityId) const;

	/**
	 * Change the label of an entity.
	 * @param ExposedEntityId the id of the entity to rename.
	 * @param NewLabel the new label to assign to the entity.
	 * @return The assigned label, which might be suffixed if the label already exists in the registry, 
	 *         or NAME_None if the entity was not found.
	 */
	FName RenameExposedEntity(const FGuid& ExposedEntityId, FName NewLabel);

	/**
	 * Get the ID of an exposed entity using its label.
	 * @return an invalid guid if the exposed entity was not found.
	 */
	FGuid GetExposedEntityId(FName Label) const;

	/**
	 * Get a field ptr using it's id.
	 */
	UE_DEPRECATED(4.27, "Use GetExposedEntity<FRemoteControlField> instead.")
	TOptional<FRemoteControlField> GetField(FGuid FieldId) const;

	 /**
	  * Get an exposed function using its label.
	  */
	UE_DEPRECATED(4.27, "Use GetExposedEntity<FRemoteControlFunction> instead.")
	TOptional<FRemoteControlFunction> GetFunction(FName FunctionLabel) const;

	/** 
	 * Get an exposed property using its label.
	 */
	UE_DEPRECATED(4.27, "Use GetExposedEntity<FRemoteControlProperty> instead.")
	TOptional<FRemoteControlProperty> GetProperty(FName PropertyLabel) const;

	/**
	 * Get an exposed function using its id.
	 */
	UE_DEPRECATED(4.27, "Use GetExposedEntity<FRemoteControlFunction> instead.")
	TOptional<FRemoteControlFunction> GetFunction(FGuid FunctionId) const;

	/**
	 * Get an exposed property using its id.
	 */
	UE_DEPRECATED(4.27, "Use GetExposedEntity<FRemoteControlProperty> instead.")
	TOptional<FRemoteControlProperty> GetProperty(FGuid PropertyId) const;

	/**
	 * Rename a field.
	 * @param OldFieldLabel the target field's label.
	 * @param NewFieldLabel the field's new label.
	 */
	UE_DEPRECATED(4.27, "Use RenameExposedEntity instead.")
	void RenameField(FName OldFieldLabel, FName NewFieldLabel);

	/**
	 * Resolve a remote controlled property to its FProperty and owner objects.
	 * @param Alias the target's alias that contains the remote controlled property.
	 * @param PropertyLabel the label of the remote controlled property.
	 * @return The resolved exposed property if found.
	 */
	UE_DEPRECATED(4.27, "Use FRemoteControlProperty::GetProperty and FRemoteControlProperty::ResolveFieldOwners instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TOptional<FExposedProperty> ResolveExposedProperty(FName PropertyLabel) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Resolve a remote controlled function to its UFunction and owner objects.
	 * @param Alias the target's alias that contains the remote controlled function.
	 * @param FunctionLabel the label of the remote controlled function.
	 * @return The resolved exposed function if found.
	 */
	UE_DEPRECATED(4.27, "Use FRemoteControlFunction::ResolveFieldOwners instead.")
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TOptional<FExposedFunction> ResolveExposedFunction(FName FunctionLabel) const;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * Unexpose an entity from the preset.
	 * @param EntityLabel The label of the entity to unexpose.
	 */
	void Unexpose(FName EntityLabel);

	/**
	 * Unexpose an entity from the preset.
	 * @param EntityId the entity's id.
	 */
	void Unexpose(const FGuid& EntityId);

	/** Cache this preset's layout data. */
	void CacheLayoutData();

	/** Cache this preset's controllers labels. */
	void CacheControllersLabels() const;

	/** Resolves exposed property/function bounded objects */
	UE_DEPRECATED(4.27, "ResolvedBoundObjects is deprecated, you can now resolve bound objects using FRemoteControlEntity.")
	TArray<UObject*> ResolvedBoundObjects(FName FieldLabel);

	/**
	 * Attempt to rebind all currently unbound properties.
	 */
	void RebindUnboundEntities();
	
	/**
	 * Given a RC Entity, rebind all entities with the same owner to a new actor.
	 */
	void RebindAllEntitiesUnderSameActor(const FGuid& EntityId, AActor* NewActor, bool bUseRebindingContext = true);

	/**
	 * Renews all exposed entity guids. Necessary when duplicating.
	 */
	void RenewEntityIds();

	/**
	 * Renews all controller guids. Necessary when duplicating.
	 */
	void RenewControllerIds();

	/**
	 * Rename the given controller with the new name if the name is unique
	 * @param InControllerGuid Id of the controller to rename
	 * @param InNewName New name for the controller
	 * @return New name of the controller
	 */
	FName SetControllerDisplayName(FGuid InControllerGuid, const FName& InNewName) const;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresetEntityEvent, URemoteControlPreset* /*Preset*/, const FGuid& /*EntityId*/);
	FOnPresetEntityEvent& OnEntityExposed() { return OnEntityExposedDelegate; }
	FOnPresetEntityEvent& OnEntityUnexposed() { return OnEntityUnexposedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresetExposedPropertiesModified, URemoteControlPreset* /*Preset*/, const TSet<FGuid>& /* ModifiedProperties */);
	/**
	 *  Delegate called with the list of exposed property that were modified in the last frame.
	 */
	FOnPresetExposedPropertiesModified& OnExposedPropertiesModified() { return OnPropertyChangedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresetEntitiesUpdatedEvent, URemoteControlPreset* /*Preset*/, const TSet<FGuid>& /* Modified Entities */);
	/**
	 * Delegate called when the exposed entity wrapper itself is updated (ie. binding change, rename)  
	 */
	FOnPresetEntitiesUpdatedEvent& OnEntitiesUpdated() { return OnEntitiesUpdatedDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnEntityRebind, const FGuid&)
	FOnEntityRebind& OnEntityRebind() { return OnEntityRebindDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresetPropertyExposed, URemoteControlPreset* /*Preset*/, FName /*ExposedLabel*/);
	UE_DEPRECATED(4.27, "This delegate is deprecated, use OnEntityExposed instead.")
	FOnPresetPropertyExposed& OnPropertyExposed() { return OnPropertyExposedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPresetPropertyUnexposed, URemoteControlPreset* /*Preset*/, FName /*ExposedLabel*/);
	UE_DEPRECATED(4.27, "This delegate is deprecated, use OnEntityUnexposed instead.")
	FOnPresetPropertyUnexposed& OnPropertyUnexposed() { return OnPropertyUnexposedDelegate; }
	
	using FGuidToGuidMap = TMap<FGuid, FGuid>;	// Workaround for delegate macro.
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPropertyIdsRenewed, URemoteControlPreset* /*Preset*/, const FGuidToGuidMap& /*OldToNewEntityIds*/);
	FOnPropertyIdsRenewed& OnPropertyIdsRenewed() { return OnPropertyIdsRenewedDelegate;}
	
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnPresetFieldRenamed, URemoteControlPreset* /*Preset*/, FName /*OldExposedLabel*/, FName /**NewExposedLabel*/);
	FOnPresetFieldRenamed& OnFieldRenamed() { return OnPresetFieldRenamed; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPresetMetadataModified, URemoteControlPreset* /*Preset*/);
	FOnPresetMetadataModified& OnMetadataModified() { return OnMetadataModifiedDelegate; }

	DECLARE_MULTICAST_DELEGATE_FourParams(FOnActorPropertyModified, URemoteControlPreset* /*Preset*/, FRemoteControlActor& /*Actor*/, UObject* /*ModifiedObject*/, FProperty* /*MemberProperty*/);
	FOnActorPropertyModified& OnActorPropertyModified() { return OnActorPropertyModifiedDelegate; }

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnPresetLayoutModified, URemoteControlPreset* /*Preset*/);
	FOnPresetLayoutModified& OnPresetLayoutModified() { return OnPresetLayoutModifiedDelegate; }

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnControllerAdded, URemoteControlPreset* /*Preset*/, const FName /*NewControllerName*/, const FGuid& /*ControllerId*/);
	FOnControllerAdded& OnControllerAdded() { return OnControllerAddedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnControllerRemoved, URemoteControlPreset* /*Preset*/, const FGuid& /*ControllerId*/);
	FOnControllerRemoved& OnControllerRemoved() { return OnControllerRemovedDelegate; }

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnControllerRenamed, URemoteControlPreset* /*Preset*/, const FName /*OldLabel*/, const FName /*NewLabel*/);
	FOnControllerRenamed& OnControllerRenamed() { return OnControllerRenamedDelegate; }

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnControllerIdsRenewed, URemoteControlPreset* /*Preset*/, const FGuidToGuidMap& /*OldToNewControllerIds*/);
	FOnControllerIdsRenewed& OnControllerIdsRenewed() { return OnControllerIdsRenewedDelegate; }
	
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnControllerModified, URemoteControlPreset* /*Preset*/, const TSet<FGuid>& /*ModifiedControllerIds*/);
	FOnControllerModified& OnControllerModified() { return OnControllerModifiedDelegate; }
	
	UE_DEPRECATED(4.27, "This function is deprecated.")
	void NotifyExposedPropertyChanged(FName PropertyLabel);

	virtual void Serialize(FArchive& Ar) override;

	/**
	* Get entity name from given Desired or UObject + Field Path 
	* @param InDesiredName		Given name, it might be none
	* @param InObject			Bound object input
	* @param InFieldPath		Bound field path
	* @return Computing name of the entity
	*/
	FName GetEntityName(const FName InDesiredName, UObject* InObject, const FRCFieldPathInfo& InFieldPath) const;

	/** Generate label from Registry */
	FName GenerateUniqueLabel(const FName InDesiredName) const;

	/** Returns the preset's signature registry */
	URCSignatureRegistry* GetSignatureRegistry() const
	{
		return Signatures;
	}

public:
	/** The visual layout for this preset. */
	UPROPERTY()
	FRemoteControlPresetLayout Layout;

	/** This preset's metadata. */
	UPROPERTY()
	TMap<FString, FString> Metadata;

	/** This preset's list of objects that are exposed or that have exposed fields. */
	UPROPERTY(EditAnywhere, Category = "Remote Control Preset")
	TArray<TObjectPtr<URemoteControlBinding>> Bindings;

	/** This preset's user data */
	UPROPERTY()
	TArray<TObjectPtr<UObject>> UserData;

#if WITH_EDITOR
	/** Current selected world used only in the editor to let you switch world */
	TWeakObjectPtr<const UWorld> SelectedWorld = nullptr;
#endif
	/** ~~~Virtual Property Wrapper Functions ~~~
	* 
	* The goal is to hide the Controller Container and provide a simple interface for Controller access to UI and Web.
	*/

	/** Fetches a controller by internal property name. */
	URCVirtualPropertyBase* GetController(const FName InPropertyName) const;

	/** Fetches a controller by unique Id. */
	URCVirtualPropertyBase* GetController(const FGuid& InId) const;

	/** Fetches all controller */
	TArray<URCVirtualPropertyBase*> GetControllers() const;

	/** Fetches a virtual property by specified name. */
	URCVirtualPropertyBase* GetControllerByDisplayName(const FName InDisplayName) const;

	/** Fetches the first found Controller matching the specified Field Id */
	URCVirtualPropertyBase* GetControllerByFieldId(const FName InFieldId) const;

	/** Fetches the first Controller matching the specified Field Id and ValueType */
	URCVirtualPropertyBase* GetControllerByFieldId(const FName InFieldId, const EPropertyBagPropertyType InType) const;

	/** Fetches an array of all Controllers with the specified Field Id. */
	TArray<URCVirtualPropertyBase*> GetControllersByFieldId(const FName InFieldId) const;

	/** Fetches an array of all Controllers Types with the specified Field Id. */
	TArray<EPropertyBagPropertyType> GetControllersTypesByFieldId(const FName InFieldId) const;

	/** Adds a Virtual Property (Controller) to the Remote Control Preset */
	URCVirtualPropertyInContainer* AddController(TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject = nullptr, const FName InPropertyName = NAME_None);

	/** Returns true if the preset contains this controller. */
	bool HasController(const URCVirtualPropertyBase* InController) const;

	/** Removes a given Virtual Property (by Name) from the Remote Control preset */
	bool RemoveController(const FName& InPropertyName);

	/** Duplicates a given Virtual Property from the Remote Control preset */
	URCVirtualPropertyInContainer* DuplicateController(URCVirtualPropertyInContainer* InVirtualProperty);

	/** Removes all virtual properties held by this Remote Control preset*/
	void ResetControllers();

	/** Returns the number of Virtual Properties contained in this Remote Control preset*/
	int32 GetNumControllers() const;

	/** Returns the Struct On Scope of the Controller Container (value ptr of the virtual properties)*
	* Currently used by the UI class SRCControllerPanelList for user value entry via the RC Controllers panel*/
	TSharedPtr<FStructOnScope> GetControllerContainerStructOnScope();

	/** Returns the Property Bag Struct of this Container */
	URCVirtualPropertyContainerBase* GetControllerContainer() const
	{
		return ControllerContainer;
	}

	/** Sets the Controller Container (holds all Virtual Properties) */
	void SetControllerContainer(URCVirtualPropertyContainerBase* InControllerContainer);

	/** Checks whether the Controller Container (which holds all Virtual Properties) is currently valid*/
	bool IsControllerContainerValid()
	{
		return ControllerContainer != nullptr;
	}

#if WITH_EDITOR
	/** Called when a virtual property's value is being scrubbed by the user in the UI.
	* This call is routed to the Controller for evaluating associated Logic & Behaviours*/
	void OnNotifyPreChangeVirtualProperty(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Called when a virtual property is modified. This call is routed to the Controller for evaluating associated Logic & Behaviours*/
	void OnModifyController(const FPropertyChangedEvent& PropertyChangedEvent);
#endif

	FOnVirtualPropertyContainerModified& OnVirtualPropertyContainerModified() const;

private:
	/** Holds virtual controllers properties, behaviours and actions */
	UPROPERTY(Instanced)
	TObjectPtr<URCVirtualPropertyContainerBase> ControllerContainer;

	/** Holds information about an exposed field. */
	struct FExposeInfo
	{
		FName Alias;
		FGuid FieldId;
		FGuid LayoutGroupId;
	};

	//~ Callbacks called by the object targets
	void OnExpose(const FExposeInfo& Info);
	void OnUnexpose(FGuid UnexposedFieldId);
	
	//~ Cache operations.
	void CacheFieldLayoutData();
	
	//~ Register/Unregister delegates
	void RegisterDelegates();
	void UnregisterDelegates();
	
	//~ Keep track of any property change to notify if one of the exposed property has changed
	void OnObjectPropertyChanged(UObject* Object, FPropertyChangedEvent& Event);
	void OnPreObjectPropertyChanged(UObject* Object, const class FEditPropertyChain& PropertyChain);
	void OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionEvent);

	/** Fix controllers labels for older presets. */
	void FixAndCacheControllersLabels() const;

#if WITH_EDITOR	
	//~ Handle events that can incur bindings to be modified.
	void OnActorDeleted(AActor* Actor);
	void OnPieEvent(bool);
	void OnReplaceObjects(const TMap<UObject*, UObject*>& ReplacementObjectMap);
	void OnMapChange(uint32 /*MapChangeFlags*/);
	void OnBlueprintRecompiled(UBlueprint* Blueprint);

	/** Handles a package reloaded, used to detect a multi-user session being joined in order to update entities. */
	void OnPackageReloaded(EPackageReloadPhase Phase, FPackageReloadedEvent* Event);

	/** Remove deleted actors from bindings */
	void CleanUpBindings();
#endif

	//~ Frame events handlers.
	void OnBeginFrame();
	void OnEndFrame();

	/** Handles dynamic level change. */
	void OnMapLoadFinished(UWorld* LoadedWorld);
	
	/** Get a field ptr using it's id. */
	FRemoteControlField* GetFieldPtr(FGuid FieldId);

	//~ Helper methods that interact with the expose registry.,
	TSharedPtr<const FRemoteControlEntity> FindEntityById(const FGuid& EntityId, const UScriptStruct* EntityType = FRemoteControlEntity::StaticStruct()) const;
	TSharedPtr<FRemoteControlEntity> FindEntityById(const FGuid& EntityId, const UScriptStruct* EntityType = FRemoteControlEntity::StaticStruct());
	TArray<TSharedPtr<FRemoteControlEntity>> GetEntities(UScriptStruct* EntityType);
	TArray<TSharedPtr<const FRemoteControlEntity>> GetEntities(UScriptStruct* EntityType) const;

public:	
	/** Expose an entity in the registry. */
	TSharedPtr<FRemoteControlEntity> Expose(FRemoteControlEntity&& Entity, UScriptStruct* EntityType, const FGuid& GroupId);

	/**
	 * Calls to the PropertyIdRegistry PerformChainReaction to update the value(s) of the property(ies) that are bound to a PropertyIdAction.
	 * @param InArgs Argument used to update the property(ies) value.
	 */
	void PerformChainReaction(const FRemoteControlPropertyIdArgs& InArgs) const;

	/**
	 * Calls to the PropertyIdRegistry UpdateIdentifiedField to updates a field from the set of identified fields.
	 * @param InFieldToIdentify the entity to identify.
	 */
	void UpdateIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify) const;

	/** Try to get a binding and creates a new one if it doesn't exist. */
	URemoteControlBinding* FindOrAddBinding(const TSoftObjectPtr<UObject>& Object);

	/**
	 * Get the PropertyIdRegistry.
	 * @return The PropertyIdRegistry of this Preset.
	 */
	TObjectPtr<URemoteControlPropertyIdRegistry> GetPropertyIdRegistry() const { return PropertyIdRegistry; }

	/** Sets how this preset modifies properties when changed by protocols */
	void SetModifyOperationFlagsForProtocols(ERCModifyOperationFlags InOperationFlags) { ModifyOperationFlagsForProtocols = InOperationFlags; }

	/** Returns how this preset modifies properties when changed by protocols */
	ERCModifyOperationFlags GetModifyOperationFlagsForProtocols() const { return ModifyOperationFlagsForProtocols; }

	/** Updates the usage of entities by controllers. Specifically the number of controlling behaviors. */
	void UpdateEntityUsage();

	/** Returns a map of the number of times exposed entities are used by behaviors. */
	const TMap<FGuid, int32>& GetEntityUsage() const
	{
		return BoundEntityIds;
	}

	/** Returns true if an entity is bound to a controller/behavior. */
	bool IsEntityBound(const FGuid& InEntityGuid) const;

	/** Returns the number of behaviors an entity is bound to. */
	int32 GetEntityBoundCount(const FGuid& InEntityGuid) const;

	/** The display index to put it at the bottom of the list. */
	int32 GetNextControllerDisplayIndex() const;

private:

	/** Find a binding that has the same boundobjectmap but that currently points to the object passed as argument. */
	URemoteControlBinding* FindMatchingBinding(const URemoteControlBinding* InBinding, UObject* InObject);

	/** Handler called upon an entity being modified. */
	void OnEntityModified(const FGuid& EntityId);

	/** Initialize entities metadata based on the module's externally registered initializers. */
	void InitializeEntitiesMetadata();
	
	/** Initialize an entity's metadata based on the module's externally registered initializers. */
	void InitializeEntityMetadata(const TSharedPtr<FRemoteControlEntity>& Entity);
	
	/** Initialize an controllers's metadata based on the module's externally registered initializers. */
	void InitializeEntityMetadata(URCVirtualPropertyBase* Controller);

	/** Register delegates for all exposed entities. */
	void RegisterEntityDelegates();

	/** Register an event triggered when the exposed function's owner blueprint is compiled. */
	void RegisterOnCompileEvent(const TSharedPtr<FRemoteControlFunction>& RCFunction);

	/** Create a property watcher that will trigger the property change delegate upon having a different value from the last frame. */
	void CreatePropertyWatcher(const TSharedPtr<FRemoteControlProperty>& RCProperty);

	/** Returns whether a given property should have a watcher that checks for property changes across frames. */
	bool PropertyShouldBeWatched(const TSharedPtr<FRemoteControlProperty>& RCProperty) const;

	/** Create property watchers for exposed properties that need them. */
	void CreatePropertyWatchers();

	/** Remove bindings that do not have properties pointing to them. */
	void RemoveUnusedBindings();

	/** Call post load function for exposed properties. */
	void PostLoadProperties();

	/** Handle a change on a ndisplay config data, used to replace bindings. */
	void HandleDisplayClusterConfigChange(UObject* DisplayClusterConfigData);
	
private:
	/** Preset unique ID */
	UPROPERTY(AssetRegistrySearchable)
	FGuid PresetId;

	/** Defines how this preset modifies properties when changed by protocols */
	UPROPERTY()
	ERCModifyOperationFlags ModifyOperationFlagsForProtocols = ERCModifyOperationFlags::SkipPropertyChangeEvents | ERCModifyOperationFlags::SkipTransactions;

	/** The cache for information about an exposed field. */
	UPROPERTY(Transient)
	TMap<FGuid, FRCCachedFieldData> FieldCache;

	/** Map of Field Name to GUID. */
	UPROPERTY(Transient)
	TMap<FName, FGuid> NameToGuidMap;

	/** Holds exposed entities on the preset. */
	UPROPERTY(Instanced)
	TObjectPtr<URemoteControlExposeRegistry> Registry = nullptr;

	/** Holds all signatures on the preset. */
	UPROPERTY(Instanced)
	TObjectPtr<URCSignatureRegistry> Signatures = nullptr;

	/** Holds identities of exposed entities on the preset. */
	UPROPERTY(Transient)
	TObjectPtr<URemoteControlPropertyIdRegistry> PropertyIdRegistry = nullptr;

	/** Delegate triggered when an entity is exposed. */
	FOnPresetEntityEvent OnEntityExposedDelegate;
	/** Delegate triggered when an entity is unexposed from the preset. */
	FOnPresetEntityEvent OnEntityUnexposedDelegate;
	/** Delegate triggered when entities are modified and may need to be re-resolved. */
	FOnPresetEntitiesUpdatedEvent OnEntitiesUpdatedDelegate;
	/** Delegate triggered when an Entity is rebound */
	FOnEntityRebind OnEntityRebindDelegate;
	/** Delegate triggered when an exposed property value has changed. */
	FOnPresetExposedPropertiesModified OnPropertyChangedDelegate;
	/** Delegate triggered when a new property has been exposed. */
	FOnPresetPropertyExposed OnPropertyExposedDelegate;
	/** Delegate triggered when a property has been unexposed. */
	FOnPresetPropertyUnexposed OnPropertyUnexposedDelegate;
	/** Delegate triggered when the property guids are renewed. */
	FOnPropertyIdsRenewed OnPropertyIdsRenewedDelegate;
	/** Delegate triggered when a field has been renamed. */
	FOnPresetFieldRenamed OnPresetFieldRenamed;
	/** Delegate triggered when the preset's metadata has been modified. */
	FOnPresetMetadataModified OnMetadataModifiedDelegate;
	/** Delegate triggered when an exposed actor's property is modified. */
	FOnActorPropertyModified OnActorPropertyModifiedDelegate;
	/** Delegate triggered when the layout is modified. */
	FOnPresetLayoutModified OnPresetLayoutModifiedDelegate;
	/** Delegate triggered when a controller is added */
	FOnControllerAdded OnControllerAddedDelegate;
	/** Delegate triggered when a controller is removed */
	FOnControllerRemoved OnControllerRemovedDelegate;
	/** Delegate triggered when a Controller is renamed */
	FOnControllerRenamed OnControllerRenamedDelegate;
	/** Delegate triggered when a Controller guids are renewed. */
	FOnControllerIdsRenewed OnControllerIdsRenewedDelegate;
	/** Delegate triggered when a Controller has been changed */
	FOnControllerModified OnControllerModifiedDelegate;

	struct FPreObjectsModifiedCache
	{
		TArray<UObject*> Objects;
		FProperty* Property;
		FProperty* MemberProperty;
	};

	/** Caches object modifications during a frame. */
	TMap<FGuid, FPreObjectsModifiedCache> PreObjectsModifiedCache;
	TMap<FGuid, FPreObjectsModifiedCache> PreObjectsModifiedActorCache;

	/** A struct representing a material object modified that is housed by a container (eg. array, set, map). */
	struct FPreMaterialModifiedCache
	{
		int32 ArrayIndex;
		bool bHadValue;
	};

	/** Caches material modifications during a frame. */
	TMap<FGuid, FPreMaterialModifiedCache> PreMaterialModifiedCache;

	/** Cache properties that were modified during a frame. */
	TSet<FGuid> PerFrameModifiedProperties;
	
	/** Cache entities updated during a frame. */
	TSet<FGuid> PerFrameUpdatedEntities; 

	/** Whether there is an ongoing remote modification happening. */
	bool bOngoingRemoteModification = false;

	/** Used for OnObjectPropertyChanged to avoid being called by itself. */
	bool bReentryGuard = false;

	/** Whether OnPostInitPropertiesRemoteControlProperties delegate is pending broadcast because it wasn't broadcast in URemoteControlPreset::PostInitProperties() */
	bool bPendingPostInitNotify = false;

	/** Transient list of the currently bound entity ids and the number of times they are bound. */
	TMap<FGuid, int32> BoundEntityIds;

	/** Holds manager that handles rebinding unbound entities upon load or map change. */
	TPimplPtr<FRemoteControlPresetRebindingManager> RebindingManager;

	/**
	 * Property watcher that triggers a delegate when the watched property is modified.
	 */
	struct FRCPropertyWatcher
	{
		FRCPropertyWatcher(const TSharedPtr<FRemoteControlProperty>& InWatchedProperty, FSimpleDelegate&& InOnWatchedValueChanged);

		/** Checks if the property value has changed since the last frame and updates the last frame value. */
		void CheckForChange();

	private:
		/** Optionally resolve the property path if possible. */
		TOptional<FRCFieldResolvedData> GetWatchedPropertyResolvedData() const;
		/** Store the latest property value in LastFrameValue. */
		void SetLastFrameValue(const FRCFieldResolvedData& ResolvedData);
	private:
		/** Delegate called when the watched property changes values. */
		FSimpleDelegate OnWatchedValueChanged;
		/** Weak pointer to the remote control property. */
		TWeakPtr<FRemoteControlProperty> WatchedProperty;
		/** Latest property value as bytes. */
		TArray<uint8> LastFrameValue;
	};
	/** Map of property watchers that should trigger the RC property change delegate upon change. */
	TMap<FGuid, FRCPropertyWatcher> PropertyWatchers;

	/** Frame counter for delaying property change checks. */
	int32 PropertyChangeWatchFrameCounter = 0;

public:

	static UWorld* GetWorld(const URemoteControlPreset* Preset = nullptr, bool bAllowPIE = false);
	UWorld* GetWorld(bool bAllowPIE = false) const;
	UWorld* GetEmbeddedWorld() const;
	
	/** Returns true if the preset is hosted within another asset. */
	bool IsEmbeddedPreset() const;

#if WITH_EDITOR
	const TArray<FName>& GetDetailsTabIdentifierOverrides() const { return DetailsTabIdentifierOverrides; }
	void SetDetailsTabIdentifierOverrides(const TArray<FName>& NewOverrides) { DetailsTabIdentifierOverrides = NewOverrides; }
#endif

	/**
	 * Transient value for the default category for new properties.
	 * Not automatically set by the preset. Used when adding controllers.
	 */
	FString NewControllerCategory;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<FName> DetailsTabIdentifierOverrides;
#endif

#if WITH_EDITOR
	/** List of blueprints for which we have registered events. */
	TSet<TWeakObjectPtr<UBlueprint>> BlueprintsWithRegisteredDelegates;

	/** List of bindings for which we need to remove stale object pointers. */
	TSet<URemoteControlBinding*> PerFrameBindingsToClean;
#endif

	friend FRemoteControlPresetLayout;
	friend FRemoteControlEntity;
	friend class FRemoteControlPresetRebindingManager;
};
