// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "Engine/Engine.h"
#include "RemoteControlPresetActor.generated.h"

/**
 * Actor that wraps a remote control preset, allows linking a specific preset to a level.
 */
UCLASS(HideCategories = (Rendering, Physics, LOD, Activation, Input, Replication, Collision, Actor, Cooking))
class REMOTECONTROL_API ARemoteControlPresetActor : public AActor
{
public:
	GENERATED_BODY()

	ARemoteControlPresetActor();

public:
	UPROPERTY(EditAnywhere, Category="General")
	TObjectPtr<URemoteControlPreset> Preset;

private:
	/** Delegate handle for actor deletion callback. */
	FDelegateHandle OnActorDeletedDelegateHandle;
};