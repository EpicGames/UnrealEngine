// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlFieldPath.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#include "RemoteControlPropertyIdRegistry.generated.h"

class FProperty;
class UClass;
class URCVirtualPropertySelfContainer;
class URemoteControlPreset;
struct FGuid;
struct FRemoteControlField;
struct FRemoteControlPropertyIdArgs;
struct FRemoteControlProperty;
template <class CPPSTRUCT>
struct TStructOpsTypeTraitsBase2;

/** Wrapper class used to serialize identities of exposable entities in a generic way. */
USTRUCT()
struct FRCPropertyIdWrapper
{
	GENERATED_BODY()

	FRCPropertyIdWrapper() = default;

	FRCPropertyIdWrapper(const TSharedRef<FRemoteControlProperty>& InRCProperty);

	/** Get the type of the wrapped entity.
	 * @return Entity Id of this PropertyIdWrapper.
	 */
	const FGuid& GetEntityId() const;

	/** Get the type of the wrapped entity.
	 * @return PropertyId of this PropertyIdWrapper.
	 */
	FName GetPropertyId() const;

	/** Get the type of the wrapped entity.
	 * @param InNewPropertyId The new PropertyId to assign to this PropertyIdWrapper.
	 */
	void SetPropertyId(FName InNewPropertyId);

	/** Get the super type of the wrapped property.
	 * @return The super type of this PropertyIdWrapper.
	 */
	const FName& GetSuperType() const;

	/** Get the sub type of the wrapped entity.
	 * @return The sub type of this PropertyIdWrapper.
	 */
	const FName& GetSubType() const;

	/** Returns whether the type and the underlying data is valid.
	 * @return True if EntityId and either SuperType or SubType is valid, false otherwise.
	 */
	bool IsValid() const;

	/** Returns whether the type and the underlying data is valid.
	 * @return True if the PropertyId is different from the default value.
	 */
	bool IsValidPropertyId() const;

public:
	friend uint32 GetTypeHash(const FRCPropertyIdWrapper& Wrapper);

	bool operator==(const FGuid& WrappedId) const;

	bool operator==(const FRCPropertyIdWrapper& Other) const;

private:
	void UpdateTypes(const TSharedRef<FRemoteControlProperty>& InRCProperty, UObject* InOwner, FProperty* InProperty);

private:
	/** Entity identifier of the remote control property. */
	UPROPERTY()
	FGuid EntityId;

	/** Holds the field identifier of the remote control property. */
	UPROPERTY()
	FName PropertyId;

	/** Holds the type of the wrapped property. */
	UPROPERTY()
	FName SuperType;

	/** Holds the type of the wrapped object. */
	UPROPERTY()
	FName SubType;

	friend class URemoteControlPropertyIdRegistry;
};

template<> struct TStructOpsTypeTraits<FRCPropertyIdWrapper> : public TStructOpsTypeTraitsBase2<FRCPropertyIdWrapper>
{
	enum
	{
		WithSerializer = false,
		WithIdenticalViaEquality = true
	};
};

/**
 *	Holds a Registry of the PropertyId PropertyId
 */
UCLASS()
class REMOTECONTROL_API URemoteControlPropertyIdRegistry : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Initialize the Registry with the given Preset.
	 */
	void Initialize();

	URemoteControlPreset* GetSourcePreset() const;

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif

	/**
	 * Update the value(s) of the property(ies) that are bound to a PropertyIdAction.
	 * @param InArgs Argument used to update the property(ies) value.
	 */
	void PerformChainReaction(const FRemoteControlPropertyIdArgs& InArgs);

	/**
	 * Add a field to the set of identified fields.
	 * @param InFieldToIdentify the entity to identify.
	 */
	void AddIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify);

	/**
	 * Check if there is any PropertyId Field.
	 * @return True if the Registry is Empty, false otherwise.
	 */
	bool IsEmpty() const;

	/**
	 * Updates a field from the set of identified fields.
	 * @param InFieldToIdentify the entity to identify.
	 */
	void UpdateIdentifiedField(const TSharedRef<FRemoteControlField>& InFieldToIdentify);

	/**
	 * Remove an identified field from the registry using its id.
	 * @param InEntityId Id of the Property to remove.
	 */
	void RemoveIdentifiedField(const FGuid& InEntityId);

	/**
	 * Get the list of existing Property Ids.
	 * @return The list of existing Property Ids that has a valid FieldId.
	 */
	TSet<FName> GetFieldIdsNameList() const;

	/**
	 * Returns all the possible Ids that can be used.
	 *
	 * @return A Set with the possible Ids to use.
	 */
	TSet<FName> GetFullPropertyIdsNamePossibilitiesList() const;

	/**
	 * Return all EntityIds that has the given PropertyId.
	 * @param InPropertyId Property Id to match.
	 * @return The list of existing EntityIds that has the given PropertyId.
	 */
	TSet<FGuid> GetEntityIdsForPropertyId(const FName& InPropertyId) const;

	/**
	 * Get the list of existing Property Ids.
	 * @return The list of existing Property Ids that has a valid FieldId.
	 */
	TSet<FGuid> GetEntityIdsList();

	/**
	 * Returns whether the 2 Ids are equal or if the Target one is contained in the Container one.\n
	 * ex: 100.A and 100 will return true but not the other way around.
	 *
	 * @param InContainerPropertyId Container PropertyId, will create sub-string from this to later check if the target one is equal to one of them.
	 * @param InTargetPropertyId Target Id to check if it is part of the container one.
	 * @return True if the PropertyIds are equal or if the Target one is contained in at least 1 sub-string of the Container one.
	 */
	bool Contains(FName InContainerPropertyId, FName InTargetPropertyId) const;

	/**
	 * Returns the possible Ids based on given one.\n
	 * ex: 100.A.B will return 100 and 100.A and 100.A.B
	 *
	 * @param InPropId Property Id used to generate all the other Ids that can be used with this one.
	 * @return A Set with the possible Ids to use.
	 */
	TSet<FName> GetPossiblePropertyIds(FName InPropId) const;

	DECLARE_MULTICAST_DELEGATE(FOnPropertyIdUpdated)
	FOnPropertyIdUpdated& OnPropertyIdUpdated() { return OnPropertyIdUpdatedDelegate; }

#if WITH_EDITOR
	/** Callback for refreshing the UI when an identity is updated. */
	DECLARE_MULTICAST_DELEGATE(FOnPropertyIdActionNeedsRefresh)
	FOnPropertyIdActionNeedsRefresh& OnPropertyIdActionNeedsRefresh() { return OnPropertyIdActionNeedsRefreshDelegate; };
#endif

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap);

private:
	/** Holds the identified fields. */
	UPROPERTY(Transient)
	TSet<FRCPropertyIdWrapper> IdentifiedFields;

	/** Delegate triggered when a property has its field Id changed. */
	FOnPropertyIdUpdated OnPropertyIdUpdatedDelegate;

#if WITH_EDITOR
	/** Delegate triggered when PropertyID UI needs to refresh. */
	FOnPropertyIdActionNeedsRefresh OnPropertyIdActionNeedsRefreshDelegate;
#endif // WITH_EDITOR
};
