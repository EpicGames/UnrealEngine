// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/UnrealString.h"
#include "RCModifyOperationFlags.h"
#include "RCPropertyUtilities.h"
#include "RCTypeTraits.h"
#include "RemoteControlCommon.h"
#include "UObject/Class.h"
#include "UObject/StructOnScope.h"

#include "RemoteControlProtocolBinding.generated.h"

class FCborWriter;
class URemoteControlPreset;

struct FRemoteControlProtocolBinding;
struct FRemoteControlProtocolEntity;
struct FRemoteControlProtocolMapping;
struct FExposedProperty;
struct FRCObjectReference;

using FGetProtocolMappingCallback = TFunctionRef<void(FRemoteControlProtocolMapping&)>;

#if WITH_EDITOR

#define EXPOSE_PROTOCOL_PROPERTY(ColumnName, ClassName, MemberName) \
	if (!ColumnsToProperties.Contains(ColumnName)) \
	{	\
		ColumnsToProperties.Add(ColumnName, GET_MEMBER_NAME_CHECKED(ClassName, MemberName)); \
	}

#endif // WITH_EDITOR

/**
 * Status of the binding 
 */
UENUM()
enum class ERCBindingStatus : uint8
{
	Unassigned,
	Awaiting,
	Bound
};

/*
 * Mapping of the range of the values for the protocol
 * This class holds a generic range buffer.
 * For example, it could be FFloatProperty 4 bytes
 * Or it could be any UScripStruct, like FVector - 12 bytes
 * Or any custom struct, arrays, maps, sets, or primitive properties
 */
USTRUCT()
struct REMOTECONTROL_API FRemoteControlProtocolMapping
{
	GENERATED_BODY()

	friend struct FRemoteControlProtocolBinding;
	friend struct FRemoteControlProtocolEntity;

	friend uint32 REMOTECONTROL_API GetTypeHash(const FRemoteControlProtocolMapping& InProtocolMapping);

public:
	FRemoteControlProtocolMapping() = default;
	FRemoteControlProtocolMapping(FProperty* InProperty, uint8 InRangeValueSize, const FGuid& InMappingId = FGuid::NewGuid());

	bool operator==(const FRemoteControlProtocolMapping& InProtocolMapping) const;
	bool operator==(FGuid InProtocolMappingId) const;

public:
	/** Get Binding Range Id. */
	const FGuid& GetId() const { return Id; }

	/** Get validity of the mapping (based on the Id) */
	bool IsValid() const { return Id.IsValid(); }

	/** Get Binding Range Value */
	template <typename ValueType>
	ValueType GetRangeValue()
	{
		check(TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType());
		check(InterpolationRangePropertyData.Num() && InterpolationRangePropertyData.Num() == sizeof(ValueType));
		return *reinterpret_cast<ValueType*>(InterpolationRangePropertyData.GetData());
	}

	/** Get Binding Range Struct Value */
	template <typename ValueType, typename PropertyType>
	typename TEnableIf<std::is_same_v<FStructProperty, PropertyType>, ValueType>::Type
	GetRangeValue()
	{
		check(InterpolationRangePropertyData.Num() && InterpolationRangePropertyData.Num() == sizeof(ValueType))
		return *reinterpret_cast<ValueType*>(InterpolationRangePropertyData.GetData());
	}

	/** Set Binding Range Value based on template value input */
	template <typename ValueType>
	void SetRangeValue(ValueType InRangeValue)
	{
		check(TRemoteControlTypeTraits<ValueType>::IsSupportedRangeType());
		check(InterpolationRangePropertyData.Num() && InterpolationRangePropertyData.Num() == sizeof(ValueType));
		*reinterpret_cast<ValueType*>(InterpolationRangePropertyData.GetData()) = InRangeValue;
	}

	/** Set Binding Range Struct Value based on templated value input */
	template <typename ValueType, typename PropertyType>
	typename TEnableIf<std::is_same_v<FStructProperty, PropertyType>, void>::Type
	SetRangeValue(ValueType InRangeValue)
	{
		check(InterpolationRangePropertyData.Num() && InterpolationRangePropertyData.Num() == sizeof(ValueType));
		*reinterpret_cast<ValueType*>(InterpolationRangePropertyData.GetData()) = InRangeValue;

		RefreshCachedData(PropertyType::StaticClass()->GetFName());
	}

	/** Returns true if one can get mapping value as primitive for the specified type */
	template <typename ValueType>
	bool CanGetMappingValueAsPrimitive() const
	{
		if constexpr (RemoteControlTypeTraits::TIsStringLikeValue<ValueType>::Value)
		{
			return TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType();
		}
		else
		{
			return 
				TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType() &&
				!InterpolationMappingPropertyData.IsEmpty() &&
				InterpolationMappingPropertyData.Num() == sizeof(ValueType);
		}
	}

	/** Get Mapping Property Value as a primitive type */
	template <typename ValueType>
	typename TEnableIf<!RemoteControlTypeTraits::TIsStringLikeValue<ValueType>::Value, ValueType>::Type
	GetMappingValueAsPrimitive()
	{
		check(TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType());
		check(InterpolationMappingPropertyData.Num() && InterpolationMappingPropertyData.Num() == sizeof(ValueType));
		return *reinterpret_cast<ValueType*>(InterpolationMappingPropertyData.GetData());
	}

	/** Get Mapping Property String-like Value as a primitive type */
	template <typename ValueType>
	typename TEnableIf<RemoteControlTypeTraits::TIsStringLikeValue<ValueType>::Value, ValueType>::Type
	GetMappingValueAsPrimitive()
	{
		check(TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType());
		return *reinterpret_cast<ValueType*>(InterpolationMappingPropertyData.GetData());
	}

	/** Get Mapping Property Struct Value as a primitive type */
	template <typename ValueType, typename PropertyType>
	typename TEnableIf<std::is_same_v<FStructProperty, PropertyType>, ValueType>::Type
	GetMappingValueAsPrimitive()
	{
		check(InterpolationMappingPropertyData.Num() && InterpolationMappingPropertyData.Num() == sizeof(ValueType));
		return *reinterpret_cast<ValueType*>(InterpolationMappingPropertyData.GetData());
	}

	/** Set primitive Mapping Property Value based on template value input */
	template <typename ValueType>
	void SetMappingValueAsPrimitive(ValueType InMappingPropertyValue)
	{
		check(TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType());
		check(InterpolationMappingPropertyData.Num() && InterpolationMappingPropertyData.Num() == sizeof(ValueType));
		*reinterpret_cast<ValueType*>(InterpolationMappingPropertyData.GetData()) = InMappingPropertyValue;

		RefreshCachedData(NAME_None);
	}

	/** Set primitive Mapping Property Struct Value based on template value input */
	template <typename ValueType, typename PropertyType>
	typename TEnableIf<std::is_same_v<FStructProperty, PropertyType>, void>::Type
	SetMappingValueAsPrimitive(ValueType InMappingPropertyValue)
	{
		check(InterpolationMappingPropertyData.Num() && InterpolationMappingPropertyData.Num() == sizeof(ValueType));
		*reinterpret_cast<ValueType*>(InterpolationMappingPropertyData.GetData()) = InMappingPropertyValue;

		RefreshCachedData(NAME_None);
	}

#if WITH_EDITOR
	/** Copies the underlying InterpolationRangePropertyData to the given destination, using the input property for type information */
	template <typename PropertyType>
	bool CopyRawRangeData(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Src{InPropertyHandle->GetProperty(), InterpolationRangePropertyData};
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InPropertyHandle};
		if(RemoteControlPropertyUtilities::Deserialize<PropertyType>(Src, Dst))
		{
			RefreshCachedData(InPropertyHandle->GetPropertyClass()->GetFName());
			return true;
		}
		return false;
	}

	/** Sets the underlying InterpolationRangePropertyData to the given source, using the input property for type information */
	template <typename PropertyType>
	bool SetRawRangeData(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InPropertyHandle->GetProperty(), InterpolationRangePropertyData};
		if(RemoteControlPropertyUtilities::Serialize<PropertyType>(InPropertyHandle, Dst))
		{
			RefreshCachedData(InPropertyHandle->GetPropertyClass()->GetFName());
			return true;
		}
		return false;
	}

	/** Sets the underlying InterpolationRangePropertyData to the given source, using the input property for type information */
	bool SetRawRangeData(URemoteControlPreset* InOwningPreset, const FProperty* InProperty, const void* InSource)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, InterpolationRangePropertyData};
		if(RemoteControlPropertyUtilities::Serialize<FProperty>({InProperty, InSource}, Dst))
		{
			RefreshCachedData(InProperty-> GetFName());
			return true;
		}
		return false;
	}

	/** Copies the underlying InterpolationMappingPropertyData to the given destination, using the input property for type information */
	template <typename PropertyType>
	bool CopyRawMappingData(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Src{InPropertyHandle->GetProperty(), InterpolationMappingPropertyData, InterpolationMappingPropertyElementNum};
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InPropertyHandle};
		if(RemoteControlPropertyUtilities::Deserialize<PropertyType>(Src, Dst))
		{
			InterpolationMappingPropertyElementNum = Dst.Num();
			RefreshCachedData(NAME_None);
			return true;
		}
		return false;
	}

	/** Sets the underlying InterpolationMappingPropertyData to the given source, using the input property for type information */
	template <typename PropertyType>
	bool SetRawMappingData(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InPropertyHandle->GetProperty(), InterpolationMappingPropertyData};
		if (RemoteControlPropertyUtilities::Serialize<PropertyType>(InPropertyHandle, Dst))
		{
			InterpolationMappingPropertyElementNum = Dst.Num();
			RefreshCachedData(NAME_None);
			return true;
		}
		return false;
	}
#endif

	/** Sets the underlying InterpolationMappingPropertyData to the given source, using the input property for type information */
	bool SetRawMappingData(URemoteControlPreset* InOwningPreset, const FProperty* InProperty, const void* InSource)
	{
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, InterpolationMappingPropertyData};
		if (RemoteControlPropertyUtilities::Serialize<FProperty>({InProperty, InSource}, Dst))
		{
			InterpolationMappingPropertyElementNum = Dst.Num();
			RefreshCachedData(NAME_None);
			return true;
		}
		return false;
	}

	/** Get Mapping value as a Struct on Scope, only in case BoundProperty is FStructProperty */
	TSharedPtr<FStructOnScope> GetMappingPropertyAsStructOnScope();

private:
	/** Checks that the source data matches the expected size of the property */
	template <typename PropertyType>
	bool PropertySizeMatchesData(const TArray<uint8>& InSource, const PropertyType* InProperty);

	/** Checks that the source data matches the expected size of the property */
	bool PropertySizeMatchesData(const TArray<uint8>& InSource, const FName& InPropertyTypeName);

	/** Refreshes (deserializes) mapping data to cache. If the RangePropertyTypeName is NAME_None, it is not set. */
	void RefreshCachedData(const FName& InRangePropertyTypeName);

private:
	/** Unique Id of the current binding */
	UPROPERTY()
	FGuid Id;

	/**
	 * The current integer value of the mapping range. It could be a float, int32, uint8, etc.
	 * That is based on protocol input data.
	 * 
	 *  For example, it could be uint8 in the case of one-byte mapping. In this case, the range value could be from 0 up to 255, which bound to InterpolationMappingPropertyData
	 */
	UPROPERTY()
	TArray<uint8> InterpolationRangePropertyData;

	/** 
	 * Holds the serialized mapped property data. 
	 * It could be a primitive value for FNumericProperty, or a text representation for a struct.
	 */
	UPROPERTY()
	TArray<uint8> InterpolationMappingPropertyData;

	/** 
	* Holds the range property data buffer. 
	* This is the cached raw data as opposed to the serialized data.
	*/
	UPROPERTY(Transient)
	TArray<uint8> InterpolationRangePropertyDataCache;

	/** 
	* Holds the mapped property data buffer. 
	* This is the cached raw data as opposed to the serialized data, so it doesn't contain type information.
	*/
	UPROPERTY(Transient)
	TArray<uint8> InterpolationMappingPropertyDataCache;

	/** The element count if the Mapping is an array. */
	UPROPERTY()
	int32 InterpolationMappingPropertyElementNum = 1;

	/** Holds the bound property path */
	UPROPERTY()
	TFieldPath<FProperty> BoundPropertyPath;
};

/** Set primitive Mapping Property String Value based on template value input */
template <>
inline void FRemoteControlProtocolMapping::SetMappingValueAsPrimitive<FString>(FString InMappingPropertyValue)
{
	check(TRemoteControlTypeTraits<FString>::IsSupportedMappingType());

	const FString Value = InMappingPropertyValue;
	TArray<uint8> Buffer;
	StringToBytes(Value, Buffer.GetData(), TNumericLimits<int16>::Max());
	InterpolationMappingPropertyData = Buffer;
}

/** Set primitive Mapping Property Name Value based on template value input */
template <>
inline void FRemoteControlProtocolMapping::SetMappingValueAsPrimitive<FName>(FName InMappingPropertyValue)
{
	check(TRemoteControlTypeTraits<FName>::IsSupportedMappingType());

	SetMappingValueAsPrimitive(InMappingPropertyValue.ToString());
}

/** Set primitive Mapping Property Text Value based on template value input */
template <>
inline void FRemoteControlProtocolMapping::SetMappingValueAsPrimitive<FText>(FText InMappingPropertyValue)
{
	check(TRemoteControlTypeTraits<FText>::IsSupportedMappingType());

	SetMappingValueAsPrimitive(InMappingPropertyValue.ToString());
}


template <>
inline bool FRemoteControlProtocolMapping::PropertySizeMatchesData<FBoolProperty>(const TArray<uint8>& InSource, const FBoolProperty* InProperty)
{
	return ensure(InProperty->GetElementSize() == InSource.Num());
}

template <>
inline bool FRemoteControlProtocolMapping::PropertySizeMatchesData<FNumericProperty>(const TArray<uint8>& InSource, const FNumericProperty* InProperty)
{
	return ensure(InProperty->GetElementSize() >= InSource.Num());
}

template <>
inline bool FRemoteControlProtocolMapping::PropertySizeMatchesData<FStructProperty>(const TArray<uint8>& InSource, const FStructProperty* InProperty)
{
	UScriptStruct* ScriptStruct = InProperty->Struct;
	return ensure(ScriptStruct->GetStructureSize() == InSource.Num());
}

template <typename PropertyType>
bool FRemoteControlProtocolMapping::PropertySizeMatchesData(const TArray<uint8>& InSource, const PropertyType* InProperty)
{
	return true;
}

/**
 * These structures serve both as properties mapping as well as UI generation
 * Protocols should implement it based on the parameters they need.
 */
USTRUCT()
struct REMOTECONTROL_API FRemoteControlProtocolEntity
{
	GENERATED_BODY()

public:
	virtual ~FRemoteControlProtocolEntity() {}

	/**
	 * Initialize after allocation
	 * @param InOwner The preset that owns this entity.
	 * @param InPropertyId exposed property id.
	 */
	void Init(URemoteControlPreset* InOwner, FGuid InPropertyId);

	/** Get exposed property id */
	const FGuid& GetPropertyId() const { return PropertyId; }

	/**
	 * Interpolate and apply protocol value to the property
	 * 
	 * @param InProtocolValue				Double value from the protocol
	 * @param ModifyOperationFlags			(optional) Flags that specify how the property is modified when the value is applied.
	 * @return								True if the value was applied successfully
	 */
	UE_DEPRECATED(5.5, "ApplyProtocolValueToProperty is deprecated. Instead please refer to UE::RemoteControl::ProtocolEntityProcessor::ProcessEntities")
	bool ApplyProtocolValueToProperty(const double InProtocolValue);

	/** 
	 * Get bound range property. For example, the range could be bound to FFloatProperty or FIntProperty, etc.
	 * It could be 0.f bound to FStructProperty or 0 bound to FBoolProperty where 0.f or 0 are Range PropertyName
	 * Each protocol defines its own binding property type.
	 */
	virtual FName GetRangePropertyName() const { return NAME_None; }

	/** Get Size of the range property value */
	virtual uint8 GetRangePropertySize() const;

	/** Get the upper most value for the range property type, used for normalization */
	virtual const FString& GetRangePropertyMaxValue() const;

	/** Get the state of the binding. */
	bool IsEnabled() const { return bIsEnabled; }

	/** Get current binding state of the entity */
	ERCBindingStatus GetBindingStatus() const { return BindingStatus; }

	/**
	 * Toggle the binding
	 * @return Current binding state
	 */
	ERCBindingStatus ToggleBindingStatus();

	/** Reset the binding set to default */
	void ResetDefaultBindingState();

	/** Set to enable or disable input. */
	void SetEnabled(bool bInEnable) { bIsEnabled = bInEnable; }

	/**
	 * Checks if this entity has the same values as the Other.
	 * Used to check for duplicate inputs.
	 */
	virtual bool IsSame(const FRemoteControlProtocolEntity* InOther) { return false; }

	/** Masking Support */

	/** Disables the given mask. */
	virtual void ClearMask(ERCMask InMaskBit);
	/** Enables the given mask. */
	virtual void EnableMask(ERCMask InMaskBit);
	/** Returns true if the given mask is enabled, false otherwise. */
	virtual bool HasMask(ERCMask InMaskBit) const;

	/** Returns the overriden masks. */
	ERCMask GetOverridenMask() const { return static_cast<ERCMask>(OverridenMasks); }

#if WITH_EDITOR
	/** Retrieves the name of property corresponding to the given column name. */
	const FName GetPropertyName(const FName& ForColumnName);
	
	/** Register(s) all the properties (to be exposed) of this protocol entity. */
	virtual void RegisterProperties() {};
#endif // WITH_EDITOR

	/** Returns the preset that owns this protocol entity */
	const TWeakObjectPtr<URemoteControlPreset>& GetOwner() const { return Owner; }

	/** Container for range and mapping value pointers, and an optional number of elements (arrays, strings). */
	struct FRangeMappingData
	{
		TArray<uint8> Range;
		TArray<uint8> Mapping;

		/** Number of elements represented in the mapping data (for arrays, etc.). */
		mutable int32 NumElements = 1;

		FRangeMappingData() = default;

		FRangeMappingData(const uint8* InRange,
						const uint8* InMapping,
						int32 InRangeSize = sizeof(uint8),
						int32 InMappingSize = sizeof(uint8),
						int32 InNumMappingElements = 1)
			: NumElements(InNumMappingElements)
		{
			// The interpolated data isn't necessarily in the same format as the persisted data, so copy
			Range = TArray<uint8>(InRange, InRangeSize);
			Mapping = TArray<uint8>(InMapping, InMappingSize);
		}

		FRangeMappingData(const TArray<uint8>& InRange,
						const uint8* InMapping,
						int32 InMappingSize = sizeof(uint8),
						const int32 InNumMappingElements = 1)
			: Range(TArray<uint8>(InRange))
			, NumElements(InNumMappingElements)
		{
			Mapping = TArray<uint8>(InMapping, InMappingSize);
		}

		FRangeMappingData(const TArray<uint8>& InRange,
						const TArray<uint8>& InMapping,
						const int32 InNumMappingElements = 1)
			: Range(TArray<uint8>(InRange))
			, Mapping(TArray<uint8>(InMapping))
			, NumElements(InNumMappingElements)
		{ }
	};

	/** Templated container for range and mapping values. */
	template <typename RangeType, typename MappingType>
	struct TRangeMappingData
	{
		RangeType Range;
		MappingType Mapping;

		TRangeMappingData() = default;

		TRangeMappingData(const RangeType& InRange, const MappingType& InMapping)
			: Range(InRange)
			, Mapping(InMapping)
		{ }

		explicit TRangeMappingData(const FRangeMappingData& InRangeMappingData)
		{
			Range = reinterpret_cast<RangeType>(InRangeMappingData.Range.GetData());
			Mapping = reinterpret_cast<MappingType>(InRangeMappingData.Mapping.GetData());
		}
	};

	/** Get Ranges and Mapping Value pointers */
	TArray<FRangeMappingData> GetRangeMappingBuffers();

	/** The current protocol value. Useful to test if the value changed */
	double ProtocolValue = 0.0;

private:
	friend struct FRemoteControlProtocolBinding;

	/**
	 * Serialize interpolated property value to Cbor buffer
	 * @param InProperty Property to apply serialization
	 * @param InProtocolValue double value from the protocol
	 * @param OutBuffer serialized buffer
	 * @return true if serialized correctly
	 */
	UE_DEPRECATED(5.5, "Deprecated in favor of the overload that specifies the OwnerObject of the property.")
	bool GetInterpolatedPropertyBuffer(FProperty* InProperty, double InProtocolValue, TArray<uint8>& OutBuffer);


protected:
	/** The preset that owns this entity. */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> Owner;

	/** Exposed property Id */
	UPROPERTY()
	FGuid PropertyId;

	/** State of the binding, set to false to ignore input.  */
	bool bIsEnabled = true;

protected:
	/** 
	 * Property mapping ranges set
	 * Stores protocol mapping for this protocol binding entity
	 */
	UPROPERTY()
	TSet<FRemoteControlProtocolMapping> Mappings;

	/** Holds the overriden masks. */
	UPROPERTY()
	uint8 OverridenMasks = static_cast<uint8>(ERCMask::NoMask);

#if WITH_EDITOR

	/**
	 * Holds column specific properties for this protocol.
	 */
	TMap<FName, FName> ColumnsToProperties;

#endif // WITH_EDITOR

private:
	/** Binding status of this protocol entity */
	UPROPERTY()
	ERCBindingStatus BindingStatus = ERCBindingStatus::Unassigned;
};

/**
 * Struct which holds the bound struct and serialized struct archive
 */
USTRUCT()
struct REMOTECONTROL_API FRemoteControlProtocolBinding
{
	GENERATED_BODY()

	friend uint32 REMOTECONTROL_API GetTypeHash(const FRemoteControlProtocolBinding& InProtocolBinding);

public:
	FRemoteControlProtocolBinding() = default;
	FRemoteControlProtocolBinding(const FName InProtocolName, const FGuid& InPropertyId, TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InRemoteControlProtocolEntityPtr, const FGuid& InBindingId = FGuid::NewGuid());

	bool operator==(const FRemoteControlProtocolBinding& InProtocolBinding) const;
	bool operator==(FGuid InProtocolBindingId) const;

public:
	/** Get protocol binding id */
	const FGuid& GetId() const { return Id; }

	/** Get protocol bound protocol name, such as MIDI, DMX, OSC, etc */
	FName GetProtocolName() const { return ProtocolName; }

	/** Get exposed property id */
	const FGuid& GetPropertyId() const { return PropertyId; }

	/** Get bound struct scope wrapper */
	TSharedPtr<FStructOnScope> GetStructOnScope() const;

	/** Get pointer to the StructOnScope */
	TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> GetRemoteControlProtocolEntityPtr() const { return RemoteControlProtocolEntityPtr; }

	/**
	 * Add binding mapping to the protocol bound struct
	 * @param InProtocolMapping input mapping with value range and value data
	 */
	void AddMapping(const FRemoteControlProtocolMapping& InProtocolMapping);

	/**
	 * Remove the bound struct mapping based on given mapping id
	 * @param InMappingId mapping unique id
	 * @return The number of elements removed.
	 */
	int32 RemoveMapping(const FGuid& InMappingId);

	/**
	 * Empty all mapping for the bound struct
	 */
	void ClearMappings();

	/**
	 * Remove the bound struct mapping based on given mapping id
	 * @param InMappingId mapping unique id
	 * @return FRemoteControlProtocolMapping struct pointer
	 */
	FRemoteControlProtocolMapping* FindMapping(const FGuid& InMappingId);

	/**
	 * Loop through all mappings
	 * @param InCallback mapping reference callback with mapping struct reference as an argument
	 * @return FRemoteControlProtocolMapping struct pointer
	 */
	void ForEachMapping(FGetProtocolMappingCallback InCallback);

	/**
	 * Set range value which is bound to mapping struct
	 * @param InMappingId mapping unique id
	 * @param InRangeValue range property value
	 * @return true if it was set successfully
	 */
	template <typename T>
	bool SetRangeToMapping(const FGuid& InMappingId, T InRangeValue)
	{
		if (FRemoteControlProtocolMapping* Mapping = FindMapping(InMappingId))
		{
			Mapping->SetRangeValue(InRangeValue);

			return true;
		}

		return false;
	}

	/**
	 * Set mapping property data to bound mapping struct.
	 * Range data could be a value container of the primitive value like FFloatProperty.
	 * And it could be more complex Properties such as FStructProperty data pointer.
	 *
	 * @param InMappingId mapping unique id
	 * @param InPropertyValuePtr property value pointer
	 * @return true if it was set successfully
	 */
	bool SetPropertyDataToMapping(const FGuid& InMappingId, const void* InPropertyValuePtr);

	/** Checks if the given ValueType is supported */
	template <typename ValueType>
	static bool IsRangeTypeSupported() { return TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType(); }

	/** Checks if the given PropertyType (FProperty) is supported */
	template <typename PropertyType>
	static bool IsRangePropertyTypeSupported() { return TRemoteControlPropertyTypeTraits<PropertyType>::IsSupportedMappingType(); }

	/** Custom struct serialize */
	bool Serialize(FArchive& Ar);
	friend FArchive& operator<<(FArchive& Ar, FRemoteControlProtocolBinding& InProtocolBinding);

	void AddStructReferencedObjects(FReferenceCollector& InCollector);

private:
	/** Return FRemoteControlProtocolEntity pointer from RemoteControlProtocolEntityPtr */
	FRemoteControlProtocolEntity* GetRemoteControlProtocolEntity();

protected:
	/** Binding Id */
	UPROPERTY()
	FGuid Id;

	/** Protocol name which we using for binding */
	UPROPERTY()
	FName ProtocolName;

	/** Property Unique ID */
	UPROPERTY()
	FGuid PropertyId;

	/** Property name which we using for protocol range mapping */
	UPROPERTY()
	FName MappingPropertyName;

private:
	/** Pointer to struct on scope */
	TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> RemoteControlProtocolEntityPtr;
};

/**
 * TypeTraits to define FRemoteControlProtocolBinding with a Serialize function
 */
template <>
struct TStructOpsTypeTraits<FRemoteControlProtocolBinding> : public TStructOpsTypeTraitsBase2<FRemoteControlProtocolBinding>
{
	enum
	{
		WithSerializer = true,
		WithAddStructReferencedObjects = true,
	};
};
