// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/StructOnScope.h"

class FProperty;
class UScriptStruct;
struct FRemoteControlProtocolEntity;

namespace UE::RemoteControl::ProtocolEntityProcessor
{
	/** Returns true if the script struct supports masking */
	bool DoesScriptStructSupportMasking(const UScriptStruct* InStruct);

	/** Returns true if the property supports masking. */
	bool DoesPropertySupportMasking(const FProperty* Property);

	/** Processes provided entities and their respective protocol values */
	REMOTECONTROL_API void ProcessEntities(const TMap<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>, double>& EntityToValueMap);
}
