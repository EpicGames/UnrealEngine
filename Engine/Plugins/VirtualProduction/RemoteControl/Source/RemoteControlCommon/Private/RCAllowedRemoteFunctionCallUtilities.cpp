// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCAllowedRemoteFunctionCallUtilities.h"
#include "Containers/ArrayView.h"
#include "RCAllowedRemoteFunctionCall.h"
#include "UObject/Class.h"
#include "UObject/Object.h"

namespace UE::RemoteControl
{

FRCRemoteFunctionCallParams MakeRemoteFunctionCallParams(const UObject* InObject, const UFunction* InFunction)
{
	FRCRemoteFunctionCallParams CallParameters;
	CallParameters.ObjectClass = InObject ? InObject->GetClass() : nullptr;
	CallParameters.FunctionName = InFunction ? InFunction->GetName() : FString();
	return CallParameters;
}

bool IsRemoteFunctionCallAllowed(const FRCRemoteFunctionCallParams& InParams, const FRCAllowedRemoteFunctionCall& InAllowedCall)
{
	if (!InParams.ObjectClass || InParams.FunctionName.IsEmpty())
	{
		return false;
	}

	// No need to load the class here. Parent classes of a child should've already been loaded. 
	// Unresolved classes here mean that it wasn't the class itself or a parent in the first place.
	const UClass* const AllowedClass = InAllowedCall.ClassPath.ResolveClass();
	if (!AllowedClass)
	{
		return false;
	}

	const bool bClassRuleSatisfied = InParams.ObjectClass == AllowedClass || (InAllowedCall.bAllowChildClasses && InParams.ObjectClass->IsChildOf(AllowedClass));

	// Ignore case for consistency with how FNames are not case-sensitive (e.g. see FindFunctionByName)
	const bool bFunctionRuleSatisfied = !InAllowedCall.FunctionName.IsSet() || InParams.FunctionName.Equals(*InAllowedCall.FunctionName, ESearchCase::IgnoreCase);

	return bClassRuleSatisfied && bFunctionRuleSatisfied;
}

bool IsRemoteFunctionCallAllowed(const FRCRemoteFunctionCallParams& InParams, TConstArrayView<FRCAllowedRemoteFunctionCall> InAllowedCalls)
{
	if (!InParams.ObjectClass || InParams.FunctionName.IsEmpty())
	{
		return false;
	}

	for (const FRCAllowedRemoteFunctionCall& AllowedCall : InAllowedCalls)
	{
		if (IsRemoteFunctionCallAllowed(InParams, AllowedCall))
		{
			return true;
		}
	}
	return false;
}

} // UE::RemoteControl
