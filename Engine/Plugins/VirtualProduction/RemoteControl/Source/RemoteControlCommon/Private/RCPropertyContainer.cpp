// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCPropertyContainer.h"

#include "RemoteControlCommonModule.h"
#include "Engine/Engine.h"
#include "Engine/MemberReference.h"
#include "UObject/EnumProperty.h"
#include "UObject/FieldPathProperty.h"
#include "UObject/TextProperty.h"

#if WITH_EDITOR
#include "BlueprintActionDatabase.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/StructureEditorUtils.h"
#endif

#define LOCTEXT_NAMESPACE "RCPropertyContainer"

namespace PropertyContainers
{
	URCPropertyContainerBase* CreateContainerForProperty(UObject* InOwner, const FProperty* InSrcProperty)
	{
		ensure(IsInGameThread());

		check(InOwner);
		check(InSrcProperty);
		check(InSrcProperty->IsValidLowLevel());

		URCPropertyContainerRegistry* ContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
		FName PropertyTypeName = InSrcProperty->GetClass()->GetFName();
		if(const FStructProperty* StructProperty = CastField<FStructProperty>(InSrcProperty))
		{
			PropertyTypeName = StructProperty->Struct->GetFName();
		}
		return ContainerRegistry->CreateContainer(InOwner, PropertyTypeName, InSrcProperty);
	}

#if WITH_EDITOR
	// Adapted from FKismetCompilerUtilities::CreatePrimitiveProperty
	FProperty* CreatePropertyInternal(const FFieldVariant& InParent, const FFieldVariant& InChild, const FName& InPropertyName)
	{
		const FFieldVariant PropertyScope = InParent;

		FProperty* NewProperty = nullptr;
		const FName FieldClassTypeName = !InParent.IsUObject() ? InParent.ToField()->GetClass()->GetFName() : NAME_None;
		if (FieldClassTypeName == NAME_Object || FieldClassTypeName == NAME_Interface || FieldClassTypeName == NAME_SoftObjectProperty)
		{
			UClass* SubType = InChild.Get<UClass>();
			if (SubType == nullptr)
			{
				// If this is from a degenerate pin, because the object type has been removed, default this to a UObject subtype so we can make a dummy term for it to allow the compiler to continue
				SubType = UObject::StaticClass();
			}

			if (SubType)
			{
				const bool bIsInterface = SubType->HasAnyClassFlags(CLASS_Interface);
				if (bIsInterface)
				{
					FInterfaceProperty* NewPropertyObj = new FInterfaceProperty(PropertyScope, InPropertyName);
					// we want to use this setter function instead of setting the 
					// InterfaceClass member directly, because it properly handles  
					// placeholder classes (classes that are stubbed in during load)
					NewPropertyObj->SetInterfaceClass(SubType);
					NewProperty = NewPropertyObj;
				}
				else
				{
					FObjectPropertyBase* NewPropertyObj = nullptr;
					if (FieldClassTypeName == NAME_SoftObjectProperty)
					{
						NewPropertyObj = new FSoftObjectProperty(PropertyScope, InPropertyName);
					}
					else
					{
						NewPropertyObj = new FObjectProperty(PropertyScope, InPropertyName);
					}

					// Is the property a reference to something that should default to instanced?
					if (SubType->HasAnyClassFlags(CLASS_DefaultToInstanced))
					{
						NewPropertyObj->SetPropertyFlags(CPF_InstancedReference);
					}

					// we want to use this setter function instead of setting the 
					// PropertyClass member directly, because it properly handles  
					// placeholder classes (classes that are stubbed in during load)
					NewPropertyObj->SetPropertyClass(SubType);
					NewPropertyObj->SetPropertyFlags(CPF_HasGetValueTypeHash);
					NewProperty = NewPropertyObj;
				}
			}
		}
		else if (FieldClassTypeName == NAME_StructProperty)
		{
			if (UScriptStruct* SubType = InChild.Get<UScriptStruct>())
			{
				FString StructureError;
				if (FStructureEditorUtils::EStructureError::Ok == FStructureEditorUtils::IsStructureValid(SubType, nullptr, &StructureError))
				{
					FStructProperty* NewPropertyStruct = new FStructProperty(PropertyScope, InPropertyName);
					NewPropertyStruct->Struct = SubType;
					NewProperty = NewPropertyStruct;

					if (SubType->StructFlags & STRUCT_HasInstancedReference)
					{
						NewProperty->SetPropertyFlags(CPF_ContainsInstancedReference);
					}

					if (FBlueprintEditorUtils::StructHasGetTypeHash(SubType))
					{
						// tag the type as hashable to avoid crashes in core:
						NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
					}
				}
				else
				{
					UE_LOGF(LogRemoteControlCommon, Error,
						"Invalid property '%ls' structure '%ls' error: %ls",
						*InPropertyName.ToString(),
						*SubType->GetName(),
						*StructureError);
				}
			}
		}
		else if (FieldClassTypeName == NAME_Class)
		{
			UClass* SubType = InChild.Get<UClass>();

			if (SubType == nullptr)
			{
				// If this is from a degenerate pin, because the object type has been removed, default this to a UObject subtype so we can make a dummy term for it to allow the compiler to continue
				SubType = UObject::StaticClass();

				UE_LOGF(LogRemoteControlCommon, Error,
					"Invalid property '%ls' class, replaced with Object.  Please fix or remove.",
					*InPropertyName.ToString());
			}

			if (SubType)
			{
				if (InParent.IsA<FSoftClassProperty>())
				{
					FSoftClassProperty* SoftClassProperty = new FSoftClassProperty(PropertyScope, InPropertyName);
					// we want to use this setter function instead of setting the 
					// MetaClass member directly, because it properly handles  
					// placeholder classes (classes that are stubbed in during load)
					SoftClassProperty->SetMetaClass(SubType);
					SoftClassProperty->PropertyClass = UClass::StaticClass();
					SoftClassProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
					NewProperty = SoftClassProperty;
				}
				else
				{
					FClassProperty* NewPropertyClass = new FClassProperty(PropertyScope, InPropertyName);
					// we want to use this setter function instead of setting the 
					// MetaClass member directly, because it properly handles  
					// placeholder classes (classes that are stubbed in during load)
					NewPropertyClass->SetMetaClass(SubType);
					NewPropertyClass->PropertyClass = UClass::StaticClass();
					NewPropertyClass->SetPropertyFlags(CPF_HasGetValueTypeHash);
					NewProperty = NewPropertyClass;
				}
			}
		}
 
		// Bool
		else if (FieldClassTypeName == NAME_BoolProperty)
		{
			FBoolProperty* BoolProperty = new FBoolProperty(PropertyScope, InPropertyName);
			BoolProperty->SetBoolSize(sizeof(bool), true);
			NewProperty = BoolProperty;
		}

		// Enum
		else if(FieldClassTypeName == NAME_EnumProperty)
		{
			UEnum* Enum = InChild.Get<UEnum>();
			if(Enum)
			{
				FEnumProperty* EnumProp = new FEnumProperty(PropertyScope, InPropertyName);
				FNumericProperty* UnderlyingProp = new FByteProperty(EnumProp, TEXT("UnderlyingType"));

				EnumProp->SetEnum(Enum);
				EnumProp->AddCppProperty(UnderlyingProp);

				NewProperty = EnumProp;
			}
		}
		
		// Numeric
		else if(FieldClassTypeName == NAME_Int8Property)
		{
			NewProperty = new FInt8Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_ByteProperty)
		{
			UEnum* Enum = InChild.Get<UEnum>();
			if (Enum && Enum->GetCppForm() == UEnum::ECppForm::EnumClass)
			{
				FEnumProperty* EnumProp = new FEnumProperty(PropertyScope, InPropertyName);
				FNumericProperty* UnderlyingProp = new FByteProperty(EnumProp, TEXT("UnderlyingType"));

				EnumProp->SetEnum(Enum);
				EnumProp->AddCppProperty(UnderlyingProp);

				NewProperty = EnumProp;
			}
			else
			{
				FByteProperty* ByteProp = new FByteProperty(PropertyScope, InPropertyName);
				ByteProp->Enum = InChild.Get<UEnum>();

				NewProperty = ByteProp;
			}
 
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if(FieldClassTypeName == NAME_Int16Property)
		{
			NewProperty = new FInt16Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if(FieldClassTypeName == NAME_UInt16Property)
		{
			NewProperty = new FUInt16Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_IntProperty)
		{ 
			NewProperty = new FIntProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if(FieldClassTypeName == NAME_UInt32Property)
		{
			NewProperty = new FUInt32Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_Int64Property)
		{
			NewProperty = new FInt64Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if(FieldClassTypeName == NAME_UInt64Property)
		{
			NewProperty = new FUInt64Property(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_FloatProperty)
		{
			NewProperty = new FFloatProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_DoubleProperty)
		{
			NewProperty = new FDoubleProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}

		// Stringish
		else if (FieldClassTypeName == NAME_StrProperty)
		{
			NewProperty = new FStrProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_NameProperty)
		{
			NewProperty = new FNameProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}
		else if (FieldClassTypeName == NAME_TextProperty)
		{
			NewProperty = new FTextProperty(PropertyScope, InPropertyName);
		}
		
		else if (InParent.IsA<FFieldPathProperty>())
		{
			FFieldPathProperty* NewFieldPathProperty = new FFieldPathProperty(PropertyScope, InPropertyName);
			NewFieldPathProperty->PropertyClass = FProperty::StaticClass();
			NewFieldPathProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
			NewProperty = NewFieldPathProperty;
		}
		else
		{
			// Failed to resolve the type-subtype, create a generic property to survive VM bytecode emission
			NewProperty = new FIntProperty(PropertyScope, InPropertyName);
			NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
		}

		return NewProperty;
	}

	// Adapted from FKismetCompilerUtilities::CreatePropertyOnScope
	FProperty* CreateProperty(const FFieldVariant& InParent, const FFieldVariant& InChild, const FName& InPropertyName)
	{
		FFieldVariant PropertyScope = InParent;

		// A name is usually required further down the chain, so set one if not already.
		const FName PropertyName = (InPropertyName == NAME_None) ? TEXT("Value") : InPropertyName;
		
		FProperty* NewProperty = nullptr;

		const FName FieldClassTypeName = !InParent.IsUObject() ? InParent.ToField()->GetClass()->GetFName() : NAME_None;

		// Handle creating a container property, if necessary
		bool bIsArray = false;
		bool bIsMap = false;
		bool bIsSet = false;
		FArrayProperty* NewArrayProperty = nullptr;
		FMapProperty* NewMapProperty = nullptr;
		FSetProperty* NewSetProperty = nullptr;		
		FProperty* NewContainerProperty = nullptr;
		if(FieldClassTypeName == NAME_ArrayProperty)
		{
			bIsArray = true;
			NewArrayProperty = new FArrayProperty(PropertyScope, PropertyName);
			PropertyScope = NewArrayProperty;
			NewContainerProperty = NewArrayProperty;
		}
		else if(FieldClassTypeName == NAME_MapProperty)
		{
			bIsMap = true;
			NewMapProperty = new FMapProperty(PropertyScope, PropertyName);
			PropertyScope = NewMapProperty;
			NewContainerProperty = NewMapProperty;			
		}
		else if(FieldClassTypeName == NAME_SetProperty)
		{
			bIsSet = true;
			NewSetProperty = new FSetProperty(PropertyScope, PropertyName);
			PropertyScope = NewSetProperty;
			NewContainerProperty = NewSetProperty;
		}

		if(FieldClassTypeName == NAME_DelegateProperty)
		{
			unimplemented();
		}
		else if(FieldClassTypeName == NAME_MulticastDelegateProperty)
		{
			unimplemented();
		}
		else
		{
			NewProperty = CreatePropertyInternal(PropertyScope, InChild, PropertyName);
		}

		if(NewContainerProperty && NewProperty && NewProperty->HasAnyPropertyFlags(CPF_ContainsInstancedReference | CPF_InstancedReference))
		{
			NewContainerProperty->SetPropertyFlags(CPF_ContainsInstancedReference);
		}

		if(bIsArray)
		{
			if(NewProperty)
			{
				NewArrayProperty->Inner = NewProperty;
				NewProperty = NewArrayProperty;
			}
			// inner property creation failed, so clean up array property
			else
			{
				delete NewArrayProperty;
				NewArrayProperty = nullptr;
			}
		}
		else if(bIsMap)
		{
			if(NewProperty)
			{
				if(!NewProperty->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
				{
					UE_LOGF(LogRemoteControlCommon, Error, "Property has key of type %ls which cannot be hashed and is therefore invalid.", TEXT("@todo"));
				}

				NewMapProperty->KeyProp = NewProperty;
				const FName ValueName = FName(*(PropertyName.GetPlainNameString() + FString(TEXT("_Value"))));
				NewMapProperty->ValueProp = CreatePropertyInternal(PropertyScope, nullptr, ValueName);
				// if unsuccessful, clean up
				if(!NewMapProperty->ValueProp)
				{
					delete NewMapProperty;
					NewMapProperty = nullptr;
					NewProperty = nullptr;
				}
				else
				{
					// propagate flags
					if(NewMapProperty->ValueProp->HasAnyPropertyFlags(CPF_ContainsInstancedReference | CPF_InstancedReference))
					{
						NewContainerProperty->SetPropertyFlags(CPF_ContainsInstancedReference);
					}

					NewProperty = NewMapProperty;
				}
			}
			// inner property creation failed, so clean up map property
			else
			{
				delete NewMapProperty;
				NewMapProperty = nullptr;
			}
		}
		else if(bIsSet)
		{
			if(NewProperty)
			{
				if(!NewProperty->HasAnyPropertyFlags(CPF_HasGetValueTypeHash))
				{
					UE_LOGF(LogRemoteControlCommon, Error, "Property is set of type %ls which cannot be hashed and is therefore invalid.", TEXT("@todo"));
					NewProperty->SetPropertyFlags(CPF_HasGetValueTypeHash);
				}
				
				NewSetProperty->ElementProp = NewProperty;
				NewProperty = NewSetProperty;
			}
			// inner property creation failed, so clean up set property
			else
			{
				delete NewSetProperty;
				NewSetProperty = nullptr;
			}
		}

		return NewProperty;
	}
#endif
}

void URCPropertyContainerBase::SetValue(const uint8* InData, const SIZE_T& InSize)
{
	this->Modify();

	FProperty* Prp = GetValueProperty();
	uint8* DstData = Prp->ContainerPtrToValuePtr<uint8>(this);
	FMemory::Memcpy(DstData, InData, InSize > 0 ? InSize : Prp->GetSize());
}

SIZE_T URCPropertyContainerBase::GetValue(uint8* OutData)
{
	check(OutData != nullptr);
	
	FProperty* Prp = GetValueProperty();
	Prp->InitializeValue(OutData);
	
	uint8* SrcData = Prp->ContainerPtrToValuePtr<uint8>(this);
	FMemory::Memcpy(OutData, SrcData, Prp->GetSize());
	return Prp->GetSize();
}

SIZE_T URCPropertyContainerBase::GetValue(TArray<uint8>& OutData)
{
	FProperty* Prp = GetValueProperty();
	OutData.SetNumZeroed(Prp->GetSize());
	Prp->InitializeValue(OutData.GetData());
	
	uint8* SrcData = Prp->ContainerPtrToValuePtr<uint8>(this);
	FMemory::Memcpy(OutData.GetData(), SrcData, Prp->GetSize());
	return Prp->GetSize();
}

FProperty* URCPropertyContainerBase::GetValueProperty()
{
	if(ValueProperty.IsValid())
	{
		return ValueProperty.Get();
	}

	static FName ValuePropertyName = TEXT("Value");
	ValueProperty = FindFProperty<FProperty>(GetClass(), ValuePropertyName);
	ensure(ValueProperty.IsValid());

	return ValueProperty.Get();
}

FName FRCPropertyContainerKey::ToClassName() const
{
	auto ValueTrimmed = ValueTypeName.ToString();
	ValueTrimmed.RemoveFromEnd("Property");
	return FName(FString::Printf(TEXT("PropertyContainer_%s"), *ValueTrimmed));	
}


URCPropertyContainerBase* URCPropertyContainerRegistry::CreateContainer(UObject* InOwner, const FName& InValueTypeName, const FProperty* InValueSrcProperty)
{
	check(InValueSrcProperty && InValueSrcProperty->IsValidLowLevel());

	const FName SanitizedName = FName(InValueTypeName.ToString().Replace(TEXT(":"), TEXT("_")));
	const TSubclassOf<URCPropertyContainerBase> ClassForPropertyType = FindOrAddContainerClass(SanitizedName, InValueSrcProperty);
	if (ClassForPropertyType)
	{
		return NewObject<URCPropertyContainerBase>(InOwner
            ? InOwner
            : static_cast<UObject*>(GetTransientPackage()), ClassForPropertyType);
	}
	else
	{
		UE_LOGF(LogRemoteControlCommon, Warning, "Could not create PropertyContainer found for %ls", *SanitizedName.ToString());
		return nullptr;
	}
}

TSubclassOf<URCPropertyContainerBase>& URCPropertyContainerRegistry::FindOrAddContainerClass(const FName& InValueTypeName, const FProperty* InValueSrcProperty)
{
	check(InValueSrcProperty);

	static const FString EmptyStr = TEXT("");
	FString PropertyPathStr = InValueSrcProperty->GetPathName();
	// format like ObjectName:PropertyName, this ensures theres a unique property container for each defined property rather than for each type
	PropertyPathStr.ReplaceInline(TEXT(":"), TEXT("_"));
	PropertyPathStr.ReplaceInline(*(InValueSrcProperty->GetOutermost()->GetPathName() + TEXT(".")), *EmptyStr);

	const FRCPropertyContainerKey Key = FRCPropertyContainerKey{FName(PropertyPathStr)};
	if(TSubclassOf<URCPropertyContainerBase>* ExistingContainerClass = CachedContainerClasses.Find(Key))
	{
		return *ExistingContainerClass;
	}

	UPackage* Outer = GetOutermost();
	UClass* ContainerClass = NewObject<UClass>(Outer, Key.ToClassName(), RF_Public | RF_Transient);
	UClass* ParentClass = URCPropertyContainerBase::StaticClass();
	ContainerClass->SetSuperStruct(ParentClass);

	FProperty* ValueProperty = CastField<FProperty>(FField::Duplicate(InValueSrcProperty, ContainerClass, "Value"));
#if WITH_EDITORONLY_DATA
	FField::CopyMetaData(InValueSrcProperty, ValueProperty);
#endif
	ValueProperty->PropertyFlags |= CPF_Edit; // add this flag(s)
	//ValueProperty->PropertyFlags &= ~CPF_BlueprintVisible; // clear this flag(s)

	ContainerClass->AddCppProperty(ValueProperty);

	ContainerClass->Bind();
	ContainerClass->StaticLink(true);
	ContainerClass->AssembleReferenceTokenStream(true);

#if WITH_EDITOR
	if (FBlueprintActionDatabase* ActionDB = FBlueprintActionDatabase::TryGet())
	{
		// Notify Blueprints that there is a new class to add to the action list
		ActionDB->RefreshClassActions(ContainerClass);
	}
#endif

	return CachedContainerClasses.Add(Key, MoveTemp(ContainerClass));	
}

#undef LOCTEXT_NAMESPACE
