// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCTypeUtilities.h"

#include "UObject/UnrealType.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

SIZE_T RemoteControlTypeUtilities::GetPropertySize(const FProperty* InProperty, void* InData)
{
	check(InProperty);

	const int32 ElementSize = InProperty->GetElementSize();
	if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		return FScriptArrayHelper(ArrayProperty, InData).Num() * ElementSize;
	}
	if (const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
	{
		return FScriptSetHelper(SetProperty, InData).Num() * ElementSize;
	}
	if (const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
	{
		return FScriptMapHelper(MapProperty, InData).Num() * ElementSize;
	}

	return ElementSize;
}

#if WITH_EDITOR
SIZE_T RemoteControlTypeUtilities::GetPropertySize(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
{
	check(InPropertyHandle.IsValid() && InPropertyHandle->IsValidHandle());

	TArray<void*> Data;
	InPropertyHandle->AccessRawData(Data);
	check(Data.IsValidIndex(0));

	return GetPropertySize(InPropertyHandle->GetProperty(), Data[0]);
}
#endif
