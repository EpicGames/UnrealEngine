// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlCommonModule.h"

#define LOCTEXT_NAMESPACE "FRemoteControlCommonModule"

DEFINE_LOG_CATEGORY(LogRemoteControlCommon);

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FRemoteControlCommonModule, RemoteControlCommon)
