// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

REMOTECONTROLCOMMON_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteControlCommon, Log, All);

class FRemoteControlCommonModule : public IModuleInterface
{
public:
};
