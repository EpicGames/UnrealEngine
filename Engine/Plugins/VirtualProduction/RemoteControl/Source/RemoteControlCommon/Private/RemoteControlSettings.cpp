// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlSettings.h"

URemoteControlSettings::URemoteControlSettings()
{
	auto AllowAnyFunction = [](const TCHAR* InObjectPath)
		{
			FRCAllowedRemoteFunctionCall AllowedEntry;
			AllowedEntry.ClassPath = FSoftClassPath(InObjectPath);
			AllowedEntry.bAllowChildClasses = false;
			return AllowedEntry;
		};

	auto AllowFunction = [](const TCHAR* InObjectPath, const TCHAR* InFunctionPath, bool bAllowChildClasses)
		{
			FRCAllowedRemoteFunctionCall AllowedEntry;
			AllowedEntry.ClassPath = FSoftClassPath(InObjectPath);
			AllowedEntry.FunctionName = InFunctionPath;
			AllowedEntry.bAllowChildClasses = bAllowChildClasses;
			return AllowedEntry;
		};

	// Functions allowed by default (Functions used by Epic Stage App)
	AllowedRemoteFunctionCalls =
		{
			// Allow any function residing in the Stage App Function library
			AllowAnyFunction(TEXT("/Script/EpicStageApp.StageAppFunctionLibrary")),

			// Disallow child classes for blueprint function libraries
			AllowFunction(TEXT("/Script/DisplayCluster.DisplayClusterBlueprintLib"), TEXT("FindLightCardsForRootActor"), /*bAllowChildClasses*/false),
			AllowFunction(TEXT("/Script/Engine.KismetSystemLibrary"), TEXT("GetDisplayName"), /*bAllowChildClasses*/false),
			AllowFunction(TEXT("/Script/RemoteControl.RemoteControlFunctionLibrary"), TEXT("ApplyColorWheelDelta"), /*bAllowChildClasses*/false),
			AllowFunction(TEXT("/Script/RemoteControl.RemoteControlFunctionLibrary"), TEXT("ApplyColorGradingWheelDelta"), /*bAllowChildClasses*/false),

			// Allow child classes for Actors/Components
			AllowFunction(TEXT("/Script/Engine.Actor"), TEXT("SetActorLabel"), /*bAllowChildClasses*/true),
			AllowFunction(TEXT("/Script/Engine.Actor"), TEXT("K2_DestroyActor"), /*bAllowChildClasses*/true),
			AllowFunction(TEXT("/Script/Engine.SceneComponent"), TEXT("K2_GetComponentToWorld"), /*bAllowChildClasses*/true),
			AllowFunction(TEXT("/Script/Engine.SceneComponent"), TEXT("K2_SetWorldTransform"), /*bAllowChildClasses*/true),
			AllowFunction(TEXT("/Script/Engine.SceneComponent"), TEXT("SetAbsolute"), /*bAllowChildClasses*/true),
		};
}
