// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "Containers/ArrayView.h"
#include "Misc/AutomationTest.h"
#include "RCAllowedRemoteFunctionCall.h"
#include "RCAllowedRemoteFunctionCallUtilities.h"
#include "UObject/Class.h"
#include "UObject/Package.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FRCAllowedRemoteFunctionCallSpec,"Plugins.RemoteControl.AllowedRemoteFunctionCall",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)

	/** Function name to use that differs from TestFunctionName */
	FString OtherFunctionName = TEXT("SomeOtherFunction");
	/** The class to test (USceneComponent) */
	const UClass* TestClass = nullptr;
	const UClass* TestSubClass = nullptr;
	/** Function within the TestClass to test (USceneComponent::IsVisible); */
	const UFunction* TestFunction = nullptr;
	/** Name of the test function "IsVisible" */
	FString TestFunctionName;
	/** The function call to allow */
	FRCAllowedRemoteFunctionCall AllowedFunctionCall;

END_DEFINE_SPEC(FRCAllowedRemoteFunctionCallSpec)

void FRCAllowedRemoteFunctionCallSpec::Define()
{
	using namespace UE::RemoteControl;

	BeforeEach([this]()
	{
		TestClass = USceneComponent::StaticClass();
		TestSubClass = UPrimitiveComponent::StaticClass();
		check(TestClass);

		TestFunctionName = GET_FUNCTION_NAME_STRING_CHECKED(USceneComponent, IsVisible);
		TestFunction = TestClass->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(USceneComponent, IsVisible));
		check(TestFunction);
	});

	Describe("MakeRemoteFunctionCallParams", [this]
	{
		It("Accepts null values, but returns invalid params where object class is null and function name is unset.", [this]
		{
			const FRCRemoteFunctionCallParams Params = MakeRemoteFunctionCallParams(nullptr, nullptr);
			TestNull("ObjectClass", Params.ObjectClass);
			TestTrue("FunctionName empty", Params.FunctionName.IsEmpty());
		});

		It("Correctly makes the parameters from valid object and function pointers", [this]
		{
			UObject* const TestObject = NewObject<UObject>(GetTransientPackage(), TestClass);

			const FRCRemoteFunctionCallParams Params = MakeRemoteFunctionCallParams(TestObject, TestFunction);
			TestEqual("ObjectClass", Params.ObjectClass, TestClass);
			TestEqual("FunctionName", Params.FunctionName, TestFunctionName);

			TestObject->MarkAsGarbage();
		});
	});

	Describe("IsRemoteFunctionCallAllowed", [this]
	{
		Describe("Allowed class path is not set", [this]
		{
			BeforeEach([this]()
			{
				// Explicitly unset here for test clarity
				AllowedFunctionCall.ClassPath = {};
				AllowedFunctionCall.FunctionName = TestFunctionName;
			});

			It("Returns false for any call parameter configuration", [this]()
			{
				FRCRemoteFunctionCallParams Params;
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				Params.FunctionName = TestFunctionName;
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				Params.ObjectClass = TestClass;
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				Params.FunctionName.Reset();
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
			});
		});

		Describe("Allowed class path is set", [this]
		{
			BeforeEach([this]()
			{
				AllowedFunctionCall.ClassPath = TestClass;
			});

			It("Returns false when object class is not specified regardless of allow rule", [this]()
			{
				FRCRemoteFunctionCallParams Params;
				Params.ObjectClass = {};
				Params.FunctionName = TestFunctionName;
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
			});

			It("Returns false when function name is not specified regardless of allow rule", [this]()
			{
				FRCRemoteFunctionCallParams Params;
				Params.ObjectClass = TestClass;
				Params.FunctionName = {};
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
			});

			It("Returns false when object class is not a child", [this]
			{
				FRCRemoteFunctionCallParams Params;
				Params.ObjectClass = UActorComponent::StaticClass(); // Actor component is not a child of Scene component.
				Params.FunctionName = TestFunctionName;
				TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
			});

			Describe("Allowing a specific function from a class", [this]
			{
				BeforeEach([this]()
				{
					// Specify a function name in addition to the object class set above. 
					AllowedFunctionCall.FunctionName = TestFunctionName;
				});

				It("Returns false when class matches but function name does not", [this]
				{
					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestClass;
					Params.FunctionName = OtherFunctionName;
					TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns true when class and function name matches", [this]
				{
					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestClass;
					Params.FunctionName = TestFunctionName;
					TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns false for a valid subclass and function name when bAllowChildClasses is disabled", [this]
				{
					AllowedFunctionCall.bAllowChildClasses = false;

					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestSubClass;
					Params.FunctionName = TestFunctionName;
					TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns true for a valid subclass and function name when bAllowChildClasses is enabled", [this]
				{
					AllowedFunctionCall.bAllowChildClasses = true;

					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestSubClass;
					Params.FunctionName = TestFunctionName;
					TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns true when class and function name match case insensitive", [this]
				{
					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestClass;
					Params.FunctionName = TestFunctionName.ToLower();
					TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});
			});

			Describe("Allowing any function from a class", [this]
			{
				BeforeEach([this]()
				{
					// Explicitly unset here for test clarity
					AllowedFunctionCall.FunctionName.Reset();
				});

				It("Returns true when class matches regardless of function name", [this]
				{
					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestClass;
					Params.FunctionName = OtherFunctionName;
					TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns false for a valid subclass when bAllowChildClasses is disabled", [this]
				{
					AllowedFunctionCall.bAllowChildClasses = false;

					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestSubClass;
					Params.FunctionName = OtherFunctionName;
					TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});

				It("Returns true for a valid subclass when bAllowChildClasses is enabled regardless of function name", [this]
				{
					AllowedFunctionCall.bAllowChildClasses = true;

					FRCRemoteFunctionCallParams Params;
					Params.ObjectClass = TestSubClass;
					Params.FunctionName = OtherFunctionName;
					TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, AllowedFunctionCall));
				});
			});
		});
	});

	Describe("IsRemoteFunctionCallAllowed_Array", [this]
	{
		It("Returns false when allowed function array is empty", [this]
		{
			FRCRemoteFunctionCallParams Params;
			Params.ObjectClass = TestClass;
			Params.FunctionName = OtherFunctionName;
			TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, TConstArrayView<FRCAllowedRemoteFunctionCall>()));
		});

		It("Returns false when none of the allowed functions in the array match", [this]
		{
			const FRCAllowedRemoteFunctionCall AllowedFunctionCalls[] = 
				{
					{
						.ClassPath = TestClass,
						.FunctionName = OtherFunctionName
					},
					{
						.ClassPath = UPrimitiveComponent::StaticClass(),
						.FunctionName = TestFunctionName,
					}
				};
			FRCRemoteFunctionCallParams Params;
			Params.ObjectClass = TestClass;
			Params.FunctionName = TestFunctionName;
			TestFalse("IsAllowed", IsRemoteFunctionCallAllowed(Params, MakeArrayView(AllowedFunctionCalls)));
		});

		It("Returns true when at least one of the allowed functions in the array matches", [this]
		{
			const FRCAllowedRemoteFunctionCall AllowedFunctionCalls[] = 
				{
					{
						.ClassPath = TestClass,
						.FunctionName = TestFunctionName
					},
					{
						.ClassPath = UPrimitiveComponent::StaticClass(),
						.FunctionName = TestFunctionName,
					}
				};
			FRCRemoteFunctionCallParams Params;
			Params.ObjectClass = TestClass;
			Params.FunctionName = TestFunctionName;
			TestTrue("IsAllowed", IsRemoteFunctionCallAllowed(Params, MakeArrayView(AllowedFunctionCalls)));
		});
	});

	AfterEach([this]()
	{
		AllowedFunctionCall = {};
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
