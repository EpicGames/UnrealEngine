// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "RCPropertyContainerTestData.generated.h"

UCLASS()
class UPropertyContainerTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	bool bSomeBool;
	
	UPROPERTY()
	uint32 SomeUInt32;

	UPROPERTY()
	float SomeFloat;

	UPROPERTY()
	FVector SomeVector;

	UPROPERTY()
	FRotator SomeRotator;

	UPROPERTY(meta = (ClampMin = 20, UIMin = 15, ClampMax = 145, UIMax = 167))
	int32 SomeClampedInt = 5;

	UPROPERTY(meta = (ClampMin = 20, UIMin = 37, ClampMax = 145, UIMax = 122))
	int32 SomeClampedInt2 = 49;

	UPROPERTY(meta = (UIMin = 31, UIMax = 139))
	int32 SomeUIClampedInt = 86;

	UPROPERTY(meta = (ClampMin = 0.2f, ClampMax = 0.92f))
	float SomeClampedFloat = 0.25f;

	UPROPERTY(meta = (UIMin = 0.26f, UIMax = 0.72f))
	float SomeUIClampedFloat = 0.38f;

	UPROPERTY(meta = (ClampMin = 0.2f, UIMin = 0.38f, ClampMax = 0.92f, UIMax = 0.83f))
	float SomeClampedFloat2 = 0.46f;

	UPROPERTY()
	FString SomeString = TEXT("string contents");

	UPROPERTY()
	FText SomeText = FText::FromString(TEXT("text contents"));

	UPROPERTY()
	TArray<float> SomeFloatArray;
	
	UPropertyContainerTestObject()
		: SomeUInt32(44),
		SomeFloat(45.0f),
		SomeVector(FVector(0.2f, 0.3f, 0.6f)),
		SomeRotator(FRotator::ZeroRotator) { }
};
