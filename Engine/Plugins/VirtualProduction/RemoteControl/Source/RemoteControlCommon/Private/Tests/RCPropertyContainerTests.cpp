// Copyright Epic Games, Inc. All Rights Reserved.

#include "Engine/Engine.h"
#include "RCPropertyContainer.h"
#include "Camera/CameraComponent.h"
#include "Misc/AutomationTest.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "RCPropertyContainerTestData.h"
#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FPropertyContainerSpec,
	"Plugins.RemoteControl.PropertyContainer",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FPropertyContainerSpec)

void FPropertyContainerSpec::Define()
{
	Describe("Registry", [this]()
	{
		It("Returns_Valid_Subsystem", [this]
		{
			URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
			TestNotNull("PropertyContainerRegistry", PropertyContainerRegistry);
		});
	});

	// Check that property metadata is copied from the source property
	Describe("Metadata", [this]
	{
		It("Clamp_Min_Max", [this]
		{
			FProperty* FloatProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeClampedFloat));
			check(FloatProperty);

			TestTrue("ClampMin", FloatProperty->HasMetaData("ClampMin"));
			TestTrue("ClampMax", FloatProperty->HasMetaData("ClampMax"));

			TestEqual("ClampMin Value", FloatProperty->GetFloatMetaData("ClampMin"), 0.2f);
			TestEqual("ClampMax Value", FloatProperty->GetFloatMetaData("ClampMax"), 0.92f);

			URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
			URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_FloatProperty, FloatProperty);

			TestNotNull("Container", Container);
		});
	});
	
	Describe("Float", [this]
	{
		It("Can_Create_Value", [this]
		{
			FProperty* FloatProperty = UCameraComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UCameraComponent, PostProcessBlendWeight));
			check(FloatProperty);
		
			URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
			URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_FloatProperty, FloatProperty);

			TestNotNull("Container", Container);
		});
		It("Can_Set_Value", [this]
		{
			UPropertyContainerTestObject* ObjectInstance = NewObject<UPropertyContainerTestObject>(GetTransientPackage());
			
		    FProperty* ValueProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeFloat));
		    check(ValueProperty);
				
		    URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
		    URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_FloatProperty, ValueProperty);

            void* ValuePtr = ValueProperty->ContainerPtrToValuePtr<void>(ObjectInstance);

			Container->SetValue((uint8*)ValuePtr);
		});
		It("Can_Get_Value", [this]
        {
            UPropertyContainerTestObject* ObjectInstance = NewObject<UPropertyContainerTestObject>(GetTransientPackage());
			
            FProperty* ValueProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeFloat));
            check(ValueProperty);
				
            URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
            URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_FloatProperty, ValueProperty);

			float InputValue = 84.3f;
			ObjectInstance->SomeFloat = InputValue;
			
            void* ValuePtr = ValueProperty->ContainerPtrToValuePtr<void>(ObjectInstance);
            Container->SetValue((uint8*)ValuePtr);

			Container->GetValue((uint8*)ValuePtr);
			float OutputValue = ObjectInstance->SomeFloat;

			TestEqual("Input and Output", OutputValue, InputValue);
        });
	});

	Describe("Vector", [this]
	{
		It("Can_Create_Value", [this]
		{
			UPropertyContainerTestObject* ObjectInstance = NewObject<UPropertyContainerTestObject>(GetTransientPackage());

			FProperty* ValueProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector));
			check(ValueProperty);

			URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
			URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_VectorProperty, ValueProperty);

			TestNotNull("Container", Container);
		});
		It("Can_Set_Value", [this]
		{
			UPropertyContainerTestObject* ObjectInstance = NewObject<UPropertyContainerTestObject>(GetTransientPackage());
			
		    FProperty* ValueProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector));
		    check(ValueProperty);
				
		    URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
		    URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_VectorProperty, ValueProperty);

            void* ValuePtr = ValueProperty->ContainerPtrToValuePtr<void>(ObjectInstance);

			Container->SetValue((uint8*)ValuePtr);
		});
		It("Can_Get_Value", [this]
        {
            UPropertyContainerTestObject* ObjectInstance = NewObject<UPropertyContainerTestObject>(GetTransientPackage());
			
            FProperty* ValueProperty = UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector));
            check(ValueProperty);
				
            URCPropertyContainerRegistry* PropertyContainerRegistry = GEngine->GetEngineSubsystem<URCPropertyContainerRegistry>();
            URCPropertyContainerBase* Container = PropertyContainerRegistry->CreateContainer(GetTransientPackage(), NAME_VectorProperty, ValueProperty);

			FVector InputValue(0.45f, 0.65f, -1.24f);
			ObjectInstance->SomeVector = InputValue;
			
            void* ValuePtr = ValueProperty->ContainerPtrToValuePtr<void>(ObjectInstance);
            Container->SetValue((uint8*)ValuePtr);

			Container->GetValue((uint8*)ValuePtr);
			FVector OutputValue = ObjectInstance->SomeVector;

			TestEqual("Input and Output", OutputValue, InputValue);
        });
	});
}

#endif
#endif
