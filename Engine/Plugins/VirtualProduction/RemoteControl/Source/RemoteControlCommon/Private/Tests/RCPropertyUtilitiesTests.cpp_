// Copyright Epic Games, Inc. All Rights Reserved.

#include "ISinglePropertyView.h"
#include "Misc/AutomationTest.h"

#include "RCPropertyContainer.h"
#include "RCPropertyUtilities.h"
#include "RCTypeTraits.h"
#include "RCTypeUtilities.h"

#if WITH_EDITOR
#include "Editor.h"
#include "RCPropertyContainerTestData.h"
#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FRCPropertyUtilitiesSpec,
	"Plugin.RemoteControl.RCPropertyUtilities",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags_ApplicationContextMask)
END_DEFINE_SPEC(FRCPropertyUtilitiesSpec)

void FRCPropertyUtilitiesSpec::Define()
{
	// Ensure that the ExpectedValue is the same when stored and retrieved
	// Takes pre-created Src and Dst PropertyVariants
	auto MakeTestForSrcDst = [this](auto* InProperty, auto InExpectedValue, RemoteControlPropertyUtilities::FRCPropertyVariant& InSrc, RemoteControlPropertyUtilities::FRCPropertyVariant& OutDst)
	{
		check(InProperty);

		using PropertyType = typename TDecay<decltype(InProperty)>::Type;
		using ValueType = typename TDecay<decltype(InExpectedValue)>::Type;

		TArray<uint8> Buffer;			
		RemoteControlPropertyUtilities::FRCPropertyVariant Src = InSrc;
		RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, Buffer};
		auto bToBinaryOk = RemoteControlPropertyUtilities::ToBinary<PropertyType>(Src, Dst);
		TestTrue("ToBinary", bToBinaryOk);

		Src = RemoteControlPropertyUtilities::FRCPropertyVariant{InProperty, Buffer};
		Dst = OutDst;
		auto bFromBinaryOk = RemoteControlPropertyUtilities::FromBinary<PropertyType>(Src, Dst);
		TestTrue("FromBinary", bFromBinaryOk);

		TestEqual<ValueType>("DeserializedValue", *Dst.GetPropertyValue<ValueType>(), InExpectedValue);
	};

	// Check that a property can be serialized from an IPropertyHandle
	Describe("ToBinaryFromHandle", [this, MakeTestForSrcDst]
	{
		UPropertyContainerTestObject* DefaultTestObject = GetMutableDefault<UPropertyContainerTestObject>();

		// Set test values here, they shouldn't match default values!
		UPropertyContainerTestObject* TestObject = NewObject<UPropertyContainerTestObject>();
		TestObject->bSomeBool = true;
		TestObject->SomeUInt32 = 43;
		TestObject->SomeFloat = 0.12f;
		TestObject->SomeVector = FVector(0.1f, 0.5f, 0.9f);
	
		// Ensure that the ExpectedValue is the same when stored and retrieved
		auto MakeTestForType = [this, MakeTestForSrcDst, DefaultTestObject, TestObject](auto* InProperty, auto InExpectedValue)
		{
			check(InProperty);

			using PropertyType = typename TDecay<decltype(InProperty)>::Type;
			using ValueType = typename TDecay<decltype(InExpectedValue)>::Type;

			It(typeid(ValueType).name(), [this, MakeTestForSrcDst, DefaultTestObject, TestObject, InProperty, InExpectedValue]
			{
				FPropertyEditorModule* PropertyEditorModule = FModuleManager::LoadModulePtr<FPropertyEditorModule>(TEXT("PropertyEditorModule"));
				const TSharedPtr<IPropertyHandle> SrcPropertyHandle = PropertyEditorModule->CreateSingleProperty(TestObject, InProperty->GetFName(), {})->GetPropertyHandle();
				const TSharedPtr<IPropertyHandle> DstPropertyHandle = PropertyEditorModule->CreateSingleProperty(DefaultTestObject, InProperty->GetFName(), {})->GetPropertyHandle();
				
				RemoteControlPropertyUtilities::FRCPropertyVariant Src{SrcPropertyHandle};
				RemoteControlPropertyUtilities::FRCPropertyVariant Dst{DstPropertyHandle};

				return MakeTestForSrcDst(InProperty, InExpectedValue, Src, Dst);
			});
		};

		MakeTestForType(
			CastField<FBoolProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, bSomeBool))),
			TestObject->bSomeBool);

		MakeTestForType(			
			CastField<FUInt32Property>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeUInt32))),
			TestObject->SomeUInt32);

		MakeTestForType(
			CastField<FFloatProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeFloat))),
			TestObject->SomeFloat);

		MakeTestForType(			
			CastField<FStructProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector))),
			TestObject->SomeVector);
	});

	// Check that property can be serialized from a source byte array
	Describe("ToBinaryFromArray", [this, MakeTestForSrcDst]
	{
		UPropertyContainerTestObject* DefaultTestObject = GetMutableDefault<UPropertyContainerTestObject>();

		// Set test values here, they shouldn't match default values!
		UPropertyContainerTestObject* TestObject = NewObject<UPropertyContainerTestObject>();
		TestObject->bSomeBool = true;
		TestObject->SomeUInt32 = 43;
		TestObject->SomeFloat = 0.12f;
		TestObject->SomeVector = FVector(0.1f, 0.5f, 0.9f);
	
		// Ensure that the ExpectedValue is the same when stored and retrieved
		auto MakeTestForType = [this, MakeTestForSrcDst, DefaultTestObject, TestObject](auto* InProperty, auto InExpectedValue)
		{
			check(InProperty);

			using PropertyType = typename TDecay<decltype(InProperty)>::Type;
			using ValueType = typename TDecay<decltype(InExpectedValue)>::Type;
			
			It(typeid(ValueType).name(), [this, MakeTestForSrcDst, DefaultTestObject, TestObject, InProperty, InExpectedValue]
			{
				TArray<uint8> PropertyData;

				// 1. Get and store value from PropertyHandle
				{
					FPropertyEditorModule* PropertyEditorModule = FModuleManager::LoadModulePtr<FPropertyEditorModule>(TEXT("PropertyEditorModule"));
					const TSharedPtr<IPropertyHandle> SrcPropertyHandle = PropertyEditorModule->CreateSingleProperty(TestObject, InProperty->GetFName(), {})->GetPropertyHandle();

					RemoteControlPropertyUtilities::FRCPropertyVariant Src{SrcPropertyHandle};
					RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, PropertyData};
					RemoteControlPropertyUtilities::ToBinary<PropertyType>(Src, Dst);
				}

				// 2. Try serializing from PropertyData
				{
					TArray<uint8> DstBuffer;
					RemoteControlPropertyUtilities::FRCPropertyVariant Src{InProperty, PropertyData};
					RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, DstBuffer};

					return MakeTestForSrcDst(InProperty, InExpectedValue, Src, Dst);
				}
			});
		};

		MakeTestForType(
			CastField<FBoolProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, bSomeBool))),
			TestObject->bSomeBool);

		MakeTestForType(			
			CastField<FUInt32Property>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeUInt32))),
			TestObject->SomeUInt32);

		MakeTestForType(
			CastField<FFloatProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeFloat))),
			TestObject->SomeFloat);

		MakeTestForType(			
			CastField<FStructProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector))),
			TestObject->SomeVector);
	});

	// Check that property CANNOT be serialized from a source pointer
	Describe("ToBinaryFromPtr", [this, MakeTestForSrcDst]
	{
		UPropertyContainerTestObject* DefaultTestObject = GetMutableDefault<UPropertyContainerTestObject>();

		// Set test values here, they shouldn't match default values!
		UPropertyContainerTestObject* TestObject = NewObject<UPropertyContainerTestObject>();
		TestObject->bSomeBool = true;
		TestObject->SomeUInt32 = 43;
		TestObject->SomeFloat = 0.12f;
		TestObject->SomeVector = FVector(0.1f, 0.5f, 0.9f);
		
		// Ensure that the ExpectedValue is the same when stored and retrieved
		auto MakeTestForType = [this, MakeTestForSrcDst, DefaultTestObject, TestObject](auto* InProperty, auto InExpectedValue)
		{
			check(InProperty);

			using PropertyType = typename TDecay<decltype(InProperty)>::Type;
			using ValueType = typename TDecay<decltype(InExpectedValue)>::Type;
			
			It(typeid(ValueType).name(), [this, MakeTestForSrcDst, DefaultTestObject, TestObject, InProperty, InExpectedValue]
			{
				TArray<uint8> PropertyData;

				// 1. Get and store value from PropertyHandle
				{
					FPropertyEditorModule* PropertyEditorModule = FModuleManager::LoadModulePtr<FPropertyEditorModule>(TEXT("PropertyEditorModule"));
					const TSharedPtr<IPropertyHandle> SrcPropertyHandle = PropertyEditorModule->CreateSingleProperty(TestObject, InProperty->GetFName(), {})->GetPropertyHandle();

					RemoteControlPropertyUtilities::FRCPropertyVariant Src{SrcPropertyHandle};
					RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, PropertyData};
					RemoteControlPropertyUtilities::ToBinary<PropertyType>(Src, Dst);
				}

				// 2. Try serializing from PropertyData ptr
				{
					TArray<uint8> DstBuffer;
					RemoteControlPropertyUtilities::FRCPropertyVariant Src{InProperty, PropertyData.GetData(), PropertyData.Num()};
					RemoteControlPropertyUtilities::FRCPropertyVariant Dst{InProperty, DstBuffer};

					return MakeTestForSrcDst(InProperty, InExpectedValue, Src, Dst);
				}
			});
		};


		MakeTestForType(
			CastField<FBoolProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, bSomeBool))),
			false);

		MakeTestForType(
			CastField<FBoolProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, bSomeBool))),
			true);

		MakeTestForType(			
			CastField<FUInt32Property>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeUInt32))),
20);

		MakeTestForType(
			CastField<FFloatProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeFloat))),
			0.4f);

		MakeTestForType(			
			CastField<FStructProperty>(UPropertyContainerTestObject::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UPropertyContainerTestObject, SomeVector))),
FVector(0.1, 0.5, 0.9));
	});
}
#endif
#endif
