// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/SoftObjectPath.h"
#include "RCAllowedRemoteFunctionCall.generated.h"

/** Describes a rule to allow a function (or all the functions) within a specified class to be called remotely */
USTRUCT()
struct FRCAllowedRemoteFunctionCall
{
	GENERATED_BODY()

	/** The path of the class to allow */
	UPROPERTY(EditAnywhere, Category="Remote Control")
	FSoftClassPath ClassPath;

	/**
	 * If set, the name of the function to allow. If unset, any function in the class will be allowed.
	 * NOTE: This only accepts the reflected UFUNCTION name, not the script name.
	 * E.g. USceneComponent::K2_GetComponentToWorld's script name is 'GetWorldTransform', but "K2_GetComponentToWorld" needs to be specified instead.
	 */
	UPROPERTY(EditAnywhere, Category="Remote Control")
	TOptional<FString> FunctionName;

	/**
	 * Determines whether child classes of ClassPath are allowed.
	 * NOTE: Prefer keeping this disabled when 'FunctionName' is not set to disallow arbitrary child classes from executing their functions because of this rule.
	 */
	UPROPERTY(EditAnywhere, Category="Remote Control", meta=(DisplayAfter="ClassPath"))
	bool bAllowChildClasses = false;
};
