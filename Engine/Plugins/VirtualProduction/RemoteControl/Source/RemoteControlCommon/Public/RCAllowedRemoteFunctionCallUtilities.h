// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"

#define UE_API REMOTECONTROLCOMMON_API

class UClass;
class UFunction;
class UObject;
struct FRCAllowedRemoteFunctionCall;

namespace UE::RemoteControl
{

struct FRCRemoteFunctionCallParams
{
	/** Required: Path to the object class containing the function prospect to call */
	const UClass* ObjectClass = nullptr;
	/** Required: Name of the function prospect to call */
	FString FunctionName;
};
/** Makes a remote function call with the given function and object. */
UE_API FRCRemoteFunctionCallParams MakeRemoteFunctionCallParams(const UObject* InObject, const UFunction* InFunction);

/** 
 * Checks whether the given parameters match the single allowed call.
 * @param InParams the call parameters (object class and function) to test
 * @param InAllowedCall the call to match
 * @return true if the remote call from the parameters is allowed
 */
UE_API bool IsRemoteFunctionCallAllowed(const FRCRemoteFunctionCallParams& InParams, const FRCAllowedRemoteFunctionCall& InAllowedCall);

/**
 * Returns true if the given remote call satisfy at least one of the given filters
 * @param InParams the call parameters (object class and function) to test
 * @param InAllowedCalls allowlist of calls
 * @return true if the remote call from the parameters is allowed
 */
UE_API bool IsRemoteFunctionCallAllowed(const FRCRemoteFunctionCallParams& InParams, TConstArrayView<FRCAllowedRemoteFunctionCall> InAllowedCalls);

} // UE::RemoteControl

#undef UE_API
