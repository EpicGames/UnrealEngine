// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Subsystems/EngineSubsystem.h"
#include "Templates/SubclassOf.h"
#include "UObject/Object.h"
#include "UObject/WeakFieldPtr.h"

#include "RCPropertyContainer.generated.h"

UCLASS(Transient, Abstract)
class REMOTECONTROLCOMMON_API URCPropertyContainerBase : public UObject
{
	GENERATED_BODY()

public:
	/** Sets the value from the incoming raw data. Provide size for array, string, etc. */
	void SetValue(const uint8* InData, const SIZE_T& InSize = 0);

	/** Sets the value as ValueType, Not necessarily valid if using incorrect ValueType. */
	template <typename ValueType>
	typename TEnableIf<TIsPointer<ValueType>::Value, void>::Type
	SetValue(const ValueType& InValue)
	{
		using DecayedValueType = typename TDecay<TRemovePointer<ValueType>>::Type;

#if WITH_EDITOR
		Modify();
		FProperty* Property = GetValueProperty();
		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		PreEditChange(EditChain);
#endif

		DecayedValueType* ValuePtr = GetValueProperty()->ContainerPtrToValuePtr<DecayedValueType>(this);
		GetValueProperty()->CopyCompleteValue(
			(void*)ValuePtr,
			(const DecayedValueType*)InValue);

#if WITH_EDITOR
		FPropertyChangedEvent EditPropertyChangeEvent(Property, EPropertyChangeType::ValueSet);
		PostEditChangeProperty(EditPropertyChangeEvent);
		FPropertyChangedChainEvent EditChangeChainEvent(EditChain, EditPropertyChangeEvent);
		PostEditChangeChainProperty(EditChangeChainEvent);
#endif
	}

	template <typename ValueType>
	typename TEnableIf<!TIsPointer<ValueType>::Value, void>::Type
	SetValue(const ValueType& InValue)
	{
		SetValue(&InValue);		
	}
	
	/** Writes to the provided raw data pointer. Returns size for array, string, etc. */
	SIZE_T GetValue(uint8* OutData);

	/** Writes to the provided data array. Returns size for array, string, etc. */
	SIZE_T GetValue(TArray<uint8>& OutData);

	/** Returns the Value as ValueType. Not necessarily valid if using incorrect ValueType. */
	template <typename ValueType>
	ValueType* GetValue()
	{
		return GetValueProperty()->ContainerPtrToValuePtr<ValueType>(this);
	}

	/** Returns the Property for Value */
	virtual FProperty* GetValueProperty();

private:
	TWeakFieldPtr<FProperty> ValueProperty;
};

/** Minimal information needed to lookup a unique property container class */
USTRUCT()
struct REMOTECONTROLCOMMON_API FRCPropertyContainerKey
{
	GENERATED_BODY()

	UPROPERTY()
	FName ValueTypeName;

	FName ToClassName() const;
};

inline uint64 GetTypeHash(const FRCPropertyContainerKey& InValue) { return GetTypeHash(InValue.ValueTypeName); }
inline bool operator==(const FRCPropertyContainerKey& Lhs, const FRCPropertyContainerKey& Rhs) { return Lhs.ValueTypeName == Rhs.ValueTypeName; }
inline bool operator!=(const FRCPropertyContainerKey& Lhs, const FRCPropertyContainerKey& Rhs) { return Lhs.ValueTypeName != Rhs.ValueTypeName ; }

/** A subsystem to provide and cache dynamically created PropertyContainer classes. */
UCLASS()
class REMOTECONTROLCOMMON_API URCPropertyContainerRegistry : public UEngineSubsystem
{
	GENERATED_BODY()
	
public:
	/** Creates a (UObject) container, with a single Value property of the given type. */
	URCPropertyContainerBase* CreateContainer(UObject* InOwner, const FName& InValueTypeName, const FProperty* InValueSrcProperty);
	
private:
	UPROPERTY(Transient)
	TMap<FRCPropertyContainerKey, TSubclassOf<URCPropertyContainerBase>> CachedContainerClasses;

	/** Finds (cached) or creates a new container for the given property. */
	TSubclassOf<URCPropertyContainerBase>& FindOrAddContainerClass(const FName& InValueTypeName, const FProperty* InValueSrcProperty);
};

namespace PropertyContainers
{
	REMOTECONTROLCOMMON_API URCPropertyContainerBase* CreateContainerForProperty(UObject* InOwner, const FProperty* InSrcProperty);

#if WITH_EDITOR
	/** @note: use with caution, currently only used for testing */
	REMOTECONTROLCOMMON_API FProperty* CreateProperty(const FFieldVariant& InParent, const FFieldVariant& InChild, const FName& InPropertyName = NAME_None, EObjectFlags InObjectFlags = EObjectFlags::RF_Public);
#endif
}
