// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RCTypeTraits.h"
#include "Serialization/BufferArchive.h"
#include "UObject/WeakFieldPtr.h"

#if WITH_EDITOR
#include "PropertyHandle.h"
#endif

namespace RemoteControlPropertyUtilities
{
	static TMap<TWeakFieldPtr<const FProperty>, TWeakObjectPtr<UFunction>> CachedSetterFunctions;
	static const FName NAME_BlueprintGetter(TEXT("BlueprintGetter"));
	static const FName NAME_BlueprintSetter(TEXT("BlueprintSetter"));

	/** Container that can hold either a PropertyHandle, or Property/Data pair. Similar to FFieldVariant */
	class FRCPropertyVariant
	{
#if WITH_EDITOR
		TSharedPtr<IPropertyHandle> PropertyHandle = nullptr;
#endif
		TWeakFieldPtr<const FProperty> Property = nullptr;
		void* PropertyData = nullptr;
		TArray<uint8>* PropertyContainer = nullptr;

		bool bHasHandle = false;
		int32 NumElements = 1;

	public:
		explicit FRCPropertyVariant() = default;

#if WITH_EDITOR
		/** Construct from an IPropertyHandle. */
		FRCPropertyVariant(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
#endif

		/** Construct from a Property, PropertyData ptr, and the expected element count (needed for arrays, strings, etc.). */
		FRCPropertyVariant(const FProperty* InProperty, const void* InPropertyData, const int32& InNumElements = 1);

		/** Construct from a Property and backing data array. Preferred over a raw ptr. */
		FRCPropertyVariant(const FProperty* InProperty, TArray<uint8>& InPropertyData, const int32& InNumElements = -1);
		
		virtual ~FRCPropertyVariant() = default;

		/** Gets the property. */
		FProperty* GetProperty() const;

		/** Gets the typed property, returns nullptr if not cast. */
		template <typename PropertyType>
		PropertyType* GetProperty() const
		{
			static_assert(TIsDerivedFrom<PropertyType, FProperty>::Value, "PropertyType must derive from FProperty");

			return CastField<PropertyType>(GetProperty());
		} 

		/** Gets the property container (byte array), if available. */
		TArray<uint8>* GetPropertyContainer() const;

		/** Gets the data pointer */
		void* GetPropertyData(const FProperty* InContainer = nullptr, int32 InIdx = 0) const;
		
		/** Returns the data as the ValueType. */
		template <typename ValueType>
		ValueType* GetPropertyValue(const FProperty* InContainer = nullptr, int32 InIdx = 0) const
		{
			return (ValueType*)(GetPropertyData(InContainer, InIdx));
		}

		/** Is this backed by an IPropertyHandle? */
		bool IsHandle() const
		{
			return bHasHandle;
		}

		/** Initialize/allocate if necessary. */
		void Init(int32 InSize = -1);
		
		/** Gets the number of elements, more than 1 if an array. */
		int32 Num() const
		{
			return NumElements;
		}
		
		/** Gets the size of the underlying data. */
		int32 Size() const
		{
			return PropertyContainer ? PropertyContainer->Num() : GetElementSize() * Num();
		}

		/** Gets individual element size (for arrays, etc.). */
		int32 GetElementSize() const;
		
		/** Calculate number of elements based on available info. If OtherProperty is specified, that's used instead. */
		void InferNum(const FRCPropertyVariant* InOtherProperty = nullptr);
		
		/** Comparison is by property, not by property instance value. */
		bool operator==(const FRCPropertyVariant& InOther) const
		{
			return GetProperty() == InOther.GetProperty();
		}
		
		bool operator!=(const FRCPropertyVariant& InOther) const
		{
			return GetProperty() != InOther.GetProperty();
		}
	};

	template <>
	FString* FRCPropertyVariant::GetPropertyValue<FString>(const FProperty* InContainer, int32 InIdx) const;
	
	/** Reads the raw data from InSrc and deserializes to OutDst. */
	template <typename PropertyType>
	typename TEnableIf<
		(
			TIsDerivedFrom<PropertyType, FProperty>::Value &&
			!std::is_same_v<PropertyType, FProperty> &&
			!std::is_same_v<PropertyType, FNumericProperty>
		) ||
		std::is_same_v<PropertyType, FEnumProperty>, bool>::Type
	Deserialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);
	
	/** Reads the property value from InSrc and serializes to OutDst. */
	template <typename PropertyType>
	typename TEnableIf<
		(
			TIsDerivedFrom<PropertyType, FProperty>::Value &&
			!std::is_same_v<PropertyType, FProperty> &&
			!std::is_same_v<PropertyType, FNumericProperty>
		) ||
		std::is_same_v<PropertyType, FEnumProperty>, bool>::Type
	Serialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);

	/** Specialization for FBoolProperty. */
	template <>
	bool Deserialize<FBoolProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);

	/** Specialization for FStructProperty. */
	template <>
	bool Deserialize<FStructProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);

	/** Specialization for FProperty casts and forwards to specializations. */
	template <typename PropertyType>
	typename TEnableIf<
			std::is_same_v<PropertyType, FProperty> ||
			std::is_same_v<PropertyType, FNumericProperty>, bool>::Type
	Deserialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);
	
	/** Specialization for FProperty casts and forwards to specializations. */
	template <typename PropertyType>
	typename TEnableIf<
		std::is_same_v<PropertyType, FProperty> ||
		std::is_same_v<PropertyType, FNumericProperty>, bool>::Type
	Serialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst);

	static FProperty* FindSetterArgument(UFunction* SetterFunction, const FProperty* PropertyToModify);

	/** LightComponent derived components use a lot of property setters, without specifying BlueprintSetter, so handle here.  */
	static FName FindLightSetterFunctionInternal(const FProperty* Property, UClass* OwnerClass);

	static UFunction* FindSetterFunctionInternal(const FProperty* Property, UClass* OwnerClass);
	
	static UFunction* FindSetterFunction(const FProperty* Property, UClass* OwnerClass = nullptr);
}

#include "RCPropertyUtilities.inl"
