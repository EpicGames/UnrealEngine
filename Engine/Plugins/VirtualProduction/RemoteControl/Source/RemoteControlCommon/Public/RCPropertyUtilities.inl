// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"

#include "RCTypeTraits.h"
#include "RCTypeUtilities.h"
#include "Components/LightComponent.h"
#include "Components/SceneComponent.h"
#include "Serialization/BufferArchive.h"
#include "Serialization/MemoryReader.h"
#include "UObject/WeakFieldPtr.h"

#if WITH_EDITOR
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#endif

namespace RemoteControlPropertyUtilities
{
#if WITH_EDITOR
	/** Construct from an IPropertyHandle. */
	inline FRCPropertyVariant::FRCPropertyVariant(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
		: bHasHandle(true)
	{
		PropertyHandle = InPropertyHandle;
		Property = PropertyHandle->GetProperty();
	}
#endif

	/** Construct from a Property, PropertyData ptr, and the expected element count (needed for arrays, strings, etc.). */
	inline FRCPropertyVariant::FRCPropertyVariant(const FProperty* InProperty, const void* InPropertyData, const int32& InNumElements)
		: NumElements(InNumElements)
	{
		Property = InProperty;
		PropertyData = const_cast<void*>(InPropertyData);
	}

	/** Construct from a Property and backing data array. Preferred over a raw ptr. */
	inline FRCPropertyVariant::FRCPropertyVariant(const FProperty* InProperty, TArray<uint8>& InPropertyData, const int32& InNumElements)
	{
		Property = InProperty;

		PropertyData = InPropertyData.GetData();
		PropertyContainer = &InPropertyData;
		NumElements = InNumElements > 0 ? InNumElements : InPropertyData.Num() / InProperty->GetSize();
	}

	/** Gets the property. */
	inline FProperty* FRCPropertyVariant::GetProperty() const
	{
		if(Property.IsValid())
		{
			// API is not const-correct
			return const_cast<FProperty*>(Property.Get());
		}

#if WITH_EDITOR
		if(bHasHandle && PropertyHandle.IsValid() && PropertyHandle->IsValidHandle())
		{
			return PropertyHandle->GetProperty();
		}
#endif

		return nullptr;
	}

	/** Gets the property container (byte array), if available. */
	inline TArray<uint8>* FRCPropertyVariant::GetPropertyContainer() const
	{
		if(PropertyContainer)
		{
			return PropertyContainer;
		}

		return nullptr;
	}
	
	/** Gets the data pointer */
	inline void* FRCPropertyVariant::GetPropertyData(const FProperty* InContainer, int32 InIdx) const
	{
#if WITH_EDITOR
		if(bHasHandle)
		{
			TArray<void*> Data;
			TSharedPtr<IPropertyHandle> InnerHandle = PropertyHandle;

			InnerHandle->AccessRawData(Data);
			check(Data.IsValidIndex(0)); // check that there's at least one set of data

			return Data[0];
		}
#endif

		if(PropertyContainer)
		{
			return PropertyContainer->GetData();
		}

		return PropertyData;
	}

	/** Initialize/allocate if necessary. */
	inline void FRCPropertyVariant::Init(int32 InSize)
	{
		InSize = InSize > 0 ? InSize : GetElementSize();		
		if(!bHasHandle)
		{
			if(PropertyContainer)
			{
				PropertyContainer->SetNumZeroed(InSize);
				PropertyData = PropertyContainer->GetData();
			}
			else
			{
				PropertyData = FMemory::Malloc(InSize, GetProperty()->GetMinAlignment());
			}
			NumElements = 1;
		}
	}

	/** Gets individual element size (for arrays, etc.). */
	inline int32 FRCPropertyVariant::GetElementSize() const
	{
		int32 ElementSize = GetProperty()->GetSize();
		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(GetProperty()))
		{
			ElementSize = ArrayProperty->Inner->GetSize();
		}
		else if(const FSetProperty* SetProperty = CastField<FSetProperty>(GetProperty()))
		{
			ElementSize = SetProperty->ElementProp->GetSize();
		}
		else if(const FMapProperty* MapProperty = CastField<FMapProperty>(GetProperty()))
		{
			ElementSize = MapProperty->ValueProp->GetSize();
		}
		
		return ElementSize;
	}

	/** Calculate number of elements based on available info. If OtherProperty is specified, that's used instead. */
	inline void FRCPropertyVariant::InferNum(const FRCPropertyVariant* InOtherProperty)
	{
		const FRCPropertyVariant* SrcVariant = InOtherProperty ? InOtherProperty : this;
		
		const int32 ElementSize = SrcVariant->GetElementSize();
		int32 TotalSize = ElementSize;

		if(SrcVariant->PropertyContainer != nullptr)
		{
			// the first element is the item count (int32) so remove to get actual element count
			//TotalSize = InOtherProperty->PropertyContainer->Num() - sizeof(int32);

			if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(SrcVariant->GetProperty()))
			{
				FScriptArrayHelper Helper(ArrayProperty, SrcVariant->GetPropertyData());
				NumElements = Helper.Num();
				return;
			}
			else if(const FSetProperty* SetProperty = CastField<FSetProperty>(SrcVariant->GetProperty()))
			{
			}
			else if(const FMapProperty* MapProperty = CastField<FMapProperty>(SrcVariant->GetProperty()))
			{
			}
		}
#if WITH_EDITOR
		else if(SrcVariant->bHasHandle)
		{
			if(const TSharedPtr<IPropertyHandleArray> ArrayHandle = SrcVariant->PropertyHandle->AsArray())
			{
				uint32 N;
				if(ArrayHandle->GetNumElements(N) == FPropertyAccess::Success)
				{
					NumElements = N;
					return;	
				}
			}
			else if(TSharedPtr<IPropertyHandleSet> SetHandle = SrcVariant->PropertyHandle->AsSet())
			{
				uint32 N;
				if(SetHandle->GetNumElements(N) == FPropertyAccess::Success)
				{
					NumElements = N;
					return;	
				}
			}
			else if(TSharedPtr<IPropertyHandleMap> MapHandle = SrcVariant->PropertyHandle->AsMap())
			{
				uint32 N;
				if(MapHandle->GetNumElements(N) == FPropertyAccess::Success)
				{
					NumElements = N;
					return;	
				}
			}
		}
#endif
	}

	template <>
	inline FString* FRCPropertyVariant::GetPropertyValue<FString>(const FProperty* InContainer, int32 InIdx) const
	{
		return GetProperty<FStrProperty>()->GetPropertyValuePtr(GetPropertyData(InContainer, InIdx));
	}

	/** Reads the raw data from InSrc and deserializes to OutDst. */
	template <typename PropertyType>
	typename TEnableIf<
		(
			TIsDerivedFrom<PropertyType, FProperty>::Value &&
			!std::is_same_v<PropertyType, FProperty> &&
			!std::is_same_v<PropertyType, FNumericProperty>
		) ||
		std::is_same_v<PropertyType, FEnumProperty>, bool>::Type
	Deserialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		using ValueType = typename TRemoteControlPropertyTypeTraits<PropertyType>::ValueType;
		
		TArray<uint8>* SrcPropertyContainer = InSrc.GetPropertyContainer();
		checkf(SrcPropertyContainer != nullptr, TEXT("Deserialize requires Src to have a backing container."));

		OutDst.Init(InSrc.Size()); // initializes only if necessary
		
		ValueType* DstCurrentValue = OutDst.GetPropertyValue<ValueType>();
		InSrc.GetProperty()->InitializeValue(DstCurrentValue);

		const int32 SrcSize = InSrc.Size();

		// The stored data size doesn't match, so cast
		if(OutDst.GetProperty()->GetElementSize() != SrcSize)
		{
			if(const FNumericProperty* DstProperty = OutDst.GetProperty<FNumericProperty>())
			{
				// @note this only works for integers
				if(DstProperty->IsInteger())
				{
					if(SrcSize == 1)
					{
						uint8* SrcValue = InSrc.GetPropertyValue<uint8>();
						if(SrcValue)
						{
							DstProperty->SetIntPropertyValue(DstCurrentValue, static_cast<uint64>(*SrcValue));
							return true;
						}
					}
					else if(SrcSize == 2)
					{
						uint16* SrcValue = InSrc.GetPropertyValue<uint16>();
						if(SrcValue)
						{
							DstProperty->SetIntPropertyValue(DstCurrentValue, static_cast<uint64>(*SrcValue));
							return true;
						}
					}
					else if(SrcSize == 4)
					{
						uint32* SrcValue = InSrc.GetPropertyValue<uint32>();
						if(SrcValue)
						{
							DstProperty->SetIntPropertyValue(DstCurrentValue, static_cast<uint64>(*SrcValue));
							return true;
						}
					}
				}
			}
		}

		FMemoryReader Reader(*SrcPropertyContainer);
		InSrc.GetProperty()->SerializeItem(FStructuredArchiveFromArchive(Reader).GetSlot(), DstCurrentValue, nullptr);

 		return true;
	}
	
	/** Reads the property value from InSrc and serializes to OutDst. */
	template <typename PropertyType>
	typename TEnableIf<
		(
			TIsDerivedFrom<PropertyType, FProperty>::Value &&
			!std::is_same_v<PropertyType, FProperty> &&
			!std::is_same_v<PropertyType, FNumericProperty>
		) ||
		std::is_same_v<PropertyType, FEnumProperty>, bool>::Type
	Serialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		using ValueType = typename TRemoteControlPropertyTypeTraits<PropertyType>::ValueType;

		ValueType* SrcValue = InSrc.GetPropertyValue<ValueType>();

		TArray<uint8>* DstPropertyContainer = OutDst.GetPropertyContainer();
		checkf(DstPropertyContainer != nullptr, TEXT("Serialize requires Dst to have a backing container."));

		DstPropertyContainer->Empty();
		OutDst.Init(InSrc.GetProperty()->GetSize()); // initializes only if necessary
		InSrc.GetProperty()->InitializeValue(DstPropertyContainer->GetData());
		
		FMemoryWriter Writer(*DstPropertyContainer);
		OutDst.GetProperty()->SerializeItem(FStructuredArchiveFromArchive(Writer).GetSlot(), SrcValue, nullptr);

		OutDst.InferNum(&InSrc);

		return true;
	}

	/** Specialization for FBoolProperty. */
	template <>
	inline bool Deserialize<FBoolProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		using ValueType = TRemoteControlPropertyTypeTraits<FBoolProperty>::ValueType;
		
		TArray<uint8>* SrcPropertyContainer = InSrc.GetPropertyContainer();
		checkf(SrcPropertyContainer != nullptr, TEXT("Deserialize requires Src to have a backing container."));

		OutDst.Init(InSrc.Size()); // initializes only if necessary

		const ValueType* SrcCurrentValue = InSrc.GetPropertyValue<ValueType>();
		ValueType* DstCurrentValue = OutDst.GetPropertyValue<ValueType>();
		OutDst.GetProperty<FBoolProperty>()->SetPropertyValue(DstCurrentValue, *SrcCurrentValue);

		return true;
	}

	/** Specialization for FStructProperty. */
	template <>
	inline bool Deserialize<FStructProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		TArray<uint8>* SrcPropertyContainer = InSrc.GetPropertyContainer();
		checkf(SrcPropertyContainer != nullptr, TEXT("Deserialize requires Src to have a backing container."));
		
		OutDst.Init(InSrc.Size()); // initializes only if necessary
		void* DstCurrentValue = OutDst.GetPropertyValue<void>();
		InSrc.GetProperty<FStructProperty>()->Struct->InitializeStruct(DstCurrentValue);
		
		FMemoryReader Reader(*SrcPropertyContainer);
		InSrc.GetProperty()->SerializeItem(FStructuredArchiveFromArchive(Reader).GetSlot(), DstCurrentValue, nullptr);

		return true;
	}

	/** Specialization for FProperty casts and forwards to specializations. */
	template <typename PropertyType>
	typename TEnableIf<
		std::is_same_v<PropertyType, FProperty> ||
		std::is_same_v<PropertyType, FNumericProperty>, bool>::Type
	Deserialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		const FProperty* Property = OutDst.GetProperty();
		FOREACH_CAST_PROPERTY(Property, Deserialize<CastPropertyType>(InSrc, OutDst))

		return true;
	}

	/** Specialization for FProperty casts and forwards to specializations. */
	template <typename PropertyType>
	typename TEnableIf<
		std::is_same_v<PropertyType, FProperty> ||
		std::is_same_v<PropertyType, FNumericProperty>, bool>::Type
	Serialize(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		const FProperty* Property = InSrc.GetProperty();
		FOREACH_CAST_PROPERTY(Property, Serialize<CastPropertyType>(InSrc, OutDst))

		return true;
	}

	static FProperty* FindSetterArgument(UFunction* SetterFunction, const FProperty* PropertyToModify)
	{
		FProperty* SetterArgument = nullptr;

		if (!ensure(SetterFunction))
		{
			return nullptr;
		}

		// Check if the first parameter for the setter function matches the parameter value.
		for (TFieldIterator<FProperty> PropertyIt(SetterFunction); PropertyIt; ++PropertyIt)
		{
			if (PropertyIt->HasAnyPropertyFlags(CPF_Parm) && !PropertyIt->HasAnyPropertyFlags(CPF_ReturnParm))
			{
				if (PropertyIt->SameType(PropertyToModify))
				{
					SetterArgument = *PropertyIt;
				}

				break;
			}
		}

		return SetterArgument;
	}

	/** LightComponent derived components use a lot of property setters, without specifying BlueprintSetter, so handle here.  */
	static FName FindLightSetterFunctionInternal(const FProperty* Property, UClass* OwnerClass)
	{
		static const FName LightAffectDynamicIndirectLightingPropertyName = TEXT("bAffectDynamicIndirectLighting");
		static const FName LightAffectTranslucentLightingPropertyName = TEXT("bAffectTranslucentLighting");
		static const FName LightBloomMaxBrightnessPropertyName = TEXT("BloomMaxBrightness");
		static const FName LightBloomScalePropertyName = TEXT("BloomScale");
		static const FName LightBloomThresholdPropertyName = TEXT("BloomThreshold");
		static const FName LightBloomTintPropertyName = TEXT("BloomTint");
		static const FName LightColorPropertyName = TEXT("LightColor");
		static const FName LightEnableLightShaftBloomPropertyName = TEXT("bEnableLightShaftBloom");
		static const FName LightForceCachedShadowsForMovablePrimitivesPropertyName = TEXT("bForceCachedShadowsForMovablePrimitives");
		static const FName LightFunctionDisabledBrightnessPropertyName = TEXT("LightFunctionDisabledBrightness");
		static const FName LightFunctionFadeDistancePropertyName = TEXT("LightFunctionFadeDistance");
		static const FName LightFunctionMaterialPropertyName = TEXT("LightFunctionMaterial");
		static const FName LightFunctionScalePropertyName = TEXT("LightFunctionScale");
		static const FName LightIESBrightnessScalePropertyName = TEXT("IESBrightnessScale");
		static const FName LightIESTexturePropertyName = TEXT("IESTexture");
		static const FName LightIndirectLightingIntensityPropertyName = TEXT("IndirectLightingIntensity");
		static const FName LightIntensityPropertyName = TEXT("Intensity");
		static const FName LightLightingChannelsPropertyName = TEXT("LightingChannels");
		static const FName LightShadowBiasPropertyName = TEXT("ShadowBias");
		static const FName LightShadowSlopeBiasPropertyName = TEXT("ShadowSlopeBias");
		static const FName LightSpecularScalePropertyName = TEXT("SpecularScale");
		static const FName LightDiffuseScalePropertyName = TEXT("DiffuseScale");
		static const FName LightTemperaturePropertyName = TEXT("Temperature");
		static const FName LightTransmissionPropertyName = TEXT("Transmission");
		static const FName LightUseIESBrightnessPropertyName = TEXT("bUseIESBrightness");
		static const FName LightUseTemperaturePropertyName = TEXT("bUseTemperature");
		static const FName LightVolumetricScatteringIntensityPropertyName = TEXT("VolumetricScatteringIntensity");

		static TMap<FName, FName> LightPropertiesToFunctions = {
#if WITH_EDITOR
			// This is used by LightComponentDetails in place of the property, but only available in editor...
			{ LightIntensityPropertyName, TEXT("SetLightBrightness") },
#else
			// ... so at runtime, use SetIntensity
			{ LightIntensityPropertyName, TEXT("SetIntensity") },
#endif
			{ LightAffectDynamicIndirectLightingPropertyName, TEXT("SetAffectDynamicIndirectLighting") },
			{ LightAffectTranslucentLightingPropertyName , TEXT("SetAffectTranslucentLighting") },
			{ LightBloomMaxBrightnessPropertyName, TEXT("SetBloomMaxBrightness") },
			{ LightBloomScalePropertyName, TEXT("SetBloomScale") },
			{ LightBloomThresholdPropertyName, TEXT("SetBloomThreshold") },
			{ LightBloomTintPropertyName, TEXT("SetBloomTint") },
			{ LightColorPropertyName, TEXT("SetLightFColor") }, // special case, setter needs to be same type as property
			{ LightEnableLightShaftBloomPropertyName, TEXT("SetEnableLightShaftBloom") },
			{ LightForceCachedShadowsForMovablePrimitivesPropertyName, TEXT("SetForceCachedShadowsForMovablePrimitives") },
			{ LightFunctionDisabledBrightnessPropertyName, TEXT("SetLightFunctionDisabledBrightness") },
			{ LightFunctionFadeDistancePropertyName, TEXT("SetLightFunctionFadeDistance") },
			{ LightFunctionMaterialPropertyName, TEXT("SetLightFunctionMaterial") },
			{ LightFunctionScalePropertyName, TEXT("SetLightFunctionScale") },
			{ LightIESBrightnessScalePropertyName, TEXT("SetIESBrightnessScale") },
			{ LightIESTexturePropertyName, TEXT("SetIESTexture") },
			{ LightIndirectLightingIntensityPropertyName, TEXT("SetIndirectLightingIntensity") },
			{ LightLightingChannelsPropertyName, TEXT("SetLightingChannels") },
			{ LightShadowBiasPropertyName, TEXT("SetShadowBias") },	
			{ LightShadowSlopeBiasPropertyName, TEXT("SetShadowSlopeBias") },
			{ LightSpecularScalePropertyName, TEXT("SetSpecularScale") },
			{ LightDiffuseScalePropertyName, TEXT("SetDiffuseScale") },
			{ LightTemperaturePropertyName, TEXT("SetTemperature") },
			{ LightTransmissionPropertyName, TEXT("SetTransmission") },
			{ LightUseIESBrightnessPropertyName, TEXT("SetUseIESBrightness") },
			{ LightUseTemperaturePropertyName, TEXT("SetUseTemperature") },
			{ LightVolumetricScatteringIntensityPropertyName, TEXT("SetVolumetricScatteringIntensity") },
		};

		if (LightPropertiesToFunctions.Contains(Property->GetFName()))
		{
			return LightPropertiesToFunctions[Property->GetFName()];
		}

		return NAME_None;	
	}

	static UFunction* FindSetterFunctionInternal(const FProperty* Property, UClass* OwnerClass)
	{
		// Check if the property setter is already cached.
		const TWeakObjectPtr<UFunction> SetterPtr = CachedSetterFunctions.FindRef(Property);
		if (SetterPtr.IsValid())
		{
			return SetterPtr.Get();
		}
		
		static const FName RelativeLocationPropertyName = USceneComponent::GetRelativeLocationPropertyName();
		static const FName RelativeRotationPropertyName = USceneComponent::GetRelativeRotationPropertyName();
		static const FName RelativeScalePropertyName = USceneComponent::GetRelativeScale3DPropertyName();

		static TMap<FName, FName> CommonPropertiesToFunctions = { 
			{ RelativeLocationPropertyName, "K2_SetRelativeLocation"},
			{ RelativeRotationPropertyName, "K2_SetRelativeRotation" },
			{ RelativeScalePropertyName, "SetRelativeScale3D" }
		};

		FString SetterName;
		UFunction* SetterFunction = nullptr;
		if(OwnerClass->IsChildOf<ULightComponent>())
		{
			const FName FoundSetterName = FindLightSetterFunctionInternal(Property, OwnerClass);
			if(!FoundSetterName.IsNone())
			{
				SetterName = FoundSetterName.ToString();				
			}
		}
#if WITH_EDITOR
		if(SetterName.IsEmpty())
		{
			SetterName = Property->GetMetaData(*NAME_BlueprintSetter.ToString());
		}
		
		if (!SetterName.IsEmpty())
		{
			SetterFunction = OwnerClass->FindFunctionByName(*SetterName);
		}
		else
		{
			if (CommonPropertiesToFunctions.Contains(Property->GetFName()))
			{
				SetterFunction = OwnerClass->FindFunctionByName(*CommonPropertiesToFunctions[Property->GetFName()].ToString());
			}
		}
#endif

		if(!SetterFunction)
		{
			FString PropertyName = Property->GetName();
			if (Property->IsA<FBoolProperty>())
			{
				PropertyName.RemoveFromStart("b", ESearchCase::CaseSensitive);
			}

			static const TArray<FString> SetterPrefixes = {
				FString("Set"),
				FString("K2_Set"),
				FString("BP_Set")
			};

			for (const FString& Prefix : SetterPrefixes)
			{
				const FName SetterFunctionName = FName(Prefix + PropertyName);
				SetterFunction = OwnerClass->FindFunctionByName(SetterFunctionName);
				if (SetterFunction)
				{
					break;
				}
			}
		}

		if (SetterFunction && FindSetterArgument(SetterFunction, Property))
		{
			CachedSetterFunctions.Add(Property, SetterFunction);
		}
		else
		{
			// Arguments are not compatible so don't use this setter.
			SetterFunction = nullptr;
		}

		return SetterFunction;
	}

	static UFunction* FindSetterFunction(const FProperty* Property, UClass* OwnerClass)
	{
		// UStruct properties cannot have setters.
		if (!ensure(Property) || !Property->GetOwnerClass())
		{
			return nullptr;
		}

		UFunction* SetterFunction = nullptr;
		if (OwnerClass && OwnerClass != Property->GetOwnerClass())
		{
			SetterFunction = FindSetterFunctionInternal(Property, OwnerClass);
		}
		
		if (!SetterFunction)
		{
			SetterFunction = FindSetterFunctionInternal(Property, Property->GetOwnerClass());
		}
		
		return SetterFunction;
	}

#if WITH_EDITOR
	template <>
	inline bool Deserialize<FProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		const FProperty* Property = OutDst.GetProperty();
		FOREACH_CAST_PROPERTY(Property, Deserialize<CastPropertyType>(InSrc, OutDst))

		return true;
	}
#endif

	template <>
	inline bool Serialize<FProperty>(const FRCPropertyVariant& InSrc, FRCPropertyVariant& OutDst)
	{
		const FProperty* Property = InSrc.GetProperty();
		FOREACH_CAST_PROPERTY(Property, Serialize<CastPropertyType>(InSrc, OutDst))

		return true;
	}
}
