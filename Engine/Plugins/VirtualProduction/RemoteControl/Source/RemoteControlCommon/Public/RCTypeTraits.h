// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/IsArithmetic.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"

namespace RemoteControlTypeTraits
{
	namespace Concepts
	{
		/** Concept to check if T has NumericLimits */
		struct CNumerical
		{
			template <typename T>
			auto Requires() -> decltype(
				std::is_same_v<typename TNumericLimits<T>::NumericType, T> &&
				!std::is_same_v<T, bool>);
		};

		/** Concept to check if PropertyType::TCppType matches ValueType */
		struct CPropertyValuePair
		{
			template <typename PropertyType, typename ValueType>
			auto Requires() -> decltype(TIsDerivedFrom<PropertyType, TProperty<ValueType, PropertyType>>::Value);
		};

		/** Concept to check if PropertyType has a TCppType */
		struct CPropertyHasTCppType
		{
			template <typename PropertyType>
			auto Requires() -> decltype(std::is_same_v<typename PropertyType::TCppType, void>);
		};
	}

	/** Catch-all for string-like property types */
	template <typename PropertyType, typename Enable = void>
	struct TIsStringLikeProperty
	{
		enum { Value = false };
	};

	template <typename PropertyType>
	struct TIsStringLikeProperty<
			PropertyType,
			typename TEnableIf<
					std::is_same_v<PropertyType, FStrProperty> ||
					std::is_same_v<PropertyType, FNameProperty> ||
					std::is_same_v<PropertyType, FTextProperty>>::Type>
	{
		enum
		{
			Value = true
		};
	};

	/** Catch-all for string-like value types */
	template <typename ValueType, typename Enable = void>
	struct TIsStringLikeValue
	{
		enum { Value = false };
	};

	template <typename ValueType>
	struct TIsStringLikeValue<
			ValueType,
			typename TEnableIf<
					std::is_same_v<ValueType, FString> ||
					std::is_same_v<ValueType, FName> ||
					std::is_same_v<ValueType, FText>>::Type>
	{
		enum
		{
			Value = true
		};
	};

	/** Ensures ValueType is a numeric type. */
	template <typename ValueType>
	constexpr bool TNumericValueConstraint_V = TModels_V<RemoteControlTypeTraits::Concepts::CNumerical, ValueType>;
}

/** Various RemoteControl type traits */
template <typename ValueType, typename Enable = void>
struct TRemoteControlTypeTraits;

/** Various RemoteControl property type traits */
template <typename InPropertyType, typename Enable = void>
struct TRemoteControlPropertyTypeTraits;

#pragma region Numeric Types

/** RemoteControlTypeTraits for integers */
template <typename ValueType>
struct TRemoteControlTypeTraits<ValueType,
	typename TEnableIf<
		std::is_arithmetic_v<ValueType> &&
		!std::is_floating_point_v<ValueType> &&
		TModels_V<RemoteControlTypeTraits::Concepts::CNumerical, ValueType>, void>::Type>
{
	using Type = ValueType;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return true; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	/** The default minimum value for newly created range (protocol, input). */
	static constexpr Type DefaultRangeValueMin() { return TNumericLimits<Type>::Min(); }

	/** The default maximum value for newly created range. */
	static constexpr Type DefaultRangeValueMax() { return TNumericLimits<Type>::Max(); }

	/** The default minimum value for newly created mapping. */
	static constexpr Type DefaultMappingValueMin() { return static_cast<Type>(0); }

	/** The default maximum value for newly created mapping. */
	static constexpr Type DefaultMappingValueMax() { return static_cast<Type>(1); }
};

/** RemoteControlTypeTraits for floats */
template <typename ValueType>
struct TRemoteControlTypeTraits<ValueType,
	typename TEnableIf<TIsFloatingPoint<ValueType>::Value, void>::Type>
{
	using Type = ValueType;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return true; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static constexpr Type DefaultRangeValueMin() { return static_cast<Type>(0.0); }

	static constexpr Type DefaultRangeValueMax() { return static_cast<Type>(1.0); }

	static constexpr Type DefaultMappingValueMin() { return static_cast<Type>(0.0); }

	static constexpr Type DefaultMappingValueMax() { return static_cast<Type>(1.0); }
};

/**
 * RemoteControlPropertyTypeTraits for FNumericProperty
 * Currently all numeric types are supported so we can shortcut the above (rather than doing a series of CastField's)
 */
template <>
struct TRemoteControlPropertyTypeTraits<FNumericProperty>
{
	using PropertyType = FNumericProperty;
	using ValueType = void;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return true; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
};

/** RemoteControlPropertyTypeTraits for any property with a TCppType  */
template <typename InPropertyType>
struct TRemoteControlPropertyTypeTraits<InPropertyType,
	typename TEnableIf<TModels_V<RemoteControlTypeTraits::Concepts::CPropertyHasTCppType, InPropertyType>, void>::Type>
{
	using PropertyType = InPropertyType;
	using ValueType = typename InPropertyType::TCppType;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return true; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
};

/** RemoteControlPropertyTypeTraits for FEnumProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FEnumProperty>
{
	using PropertyType = FEnumProperty;
	using ValueType = uint8;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
};

#pragma endregion Numeric Types

/** RemoteControlTypeTraits for bool */
template <>
struct TRemoteControlTypeTraits<bool>
{
	using Type = bool;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static constexpr Type DefaultRangeValueMin() { return false; }

	static constexpr Type DefaultRangeValueMax() { return true; }
	
	static constexpr Type DefaultMappingValueMin() { return false; }
	
	static constexpr Type DefaultMappingValueMax() { return true; }
};

/** RemoteControlPropertyTypeTraits for FBoolProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FBoolProperty>
{
	using PropertyType = FBoolProperty;
	using ValueType = bool;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return TRemoteControlTypeTraits<bool>::IsSupportedRangeType(); }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return TRemoteControlTypeTraits<bool>::IsSupportedMappingType(); }

	/** The default minimum value for newly created range (protocol, input). */
	static constexpr ValueType DefaultRangeValueMin() { return TRemoteControlTypeTraits<bool>::DefaultRangeValueMin(); }

	/** The default maximum value for newly created range. */
	static constexpr ValueType DefaultRangeValueMax() { return TRemoteControlTypeTraits<bool>::DefaultRangeValueMax(); }

	/** The default minimum value for newly created mapping. */
	static constexpr ValueType DefaultMappingValueMin() { return TRemoteControlTypeTraits<bool>::DefaultMappingValueMin(); }

	/** The default maximum value for newly created mapping. */
	static constexpr ValueType DefaultMappingValueMax() { return TRemoteControlTypeTraits<bool>::DefaultMappingValueMax(); }
};

#pragma region String Types

/** RemoteControlTypeTraits for FString */
template <>
struct TRemoteControlTypeTraits<FString>
{
	using Type = FString;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static Type DefaultRangeValueMin() { return {}; }

	static Type DefaultRangeValueMax() { return {}; }
	
	static Type DefaultMappingValueMin() { return {}; }
	
	static Type DefaultMappingValueMax() { return {}; }
};

/** RemoteControlPropertyTypeTraits for FStrProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FStrProperty>
{
	using PropertyType = FStrProperty;
	using ValueType = FString;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return TRemoteControlTypeTraits<FString>::IsSupportedRangeType(); }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return TRemoteControlTypeTraits<FString>::IsSupportedMappingType(); }

	/** The default minimum value for newly created range (protocol, input). */
	static ValueType DefaultRangeValueMin() { return TRemoteControlTypeTraits<FString>::DefaultRangeValueMin(); }

	/** The default maximum value for newly created range. */
	static ValueType DefaultRangeValueMax() { return TRemoteControlTypeTraits<FString>::DefaultRangeValueMax(); }

	/** The default minimum value for newly created mapping. */
	static ValueType DefaultMappingValueMin() { return TRemoteControlTypeTraits<FString>::DefaultMappingValueMin(); }

	/** The default maximum value for newly created mapping. */
	static ValueType DefaultMappingValueMax() { return TRemoteControlTypeTraits<FString>::DefaultMappingValueMax(); }
};

/** RemoteControlTypeTraits for FName */
template <>
struct TRemoteControlTypeTraits<FName>
{
	using Type = FName;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static Type DefaultRangeValueMin() { return {}; }

	static Type DefaultRangeValueMax() { return {}; }
	
	static Type DefaultMappingValueMin() { return {}; }
	
	static Type DefaultMappingValueMax() { return {}; }
};

/** RemoteControlPropertyTypeTraits for FNameProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FNameProperty>
{
	using PropertyType = FNameProperty;
	using ValueType = FName;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return TRemoteControlTypeTraits<ValueType>::IsSupportedRangeType(); }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType(); }

	/** The default minimum value for newly created range (protocol, input). */
	static ValueType DefaultRangeValueMin() { return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin(); }

	/** The default maximum value for newly created range. */
	static ValueType DefaultRangeValueMax() { return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax(); }

	/** The default minimum value for newly created mapping. */
	static ValueType DefaultMappingValueMin() { return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin(); }

	/** The default maximum value for newly created mapping. */
	static ValueType DefaultMappingValueMax() { return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax(); }
};

/** RemoteControlTypeTraits for FText */
template <>
struct TRemoteControlTypeTraits<FText>
{
	using Type = FText;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static Type DefaultRangeValueMin() { return {}; }

	static Type DefaultRangeValueMax() { return {}; }
	
	static Type DefaultMappingValueMin() { return {}; }
	
	static Type DefaultMappingValueMax() { return {}; }
};

/** RemoteControlPropertyTypeTraits for FTextProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FTextProperty>
{
	using PropertyType = FTextProperty;
	using ValueType = FText;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return TRemoteControlTypeTraits<ValueType>::IsSupportedRangeType(); }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return TRemoteControlTypeTraits<ValueType>::IsSupportedMappingType(); }

	/** The default minimum value for newly created range (protocol, input). */
	static ValueType DefaultRangeValueMin() { return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin(); }

	/** The default maximum value for newly created range. */
	static ValueType DefaultRangeValueMax() { return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax(); }

	/** The default minimum value for newly created mapping. */
	static ValueType DefaultMappingValueMin() { return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin(); }

	/** The default maximum value for newly created mapping. */
	static ValueType DefaultMappingValueMax() { return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax(); }
};

#pragma endregion String Types

/** RemoteControlPropertyTypeTraits for FArrayProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FArrayProperty>
{
	using PropertyType = FArrayProperty;
	using ValueType = TArray<uint8>;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return false; }
};

/** RemoteControlPropertyTypeTraits for FSetProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FSetProperty>
{
	using PropertyType = FSetProperty;
	using ValueType = TArray<uint8>;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return false; }
};

/** RemoteControlPropertyTypeTraits for FMapProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FMapProperty>
{
	using PropertyType = FMapProperty;
	using ValueType = TArray<uint8>;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return false; }
};

#pragma region Structs (Built-in)

/** RemoteControlPropertyTypeTraits for FStructProperty */
template <>
struct TRemoteControlPropertyTypeTraits<FStructProperty>
{
	using PropertyType = FStructProperty;
	using ValueType = void;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
};

/** RemoteControlTypeTraits for FVector */
template <>
struct TRemoteControlTypeTraits<FVector>
{
	using Type = FVector;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type::ZeroVector; }

	static Type DefaultMappingValueMax() { return Type::OneVector; }
};

/** RemoteControlTypeTraits for FVector2D */
template <>
struct TRemoteControlTypeTraits<FVector2D>
{
	using Type = FVector2D;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }

	static Type DefaultMappingValueMin() { return Type::ZeroVector; }
	
	static Type DefaultMappingValueMax() { return Type::UnitVector; }
};

/** RemoteControlTypeTraits for FVector4 */
template <>
struct TRemoteControlTypeTraits<FVector4>
{
	using Type = FVector4;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type(EForceInit::ForceInitToZero); }

	static Type DefaultMappingValueMax() { return Type(1.0f, 1.0f, 1.0f, 1.0f); }
};

/** RemoteControlTypeTraits for FRotator */
template <>
struct TRemoteControlTypeTraits<FRotator>
{
	using Type = FRotator;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type::ZeroRotator; }

	static Type DefaultMappingValueMax() { return Type(90.0f, 90.0f, 90.0f); }
};

/** RemoteControlTypeTraits for FQuat */
template <>
struct TRemoteControlTypeTraits<FQuat>
{
	using Type = FQuat;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type(0.0f, 0.0f, 0.0f, 0.0f); }

	static Type DefaultMappingValueMax() { return Type::Identity; }
};

/** RemoteControlTypeTraits for FTransform */
template <>
struct TRemoteControlTypeTraits<FTransform>
{
	using Type = FTransform;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin()
	{
		return Type(
			TRemoteControlTypeTraits<FRotator>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax()); // scale is Max cause it shouldn't be zero
	}

	static Type DefaultMappingValueMax()
	{
		return Type(
			TRemoteControlTypeTraits<FRotator>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax()); // scale is Max cause it shouldn't be zero
	}		
};

/** RemoteControlTypeTraits for FIntPoint */
template <>
struct TRemoteControlTypeTraits<FIntPoint>
{
	using Type = FIntPoint;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type::ZeroValue; }

	static Type DefaultMappingValueMax() { return Type(1, 1); }
};

/** RemoteControlTypeTraits for FIntVector */
template <>
struct TRemoteControlTypeTraits<FIntVector>
{
	using Type = FIntVector;

	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type::ZeroValue; }
	
	static Type DefaultMappingValueMax() { return Type(1, 1, 1); }
};

/** RemoteControlTypeTraits for FBox */
template <>
struct TRemoteControlTypeTraits<FBox>
{
	using Type = FBox;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin()
	{
		return Type(
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax());
	}

	static Type DefaultMappingValueMax()
	{
		return Type(
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<FVector>::DefaultMappingValueMax() * 2.0f);
	}
};

/** RemoteControlTypeTraits for FBox2D */
template <>
struct TRemoteControlTypeTraits<FBox2D>
{
	using Type = FBox2D;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin()
	{
		return Type(
			TRemoteControlTypeTraits<FVector2D>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<FVector2D>::DefaultMappingValueMax());
	}

	static Type DefaultMappingValueMax()
	{
		return Type(
			TRemoteControlTypeTraits<FVector2D>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<FVector2D>::DefaultMappingValueMax() * 2.0f);
	}
};

/** RemoteControlTypeTraits for FBoxSphereBounds */
template <>
struct TRemoteControlTypeTraits<FBoxSphereBounds>
{
	using Type = FBoxSphereBounds;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin() { return Type(TRemoteControlTypeTraits<FBox>::DefaultMappingValueMin()); }

	static Type DefaultMappingValueMax() { return Type(TRemoteControlTypeTraits<FBox>::DefaultMappingValueMax()); }
};

/** RemoteControlTypeTraits for FColor */
template <>
struct TRemoteControlTypeTraits<FColor>
{
	using Type = FColor;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin()
	{
		return Type(TRemoteControlTypeTraits<uint8>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<uint8>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<uint8>::DefaultMappingValueMin());
	}
		
	static Type DefaultMappingValueMax()
	{
		// @note: the use of Range instead of Mapping is intentional, by default Mapping is 0-1, but in this case we want 0-255
		return Type(TRemoteControlTypeTraits<uint8>::DefaultRangeValueMax(),
			TRemoteControlTypeTraits<uint8>::DefaultRangeValueMax(),
			TRemoteControlTypeTraits<uint8>::DefaultRangeValueMax());
	}
};

/** RemoteControlTypeTraits for FLinearColor */
template <>
struct TRemoteControlTypeTraits<FLinearColor>
{
	using Type = FLinearColor;
	
	/** Is ValueType supported as a range (protocol input) value? */
	static constexpr bool IsSupportedRangeType() { return false; }

	/** Is ValueType supported as a mapping (property output) value? */
	static constexpr bool IsSupportedMappingType() { return true; }
	
	static Type DefaultMappingValueMin()
	{
		return Type(TRemoteControlTypeTraits<float>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<float>::DefaultMappingValueMin(),
			TRemoteControlTypeTraits<float>::DefaultMappingValueMin());
	}

	static Type DefaultMappingValueMax()
	{
		return Type(TRemoteControlTypeTraits<float>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<float>::DefaultMappingValueMax(),
			TRemoteControlTypeTraits<float>::DefaultMappingValueMax());
	}
};

#pragma endregion Structs (Built-in)

#pragma region Name to Type

template <EName TypeName>
struct TRCTypeNameToType { using Type = void; };

template <> struct TRCTypeNameToType<NAME_Int8Property> { using Type = int8; };
template <> struct TRCTypeNameToType<NAME_ByteProperty> { using Type = uint8; };
template <> struct TRCTypeNameToType<NAME_Int16Property> { using Type = int16; };
template <> struct TRCTypeNameToType<NAME_UInt16Property> { using Type = uint16; };
template <> struct TRCTypeNameToType<NAME_IntProperty> { using Type = int32; };
template <> struct TRCTypeNameToType<NAME_Int32Property> { using Type = int32; };
template <> struct TRCTypeNameToType<NAME_UInt32Property> { using Type = uint32; };
template <> struct TRCTypeNameToType<NAME_Int64Property> { using Type = int64; };
template <> struct TRCTypeNameToType<NAME_UInt64Property> { using Type = uint64; };

template <> struct TRCTypeNameToType<NAME_FloatProperty> { using Type = float; };
template <> struct TRCTypeNameToType<NAME_DoubleProperty> { using Type = double; };
template <> struct TRCTypeNameToType<NAME_Double> { using Type = double; };

template <> struct TRCTypeNameToType<NAME_BoolProperty> { using Type = bool; };
template <> struct TRCTypeNameToType<NAME_EnumProperty> { using Type = UEnum; };

template <> struct TRCTypeNameToType<NAME_StrProperty> { using Type = FString; };
template <> struct TRCTypeNameToType<NAME_NameProperty> { using Type = FName; };
template <> struct TRCTypeNameToType<NAME_Name> { using Type = FName; };
template <> struct TRCTypeNameToType<NAME_TextProperty> { using Type = FText; };

template <> struct TRCTypeNameToType<NAME_ArrayProperty> { using Type = FScriptArray; };
template <> struct TRCTypeNameToType<NAME_SetProperty> { using Type = FScriptSet; };
template <> struct TRCTypeNameToType<NAME_MapProperty> { using Type = FScriptMap; };

template <> struct TRCTypeNameToType<NAME_StructProperty> { using Type = UScriptStruct; };
template <> struct TRCTypeNameToType<NAME_VectorProperty> { using Type = FVector; };
template <> struct TRCTypeNameToType<NAME_Vector> { using Type = FVector; };
template <> struct TRCTypeNameToType<NAME_Vector2D> { using Type = FVector2D; };
template <> struct TRCTypeNameToType<NAME_RotatorProperty> { using Type = FRotator; };
template <> struct TRCTypeNameToType<NAME_Rotator> { using Type = FRotator; };
template <> struct TRCTypeNameToType<NAME_ObjectProperty> { using Type = UObject; };
template <> struct TRCTypeNameToType<NAME_Box> { using Type = FBox; };
template <> struct TRCTypeNameToType<NAME_BoxSphereBounds> { using Type = FBoxSphereBounds; };
template <> struct TRCTypeNameToType<NAME_Color> { using Type = FColor; };
template <> struct TRCTypeNameToType<NAME_IntPoint> { using Type = FIntPoint; };
template <> struct TRCTypeNameToType<NAME_IntRect> { using Type = FIntRect; };
template <> struct TRCTypeNameToType<NAME_LinearColor> { using Type = FLinearColor; };
template <> struct TRCTypeNameToType<NAME_Matrix> { using Type = FMatrix; };
template <> struct TRCTypeNameToType<NAME_Plane> { using Type = FPlane; };
template <> struct TRCTypeNameToType<NAME_Quat> { using Type = FQuat; };
template <> struct TRCTypeNameToType<NAME_Sphere> { using Type = FSphere; };
template <> struct TRCTypeNameToType<NAME_Transform> { using Type = FTransform; };
template <> struct TRCTypeNameToType<NAME_Vector4> { using Type = FVector4; };

#pragma endregion Name to Type
