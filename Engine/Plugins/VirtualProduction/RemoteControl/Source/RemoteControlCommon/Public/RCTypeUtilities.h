// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include <type_traits>

#include "RCTypeTraits.h"
#include "Math/NumericLimits.h"
#include "Math/UnrealMathUtility.h"
#include "Templates/UnrealTypeTraits.h"
#include "UObject/StructOnScope.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"

class IPropertyHandle;

namespace RemoteControlTypeUtilities
{
	#pragma region FOREACH_CAST_PROPERTY
	#ifndef FOREACH_CAST_PROPERTY
	
	/**
	 * Iterate and attempt to cast Property to different types (as CastPropertyType), where the instance will be CastProperty
	 * Usage:
	 * FProperty* SomePropertyInstance;
	 *
	 * template <>
	 * void SomeFunc<FNumericProperty>(const FNumericProperty*);
	 * etc..
	 * 
	 * FOREACH_CAST_PROPERTY(SomePropertyInstance, SomeFunc<CastPropertyType>(CastProperty))
	 */
#define FOREACH_CAST_PROPERTY(Property, Func)                                                                          \
    if (const FBoolProperty* CastProperty = CastField<FBoolProperty>(Property))                                        \
    {                                                                                                                  \
        using CastPropertyType = FBoolProperty;                                                                        \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FEnumProperty* CastProperty = CastField<FEnumProperty>(Property))                                        \
    {                                                                                                                  \
        using CastPropertyType = FEnumProperty;                                                                        \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FNumericProperty* NumericCastProperty = CastField<FNumericProperty>(Property))                           \
    {                                                                                                                  \
        if (const FInt8Property* CastProperty = CastField<FInt8Property>(Property))                                    \
        {                                                                                                              \
            using CastPropertyType = FInt8Property;                                                                    \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FByteProperty* CastProperty = CastField<FByteProperty>(Property))                                    \
        {                                                                                                              \
            using CastPropertyType = FByteProperty;                                                                    \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FInt16Property* CastProperty = CastField<FInt16Property>(Property))                                  \
        {                                                                                                              \
            using CastPropertyType = FInt16Property;                                                                   \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FUInt16Property* CastProperty = CastField<FUInt16Property>(Property))                                \
        {                                                                                                              \
            using CastPropertyType = FUInt16Property;                                                                  \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FIntProperty* CastProperty = CastField<FIntProperty>(Property))                                      \
        {                                                                                                              \
            using CastPropertyType = FIntProperty;                                                                     \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FUInt32Property* CastProperty = CastField<FUInt32Property>(Property))                                \
        {                                                                                                              \
            using CastPropertyType = FUInt32Property;                                                                  \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FInt64Property* CastProperty = CastField<FInt64Property>(Property))                                  \
        {                                                                                                              \
            using CastPropertyType = FInt64Property;                                                                   \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FUInt64Property* CastProperty = CastField<FUInt64Property>(Property))                                \
        {                                                                                                              \
            using CastPropertyType = FUInt64Property;                                                                  \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FFloatProperty* CastProperty = CastField<FFloatProperty>(Property))                                  \
        {                                                                                                              \
            using CastPropertyType = FFloatProperty;                                                                   \
            return Func;                                                                                               \
        }                                                                                                              \
        if (const FDoubleProperty* CastProperty = CastField<FDoubleProperty>(Property))                                \
        {                                                                                                              \
            using CastPropertyType = FDoubleProperty;                                                                  \
            return Func;                                                                                               \
        }                                                                                                              \
        if(const FNumericProperty* CastProperty = NumericCastProperty)												   \
        {                                                                                                              \
            using CastPropertyType = FNumericProperty;                                                                 \
            return Func;                                                                                               \
        }                                                                                                              \
    }                                                                                                                  \
    if (const FStructProperty* CastProperty = CastField<FStructProperty>(Property))                                    \
    {                                                                                                                  \
        using CastPropertyType = FStructProperty;                                                                      \
        return Func;                                                                                                   \
    }                                                                                                                  \
                                                                                                                       \
    if (const FStrProperty* CastProperty = CastField<FStrProperty>(Property))                                          \
    {                                                                                                                  \
        using CastPropertyType = FStrProperty;                                                                         \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FNameProperty* CastProperty = CastField<FNameProperty>(Property))                                        \
    {                                                                                                                  \
        using CastPropertyType = FNameProperty;                                                                        \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FTextProperty* CastProperty = CastField<FTextProperty>(Property))                                        \
    {                                                                                                                  \
        using CastPropertyType = FTextProperty;                                                                        \
        return Func;                                                                                                   \
    }                                                                                                                  \
                                                                                                                       \
    if (const FArrayProperty* CastProperty = CastField<FArrayProperty>(Property))                                      \
    {                                                                                                                  \
        using CastPropertyType = FArrayProperty;                                                                       \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FSetProperty* CastProperty = CastField<FSetProperty>(Property))                                          \
    {                                                                                                                  \
        using CastPropertyType = FSetProperty;                                                                         \
        return Func;                                                                                                   \
    }                                                                                                                  \
    if (const FMapProperty* CastProperty = CastField<FMapProperty>(Property))                                          \
    {                                                                                                                  \
        using CastPropertyType = FMapProperty;                                                                         \
        return Func;                                                                                                   \
    }

	#endif
	#pragma endregion FOREACH_CAST_PROPERTY
			
	/** Metadata Keys. */
	
	static FName ClampMinKey = "ClampMin";
	static FName ClampMaxKey = "ClampMax";

	static FName UIMinKey = "UIMin";
	static FName UIMaxKey = "UIMax";

	/** Delegated Min operator. */
	template <typename ValueType>
	static auto MinOp = [](auto&& InValues) { return FMath::Min(InValues); };

	/** Delegated Max operator. */
	template <typename ValueType>
	static auto MaxOp = [](auto&& InValues) { return FMath::Max(InValues); };

	/** Returns typed metadata for numeric property if it exists, otherwise returns the DefaultValue. */
	template <
		typename PropertyType,
		typename ValueType
		UE_REQUIRES(std::is_base_of_v<FProperty, PropertyType>)
	>
	constexpr static ValueType GetMetadataValue(const PropertyType* InProperty, const FName& InKey, const ValueType& InDefaultValue)
	{
#if WITH_EDITORONLY_DATA
		if constexpr (RemoteControlTypeTraits::TNumericValueConstraint_V<ValueType>)
		{
			if (InProperty->HasMetaData(InKey))
			{
				if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
				{
					if (NumericProperty->IsInteger())
					{
						return static_cast<ValueType>(NumericProperty->GetIntMetaData(InKey));
					}

					if (NumericProperty->IsFloatingPoint())
					{
						return static_cast<ValueType>(NumericProperty->GetFloatMetaData(InKey));
					}
				}
			}
		}
#endif

		return InDefaultValue;
	}

	template <class Func, class PropertyType, class ValueType>
	static ValueType GetClampedValue(Func InFunc, const PropertyType* InProperty, ValueType InValue, const TArray<FName>& InMetaKeys)
	{
		TArray<ValueType> Params;
		Params.Reserve(InMetaKeys.Num());
		for(const FName& Key : InMetaKeys)
		{
			Params.Add(
				RemoteControlTypeUtilities::GetMetadataValue(
					InProperty,
					Key,
					InValue));
		}

		return InFunc(Params);
	}

	/** Clamps the value to the PropertyType, for example if an int64 is requested but the PropertyType is UInt16Property. */
	template <typename PropertyType, typename ValueType>
	static ValueType ClampToPropertyType(const FNumericProperty* InProperty, ValueType& InOutValue)
	{
		check(InProperty);

		const FName PropertyTypeName = InProperty->GetClass()->GetFName();
		if(InProperty->IsInteger())
		{
			if(PropertyTypeName == NAME_ByteProperty)
			{
				using T = uint8;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_Int8Property)
			{
				using T = int8;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_Int16Property)
			{
				using T = int16;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_UInt16Property)
			{
				using T = uint16;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_Int32Property)
			{
				using T = int32;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_UInt32Property)
			{
				using T = uint32;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_Int64Property)
			{
				using T = int64;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_UInt64Property)
			{
				using T = uint64;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Min(), TNumericLimits<T>::Max());
			}
		}
		else if(InProperty->IsFloatingPoint())
		{
			// Special case for floats where min value isn't precisely 0.0
			if(PropertyTypeName == NAME_FloatProperty)
			{
				using T = float;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Lowest(), TNumericLimits<T>::Max());
			}
			else if(PropertyTypeName == NAME_DoubleProperty)
			{
				using T = double;
				InOutValue = FMath::Clamp<T>(InOutValue, TNumericLimits<T>::Lowest(), TNumericLimits<T>::Max());
			}
		}

		return InOutValue;
	}
	
	/** The utilities below wrap TRemoteControlTypeTraits */

	template <typename PropertyType>
	static bool IsSupportedRangeType(const PropertyType* InProperty);

	/** Is ValueType supported as a range (protocol input) value? */
	template <typename PropertyType>
	static bool IsSupportedRangeType(const PropertyType* InProperty)
	{
		check(InProperty);

		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FArrayProperty>::IsSupportedRangeType() && IsSupportedRangeType(ArrayProperty->Inner);
		}
		
		if(const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FArrayProperty>::IsSupportedRangeType() && IsSupportedRangeType(SetProperty->ElementProp);
		}
		
		if(const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FArrayProperty>::IsSupportedRangeType() && IsSupportedRangeType(MapProperty->KeyProp);
		}

		FOREACH_CAST_PROPERTY(InProperty, TRemoteControlPropertyTypeTraits<CastPropertyType>::IsSupportedRangeType())
		
		return false;
	}

	/** Is ValueType supported as a mapping (property output) value? */
	template <typename PropertyType>
	static bool IsSupportedMappingType(const PropertyType* InProperty);

	/** Is ValueType supported as a mapping (property output) value? */
	template <typename PropertyType>
	static bool IsSupportedMappingType(const PropertyType* InProperty)
	{
		check(InProperty);

		if(const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FArrayProperty>::IsSupportedMappingType() && IsSupportedMappingType(ArrayProperty->Inner);
		}
		
		if(const FSetProperty* SetProperty = CastField<FSetProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FSetProperty>::IsSupportedMappingType() && IsSupportedMappingType(SetProperty->ElementProp);
		}
		
		if(const FMapProperty* MapProperty = CastField<FMapProperty>(InProperty))
		{
			return TRemoteControlPropertyTypeTraits<FMapProperty>::IsSupportedMappingType() &&  IsSupportedMappingType(MapProperty->ValueProp);
		}

		FOREACH_CAST_PROPERTY(InProperty, TRemoteControlPropertyTypeTraits<CastPropertyType>::IsSupportedMappingType())

		return false;
	}

	template <typename ValueType>
	static constexpr ValueType GetDefaultRangeValueMin()
	{
		return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin();
	}

	template <typename ValueType>
	static ValueType GetDefaultRangeValueMin(const FProperty* InProperty)
	{
		check(InProperty);

		if constexpr (RemoteControlTypeTraits::TNumericValueConstraint_V<ValueType>)
		{
			// Choose greater than the 3 values: (Traits defined) Default, ClampMin, or UIMin
			ValueType ValueResult = GetClampedValue(MaxOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin(), { ClampMinKey, UIMinKey });

			// ValueType might not match property, so clamp to min/max of property type
			if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
			{
				ClampToPropertyType<ValueType>(NumericProperty, ValueResult);
			}

			return ValueResult;
		}
		else
		{
			return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin();
		}
	}

	template <typename ValueType>
	static constexpr ValueType GetDefaultRangeValueMax()
	{
		return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax();
	}

	template <typename ValueType>
	static ValueType GetDefaultRangeValueMax(const FProperty* InProperty)
	{
		check(InProperty);
		
		if constexpr (RemoteControlTypeTraits::TNumericValueConstraint_V<ValueType>)
		{
			// Choose lesser than the 3 values: (Traits defined) Default, ClampMin, or UIMin
			ValueType ValueResult = GetClampedValue(MinOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax(), { ClampMaxKey, UIMaxKey });

			// ValueType might not match property, so clamp to min/max of property type
			if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
			{
				ClampToPropertyType<ValueType>(NumericProperty, ValueResult);
			}

			return ValueResult;
		}
		else
		{
			return TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax();
		}
	}

	template <typename ValueType>
	static constexpr ValueType GetDefaultMappingValueMin()
	{
		return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin();
	}

	template <typename ValueType>
	static ValueType GetDefaultMappingValueMin(const FProperty* InProperty)
	{
		check(InProperty);

		if constexpr (RemoteControlTypeTraits::TNumericValueConstraint_V<ValueType>)
		{
			ValueType ValueResult = GetClampedValue(MaxOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin(), { ClampMinKey, UIMinKey });

			// ValueType might not match property, so clamp to min/max of property type
			if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
			{
				ClampToPropertyType<ValueType>(NumericProperty, ValueResult);
			}

			return ValueResult;
		}
		else
		{
			return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin();
		}
	}

	template <
		typename ValueType
		UE_REQUIRES(std::is_same_v<ValueType, FStructOnScope>)
	>
	static TSharedPtr<ValueType> GetDefaultMappingValueMin(const FStructProperty* InProperty)
	{
		check(InProperty);

		TSharedPtr<FStructOnScope> PopulatedStruct = MakeShared<FStructOnScope>(InProperty->Struct);

		/*
		// See UETOOL-3502
		for (TFieldIterator<FProperty> It(InProperty->Struct, EFieldIteratorFlags::IncludeSuper); It; ++It)
		{
			FProperty* Property = *It;
			uint8* PropertyAddr = Property->ContainerPtrToValuePtr<uint8>(PopulatedStruct->GetStructMemory());
		}
		*/

		return PopulatedStruct;
	}

	template <typename ValueType>
	static constexpr ValueType GetDefaultMappingValueMax()
	{
		return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax();		
	}

	template <typename ValueType>
	static ValueType GetDefaultMappingValueMax(const FProperty* InProperty)
	{
		check(InProperty);

		if constexpr (RemoteControlTypeTraits::TNumericValueConstraint_V<ValueType>)
		{
			ValueType ValueResult = GetClampedValue(MinOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax(), { ClampMaxKey, UIMaxKey });

			// ValueType might not match property, so clamp to min/max of property type
			if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProperty))
			{
				ClampToPropertyType<ValueType>(NumericProperty, ValueResult);
			}

			return ValueResult;
		}
		else
		{
			return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax();
		}
	}

	template <
		typename ValueType
		UE_REQUIRES(std::is_same_v<ValueType, FStructOnScope>)
	>
	static TSharedPtr<ValueType> GetDefaultMappingValueMax(const FStructProperty* InProperty)
	{
		check(InProperty);

		return TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax();
	}

	/** Various RemoteControl property type Traits (wraps internal implementations). This resolves the property type for you. */

	template <typename PropertyType>
	constexpr bool TPropertyConstraint_V = std::is_base_of_v<FProperty, PropertyType>;
	
	template <
		typename PropertyType,
		typename ValueType
		UE_REQUIRES(TPropertyConstraint_V<PropertyType>)
	>
	static ValueType GetDefaultRangeValueMin(const PropertyType* InProperty)
	{
		check(InProperty);
		
		using FTraits = TRemoteControlTypeTraits<ValueType>;
		
		return GetClampedValue(MaxOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMin(), { ClampMinKey, UIMinKey });
	}

	template <
		typename PropertyType,
		typename ValueType
		UE_REQUIRES(TPropertyConstraint_V<PropertyType>)
	>
	static ValueType GetDefaultRangeValueMax(const FProperty* InProperty)
	{
		check(InProperty);

		using FTraits = TRemoteControlTypeTraits<ValueType>;

		return GetClampedValue(MinOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultRangeValueMax(), { ClampMaxKey, UIMaxKey });
	}

	template <
		typename PropertyType,
		typename ValueType
		UE_REQUIRES(TPropertyConstraint_V<PropertyType>)
	>
	static ValueType GetDefaultMappingValueMin(const FProperty* InProperty)
	{
		check(InProperty);
		
		using FTraits = TRemoteControlTypeTraits<ValueType>;
		
		return GetClampedValue(MaxOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMin(), { ClampMinKey, UIMinKey });
	}

	template <
		typename PropertyType,
		typename ValueType
		UE_REQUIRES(TPropertyConstraint_V<PropertyType>)
	>
	static ValueType GetDefaultMappingValueMax(const FProperty* InProperty)
	{
		check(InProperty);
		
		using FTraits = TRemoteControlTypeTraits<ValueType>;
		
		return GetClampedValue(MinOp<ValueType>, InProperty, TRemoteControlTypeTraits<ValueType>::DefaultMappingValueMax(), { ClampMaxKey, UIMaxKey });
	}

	/** Size of given property + data, accounting for item count (arrays, strings, etc.) */
	REMOTECONTROLCOMMON_API SIZE_T GetPropertySize(const FProperty* InProperty, void* InData);

#if WITH_EDITOR
	/** Size of given property + data, accounting for item count (arrays, strings, etc.) */
	REMOTECONTROLCOMMON_API SIZE_T GetPropertySize(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
#endif
}
