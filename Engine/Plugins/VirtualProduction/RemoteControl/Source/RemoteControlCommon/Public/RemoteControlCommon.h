// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RemoteControlCommon.generated.h"

UENUM()
namespace ERCProtocolBinding
{
	enum Op : int
	{
		Added,

		Removed
	};
}

UENUM()
enum class ERCMask : uint8
{
	NoMask = 0x00,

	MaskA = 0x01,

	MaskB = 0x02,

	MaskC = 0x04,

	MaskD = 0x08,
};

/** Mask for all remote control masking flags */
#define RC_AllMasks (ERCMask)0xFF // All masks, used mainly for initialization

ENUM_CLASS_FLAGS(ERCMask);
