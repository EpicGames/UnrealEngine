// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "RCAllowedRemoteFunctionCall.h"

#include "RemoteControlSettings.generated.h"

/**
 * Utility struct to represent IPv4 Network addresses.
 */
USTRUCT(BlueprintType)
struct FRCNetworkAddress
{
	GENERATED_BODY()

	FRCNetworkAddress() = default;
	FRCNetworkAddress(uint8 InClassA, uint8 InClassB, uint8 InClassC, uint8 InClassD)
		: ClassA(InClassA), ClassB(InClassB), ClassC(InClassC), ClassD(InClassD)
	{}

	bool operator==(const FRCNetworkAddress& OtherNetworkAddress) const
	{
		return ClassA == OtherNetworkAddress.ClassA
			&& ClassB == OtherNetworkAddress.ClassB
			&& ClassC == OtherNetworkAddress.ClassC
			&& ClassD == OtherNetworkAddress.ClassD;
	}

	/**
	 * Calculates the hash for a Network Address.
	 *
	 * @param NetworkAddress The Network address to calculate the hash for.
	 * @return The hash.
	 */
	friend uint32 GetTypeHash(const FRCNetworkAddress& NetworkAddress)
	{
		const uint32 NetworkPartHash = HashCombine(GetTypeHash(NetworkAddress.ClassA), GetTypeHash(NetworkAddress.ClassB));
		const uint32 HostPartHash = HashCombine(GetTypeHash(NetworkAddress.ClassC), GetTypeHash(NetworkAddress.ClassD));

		return HashCombine(NetworkPartHash, HostPartHash);
	}

	/**
	 * Retrieves the network address as string e.g. 192.168.1.1
	 */
	const FString ToString() const
	{
		return FString::Printf(TEXT("%d.%d.%d.%d"), ClassA, ClassB, ClassC, ClassD);
	}

	/** Denotes the first octet of the IPv4 address (0-255.xxx.xxx.xxx) */
	UPROPERTY(EditAnywhere, Category="Network Address")
	uint8 ClassA = 192;

	/** Denotes the second octet of the IPv4 address (xxx.0-255.xxx.xxx) */
	UPROPERTY(EditAnywhere, Category = "Network Address")
	uint8 ClassB = 168;

	/** Denotes the third octet of the IPv4 address (xxx.xxx.0-255.xxx) */
	UPROPERTY(EditAnywhere, Category = "Network Address")
	uint8 ClassC = 1;

	/** Denotes the fourth octet of the IPv4 address (xxx.xxx.xxx.0-255) */
	UPROPERTY(EditAnywhere, Category = "Network Address")
	uint8 ClassD = 1;
};

/**
 * Utility struct to represent range of IPv4 Network addresses.
 */
USTRUCT(BlueprintType)
struct FRCNetworkAddressRange
{
	GENERATED_BODY()

	FRCNetworkAddressRange() = default;
	FRCNetworkAddressRange(FRCNetworkAddress InLowerBound, FRCNetworkAddress InUpperBound)
		: LowerBound(InLowerBound), UpperBound(InUpperBound)
	{}

	static FRCNetworkAddressRange AllowAllIPs()
	{
		FRCNetworkAddressRange Range;
		Range.LowerBound = { 0, 0, 0, 0 };
		Range.UpperBound = { 255, 255, 255, 255 };
		return Range;
	}

	bool operator==(const FRCNetworkAddressRange& OtherNetworkAddressRange) const
	{
		return LowerBound == OtherNetworkAddressRange.LowerBound
			&& UpperBound == OtherNetworkAddressRange.UpperBound;
	}
	
	/**
	 * Calculates the hash for a Network Address Range.
	 *
	 * @param NetworkAddress The Network address range to calculate the hash for.
	 * @return The hash.
	 */
	friend uint32 GetTypeHash(const FRCNetworkAddressRange& NetworkAddressRange)
	{
		const uint32 LowerBoundHash = GetTypeHash(NetworkAddressRange.LowerBound);
		const uint32 UpperBoundHash = GetTypeHash(NetworkAddressRange.UpperBound);

		return HashCombine(LowerBoundHash, UpperBoundHash);
	}

	bool IsInRange(const FString& InNetworkAddress) const
	{
		TArray<FString> IndividualClasses;

		if (InNetworkAddress.ParseIntoArray(IndividualClasses, TEXT(".")) == 4)
		{
			return IsInRange_Internal(FCString::Atoi(*IndividualClasses[0])
				, FCString::Atoi(*IndividualClasses[1])
				, FCString::Atoi(*IndividualClasses[2])
				, FCString::Atoi(*IndividualClasses[3])
			);
		}

		return false;
	}
	
	bool IsInRange(const FRCNetworkAddress& InNetworkAddress) const
	{
		return IsInRange_Internal(InNetworkAddress.ClassA
			, InNetworkAddress.ClassB
			, InNetworkAddress.ClassC
			, InNetworkAddress.ClassD
		);
	}

private:

	bool IsInRange_Internal(uint8 InClassA, uint8 InClassB, uint8 InClassC, uint8 InClassD) const
	{
		bool bClassAIsInRange = InClassA >= LowerBound.ClassA && InClassA <= UpperBound.ClassA;
		bool bClassBIsInRange = InClassB >= LowerBound.ClassB && InClassB <= UpperBound.ClassB;
		bool bClassCIsInRange = InClassC >= LowerBound.ClassC && InClassC <= UpperBound.ClassC;
		bool bClassDIsInRange = InClassD >= LowerBound.ClassD && InClassD <= UpperBound.ClassD;
		
		return bClassAIsInRange && bClassBIsInRange && bClassCIsInRange && bClassDIsInRange;
	}

public:

	/** Denotes the lower bound IPv4 address. */
	UPROPERTY(EditAnywhere, Category="Network Address Range")
	FRCNetworkAddress LowerBound = { 192, 168, 1, 1 };

	/** Denotes the upper bound IPv4 address. */
	UPROPERTY(EditAnywhere, Category = "Network Address Range")
	FRCNetworkAddress UpperBound = { 192, 168, 255, 255 };
};

/**
 * Passphrase Struct
 */
USTRUCT(BlueprintType)
struct FRCPassphrase
{
	GENERATED_BODY()
	
	FRCPassphrase(){}

	UPROPERTY(EditAnywhere, Category="Passphrase")
	FString Identifier;
	
	UPROPERTY(EditAnywhere, Category="Passphrase")
	FString Passphrase;
};

/**
 * Global remote control settings
 */
UCLASS(config = RemoteControl)
class REMOTECONTROLCOMMON_API URemoteControlSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	URemoteControlSettings();

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const
	{
		return "Project";
	}
	
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const
	{
		return "Plugins";
	}
	
	/** The unique name for your section of settings, uses the class's FName. */
	virtual FName GetSectionName() const
	{
		return "Remote Control";
	}

	virtual FText GetSectionText() const
	{
		return NSLOCTEXT("RemoteControlSettings", "RemoteControlSettingsSection", "Remote Control");
	}

	virtual TArray<FString> GetHashedPassphrases() const
	{
		TArray<FString> OutArray;

		for (const FRCPassphrase& Passphrase : Passphrases)
		{
			OutArray.Add(Passphrase.Passphrase);
		}
		
		return OutArray;
	}

	virtual void AllowClient(const FString& InClientAddressStr)
	{
		// Return early if we have this in range.
		if (IsClientAllowed(InClientAddressStr))
		{
			return;
		}

		// Add a new range.
		TArray<FString> IndividualClasses;

		if (InClientAddressStr.ParseIntoArray(IndividualClasses, TEXT(".")) == 4)
		{
			FRCNetworkAddress LowerAndUpperBounds { (uint8)FCString::Atoi(*IndividualClasses[0]) // Class A
				, (uint8)FCString::Atoi(*IndividualClasses[1]) // Class B
				, (uint8)FCString::Atoi(*IndividualClasses[2]) // Class C
				, (uint8)FCString::Atoi(*IndividualClasses[3]) // Class D
			};

			FRCNetworkAddressRange NewRange = { LowerAndUpperBounds, LowerAndUpperBounds };

			AllowlistedClients.Add(NewRange);
		}
	}
	
	virtual bool IsClientAllowed(const FString& InClientAddressStr) const
	{
		for (const FRCNetworkAddressRange& AllowlistedClient : AllowlistedClients)
		{
			if (AllowlistedClient.IsInRange(InClientAddressStr))
			{
				return true;
			}
		}

		return false;
	}

	/** Returns the names of the columns visualized in the RC Panel Entities list. */
	static const TSet<FName>& GetExposedEntitiesColumnNames()
	{
		static TSet<FName> ColumnNames =
			{ TEXT("PropertyID")
			, TEXT("OwnerName")
			, TEXT("Subobject Path")
			, TEXT("Description")
			, TEXT("Value")};

		return ColumnNames;
	}

	/**
	 *  DEPRECATED 5.5
	 * 
	 * Should transactions be generated for events received through protocols (ie. MIDI, DMX etc.)
	 * Disabling transactions improves performance but will prevent events from being transacted to Multi-User
	 * unless using the Remote Control Interception feature.
	 */
	UE_DEPRECATED(5.5, "Protocol Generate Transactions is now a per Preset option. See URemoteControlPreset::GetProtocolModifyOperationFlags.")
	UPROPERTY(config)
	bool bProtocolsGenerateTransactions_DEPRECATED = true;

	/** The remote control web app bind address. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Interface", DisplayName = "Remote Control Web Interface bind address")
	FString RemoteControlWebInterfaceBindAddress = TEXT("0.0.0.0");

	/** The remote control web app http port. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Interface", DisplayName = "Remote Control Web Interface http Port")
	uint32 RemoteControlWebInterfacePort = 30000;
	
	/** Should force a build of the WebApp at startup. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Interface", DisplayName = "Force WebApp build at startup")
	bool bForceWebAppBuildAtStartup = false;

	/** Should WebApp log timing. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Interface", DisplayName = "Log WebApp requests handle duration")
	bool bWebAppLogRequestDuration = false;

	/** Whether web server is started automatically. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Server")
	bool bAutoStartWebServer = true;

	/** Whether web socket server is started automatically. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Server")
	bool bAutoStartWebSocketServer = true;

	/** The web remote control HTTP server's port. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Server", DisplayName = "Remote Control HTTP Server Port")
	uint32 RemoteControlHttpServerPort = 30010;

	/** The address to bind the websocket server to. 0.0.0.0 will open the connection to everyone on your network, while 127.0.0.1 will only allow local requests to come through.  */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Server", DisplayName = "Remote Control Websocket Bind Address")
	FString RemoteControlWebsocketServerBindAddress = TEXT("0.0.0.0");

	/** The web remote control WebSocket server's port. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Web Server", DisplayName = "Remote Control WebSocket Server Port")
	uint32 RemoteControlWebSocketServerPort = 30020;
	
	/** Show a warning icon for exposed editor-only fields. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Show a warning when exposing editor-only entities.")
	bool bDisplayInEditorOnlyWarnings = false;

	/** The split widget control ratio between entity tree and details/protocol binding list. */
	UPROPERTY(config)
	float TreeBindingSplitRatio = 0.7f;

	/** The split widget control ratio between entity tree and details/protocol binding list. */
	UPROPERTY(config)
	float ActionPanelSplitRatio = 0.6f;

	UPROPERTY(config)
	bool bUseRebindingContext = true;

	UPROPERTY(config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Ignore Remote Control Protected Check")
	bool bIgnoreProtectedCheck = false;

	UPROPERTY(Config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Ignore Remote Control Getter/Setter Check")
	bool bIgnoreGetterSetterCheck = false;

	UPROPERTY(Config, EditAnywhere, Category = "Remote Control Preset")
	bool bIgnoreWarnings = false;

	/** 
	 * Enables any remote function execution, enabling this could open you up to vulnerabilities if an outside actor has access to your server. 
	 * If true, any remote function call will be allowed.
	 * If false, only the built-in allowed functions and those specified in 'CustomAllowedRemoteFunctionCalls' will be considered.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security")
	bool bAllowAnyRemoteFunctionCall = false;

	/** Whether to restrict access to a list of hostname/IPs in the AllowedOrigins setting. */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security")
	bool bRestrictServerAccess = false;

	/** Enable remote python execution, enabling this could open you up to vulnerabilities if an outside actor has access to your server. */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security", meta = (editCondition = bRestrictServerAccess))
	bool bEnableRemotePythonExecution = false;

	/** Enable calling 'ExecuteConsoleCommand' through the web api. Enabling this could open you up to vulnerabilities if an outside actor has access to your server. */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security", meta = (editCondition = bRestrictServerAccess))
	bool bAllowConsoleCommandRemoteExecution = false;

	/**
	 * Specifies built-in function calls that are allowed to be called remotely. This is only considered if `bAllowAnyRemoteFunctionCall` is false 
	 * NOTE: For console command and python execution, 'bAllowConsoleCommandRemoteExecution' and 'bEnableRemotePythonExecution' are still required to be enabled respectively.
	 */
	UPROPERTY()
	TArray<FRCAllowedRemoteFunctionCall> AllowedRemoteFunctionCalls;

	/**
	 * Specifies custom function calls that are allowed to be called remotely. This is only considered if `bAllowAnyRemoteFunctionCall` is false 
	 * NOTE: For console command and python execution, 'bAllowConsoleCommandRemoteExecution' and 'bEnableRemotePythonExecution' are still required to be enabled respectively.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security", meta = (EditCondition="!bAllowAnyRemoteFunctionCall", DisplayAfter="bAllowAnyRemoteFunctionCall"))
	TArray<FRCAllowedRemoteFunctionCall> CustomAllowedRemoteFunctionCalls;

	/** List of IP Addresses that are allowed to access the Web API without authentication. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control | Security", DisplayName = "Range of Allowlisted Clients", Meta = (EditCondition = bRestrictServerAccess, EditConditionHides))
	TSet<FRCNetworkAddressRange> AllowlistedClients = { FRCNetworkAddressRange::AllowAllIPs() };

	/** 
	 * Origin that can make requests to the remote control server. Should contain the hostname or IP of the server making requests to remote control. ie. "http://yourdomain.com", or "*" to allow all origins. 
	 * @Note: This is used to block requests coming from a browser (ie. Coming from a script running on a website), ideally you should use both this setting and AllowListedClients, as a request coming from a websocket client can have an empty Origin.
	 * @Note Supports wildcards (ie. *.yourdomain.com)
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security", meta=(EditCondition=bRestrictServerAccess))
	FString AllowedOrigin = TEXT("*");

	/**
	 * Controls whether a passphrase should be required when remote control is accessed by a client outside of localhost.
	 */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control | Security", meta = (EditCondition = bRestrictServerAccess))
	bool bEnforcePassphraseForRemoteClients = true;

	/** List of passphrases used for accessing remote control outside of localhost. */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control | Security", DisplayName = "Remote Control Passphrase", meta = (EditCondition=bEnforcePassphraseForRemoteClients, EditConditionHides))
	TArray<FRCPassphrase> Passphrases = {};
	
	/** Whether the User should be warned that Passphrase usage is disabled or now. Initially activated */
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control | Security", DisplayName = "Warn that Passphrase might be disabled ", meta=(EditCondition=bRestrictServerAccess))
	bool bShowPassphraseDisabledWarning = true;

	UPROPERTY(Config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Hide or Show the Logic panel by default")
	bool bLogicPanelVisibility = false;

	UPROPERTY(Config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Which columns to hide in the exposed entities list")
	TSet<FName> EntitiesListHiddenColumns;
	
	UPROPERTY(Config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Default mode (e.g. Setup/Operation)")
	FName DefaultPanelMode = TEXT("Setup");

	/** Refresh all widgets in the exposed properties list when object properties are updated */
	UPROPERTY(config, EditAnywhere, Category = "Remote Control Preset", DisplayName = "Refresh all exposed entities widgets on object properties update")
	bool bRefreshExposedEntitiesOnObjectPropertyUpdate = true;
	
private:
	UPROPERTY(config)
    bool bSecuritySettingsReviewed = false;
};
