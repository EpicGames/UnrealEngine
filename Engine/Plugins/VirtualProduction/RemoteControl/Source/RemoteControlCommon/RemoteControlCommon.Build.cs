// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

/** Shared functionality between RemoteControlUI and RemoteControlProtocolWidgets */
public class RemoteControlCommon : ModuleRules
{
	public RemoteControlCommon(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore"
			}
		);
		
		if (Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"PropertyEditor",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AssetRegistry",
					"AssetTools",
					"BlueprintGraph",
					"DeveloperSettings",
					"EditorWidgets",
					
					"EditorSubsystem",
					"Projects",
					"PropertyEditor",
					"Slate",
					"SlateCore",
					"UnrealEd",
				}
			);
		}
    }
}
