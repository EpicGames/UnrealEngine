// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/Bind/RCCustomBindActionUtilities.h"
#include "AssetRegistry/AssetData.h"
#include "Engine/Level.h"
#include "Engine/Texture2D.h"
#include "IRemoteControlPropertyHandle.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Misc/Paths.h"
#include "RCCustomBindActionUtilitiesPrivate.h"
#include "RCExternalTexture.h"
#include "RemoteControlField.h"

namespace UE::RCCustomBindActionUtilities::Private
{
	/**
	 * Expand some common path tokens.
	 * Can be found elsewhere in engine: mrq, render grid, img media or python for instance.
	 */
	FString ExpandSequencePathTokens(const FString& InPath)
	{
		// Note: Replace ignores case by default (intended).
		FString ExpandedPath = InPath
			.Replace(TEXT("{project_dir}"), *FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()));

		FPaths::NormalizeDirectoryName(ExpandedPath);
		FPaths::RemoveDuplicateSlashes(ExpandedPath);
		return ExpandedPath;
	}

	/**
	 * Try to resolve the given string to a valid absolute path.
	 */
	FString TryResolvePath(const FString& InPath)
	{
		// 1- Can be already an absolute path
		if (FPaths::FileExists(InPath))
		{
			return InPath;
		}

		// 2- Try Relative to content
		const FString ResolvedPath = FPaths::Combine(FPaths::ProjectContentDir(), InPath);
		if (FPaths::FileExists(ResolvedPath))
		{
			return FPaths::ConvertRelativePathToFull(ResolvedPath);
		}

		// 3- Using other explicit path tokens (engine, project, ...)
		return ExpandSequencePathTokens(InPath);
	}

	static FString TryMakeRelativeToContent(const FString& InPath)
	{
		const FString ContentPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
		FString ConvertedPath = ExpandSequencePathTokens(InPath);
		if (FPaths::IsUnderDirectory(ConvertedPath, ContentPath))
		{
			if (FPaths::MakePathRelativeTo(ConvertedPath, *ContentPath))
			{
				return ConvertedPath;
			}
		}
		return InPath;
	}
	
	/**
	 * Generate a unique readable object name from the path. 
	 */
	static FString GetTextureObjectNameFromPath(const FString& InPath)
	{
		// This name appears in the asset picker
		FString TextureName = TEXT("RCExtTexture_");

		// Try to make a cleaner name if under content.
		FString ConvertedPath = TryMakeRelativeToContent(InPath);

		// Clean up the other path tokens
		ConvertedPath.ReplaceInline(TEXT("{project_dir}"), TEXT("Project_"));

		// Replace forbidden characters in the path (can't be in an object name)
		for (const TCHAR* InvalidCharacters = INVALID_OBJECTNAME_CHARACTERS; *InvalidCharacters; ++InvalidCharacters)
		{
			ConvertedPath.ReplaceCharInline(*InvalidCharacters, TEXT('_'));
		}

		TextureName.Append(ConvertedPath);

		// Hash the full original filepath to ensure uniqueness.
		TextureName.Appendf(TEXT("_%x"), GetTypeHash(InPath));
		return TextureName;
	}

	/**
	 * Check if the given string is a valid asset path, if so, load it. 
	 */
	static UTexture2D* LoadAsTextureAsset(const FString& InPath)
	{
		if (FPackageName::IsValidTextForLongPackageName(InPath))
		{
			return Cast<UTexture2D>(StaticLoadObject(UTexture2D::StaticClass(), nullptr, *InPath));
		}
		return nullptr;
	}

	UTexture2D* LoadExternalAsTransientTexture(const FString& InPath)
	{
		const FString TextureName = Private::GetTextureObjectNameFromPath(InPath);
	
		if (UTexture2D* Texture = Cast<UTexture2D>(StaticFindObject(UTexture2D::StaticClass(), GetTransientPackage(), *TextureName)))
		{
			return Texture;
		}

		const FString ResolvedPath = TryResolvePath(InPath);

		if (FPaths::FileExists(ResolvedPath))
		{
			// Todo: generate mipmaps.
			UTexture2D* Texture = UKismetRenderingLibrary::ImportFileAsTexture2D(nullptr, ResolvedPath);
		
			if (Texture)
			{
				Texture->Rename(*TextureName, GetTransientPackage(), REN_DontCreateRedirectors | REN_DoNotDirty);
			}
			return Texture;
		}
		return nullptr;
	}
	
}

UTexture* UE::RCCustomBindActionUtilities::LoadTextureFromPath(UObject* InOuter, const FString& InPath)
{
	// Check if the string is an asset.
	if (UTexture* TextureAsset = Private::LoadAsTextureAsset(InPath))
	{
		return TextureAsset;
	}
	
	// No outer, most likely for UI thumbnail.
	if (!InOuter)
	{
		return Private::LoadExternalAsTransientTexture(InPath);
	}

	// Check if the RCP is part of a level instance
	if (const ULevel* Level = InOuter->GetTypedOuter<ULevel>())
	{
		if (Level->IsInstancedLevel())
		{
			return Private::LoadExternalAsTransientTexture(InPath);
		}
	}

	const FString ResolvedPath = Private::TryResolvePath(InPath);

	// We have an outer and it is not a level instance -> load the texture embedded in the outer.
	// This will ensure there is a valid texture reference for materials in the level template.
	if (FPaths::FileExists(ResolvedPath))
	{
		const FString TextureName = Private::GetTextureObjectNameFromPath(InPath);

		if (UTexture* Texture = Cast<UTexture>(StaticFindObject(UTexture::StaticClass(), InOuter, *TextureName)))
		{
			return Texture;
		}

		// Using an embedded "external" texture object that will automatically load it's content
		// from the external file.
		UTexture* Texture = URCExternalTexture::Create(ResolvedPath);

		if (Texture && InOuter)
		{
			Texture->Rename(*TextureName, InOuter, REN_DontCreateRedirectors | REN_DoNotDirty);
			Texture->ClearFlags(RF_Transient);
		}
		return Texture;
	}
	return nullptr;
}

UTexture2D* UE::RCCustomBindActionUtilities::LoadTextureFromPath(const FString& InPath)
{
	// Check if the string is an asset.
	if (UTexture2D* TextureAsset = Private::LoadAsTextureAsset(InPath))
	{
		return TextureAsset;
	}

	// Consider the string as an external path.
	return Private::LoadExternalAsTransientTexture(InPath);
}

void UE::RCCustomBindActionUtilities::SetTexturePropertyFromPath(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, const FString& InPath)
{
	if (UTexture2D* LoadedTexture = LoadTextureFromPath(InPath))
	{
		SetTextureProperty(InRemoteControlEntityAsProperty, LoadedTexture);
	}
}

bool UE::RCCustomBindActionUtilities::SetTexturePropertyFromPath(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, UObject* InOuter, const FString& InPath)
{
	if (UTexture* LoadedTexture = LoadTextureFromPath(InOuter, InPath))
	{
		return SetTextureProperty(InRemoteControlEntityAsProperty, LoadedTexture);
	}
	return false;
}

bool UE::RCCustomBindActionUtilities::SetTextureProperty(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, UTexture* InTexture)
{
	const FProperty* RemoteControlProperty = InRemoteControlEntityAsProperty->GetProperty();
	if (RemoteControlProperty == nullptr)
	{
		return false;
	}

	const TSharedPtr<IRemoteControlPropertyHandle>& RemoteControlHandle = InRemoteControlEntityAsProperty->GetPropertyHandle();
	if (!ensure(RemoteControlHandle))
	{
		return false;
	}

	return RemoteControlHandle->SetValue(InTexture);
}
