// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UTexture;
struct FRemoteControlProperty;

namespace UE::RCCustomBindActionUtilities
{
	UTexture* LoadTextureFromPath(UObject* InOuter, const FString& InPath);

	/**
	 * Loads the texture found at the specified path, and uses it to set a Texture value in the specified RC Property
	 * Useful e.g. for Custom External Texture Controller
	 */
	bool SetTexturePropertyFromPath(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, UObject* InOuter, const FString& InPath);

	bool SetTextureProperty(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, UTexture* InTexture);
};
