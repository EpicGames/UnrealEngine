// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCExternalTexture.h"

#include "ImageCoreUtils.h"
#include "ImageUtils.h"
#include "TextureResource.h"
#include "UObject/Package.h"

namespace UE::RCExternalTexture
{
	struct FResourceCacheEntry
	{
		FRCExternalTextureInfo CachedInfo;
		FTextureRHIRef TextureRHI;
		FSamplerStateRHIRef SamplerStateRHI;

		FResourceCacheEntry() = default;

		FResourceCacheEntry(const FRCExternalTextureInfo& InCachedInfo)
			: CachedInfo(InCachedInfo) {}
	};

	/**
	 * This class implements a simple cache for the RHI resources.
	 * It allows multiple instance of an external texture to use the same RHI resources.
	 */
	class FResourceCache
	{
	public:
		TSharedPtr<FResourceCacheEntry> Find(const FString& InPath) const
		{
			FScopeLock ScopeLock(&EntriesSection);
			const TWeakPtr<FResourceCacheEntry>* Found = EntriesWeak.Find(InPath);
			return Found ? (*Found).Pin() : nullptr;
		}

		void Add(const FString& InPath, const TSharedRef<FResourceCacheEntry>& InEntry)
		{
			FScopeLock ScopeLock(&EntriesSection);
			EntriesWeak.Add(InPath, InEntry);
		}

		// Todo: Multi-gpu support.
		// Do we need a cache per GPU or is this internally handled by RHI?
		static TSharedPtr<FResourceCache> Get()
		{
			static TSharedRef<FResourceCache> Instance = MakeShared<FResourceCache>();
			return Instance;
		}
	
	private:
		mutable FCriticalSection EntriesSection;
		TMap<FString, TWeakPtr<FResourceCacheEntry>> EntriesWeak;
	};
}

/**
 * This implementation of texture resource, while based on Texture2DDynamic
 * also support sharing the RHI resource for all textures of the same path
 * so the actual texture is loaded only once and reused in each RCExternalTexture instance.
 */
class FRCExternalTextureResource : public FTexture2DDynamicResource
{
public:
	using FResourceCacheEntry = UE::RCExternalTexture::FResourceCacheEntry; 
	using FResourceCache = UE::RCExternalTexture::FResourceCache; 
	
	FRCExternalTextureResource(URCExternalTexture* InOwner, const FString& InPath)
		: FTexture2DDynamicResource(InOwner)
		, RCTextureOwner(InOwner)
		, Path(InPath)
	{
		
	}

	virtual uint32 GetSizeX() const override
	{
		return RCTextureOwner->SizeX;
	}

	virtual uint32 GetSizeY() const override
	{
		return RCTextureOwner->SizeY;
	}

	virtual void InitRHI(FRHICommandListBase& RHICmdList) override
	{
		const TSharedPtr<FResourceCache> RHICache = FResourceCache::Get();
		CacheEntry = RHICache->Find(Path);

		// Fast path: texture already cached.
		if (CacheEntry && CacheEntry->TextureRHI.IsValid())
		{
			UpdateRHIFromCacheEntry(RHICmdList);
			return;
		}
		
		FTexture2DDynamicResource::InitRHI(RHICmdList);

		UpdateCacheEntry();
	}
	
	virtual void ReleaseRHI() override
	{
		CacheEntry.Reset();
		FTexture2DDynamicResource::ReleaseRHI();
	}

	void SetCacheEntry(FRHICommandListBase& RHICmdList, const FString& InNewPath, const TSharedPtr<FResourceCacheEntry>& InNewEntry)
	{
		ReleaseRHI();

		Path = InNewPath;
		CacheEntry = InNewEntry;
		UpdateRHIFromCacheEntry(RHICmdList);
	}

	void UpdateRHIFromCacheEntry(FRHICommandListBase& RHICmdList)
	{
		if (CacheEntry)
		{
			TextureRHI = CacheEntry->TextureRHI;
			SamplerStateRHI = CacheEntry->SamplerStateRHI;
			RHICmdList.UpdateTextureReference(RCTextureOwner->TextureReference.TextureReferenceRHI,TextureRHI);
		}
	}

	void UpdateCacheEntry()
	{
		if (CacheEntry)
		{
			CacheEntry->TextureRHI = GetTextureRHI();
			CacheEntry->SamplerStateRHI = SamplerStateRHI;
		}
	}

	URCExternalTexture* RCTextureOwner;
	FString Path;
	TSharedPtr<FResourceCacheEntry> CacheEntry;
};

void URCExternalTexture::PostLoad()
{
	SizeX = CachedInfo.SizeX;
	SizeY = CachedInfo.SizeY;
	Format = CachedInfo.Format;
	NumMips = 1;
	bIsResolveTarget = false;

	Super::PostLoad(); // will call UpdateResource	

	if (!Path.IsEmpty())
	{
		LoadFromPath(Path);
	}
}

FTextureResource* URCExternalTexture::CreateResource()
{
	return new FRCExternalTextureResource(this, Path);
}

void URCExternalTexture::LoadFromPath(const FString& InPath)
{
	using namespace UE::RCExternalTexture;
	// Fast path: Check if texture is already in cache.
	if (const TSharedPtr<FResourceCacheEntry> ExistingEntry = FResourceCache::Get()->Find(InPath))
	{
		Path = InPath;
		CachedInfo = ExistingEntry->CachedInfo;

		// Only recreate resource if something changed.
		if (SizeX != CachedInfo.SizeX || SizeY != CachedInfo.SizeY || Format != CachedInfo.Format || !GetResource())
		{
			Init(CachedInfo.SizeX, CachedInfo.SizeY, CachedInfo.Format);	// Calls UpdateResource
		}
		
		if (FRCExternalTextureResource* TextureResource = static_cast<FRCExternalTextureResource*>(GetResource()))
		{
			ENQUEUE_RENDER_COMMAND(FUpdateSharedTextureEntry)(
				[TextureResource, ExistingEntry, NewPath=Path](FRHICommandListImmediate& RHICmdList)
				{
					TextureResource->SetCacheEntry(RHICmdList, NewPath, ExistingEntry);
				});
		}
		return;
	}


	// This will be loaded on the main thread for now. Most likely causing game thread spike.
	// As an optimization for loading, we could try to hook this up to a streaming manager.
	// The RCExternalTexture is typically embedded in a level, can get access with GetTypedOuter<ULevel>.
	
	FImage Image;	
	if (FImageUtils::LoadImage(*InPath, Image))
	{
		// Todo: keep file time and hash to detect content change and reload if necessary.
		Path = InPath;
		
		SRGB = Image.GetGammaSpace() == EGammaSpace::Linear ? 0 : 1;
		
		ERawImageFormat::Type PixelFormatRawFormat;
		const EPixelFormat PixelFormat = FImageCoreUtils::GetPixelFormatForRawImageFormat(Image.Format,&PixelFormatRawFormat);
		CachedInfo.Set(Image.GetWidth(), Image.GetHeight(), PixelFormat);
		
		if (SizeX != Image.GetWidth() || SizeY != Image.GetHeight() || Format != PixelFormat || !GetResource())
		{
			Init(Image.GetWidth(), Image.GetHeight(), PixelFormat);
		}

		if (FRCExternalTextureResource* TextureResource = static_cast<FRCExternalTextureResource*>(GetResource()))
		{
			TSharedRef<FResourceCacheEntry> NewEntry = MakeShared<FResourceCacheEntry>(CachedInfo);
			FResourceCache::Get()->Add(Path, NewEntry);
			
			ENQUEUE_RENDER_COMMAND(FWriteRawDataToTexture)(
				[TextureResource, NewEntry, NewPath=Path, NewImage = MoveTemp(Image)](FRHICommandListImmediate& InRHICmdList)
				{
#if !UE_SERVER
					// Todo: support mipmaps. some format like dds can have mipmaps (but not supported by FImageUtils for now).
					TextureResource->WriteRawToTexture_RenderThread(NewImage.RawData);
#endif					
					TextureResource->Path = NewPath;
					TextureResource->CacheEntry = NewEntry;
					TextureResource->UpdateCacheEntry();
				});
		}
		return;
	}
	
	// Fallback: Allocate texture with no data.
	// Todo: should we create a checkerboard pattern?
	Path.Reset();
	Init(CachedInfo.SizeX, CachedInfo.SizeY, CachedInfo.Format);
}

URCExternalTexture* URCExternalTexture::Create(const FTexture2DDynamicCreateInfo& InCreateInfo)
{
	URCExternalTexture* NewTexture = NewObject<URCExternalTexture>(GetTransientPackage(), NAME_None, RF_Transient);
	if (NewTexture)
	{
		NewTexture->Filter = InCreateInfo.Filter;
		NewTexture->SamplerAddressMode = InCreateInfo.SamplerAddressMode;
		NewTexture->SRGB = InCreateInfo.bSRGB;

		// Disable compression
		NewTexture->CompressionSettings = TC_Default;
#if WITH_EDITORONLY_DATA
		NewTexture->CompressionNone = true;
		NewTexture->MipGenSettings = TMGS_NoMipmaps;
		NewTexture->CompressionNoAlpha = true;
		NewTexture->DeferCompression = false;
#endif // #if WITH_EDITORONLY_DATA
		if (InCreateInfo.bIsResolveTarget)
		{
			NewTexture->bNoTiling = false;
		}
		else
		{
			// Untiled format
			NewTexture->bNoTiling = true;
		}
		NewTexture->bIsResolveTarget = InCreateInfo.bIsResolveTarget;
	}
	return NewTexture;
}

URCExternalTexture* URCExternalTexture::Create(const FString& InPath, const FTexture2DDynamicCreateInfo& InCreateInfo)
{
	URCExternalTexture* NewTexture = Create(InCreateInfo);
	if (NewTexture)
	{
		NewTexture->LoadFromPath(InPath);
	}
	return NewTexture;
}
