// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2DDynamic.h"
#include "RCExternalTexture.generated.h"

/** Cached information on the external texture. */
USTRUCT()
struct FRCExternalTextureInfo
{
	GENERATED_BODY()
	
	UPROPERTY()
	int32 SizeX = 1;

	UPROPERTY()
	int32 SizeY = 1;

	UPROPERTY()
	TEnumAsByte<enum EPixelFormat> Format = PF_B8G8R8A8;

	void Set(int32 InSizeX, int32 InSizeY, EPixelFormat InFormat)
	{
		SizeX = InSizeX;
		SizeY = InSizeY;
		Format = InFormat;
	}
};

/**
 *	Texture object that automatically loads the content from an external file.
 *	The texture object itself does not store anything, except the filename.
 *
 *	This is intended to be used with texture controllers in external mode.
 *	The texture is embedded in the RCP itself to have a valid texture object
 *	during loading. This avoids breaking the materials that
 *	require valid texture objects.
 */
UCLASS(hidecategories=Object)
class URCExternalTexture : public UTexture2DDynamic
{
	GENERATED_BODY()

public:

	/**
	 * Path to the external texture file.
	 */
	UPROPERTY()
	FString Path;

	/*
	 *	Cached information from the last time the texture was loaded.
	 *	This is only used as backed up the texture file is not accessible.
	 */
	UPROPERTY()
	FRCExternalTextureInfo CachedInfo;

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

	//~ Begin UTexture
	virtual FTextureResource* CreateResource() override;
	//~ End UTexture
	
	/**
	 * @brief Load the content of the texture from the given path.
	 * @param InPath Path of the texture file.
	 */
	void LoadFromPath(const FString& InPath);

	/**
	 * @brief Utility function to create an empty, uninitialized external texture. Need to call Init() or LoadFromPath() after this.
	 * @param InCreateInfo creation parameters
	 * @return created texture
	 */
	static URCExternalTexture* Create(const FTexture2DDynamicCreateInfo& InCreateInfo = FTexture2DDynamicCreateInfo());

	/**
	 * @brief Utility function to create and initialize the texture from the given file path.
	 * @param InCreateInfo creation parameters
	 * @return created and initialized texture
	 */
	static URCExternalTexture* Create(const FString& InPath, const FTexture2DDynamicCreateInfo& InCreateInfo = FTexture2DDynamicCreateInfo());
};