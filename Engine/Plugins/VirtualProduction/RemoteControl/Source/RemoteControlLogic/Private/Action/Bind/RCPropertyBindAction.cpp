// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/Bind/RCPropertyBindAction.h"
#include "Action/Bind/RCCustomBindActionUtilitiesPrivate.h"
#include "Components/SceneComponent.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "IRemoteControlPropertyHandle.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"

#if WITH_EDITOR
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#endif

static void SetStructPropertyFromController(const FProperty* RemoteControlProperty, TSharedRef<IRemoteControlPropertyHandle> RemoteControlHandle, const URCController* Controller)
{
	const FStructProperty* SourceStructProperty = CastField<FStructProperty>(Controller->GetProperty());
	const FStructProperty* TargetStructProperty = CastField<FStructProperty>(RemoteControlProperty);

	if (ensure(SourceStructProperty && TargetStructProperty))
	{
		// Pass 1 - Structs of exactly the same type
		if (SourceStructProperty->Struct == TargetStructProperty->Struct)
		{
			// FVector
			if (TargetStructProperty->Struct == TBaseStructure<FVector>::Get())
			{
				FVector VectorValue;
				Controller->GetValueVector(VectorValue);

				RemoteControlHandle->SetValue(VectorValue);
			}
			// FVector2D
			else if (TargetStructProperty->Struct == TBaseStructure<FVector2D>::Get())
			{
				FVector2D Vector2DValue;
				Controller->GetValueVector2D(Vector2DValue);

				RemoteControlHandle->SetValue(Vector2DValue);
			}
			// FColor
			else if (TargetStructProperty->Struct == TBaseStructure<FColor>::Get())
			{
				FColor ColorValue;
				Controller->GetValueColor(ColorValue);

				RemoteControlHandle->SetValue(ColorValue);
			}
			// FRotator
			else if (TargetStructProperty->Struct == TBaseStructure<FRotator>::Get())
			{
				FRotator RotatorValue;
				Controller->GetValueRotator(RotatorValue);

				RemoteControlHandle->SetValue(RotatorValue);
			}
		}
		// Pass 2 - Structs that support conversion to other (related) Structs
		else
		{
			// Source: FColor
			if (SourceStructProperty->Struct == TBaseStructure<FColor>::Get())
			{
				// Target: FLinearColor
				if (TargetStructProperty->Struct == TBaseStructure<FLinearColor>::Get())
				{
					FColor ColorValue;
					Controller->GetValueColor(ColorValue);

					RemoteControlHandle->SetValue(FLinearColor(ColorValue));
				}
			}
		}
	}
}

// Workaround method to extract the index of an Array Property element from its name
// e.g. MaterialOverride[3] will return 3, MaterialOverride[12] will return 12, etc
static int32 GetPropertyIndexInArray(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty)
{
	FString FieldName = InRemoteControlEntityAsProperty->FieldName.ToString();

	const TSharedPtr<IRemoteControlPropertyHandle>& RemoteControlHandle = InRemoteControlEntityAsProperty->GetPropertyHandle();
	if (!ensure(RemoteControlHandle))
	{
		return -1;
	}

	const TSharedPtr<IRemoteControlPropertyHandleArray>& AsArray = RemoteControlHandle->AsArray();
	const int32 ElementsNum = AsArray->GetNumElements();

	FieldName.RemoveFromEnd(TEXT("]"));
	
	// Number of integer digits, so that we know how many characters to keep at the end of the string
	const int32 Digits = FMath::Floor(FMath::LogX(10.0f, ElementsNum) + 1);
	const int32 CharactersToRemove = FieldName.Len() - Digits;
	FieldName.RemoveAt(0, CharactersToRemove, EAllowShrinking::Yes);
	const int32 Index = FCString::Atoi(*FieldName);

	return Index;
}

static void SetObjectPropertyFromController(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, const URCController* Controller)
{
	FProperty* RemoteControlProperty = InRemoteControlEntityAsProperty->GetProperty();
	if (RemoteControlProperty == nullptr)
	{
		return;
	}

	const TSharedPtr<IRemoteControlPropertyHandle>& RemoteControlHandle = InRemoteControlEntityAsProperty->GetPropertyHandle();
	if (!ensure(RemoteControlHandle))
	{
		return;
	}
	
	const FObjectProperty* SourceObjectProperty = CastField<FObjectProperty>(Controller->GetProperty());
	const FObjectProperty* TargetObjectProperty = CastField<FObjectProperty>(RemoteControlProperty);
	
	if (!TargetObjectProperty && RemoteControlProperty->IsA(FArrayProperty::StaticClass()))
	{
		if (const FArrayProperty* TargetArrayProperty = CastField<FArrayProperty>(RemoteControlProperty))
		{
			if (FProperty* InnerProperty = TargetArrayProperty->Inner)
			{
				TargetObjectProperty = CastField<FObjectProperty>(InnerProperty);
			}
		}
	}

	if (ensure(SourceObjectProperty && TargetObjectProperty))
	{
		bool bApplyValue = false;
		
		// Pass 1 - Objects of exactly the same type
		if (SourceObjectProperty->SameType(TargetObjectProperty))
		{
			bApplyValue = true;
		}
		// Pass 2 - Objects that support conversion to other (related) Objects
		else
		{
			// Nothing for now
		}

		if (bApplyValue)
		{
			if (UObject* ObjectValue = Controller->GetValueObject())
			{
				if (RemoteControlProperty->IsA(FArrayProperty::StaticClass()))
				{
					// Extract index in array, based on field name
					const int32 Index = GetPropertyIndexInArray(InRemoteControlEntityAsProperty);	
					if (const TSharedPtr<IRemoteControlPropertyHandle>& ElementToUpdate = RemoteControlHandle->AsArray()->GetElement(Index))
					{
						ElementToUpdate->SetValueInArray(ObjectValue, Index);					
					}
				}
				else
				{
					RemoteControlHandle->SetValue(ObjectValue);
				}		
			}
		}
	}
}

void URCPropertyBindAction::Execute() const
{
	if (!ensure(Controller))
	{
		return;
	}

	if (TSharedPtr<FRemoteControlProperty> RemoteControlEntityAsProperty = GetRemoteControlProperty())
	{
		const FProperty* ControllerAsProperty = Controller->GetProperty();
		FProperty* RemoteControlProperty = RemoteControlEntityAsProperty->GetProperty();
		if (RemoteControlProperty == nullptr)
		{
			return;
		}

		TSharedPtr<IRemoteControlPropertyHandle> Handle = RemoteControlEntityAsProperty->GetPropertyHandle();
		if (!ensure(Handle))
		{
			return;
		}

		// String Controller
		if (ControllerAsProperty->IsA(FStrProperty::StaticClass()))
		{
			FString StringValue;
			Controller->GetValueString(StringValue);

			// String to String/Name/Text
			if (RemoteControlProperty->IsA(FStrProperty::StaticClass()) ||
				RemoteControlProperty->IsA(FTextProperty::StaticClass()) ||
				RemoteControlProperty->IsA(FNameProperty::StaticClass()))
			{
				// Set value directly. Implicitly converts to Text, Name, String
				// Also provides replication support via IRemoteControlPropertyHandle
				Handle->SetValue(StringValue);
			}
			else
			{
				const float NumericValue = FCString::Atof(*StringValue);

				// String to Byte/Enum
				if (RemoteControlProperty->IsA(FByteProperty::StaticClass()))
				{
					Handle->SetValue((uint8)NumericValue);
				}
				// String to Numeric
				else if (RemoteControlProperty->IsA(FNumericProperty::StaticClass()))
				{
					Handle->SetValue(NumericValue);
				}
				// String to Boolean
				else if (RemoteControlProperty->IsA(FBoolProperty::StaticClass()))
				{
					Handle->SetValue((bool)NumericValue);
				}
				else // This might be a Custom Controller
				{
					const FString& CustomControllerType = UE::RCCustomControllers::GetCustomControllerTypeName(Controller);
					
					if (!CustomControllerType.IsEmpty())
					{
						// External Texture controller (internally a String controller)
						if (CustomControllerType == UE::RCCustomControllers::CustomTextureControllerName)
						{
							if (RemoteControlProperty->IsA(FObjectProperty::StaticClass()))
							{
								UE::RCCustomBindActionUtilities::SetTexturePropertyFromPath(RemoteControlEntityAsProperty.ToSharedRef(), Controller, StringValue);
							}
						}
					}
				}
			}
			// String Controller (end)
		}
		// Numeric Controller (Float/Integer)
		else if (ControllerAsProperty->IsA(FNumericProperty::StaticClass()))
		{
			// Extract Numeric Value
			float NumericValue = 0.f; // placate compiler warning
			if (ControllerAsProperty->IsA(FIntProperty::StaticClass()))
			{
				int32 IntValue;
				Controller->GetValueInt32(IntValue);
				NumericValue = IntValue;
			}
			else if (ControllerAsProperty->IsA(FFloatProperty::StaticClass()))
			{
				Controller->GetValueFloat(NumericValue);
			}

			// Numeric to Byte/Enum
			if (RemoteControlProperty->IsA(FByteProperty::StaticClass()) ||
				RemoteControlProperty->IsA(FEnumProperty::StaticClass()))
			{
				Handle->SetValue((uint8)NumericValue);
			}
			
			// Numeric To Numeric
			if (RemoteControlProperty->IsA(FNumericProperty::StaticClass()))
			{
				if (RemoteControlProperty->IsA(FFloatProperty::StaticClass()))
				{
					Handle->SetValue(NumericValue);
				}
				else if (RemoteControlProperty->IsA(FIntProperty::StaticClass()))
				{
					Handle->SetValue((int32)NumericValue);
				}
				else if (RemoteControlProperty->IsA(FDoubleProperty::StaticClass()))
				{
					Handle->SetValue((double)NumericValue);
				}
			}
			// Numeric To Boolean
			else if (RemoteControlProperty->IsA(FBoolProperty::StaticClass()))
			{
				Handle->SetValue((bool)NumericValue);
			}
			// Numeric to String (explicit conversion)
			else if (RemoteControlProperty->IsA(FStrProperty::StaticClass()) ||
				RemoteControlProperty->IsA(FTextProperty::StaticClass()) ||
				RemoteControlProperty->IsA(FNameProperty::StaticClass()))
			{
				FString StringValue;

				if (ControllerAsProperty->IsA(FIntProperty::StaticClass()))
				{
					StringValue = FString::Printf(TEXT("%d"), (int32)NumericValue);
					Handle->SetValue(StringValue);
				}
				else
				{
					// %g is used over %f to discard extra zero value fractional digits from the float (for visual clarity)
					StringValue = FString::Printf(TEXT("%g"), NumericValue);
				}

				Handle->SetValue(StringValue);
			}
			// Numeric to Vector/Vector2D/Rotator
			else if (const FStructProperty* TargetStructProperty = CastField<FStructProperty>(RemoteControlProperty))
			{
				// FVector
				if (TargetStructProperty->Struct == TBaseStructure<FVector>::Get())
				{
					const FVector VectorValue(NumericValue, NumericValue, NumericValue);
					Handle->SetValue(VectorValue);
				}
				// FVector2D
				else if (TargetStructProperty->Struct == TBaseStructure<FVector2D>::Get())
				{
					const FVector2D Vector2DValue(NumericValue, NumericValue);
					Handle->SetValue(Vector2DValue);
				}
				// FRotator
				else if (TargetStructProperty->Struct == TBaseStructure<FRotator>::Get())
				{
					const FRotator RotatorValue(NumericValue, NumericValue, NumericValue);
					Handle->SetValue(RotatorValue);
				} 
			}
		}
		// Boolean Controller
		else if (ControllerAsProperty->IsA(FBoolProperty::StaticClass()))
		{
			bool BoolValue;
			Controller->GetValueBool(BoolValue);

			if (RemoteControlProperty->IsA(FBoolProperty::StaticClass()))
			{
				Handle->SetValue(BoolValue);
			}
			else if (RemoteControlProperty->IsA(FNumericProperty::StaticClass()))
			{
				Handle->SetValue((float)BoolValue);
			}
			// Note: Explicit conversion from Boolean to String/Text/Name currently not deemed as a requirement, so hasn't been implemented here
		}
		else if (ControllerAsProperty->IsA(FStructProperty::StaticClass()))
		{
			SetStructPropertyFromController(RemoteControlProperty, Handle.ToSharedRef(), Controller);
		}
		else if (ControllerAsProperty->IsA(FObjectProperty::StaticClass()))
		{
			SetObjectPropertyFromController(RemoteControlEntityAsProperty.ToSharedRef(), Controller);
		}

		// Editor specific updates
	#if WITH_EDITOR
		for (UObject* Object : RemoteControlEntityAsProperty->GetBoundObjects())
		{
			// For Bind behaviour we want to refresh the pivot widget if we have just manipulated a Location vector via Controller
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object))
			{
				if (AActor* Actor = SceneComponent->GetOwner())
				{
					if (Actor->IsSelectedInEditor())
					{
						// Note: For regular Exposed Properties this is handled via RefreshEditorPostSetObjectProperties in FRemoteControlModule::SetObjectProperties
						// Bind needs this explicit path because the Controller's property name is not guaranteed to be "RelativeLocation" (which is the pattern match used in that function).
						// This can be revisited in the future depending on the number of related usecases that arise in this space.
						GUnrealEd->UpdatePivotLocationForSelection();
					}
				}
			}
		}
	#endif
	}
}
