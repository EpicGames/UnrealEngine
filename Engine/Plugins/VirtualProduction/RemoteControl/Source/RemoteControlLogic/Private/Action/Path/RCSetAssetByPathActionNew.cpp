// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/Path/RCSetAssetByPathActionNew.h"

#include "Action/Bind/RCCustomBindActionUtilitiesPrivate.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Controller/RCController.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "Engine/World.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlPropertyHandle.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlLogger.h"
#include "RemoteControlPreset.h"

void URCSetAssetByPathActionNew::Execute() const
{
	if (IsInternal())
	{
		const_cast<URCSetAssetByPathActionNew*>(this)->UpdateInternal();
	}
	else
	{
		const_cast<URCSetAssetByPathActionNew*>(this)->UpdateExternal();
	}
}

URCSetAssetByPathBehaviorNew* URCSetAssetByPathActionNew::GetSetAssetByPathBehavior() const
{
	return Cast<URCSetAssetByPathBehaviorNew>(GetParentBehaviour());
}

bool URCSetAssetByPathActionNew::IsInternal() const
{
	if (URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = GetSetAssetByPathBehavior())
	{
		return SetAssetByPathBehavior->bInternal;
	}

	return true;
}

FString URCSetAssetByPathActionNew::GetPath() const
{
	URCSetAssetByPathBehaviorNew* SetAssetByPath = GetSetAssetByPathBehavior();
	if (!SetAssetByPath)
	{
		return FString();
	}

	FString BaseAssetPath = SetAssetByPath->GetCurrentPath();
	if (BaseAssetPath.IsEmpty())
	{
		return FString();
	}

	return BaseAssetPath.Replace(TEXT("//"), TEXT("/"));;
}

bool URCSetAssetByPathActionNew::UpdateInternal()
{
	const FString AssetPath = GetPath();

	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = GetRemoteControlProperty();
	if (!RemoteControlProperty.IsValid())
	{
		return false;
	}

	FProperty* Property = RemoteControlProperty->GetProperty();
	if (!Property)
	{
		return false;
	}

	FSoftObjectPath MainObjectRef(AssetPath);

	UObject* Object = MainObjectRef.TryLoad();

	if (!Object)
	{
		return false;
	}

	using namespace UE::RemoteControl::Logic;

	FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, [MainObjectRef]
		{
			return FText::FromString(FString("Path Behavior attempts to set Asset to %s") + *MainObjectRef.GetAssetPathString());
		});

	return SetInternalAsset(Object);
}

bool URCSetAssetByPathActionNew::UpdateExternal()
{
	const FString AssetPath = GetPath();

	using namespace UE::RemoteControl::Logic;

	FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, [AssetPath]
		{
			return FText::FromString(FString("Path Behavior attempts to set Asset to %s") + *AssetPath);
		});

#if !WITH_EDITOR
	return SetExternalAsset(AssetPath);
#else
	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = GetRemoteControlProperty();
	if (!RemoteControlProperty.IsValid())
	{
		return false;
	}

	FProperty* Property = RemoteControlProperty->GetProperty();
	if (!Property)
	{
		return false;
	}

	FEditPropertyChain PropertyChain;
	PropertyChain.AddHead(Property);
	RemoteControlProperty->GetBoundObject()->PreEditChange(PropertyChain);

	if (SetExternalAsset(AssetPath))
	{
		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		RemoteControlProperty->GetBoundObject()->PostEditChangeProperty(ChangedEvent);

		return true;
	}

	return false;
#endif
}

bool URCSetAssetByPathActionNew::SetTextureFromPath(TSharedPtr<FRemoteControlProperty> InRCPropertyToSet, const FString& InFileName)
{
	if (InRCPropertyToSet.IsValid())
	{
		if (URCBehaviour* Behavior = GetParentBehaviour())
		{
			if (URCController* Controller = Behavior->ControllerWeakPtr.Get())
			{
				UE::RCCustomBindActionUtilities::SetTexturePropertyFromPath(InRCPropertyToSet.ToSharedRef(), Controller, InFileName);
			}
		}

		return true;
	}

	return false;
}

bool URCSetAssetByPathActionNew::SetInternalAsset(UObject* InSetterObject)
{
	using namespace UE::RemoteControl::Logic;

	if (!InSetterObject)
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, [InSetterObject]
			{
				return FText::FromString(FString("Path Behavior fails to set Asset: Null"));
			}, EVerbosityLevel::Error);

		return false;
	}

	URCSetAssetByPathBehaviorNew* SetAssetByPath = GetSetAssetByPathBehavior();
	if (!SetAssetByPath)
	{
		return false;
	}

	URCController* Controller = SetAssetByPath->ControllerWeakPtr.Get();
	if (!Controller)
	{
		return false;
	}

	URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get();
	if (!Preset)
	{
		return false;
	}

	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = GetRemoteControlProperty();

	if (!RemoteControlProperty.IsValid())
	{
		return false;
	}

	if (InSetterObject->IsA(UBlueprint::StaticClass()))
	{
		if (AActor* OldActor = Cast<AActor>(RemoteControlProperty->GetBoundObject()))
		{
			UWorld* World = OldActor->GetWorld();

			OldActor->UnregisterAllComponents();

			const FVector OldLocation = OldActor->GetActorLocation();
			const FRotator OldRotation = OldActor->GetActorRotation();
			const FName OldActorName = OldActor->GetFName();

			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = OldActorName;

			const FName OldActorReplacedNamed = MakeUniqueObjectName(OldActor->GetOuter(), OldActor->GetClass(), *FString::Printf(TEXT("%s_REPLACED"), *OldActor->GetFName().ToString()));
			OldActor->Rename(*OldActorReplacedNamed.ToString(), OldActor->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

			if (AActor* NewActor = World->SpawnActor(Cast<UBlueprint>(InSetterObject)->GeneratedClass, &OldLocation, &OldRotation, SpawnParams))
			{
				World->DestroyActor(OldActor);

				UE_LOGF(LogRemoteControl, Verbose, "Path Behavior sets Blueprint Asset %ls", *InSetterObject->GetName());
				return true;
			}

			UE_LOGF(LogRemoteControl, Error, "Path Behavior unable to delete old Actor %ls", *OldActor->GetName());
			OldActor->UObject::Rename(*OldActorName.ToString(), OldActor->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			OldActor->RegisterAllComponents();

			return false;
		}
	}

	if (TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle = RemoteControlProperty->GetPropertyHandle())
	{
		UClass* SetterObjectClass = InSetterObject->GetClass();

		if (TSharedPtr<IRemoteControlPropertyHandleArray> ArrayHandle = PropertyHandle->AsArray())
		{
			const int32 Index = PropertyHandle->GetIndexInArray();

			if (Index >= 0)
			{
				if (ArrayHandle->GetNumElements() > 0)
				{
					if (TSharedPtr<IRemoteControlPropertyHandle> SubPropertyHandle = ArrayHandle->GetElement(Index))
					{
						if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(SubPropertyHandle->GetProperty()))
						{
							if (ObjectProperty->PropertyClass && (SetterObjectClass == ObjectProperty->PropertyClass || SetterObjectClass->IsChildOf(ObjectProperty->PropertyClass)))
							{
								UE_LOGF(LogRemoteControl, Verbose, "Path Behavior sets Array Property %ls", *InSetterObject->GetName());
								SubPropertyHandle->SetValue(InSetterObject);
								return true;
							}
						}
					}
				}
			}
		}
		else if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(PropertyHandle->GetProperty()))
		{
			if (ObjectProperty->PropertyClass && (SetterObjectClass == ObjectProperty->PropertyClass || SetterObjectClass->IsChildOf(ObjectProperty->PropertyClass)))
			{
				UE_LOGF(LogRemoteControl, Verbose, "Path Behavior sets Property %ls", *InSetterObject->GetName());
				PropertyHandle->SetValue(InSetterObject);
				return true;
			}
		}
	}

	UE_LOGF(LogRemoteControl, Error, "Path Behavior fails to set Asset %ls", *InSetterObject->GetName());
	return false;
}

bool URCSetAssetByPathActionNew::SetExternalAsset(const FString& InExternalPath)
{
	using namespace UE::RemoteControl::Logic;

	if (InExternalPath.IsEmpty())
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, [InExternalPath]
			{
				return FText::FromString(FString("Path Behavior fails to set Asset: %s") + *InExternalPath);
			}, EVerbosityLevel::Error);

		return false;
	}

	URCSetAssetByPathBehaviorNew* SetAssetByPath = GetSetAssetByPathBehavior();
	if (!SetAssetByPath)
	{
		return false;
	}

	URCController* Controller = SetAssetByPath->ControllerWeakPtr.Get();
	if (!Controller)
	{
		return false;
	}

	URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get();
	if (!Preset)
	{
		return false;
	}

	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = GetRemoteControlProperty();

	if (!RemoteControlProperty.IsValid())
	{
		return false;
	}

	if (SetTextureFromPath(RemoteControlProperty, InExternalPath))
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, [InExternalPath]
		{
			return FText::FromString(FString("Path Behavior Set external Asset: ") + *InExternalPath);
		});

		return true;
	}

	FRemoteControlLogger::Get().Log(SetAssetByPathNew::SetAssetByPathBehaviour, []
	{
		return FText::FromString(TEXT("Path Behavior Set external Asset failed"));
	});

	return false;
}
