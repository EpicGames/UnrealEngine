// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/RCActionContainer.h"

#include "Action/RCAction.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyIdAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Behaviour/RCBehaviour.h"
#include "Containers/Ticker.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "IRemoteControlModule.h"
#include "RCVirtualProperty.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"

void URCActionContainer::ExecuteActions()
{
	for (const URCAction* Action : Actions)
	{
		Action->Execute();
	}
}

TRCActionUniquenessTest URCActionContainer::GetDefaultActionUniquenessTest(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	const FGuid FieldId = InRemoteControlField->GetId();

	return [FieldId](const TSet<TObjectPtr<URCAction>>& InActions)
	{
		for (const URCAction* Action : InActions)
		{
			if (Action->ExposedFieldId == FieldId)
			{
				return false;
			}
		}

		return true;
	};
}

URCAction* URCActionContainer::AddAction()
{
	// Create new PropertyIdAction
	URCPropertyIdAction* NewPropertyIdAction = NewObject<URCPropertyIdAction>(this);
	NewPropertyIdAction->PresetWeakPtr = PresetWeakPtr;
	NewPropertyIdAction->Id = FGuid::NewGuid();
	NewPropertyIdAction->Initialize();
	NewPropertyIdAction->UpdatePropertyId();
	AddAction(NewPropertyIdAction);
	return NewPropertyIdAction;
}

URCAction* URCActionContainer::AddAction(FName InFieldId)
{
	// Create new PropertyIdAction
	URCPropertyIdAction* NewPropertyIdAction = NewObject<URCPropertyIdAction>(this);
	NewPropertyIdAction->PresetWeakPtr = PresetWeakPtr;
	NewPropertyIdAction->Id = FGuid::NewGuid();
	NewPropertyIdAction->PropertyId = InFieldId;
	NewPropertyIdAction->Initialize();
	NewPropertyIdAction->UpdatePropertyId();
	AddAction(NewPropertyIdAction);
	return NewPropertyIdAction;
}

URCAction* URCActionContainer::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	return AddAction(GetDefaultActionUniquenessTest(InRemoteControlField), InRemoteControlField);
}


URCAction* URCActionContainer::AddAction(TRCActionUniquenessTest InUniquenessTest, const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	if (!InUniquenessTest(Actions))
	{
		return nullptr;
	}

	URCAction* NewAction = nullptr;

	if (InRemoteControlField->FieldType == EExposedFieldType::Property)
	{
		NewAction = AddPropertyAction(StaticCastSharedRef<const FRemoteControlProperty>(InRemoteControlField));
	}
	else if (InRemoteControlField->FieldType == EExposedFieldType::Function)
	{
		NewAction = AddFunctionAction(StaticCastSharedRef<const FRemoteControlFunction>(InRemoteControlField));
	}

	return NewAction;
}

URCBehaviour* URCActionContainer::GetParentBehaviour()
{
	return Cast<URCBehaviour>(GetOuter());
}

void URCActionContainer::ForEachAction(TFunctionRef<void(URCAction*)> InActionFunction, bool bInRecursive)
{
	for (URCAction* Action : GetActions())
	{
		if (Action)
		{
			InActionFunction(Action);
		}
	}

	if (bInRecursive)
	{
		for (URCActionContainer* ChildActionContainer : ActionContainers)
		{
			if (ChildActionContainer)
			{
				ChildActionContainer->ForEachAction(InActionFunction, bInRecursive);
			}
		}
	}
}

TArray<const URCPropertyAction*> URCActionContainer::GetPropertyActions() const
{
	TArray<const URCPropertyAction*> PropertyActions;

	Algo::TransformIf(Actions, PropertyActions,
		[](const URCAction* Action) { return Action->IsA(URCPropertyAction::StaticClass()); },
		[](const URCAction* Action) { return Cast<URCPropertyAction>(Action); });

	return PropertyActions;
}

URCPropertyAction* URCActionContainer::AddPropertyAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty)
{
	// Create new Property
	URCPropertyAction* NewPropertyAction = NewObject<URCPropertyAction>(this);
	NewPropertyAction->PresetWeakPtr = PresetWeakPtr;
	NewPropertyAction->ExposedFieldId = InRemoteControlProperty->GetId();
	NewPropertyAction->Id = FGuid::NewGuid();

	if (!InRemoteControlProperty->GetBoundObjects().Num())
	{
		// This is possible if an exposed property was either deleted directly by the user, or if a project exits without saving the linked actor, etc

		return nullptr;
	}

	bool bFoundMatchingContainer = false;

	if (FProperty* Property = InRemoteControlProperty->GetProperty())
	{
		// Reusing input property is only applicable for Array properties
		// Eg: "Override Materials" will be represented by a single virtual property Array with each Action occupying a different index slot.
		if(Property->IsA(FArrayProperty::StaticClass()))
		{
			for (const URCPropertyAction* PropertyAction : GetPropertyActions())
			{
				if (FProperty* InputFieldProperty = PropertyAction->GetProperty())
				{
					if (InputFieldProperty->GetFName() == Property->GetFName())
					{
						if (const TSharedPtr<const FRemoteControlProperty> RemoteControlPropertyForAction = PropertyAction->GetRemoteControlProperty())
						{
							if (InRemoteControlProperty->GetBoundObject() == RemoteControlPropertyForAction->GetBoundObject())
							{
								// We already have an Action associated with this Array container property.
								// The array container will be reused, both the UI widget and Execute will operate on a unique index associated with the respective Action
								NewPropertyAction->PropertySelfContainer = PropertyAction->PropertySelfContainer;

								bFoundMatchingContainer = true;
							}
						}
					}
				}
			}
		}
	}

	if(!bFoundMatchingContainer)
	{
		// Check both Reading and Writing since if one of the two is not valid then the action won't work
		FRCObjectReference ObjectRefReading;
		const bool bResolveForReading = IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, InRemoteControlProperty->GetBoundObjects()[0], InRemoteControlProperty->FieldPathInfo.ToString(), ObjectRefReading);

		FRCObjectReference ObjectRefWriting;
		const bool bResolveForWriting = IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::WRITE_ACCESS, InRemoteControlProperty->GetBoundObjects()[0], InRemoteControlProperty->FieldPathInfo.ToString(), ObjectRefWriting);

		// Create an input field for the Action by duplicating the Remote Control Property associated with it
		if (bResolveForReading && bResolveForWriting)
		{
			const FName& PropertyName = InRemoteControlProperty->GetProperty()->GetFName();
			NewPropertyAction->PropertySelfContainer->DuplicatePropertyWithCopy(PropertyName, InRemoteControlProperty->GetProperty(), (uint8*)ObjectRefReading.ContainerAdress);
			NewPropertyAction->PropertySelfContainer->PresetWeakPtr = PresetWeakPtr;
		}
	}

	AddAction(NewPropertyAction);
	
	return NewPropertyAction;
}

URCFunctionAction* URCActionContainer::AddFunctionAction(const TSharedRef<const FRemoteControlFunction> InRemoteControlFunction)
{
	// Create new Function Action
	URCFunctionAction* NewFunctionAction = NewObject<URCFunctionAction>(this);
	NewFunctionAction->PresetWeakPtr = PresetWeakPtr;
	NewFunctionAction->ExposedFieldId = InRemoteControlFunction->GetId();
	NewFunctionAction->Id = FGuid::NewGuid();

	AddAction(NewFunctionAction);
	
	return NewFunctionAction;
}

void URCActionContainer::AddAction(URCAction* NewAction)
{
	Actions.Add(NewAction);
}

#if WITH_EDITOR
void URCActionContainer::PostEditUndo()
{
	Super::PostEditUndo();

	OnActionsListModified.Broadcast();
}
#endif

URCAction* URCActionContainer::FindActionByFieldId(const FGuid InId) const
{
	for (URCAction* Action :  Actions)
	{
		if (Action->ExposedFieldId == InId)
		{
			return Action;
		}
	}

	return nullptr;
}

URCAction* URCActionContainer::FindActionByField(const TSharedRef<const FRemoteControlField> InRemoteControlField) const
{
	return FindActionByFieldId(InRemoteControlField->GetId());
}

int32 URCActionContainer::RemoveAction(const FGuid InExposedFieldId)
{
	int32 RemoveCount = 0;
	
	for (auto ActionsIt = Actions.CreateIterator(); ActionsIt; ++ActionsIt)
	{
		if (const URCAction* Action = *ActionsIt; Action->ExposedFieldId == InExposedFieldId)
		{
			ActionsIt.RemoveCurrent();
			RemoveCount++;
		}
	}

	return RemoveCount;
}

int32 URCActionContainer::RemoveAction(URCAction* InAction)
{
	return Actions.Remove(InAction);
}

void URCActionContainer::EmptyActions()
{
	Actions.Empty();
}

void URCActionContainer::UpdateEntityUsage(const URCBehaviour* InBehavior, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const
{
	for (URCActionContainer* ActionContainer : ActionContainers)
	{
		ActionContainer->UpdateEntityUsage(InBehavior, InEntityToBehavior);
	}

	for (URCAction* Action : Actions)
	{
		Action->UpdateEntityUsage(InBehavior, InEntityToBehavior);
	}
}
