// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/RCFunctionAction.h"

#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "StructSerializer.h"
#include "Backends/CborStructSerializerBackend.h"

void URCFunctionAction::Execute() const
{
	if (!PresetWeakPtr.IsValid())
	{
		return;
	}

	if (!ExposedFieldId.IsValid())
	{
		return;
	}
	
	if (TSharedPtr<FRemoteControlFunction> RemoteControlFunction = PresetWeakPtr->GetExposedEntity<FRemoteControlFunction>(ExposedFieldId).Pin())
	{
		// Copy the default arguments.
		FStructOnScope FunctionArgs{ RemoteControlFunction->GetFunction() };
		for (TFieldIterator<FProperty> It(RemoteControlFunction->GetFunction()); It; ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm))
			{
				It->CopyCompleteValue_InContainer(FunctionArgs.GetStructMemory(), RemoteControlFunction->FunctionArguments->GetStructMemory());
			}
		}

		TArray<uint8> FunctionPayload;
		FMemoryWriter Writer(FunctionPayload);

		FCborStructSerializerBackend SerializerBackend{ Writer, EStructSerializerBackendFlags::Default };
		FStructSerializer::Serialize(FunctionArgs.GetStructMemory(), *const_cast<UStruct*>(FunctionArgs.GetStruct()), SerializerBackend);

		for (UObject* Object : RemoteControlFunction->GetBoundObjects())
		{
			FRCCallReference CallRef;
			CallRef.Object = Object;
			CallRef.Function = RemoteControlFunction->GetFunction();

			FRCCall Call;
			Call.CallRef = MoveTemp(CallRef);
			Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
			Call.ParamStruct = FStructOnScope(FunctionArgs.GetStruct(), FunctionArgs.GetStructMemory());
		
			IRemoteControlModule::Get().InvokeCall(Call, ERCPayloadType::Cbor, FunctionPayload);
		}
	}
	
	Super::Execute();
}
