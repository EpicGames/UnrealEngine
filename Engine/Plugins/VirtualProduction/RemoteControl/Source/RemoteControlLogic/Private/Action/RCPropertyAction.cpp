// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/RCPropertyAction.h"

#include "Action/RCActionContainer.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlPropertyHandle.h"
#include "IStructSerializerBackend.h"
#include "RCVirtualProperty.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "StructSerializer.h"
#include "Action/RCAction.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"

URCPropertyAction::URCPropertyAction()
{
	PropertySelfContainer = CreateDefaultSubobject<URCVirtualPropertySelfContainer>(FName("VirtualPropertySelfContainer"));
}

URCBehaviour* URCAction::GetParentBehaviour() const
{
	URCActionContainer* ActionContainer = Cast<URCActionContainer>(GetOuter());
	if (ensure(ActionContainer))
	{
		return ActionContainer->GetParentBehaviour();
	}

	return nullptr;
}

void URCAction::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	if (const FGuid* FoundId = InEntityIdMap.Find(ExposedFieldId))
	{
		ExposedFieldId = *FoundId;
	}
}

void URCAction::UpdateEntityUsage(const URCBehaviour* InBehavior, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const
{
	URCBehaviour* Behavior = GetParentBehaviour();

	if (!Behavior || Behavior != InBehavior || !InBehavior->HasAction(this))
	{
		return;
	}

	if (!ExposedFieldId.IsValid())
	{
		return;
	}

	InEntityToBehavior.FindOrAdd(ExposedFieldId).Add(Behavior);
}

void URCAction::NotifyActionValueChanged()
{
	if (URCBehaviour* Behaviour = GetParentBehaviour())
	{
		Behaviour->NotifyActionValueChanged(this);
	}
}

void URCPropertyAction::Execute() const
{
	if (!PresetWeakPtr.IsValid())
	{
		return;
	}
	
	if (TSharedPtr<FRemoteControlProperty> RemoteControlProperty = PresetWeakPtr->GetExposedEntity<FRemoteControlProperty>(ExposedFieldId).Pin())
	{
		FRCObjectReference ObjectRef;
		ObjectRef.Property = RemoteControlProperty->GetProperty();
		ObjectRef.Access = ERCAccess::WRITE_ACCESS;
	
		ObjectRef.PropertyPathInfo = RemoteControlProperty->FieldPathInfo.ToString();
	
		bool bSuccess = true;
		for (UObject* Object : RemoteControlProperty->GetBoundObjects())
		{
			if (IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef))
			{
				TArray<uint8> Buffer;
				FMemoryWriter Writer(Buffer);
	
				FCborStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);
	
				FStructSerializerPolicies Policies; 
				Policies.MapSerialization = EStructSerializerMapPolicies::Array;
				
				PropertySelfContainer->SerializeToBackend(SerializerBackend);
	
				// Deserialization
				FMemoryReader Reader(Buffer);
				FCborStructDeserializerBackend DeserializerBackend(Reader);
	
				IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Cbor, Buffer);
			}
		}
	}
	
	Super::Execute();
}

void URCPropertyAction::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	if (PropertySelfContainer)
	{
		PropertySelfContainer->UpdateEntityIds(InEntityIdMap);
	}
	
	Super::UpdateEntityIds(InEntityIdMap);	
}

FProperty* URCPropertyAction::GetProperty() const
{
	if (ensure(PropertySelfContainer))
	{
		return PropertySelfContainer->GetProperty();
	}

	return nullptr;
}

void URCPropertyAction::UpdateValueBasedOnRCProperty() const
{
	if (const TSharedPtr<FRemoteControlProperty> RCExposedProperty = GetRemoteControlProperty())
	{
		const FProperty* RCProperty = RCExposedProperty->GetProperty();
		if (PropertySelfContainer)
		{
			PropertySelfContainer->UpdateValueWithProperty(RCProperty, RCExposedProperty->GetFieldContainerAddress());
		}
	}
}

FName URCAction::GetExposedFieldLabel() const
{
	if (const URemoteControlPreset* Preset = PresetWeakPtr.Get())
	{
		if (TSharedPtr<const FRemoteControlField> RemoteControlField = Preset->GetExposedEntity<FRemoteControlField>(ExposedFieldId).Pin())
		{
			return RemoteControlField->GetLabel();
		}
	}

	return NAME_None;
}

TSharedPtr<FRemoteControlProperty> URCPropertyAction::GetRemoteControlProperty() const
{
	if (URemoteControlPreset* Preset = PresetWeakPtr.Get())
	{
		if (TSharedPtr<FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(ExposedFieldId).Pin())
		{
			return RemoteControlProperty;
		}
	}

	return nullptr;
}