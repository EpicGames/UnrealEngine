// Copyright Epic Games, Inc. All Rights Reserved.

#include "Action/RCPropertyIdAction.h"

#include "Engine/StaticMesh.h"
#include "IPropertyIdHandler.h"
#include "IRemoteControlModule.h"
#include "Materials/MaterialInterface.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "RemoteControlPropertyIdRegistry.h"

TSet<FName> URCPropertyIdAction::AllowedStructNameToCopy
{
	NAME_Color,
	NAME_LinearColor,
	NAME_Vector,
	NAME_Vector2D,
	NAME_Rotation,
	NAME_Rotator
};

URCPropertyIdAction::~URCPropertyIdAction()
{
	if (PresetWeakPtr.IsValid())
	{
		PresetWeakPtr->OnEntityUnexposed().RemoveAll(this);
		if (const TObjectPtr<URemoteControlPropertyIdRegistry> Registry = PresetWeakPtr->GetPropertyIdRegistry())
		{
			Registry->OnPropertyIdUpdated().RemoveAll(this);
		}
	}
}

void URCPropertyIdAction::Execute() const
{
	if (!PresetWeakPtr.IsValid())
	{
		return;
	}
	
	if (PropertySelfContainer.IsEmpty())
	{
		return;
	}

	for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainer : PropertySelfContainer)
	{
		if (FProperty* Property = PropertyContainer.Value->GetProperty())
		{
			FRemoteControlPropertyIdArgs PropertyIdArgs;
			PropertyIdArgs.VirtualProperty = PropertyContainer.Value;
			PropertyIdArgs.PropertyId = PropertyContainer.Key.PropertyId;
			PropertyIdArgs.RealProperties = RealPropertySelfContainer;

			TSharedPtr<IPropertyIdHandler> PropertyIdHandler = IRemoteControlModule::Get().GetPropertyIdHandlerFor(Property);
			if (!PropertyIdHandler.IsValid())
			{
				continue;
			}

			PropertyIdArgs.SuperType = PropertyIdHandler->GetPropertySuperTypeName(Property);
			const EPropertyBagPropertyType PropertyBagType = PropertyIdHandler->GetPropertyType(Property);

			if (PropertyBagType == EPropertyBagPropertyType::Enum ||
				PropertyBagType == EPropertyBagPropertyType::Object ||
				PropertyBagType == EPropertyBagPropertyType::Struct)
			{
				PropertyIdArgs.SubType = PropertyIdHandler->GetPropertySubTypeName(Property);
			}

			PresetWeakPtr->PerformChainReaction(PropertyIdArgs);
		}
	}
	Super::Execute();
}

void URCPropertyIdAction::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainerEntry : PropertySelfContainer)
	{
		if (PropertyContainerEntry.Value)
		{
			PropertyContainerEntry.Value->UpdateEntityIds(InEntityIdMap);
		}
	}
	for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainerEntry : CachedPropertySelfContainer)
	{
		if (PropertyContainerEntry.Value)
		{
			PropertyContainerEntry.Value->UpdateEntityIds(InEntityIdMap);
		}
	}
	
	Super::UpdateEntityIds(InEntityIdMap);
}

void URCPropertyIdAction::PostLoad()
{
	UObject::PostLoad();
	Initialize();
}

#if WITH_EDITOR
void URCPropertyIdAction::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);
	if (const FProperty* Property = PropertyChangedEvent.Property)
	{
		if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(URCPropertyIdAction, PropertyId))
		{
			UpdatePropertyId();
		}
	}
}
#endif // WITH_EDITOR

void URCPropertyIdAction::UpdatePropertyId()
{
	if (URemoteControlPreset* Preset = PresetWeakPtr.Get())
	{
		PropertySelfContainer.Empty();
		TArray<FGuid> CurrentRCGuidsInAction;
		const TObjectPtr<URemoteControlPropertyIdRegistry> PropertyIdRegistry = Preset->GetPropertyIdRegistry();

		for (const FGuid& TargetProperty : PropertyIdRegistry->GetEntityIdsList())
		{
			if (const TSharedPtr<FRemoteControlProperty> TargetRCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(TargetProperty).Pin())
			{
				if (PropertyIdRegistry->Contains(TargetRCProperty->PropertyId, PropertyId))
				{
					if (FProperty* Property = TargetRCProperty->GetProperty())
					{
						TSharedPtr<IPropertyIdHandler> PropertyIdHandler = IRemoteControlModule::Get().GetPropertyIdHandlerFor(Property);
						if (!PropertyIdHandler.IsValid())
						{
							continue;
						}

						const FName PropertyClassName = *(TargetRCProperty->PropertyId.ToString() + TEXT(".") + PropertyIdHandler->GetPropertySubTypeName(Property).ToString());

						// Set the property name while sanitizing it with valid object characters
						const FName NewPropertyIdName = *SlugStringForValidName(TargetRCProperty->PropertyId.ToString() + TEXT("_") + PropertyClassName.ToString(), TEXT("_"));

						FPropertyIdContainerKey CurrentKey = FPropertyIdContainerKey(TargetRCProperty->PropertyId, PropertyClassName);

						TArray<UObject*> BoundObjects = TargetRCProperty->GetBoundObjects();
						if (BoundObjects.IsEmpty() || !BoundObjects[0])
						{
							continue;
						}

						FRCObjectReference ObjectRefReading;
						const bool bResolveForReading = IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, BoundObjects[0], TargetRCProperty->FieldPathInfo.ToString(), ObjectRefReading);
						const FProperty* PropToDuplicate = PropertyIdHandler->GetPropertyInsideContainer(Property);

						if (!bResolveForReading)
						{
							continue;
						}

						const FName& PropertyName = TargetRCProperty->GetProperty()->GetFName();
						const uint8* RealPropAddress = (uint8*)ObjectRefReading.ContainerAdress;

						if (RealPropAddress && PropToDuplicate != Property)
						{
							RealPropAddress = PropToDuplicate->ContainerPtrToValuePtr<uint8>(ObjectRefReading.ContainerAdress);
						}

						if (!RealPropAddress)
						{
							continue;
						}

						FGuid RCGuid = TargetRCProperty->GetId();
						CurrentRCGuidsInAction.Add(RCGuid);
						if (!RealPropertySelfContainer.Contains(RCGuid))
						{
							RealPropertySelfContainer.Add(RCGuid, NewObject<URCVirtualPropertySelfContainer>(this));
							RealPropertySelfContainer[RCGuid]->DuplicateProperty(PropertyName, PropToDuplicate);
							RealPropertySelfContainer[RCGuid]->PresetWeakPtr = PresetWeakPtr;
						}

						if (CachedPropertySelfContainer.Contains(CurrentKey))
						{
							PropertySelfContainer.Add(CurrentKey, CachedPropertySelfContainer[CurrentKey]);
						}
						else
						{
							CachedPropertySelfContainer.Add(CurrentKey, NewObject<URCVirtualPropertySelfContainer>(this));

							bool bIsSpecialCase = false;
							FName StructName = NAME_None;

							// float property and double ones are treated as one the same goes for FLinearColor and FColor
							if (PropToDuplicate->IsA<FFloatProperty>())
							{
								CachedPropertySelfContainer[CurrentKey]->AddProperty(NewPropertyIdName,
								PropertyIdHandler->GetPropertyType(Property),
								PropertyIdHandler->GetPropertyTypeObject(Property));

								bIsSpecialCase = true;
							}
							else if (const FStructProperty* StructProp = CastField<FStructProperty>(PropToDuplicate))
							{
								if (StructProp->Struct)
								{
									StructName = StructProp->Struct->GetFName();
									if (StructName == NAME_LinearColor ||
										StructName == NAME_Color)
									{
										CachedPropertySelfContainer[CurrentKey]->AddProperty(NewPropertyIdName,
										PropertyIdHandler->GetPropertyType(Property),
										PropertyIdHandler->GetPropertyTypeObject(Property));
#if WITH_EDITORONLY_DATA
										FProperty* BagProperty = CachedPropertySelfContainer[CurrentKey]->GetProperty();
										FStructProperty* StructProperty = CastField<FStructProperty>(BagProperty);
										if (StructProperty && StructProperty->Struct &&
											!StructProperty->HasMetaData("OnlyUpdateOnInteractionEnd"))
										{
											StructProperty->AppendMetaData({{FName("OnlyUpdateOnInteractionEnd"), TEXT("true")}});
										}
#endif
										bIsSpecialCase = true;
									}
								}
							}

							if (!bIsSpecialCase)
							{
								CachedPropertySelfContainer[CurrentKey]->DuplicateProperty(NewPropertyIdName, PropToDuplicate);
#if WITH_EDITORONLY_DATA
								// In case the property is a Vector/Rotator we don't want the Lock widget
								if (FProperty* CachedProperty = CachedPropertySelfContainer[CurrentKey]->GetProperty())
								{
									CachedProperty->RemoveMetaData(TEXT("AllowPreserveRatio"));
								}
#endif
							}

							// Do this the first time it is created so that it will have a better default value except for Object
							// Some ObjectProperty won't work correctly with this copy for example the Material one, so we skip it
							// The same for some struct so we copy only the AllowedOnes
							bool bCanCopyValue = true;
							if (PropToDuplicate->IsA<FObjectProperty>())
							{
								bCanCopyValue = false;
							}

							if (StructName != NAME_None)
							{
								if (!AllowedStructNameToCopy.Contains(StructName))
								{
									bCanCopyValue = false;
								}
							}

							if (bCanCopyValue)
							{
								CachedPropertySelfContainer[CurrentKey]->UpdateValueWithProperty(PropToDuplicate, RealPropAddress);
							}
							CachedPropertySelfContainer[CurrentKey]->PresetWeakPtr = PresetWeakPtr;
							PropertySelfContainer.Add(CurrentKey, CachedPropertySelfContainer[CurrentKey]);
						}
					}
				}
			}
		}

		// Remove RealContainers that are not anymore part of this action
		TArray<FGuid> CurrentRealPropertyGuids;
		RealPropertySelfContainer.GetKeys(CurrentRealPropertyGuids);

		CurrentRealPropertyGuids.RemoveAll([CurrentRCGuidsInAction] (const FGuid& InGuid) { return CurrentRCGuidsInAction.Contains(InGuid); });
		for (const FGuid& Guid : CurrentRealPropertyGuids)
		{
			RealPropertySelfContainer.Remove(Guid);
		}

#if WITH_EDITOR
		if (PresetWeakPtr.IsValid())
		{
			PresetWeakPtr->GetPropertyIdRegistry()->OnPropertyIdActionNeedsRefresh().Broadcast();
		}
#endif
	}
}

void URCPropertyIdAction::Initialize()
{
	if (PresetWeakPtr.IsValid())
	{
		PresetWeakPtr->OnEntityUnexposed().AddUObject(this, &URCPropertyIdAction::OnEntityUnexposed);
		if (const TObjectPtr<URemoteControlPropertyIdRegistry> Registry = PresetWeakPtr->GetPropertyIdRegistry())
		{
			Registry->OnPropertyIdUpdated().AddUObject(this, &URCPropertyIdAction::UpdatePropertyId);
		}
	}
}

void URCPropertyIdAction::OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InGuid)
{
	if (InPreset)
	{
		const TWeakPtr<FRemoteControlField> UnexposedEntity = StaticCastWeakPtr<FRemoteControlField>(InPreset->GetExposedEntity(InGuid));
		if (UnexposedEntity.IsValid())
		{
			//If the propertyId of the property unexposed is the same of this PropertyIdAction refresh, otherwise don't update it
			if (UnexposedEntity.Pin()->PropertyId == PropertyId)
			{
				InPreset->GetPropertyIdRegistry()->RemoveIdentifiedField(UnexposedEntity.Pin()->GetId());
				InPreset->GetPropertyIdRegistry()->OnPropertyIdUpdated().Broadcast();
			}
			return;
		}
	}
	//If there is anything wrong with the InPreset or InGuid it won't be able to check the PropertyId
	//We refresh to be sure to not miss anything
	UpdatePropertyId();
}
