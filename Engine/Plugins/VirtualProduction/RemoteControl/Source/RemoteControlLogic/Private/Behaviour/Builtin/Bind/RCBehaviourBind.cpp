// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"

#include "Action/Bind/RCPropertyBindAction.h"
#include "Action/RCAction.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "Engine/Texture2D.h"
#include "IRemoteControlPropertyHandle.h"
#include "StructUtils/PropertyBag.h"
#include "RCVirtualProperty.h"
#include "RemoteControlField.h"

URCBehaviourBind::URCBehaviourBind()
{
	// For Bind Behaviour we want the users to be able to scrub UI widgets like float slides and watch the Exposed Property (and their level rendering like Location, Light intenstiy, etc) update in realtime
	bExecuteBehavioursDuringPreChange = true;
}

void URCBehaviourBind::Initialize()
{
	Super::Initialize();
}

URCPropertyBindAction* URCBehaviourBind::AddPropertyBindAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty)
{
	if (!ensure(ControllerWeakPtr.IsValid() && ActionContainer))
	{
		return nullptr;
	}

	URCPropertyBindAction* BindAction = NewObject<URCPropertyBindAction>(ActionContainer);
	BindAction->PresetWeakPtr = ActionContainer->PresetWeakPtr;
	BindAction->ExposedFieldId = InRemoteControlProperty->GetId();
	BindAction->Controller = ControllerWeakPtr.Get();
	BindAction->Id = FGuid::NewGuid();

	// Add action to array
	ActionContainer->AddAction(BindAction);

	return BindAction;
}

URCAction* URCBehaviourBind::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	const TSharedRef<const FRemoteControlProperty> RemoteControlProperty = StaticCastSharedRef<const FRemoteControlProperty>(InRemoteControlField);

	URCAction* BindAction = AddPropertyBindAction(RemoteControlProperty);

	if (ensure(BindAction))
	{
		// For Bind Behaviour we want to pick up the Controller's current value immediately.
		// Subsequent value updates from Controller to Action will be propagated via the usual OnModifyPropertyValue code path

		BindAction->Execute();
	}

	return BindAction;
}

bool URCBehaviourBind::CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const
{
	if (!InRemoteControlField.IsValid())
	{
		return false;
	}

	// Basic check (uniqueness)
	if (ActionContainer->FindActionByFieldId(InRemoteControlField->GetId()))
	{
		return false; // already exists!
	}

	// Only property type will be allowed so we check before to avoid a cast
	if (InRemoteControlField->FieldType != EExposedFieldType::Property)
	{
		return false;
	}

	// Advanced checks (by Controller type and Target type)
	if (const TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(InRemoteControlField))
	{
		if (URCController* Controller = ControllerWeakPtr.Get())
		{
			if (URCBehaviourBind::CanHaveActionForField(Controller, RCProperty.ToSharedRef(), bAllowNumericInputAsStrings))
			{
				if (const FProperty* Property = RCProperty->GetProperty())
				{
					const FPropertyBagPropertyDesc PropertyBagDesc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
					return RCProperty->IsEditable() && PropertyBagDesc.ValueType != EPropertyBagPropertyType::None;
				}
			}
		}
	}

	return false;
}

static bool EvaluateBindCompatibility(const TMap<FFieldClass*, TArray<FFieldClass*>>& BindingsMap, const FFieldClass* ControllerPropertyClass, const FFieldClass* InRemoteControlPropertyClass)
{
	TArray< FFieldClass*> Keys;
	BindingsMap.GenerateKeyArray(Keys);

	for (const FFieldClass* PropertyType : Keys)
	{
		if (ControllerPropertyClass->IsChildOf(PropertyType))
		{
			const TArray<FFieldClass*>* SupportedBindings = BindingsMap.Find(PropertyType);

			if (SupportedBindings)
			{
				for (const FFieldClass* SupportedBinding : *SupportedBindings)
				{
					if (InRemoteControlPropertyClass->IsChildOf(SupportedBinding))
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

static bool EvaluateBindCompatibility(const TMap<FFieldClass*, TArray<UScriptStruct*>>& InBindingsMap, const FFieldClass* InControllerPropertyClass, const FProperty* InRemoteControlProperty)
{
	TArray<FFieldClass*> Keys;
	InBindingsMap.GenerateKeyArray(Keys);

	if (const FStructProperty* RCFieldStructProperty = CastField<FStructProperty>(InRemoteControlProperty))
	{
		for (const FFieldClass* PropertyType : Keys)
		{
			if (InControllerPropertyClass->IsChildOf(PropertyType))
			{
				if (const TArray<UScriptStruct*>* SupportedBindings = InBindingsMap.Find(PropertyType))
				{
					for (const UScriptStruct* SupportedBinding : *SupportedBindings)
					{
						if (RCFieldStructProperty->Struct && RCFieldStructProperty->Struct->IsChildOf(SupportedBinding))
						{
							return true;
						}
					}
				}
			}
		}
	}

	return false;
}

static bool EvaluateCustomControllerBindCompatibility(const URCController* InController,  const FFieldClass* InControllerPropertyClass, const FFieldClass* InRemoteControlPropertyClass, const FProperty* InRemoteControlProperty)
{
	if (!InController)
	{
		return false;
	}
	
	const FString& CustomName = UE::RCCustomControllers::GetCustomControllerTypeName(InController);

	if (CustomName.IsEmpty())
	{
		return false;
	}

	// External Texture for image files (controller string, rc exposed property is object
	if (CustomName == UE::RCCustomControllers::CustomTextureControllerName && InControllerPropertyClass == FStrProperty::StaticClass())
	{
		// This cast makes sure we catch both UObject* and TObjectPtr
		if (const FObjectProperty* TargetObjectProperty = CastField<FObjectProperty>(InRemoteControlProperty))
		{
			// Target type is texture - we're good to go
			if (TargetObjectProperty->PropertyClass == UTexture::StaticClass())
			{
				return true;
			}
		}
	}

	return false;
}

bool URCBehaviourBind::CanHaveActionForField(URCController* Controller, TSharedRef<const FRemoteControlField> InRemoteControlField, const bool bInAllowNumericInputAsStrings)
{
	// Bind behaviour is only relevant for Exposed Properties
	if (InRemoteControlField->FieldType != EExposedFieldType::Property)
	{
		return false;
	}

	TSharedRef<const FRemoteControlProperty> RemoteControlEntityAsProperty = StaticCastSharedRef<const FRemoteControlProperty>(InRemoteControlField);

	const FProperty* ControllerAsProperty = Controller->GetProperty();
	const FProperty* RemoteControlProperty = RemoteControlEntityAsProperty->GetProperty();
	const FStructProperty* RemoteControlEntityAsStructProperty = CastField<FStructProperty>(RemoteControlProperty);

	if (!ControllerAsProperty || !RemoteControlProperty)
	{
		return false; // Unbound Property
	}

	const UEnum* Enum = nullptr;

	if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(RemoteControlProperty))
	{
		Enum = EnumProperty->GetEnum();
	}
	// BP enum properties are bytes
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(RemoteControlProperty))
	{
		Enum = ByteProperty->Enum;
	}

	// Enums with base other than 8bits should be ignored, so let's make sure of that
	if (Enum)
	{
		const int64 MaxEnumValue = Enum->GetMaxEnumValue();
		const uint32 NeededBits = FMath::RoundUpToPowerOfTwo(MaxEnumValue);

		// 8 bits enums only
		return NeededBits <= 256;
	}
	
	// Indirect Binding (related types)
	//
	const static TMap<FFieldClass*, TArray<FFieldClass*>> SupportedIndirectBindsMap =
	{
		/* Controller Type */                           /* Supported Remote Control Property Types */

		{ FStrProperty::StaticClass(),      /* --> */   { FTextProperty::StaticClass(),     FNameProperty::StaticClass()}},
		{ FNumericProperty::StaticClass(),  /* --> */   { FNumericProperty::StaticClass(),  FBoolProperty::StaticClass(),  FByteProperty::StaticClass(), FEnumProperty::StaticClass() } },
		{ FBoolProperty::StaticClass(),     /* --> */   { FFloatProperty::StaticClass(),    FIntProperty::StaticClass(),   FBoolProperty::StaticClass() } }
	};

	// Indirect Binding (via numeric conversion)
	//
	const static TMap<FFieldClass*, TArray<FFieldClass*>> SupportedNumericConversionsMap =
	{
		/* Controller Type */                           /* Supported Remote Control Property Types */

		{ FStrProperty::StaticClass(),      /* --> */   { FNumericProperty::StaticClass(), FBoolProperty::StaticClass(),  FByteProperty::StaticClass()  } },
		{ FNumericProperty::StaticClass(),  /* --> */   { FStrProperty::StaticClass(),     FTextProperty::StaticClass(),  FNameProperty::StaticClass() } }
	};

	// Indirect Binding (via numeric conversion)
	//
	const static TMap<FFieldClass*, TArray<UScriptStruct*>> SupportedNumericConversionsStructMap =
	{
		/* Controller Type */                           /* Supported Remote Control Property Types */
		{ FNumericProperty::StaticClass(),  /* --> */   { TBaseStructure<FVector>::Get(), TBaseStructure<FVector2D>::Get(), TBaseStructure<FRotator>::Get(), } }
	};

	// Indirect Binding (for Structs)
	//
	// todo: unify with supported controller structs in RC Config
	const static TMap<UScriptStruct*, TArray<UScriptStruct*>> SupportedStructConversions =
	{
		/* Struct Type */                                 /* Supported Struct Types */

		{ TBaseStructure<FColor>::Get(),      /* --> */   { TBaseStructure<FLinearColor>::Get()  } }
	};

	const FFieldClass* ControllerPropertyClass = ControllerAsProperty->GetClass();
	const FFieldClass* RemoteControlPropertyClass = RemoteControlProperty->GetClass();
	const bool bIsStructController = ControllerAsProperty->IsA(FStructProperty::StaticClass());
	const bool bIsObjectController = ControllerAsProperty->IsA(FObjectProperty::StaticClass());

	// Direct Binding (same type)
	if (ControllerPropertyClass == RemoteControlPropertyClass && !bIsStructController) // For Structs we need to check the inner component as well
	{
		return true;
	}
	// Indirect Binding (related types)
	else if (EvaluateBindCompatibility(SupportedIndirectBindsMap, ControllerPropertyClass, RemoteControlPropertyClass))
	{
		return true;
	}
	// Indirect Binding (Numeric to struct)
	else if (EvaluateBindCompatibility(SupportedNumericConversionsStructMap, ControllerPropertyClass, RemoteControlProperty))
	{
		return true;
	}
	// Indirect Binding (Numeric Conversion)
	else if (bInAllowNumericInputAsStrings)
	{
		if (EvaluateBindCompatibility(SupportedNumericConversionsMap, ControllerPropertyClass, RemoteControlPropertyClass))
		{
			return true;
		}
	}

	// Structs:
	if (bIsStructController)
	{
		if (const FStructProperty* StructProperty = CastField<FStructProperty>(ControllerAsProperty))
		{
			if (const FStructProperty* RCFieldStructProperty = CastField<FStructProperty>(RemoteControlProperty))
			{
				if (RCFieldStructProperty->Struct == StructProperty->Struct)
				{
					return true; // Binding via matching Struct
				}
				else
				{
					if (const TArray<UScriptStruct*>* SupportedConversions = SupportedStructConversions.Find(StructProperty->Struct))
					{
						if (SupportedConversions->Contains(RCFieldStructProperty->Struct))
						{
							return true; // Bind via explicit conversion
						}
					}
				}
			}
		}
	}

	// Objects
	if (bIsObjectController)
	{
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ControllerAsProperty))
		{
			if (const FObjectProperty* RCFieldObjectProperty = CastField<FObjectProperty>(RemoteControlProperty))
			{
				if (ObjectProperty->SameType(RCFieldObjectProperty))
				{
					return true;
				}
			}
			else if (const FArrayProperty* RCFieldArrayProperty = CastField<FArrayProperty>(RemoteControlProperty))
			{
				if (ObjectProperty->SameType(RCFieldArrayProperty->Inner))
				{
					return true;
				}
			}
		}
	}

	// at the end, let's check if this is a Custom controller (a default controller with additional metadata)
	if (EvaluateCustomControllerBindCompatibility(Controller, ControllerPropertyClass, RemoteControlPropertyClass, RemoteControlProperty))
	{
		return true;
	}

	return false; // Unsupported types!
}

bool URCBehaviourBind::GetPropertyBagTypeFromFieldProperty(const FProperty* InProperty, EPropertyBagPropertyType& OutPropertyBagType, UObject*& OutStructObject)
{
	const static TMap<EPropertyBagPropertyType, TArray<FFieldClass*>> PropertyBagTypesMap =
	{
		/* Property Bag Type */                           /* Input - Remote Control Property Type */
		{ EPropertyBagPropertyType::String,  /* --> */    { FStrProperty::StaticClass(),      FTextProperty::StaticClass(),     FNameProperty::StaticClass()}},
		{ EPropertyBagPropertyType::Int32,   /* --> */    { FIntProperty::StaticClass(),      FInt64Property::StaticClass(),    FInt16Property::StaticClass(),   FUInt32Property::StaticClass(),
														   FUInt64Property::StaticClass(),   FUInt16Property::StaticClass(),    FByteProperty::StaticClass() } },
		{ EPropertyBagPropertyType::Float,   /* --> */    { FFloatProperty::StaticClass(),    FDoubleProperty::StaticClass() } },
		{ EPropertyBagPropertyType::Bool,    /* --> */    { FBoolProperty::StaticClass() } },
		{ EPropertyBagPropertyType::Enum,    /* --> */    { FEnumProperty::StaticClass() } },
		{ EPropertyBagPropertyType::Struct,  /* --> */    { FStructProperty::StaticClass() }, },
		{ EPropertyBagPropertyType::Object,  /* --> */    { FObjectProperty::StaticClass() } /* this will probably never occur, so we manually set the out bag type later */ }
	};

	OutPropertyBagType = EPropertyBagPropertyType::None;

	TArray<EPropertyBagPropertyType> Keys;
	PropertyBagTypesMap.GenerateKeyArray(Keys);

	for (const EPropertyBagPropertyType PropertyBagType : Keys)
	{
		if (PropertyBagTypesMap.Find(PropertyBagType)->Contains(InProperty->GetClass()))
		{
			OutPropertyBagType = PropertyBagType;
		}
	}

	// Extract the inner Struct object if this is a Struct Property
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		OutStructObject = StructProperty->Struct;
	}
	// Extract the Object type if this is an Object Property
	else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty))
	{
		if (UClass* ObjectClass = ObjectProperty->PropertyClass)
		{
			OutStructObject = ObjectClass;
			OutPropertyBagType = EPropertyBagPropertyType::Object;
		}
	}
	// Handle an array property, and then treat the inner property in case it's an Object Property
	else if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(InProperty))
	{
		if (FProperty* InnerProperty = ArrayProperty->Inner)
		{
			if (const FObjectProperty* InnerObjectProperty = CastField<FObjectProperty>(InnerProperty))
			{
				if (UClass* InnerObjectClass = InnerObjectProperty->PropertyClass)
				{
					OutStructObject = InnerObjectClass;
					OutPropertyBagType = EPropertyBagPropertyType::Object;
				}
			}
		}
	}
	// Handle getting the enum from enum properties
	else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(InProperty))
	{
		OutStructObject = EnumProperty->GetEnum();
	}
	// Handle BP-based enums that are stored as byte propertties
	else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(InProperty))
	{
		if (ByteProperty->Enum)
		{
			OutPropertyBagType = EPropertyBagPropertyType::Enum;
			OutStructObject = ByteProperty->Enum;
		}
	}

	if (OutStructObject == UTexture::StaticClass())
	{
		OutPropertyBagType = EPropertyBagPropertyType::String;
	}
	// When creating from linear color properties, create FColor controller instead
	// todo: unify with supported controller structs in RC Config
	else if (OutStructObject == TBaseStructure<FLinearColor>::Get())
	{
		OutStructObject = TBaseStructure<FColor>::Get();
	}

	return OutPropertyBagType != EPropertyBagPropertyType::None;
}

bool URCBehaviourBind::CopyPropertyValueToController(URCController* InController, TSharedRef<const FRemoteControlProperty> InRemoteControlProperty)
{
	FProperty* ControllerAsProperty = InController->GetProperty();
	FProperty* RemoteControlProperty = InRemoteControlProperty->GetProperty();
	if (!ensure(ControllerAsProperty && RemoteControlProperty))
	{
		return false;
	}

	TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle = InRemoteControlProperty->GetPropertyHandle();
	if (!ensure(PropertyHandle))
	{
		return false;
	}

	// String Controller
	if (ControllerAsProperty->IsA(FStrProperty::StaticClass()))
	{
		FString StringValue;
		if (RemoteControlProperty->IsA(FStrProperty::StaticClass()) || RemoteControlProperty->IsA(FNameProperty::StaticClass()))
		{
			PropertyHandle->GetValue(StringValue);
		}
		else if (RemoteControlProperty->IsA(FTextProperty::StaticClass()))
		{
			FText TextValue;
			PropertyHandle->GetValue(TextValue);
			StringValue = TextValue.ToString();
		}

		InController->SetValueString(StringValue);

		return true;
	}
	// Int32 Controller
	else if (ControllerAsProperty->IsA(FIntProperty::StaticClass()))
	{
		int32 IntValue;
		PropertyHandle->GetValue(IntValue);

		InController->SetValueDouble(IntValue);

		return true;
	}
	// Float Controller
	else if (ControllerAsProperty->IsA(FFloatProperty::StaticClass()))
	{
		// check also the property handle type since double can be bound to float controllers
		if (RemoteControlProperty->IsA(FFloatProperty::StaticClass()))
		{
			float FloatValue;
			PropertyHandle->GetValue(FloatValue);
			InController->SetValueFloat(FloatValue);
		}
		else if (RemoteControlProperty->IsA(FDoubleProperty::StaticClass()))
		{
			double DoubleValue;
			PropertyHandle->GetValue(DoubleValue);
			InController->SetValueFloat(DoubleValue);
		}

		return true;
	}
	// Bool Controller
	else if (ControllerAsProperty->IsA(FBoolProperty::StaticClass()))
	{
		bool BoolValue;
		PropertyHandle->GetValue(BoolValue);

		InController->SetValueBool(BoolValue);

		return true;
	}
	else if (FStructProperty* ControllerAsStructProperty = CastField<FStructProperty>(ControllerAsProperty))
	{
		if (ControllerAsStructProperty->Struct == TBaseStructure<FVector>::Get())
		{
			FVector VectorValue;
			PropertyHandle->GetValue(VectorValue);
			InController->SetValueVector(VectorValue);
		}
		if (ControllerAsStructProperty->Struct == TBaseStructure<FVector2D>::Get())
		{
			FVector2D Vector2DValue;
			PropertyHandle->GetValue(Vector2DValue);
			InController->SetValueVector2D(Vector2DValue);
		}
		else if (ControllerAsStructProperty->Struct == TBaseStructure<FRotator>::Get())
		{
			FRotator RotatorValue;
			PropertyHandle->GetValue(RotatorValue);
			InController->SetValueRotator(RotatorValue);
		}
		else if (ControllerAsStructProperty->Struct == TBaseStructure<FColor>::Get())
		{
			FColor ColorValue;
			PropertyHandle->GetValue(ColorValue);
			InController->SetValueColor(ColorValue);
		}
	}

	return false;
}
