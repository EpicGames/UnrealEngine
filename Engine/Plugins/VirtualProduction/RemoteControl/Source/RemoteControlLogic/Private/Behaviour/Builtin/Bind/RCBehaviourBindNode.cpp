// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Bind/RCBehaviourBindNode.h"

#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "Controller/RCStatelessEventController.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"

#define LOCTEXT_NAMESPACE "RemoteControlBehaviours"

URCBehaviourBindNode::URCBehaviourBindNode()
{
	DisplayName = LOCTEXT("BehaviorNameBind", "Bind");
	BehaviorDescription = LOCTEXT("BehaviorDescBind", "Bind exposed properties to match the value of the controller. Properties added to the action list will not receive input values");
}

bool URCBehaviourBindNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	return true;
}

bool URCBehaviourBindNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	if (const URCController* RCController = InBehaviour->ControllerWeakPtr.Get())
	{
		return !FRCStatelessEventController::IsStatelessEventController(RCController);
	}

	return false;
}

UClass* URCBehaviourBindNode::GetBehaviourClass() const
{
	return URCBehaviourBind::StaticClass();
}

void URCBehaviourBindNode::OnPassed_Implementation(URCBehaviour* InBehaviour) const
{
}

#undef LOCTEXT_NAMESPACE