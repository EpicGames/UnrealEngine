// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"

#include "Action/RCAction.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditionalNode.h"
#include "StructUtils/PropertyBag.h"
#include "RCVirtualProperty.h"
#include "RemoteControlField.h"
#include "Controller/RCController.h"

URCBehaviourConditional::URCBehaviourConditional()
{
}

void URCBehaviourConditional::Initialize()
{
	Super::Initialize();
}

URCAction* URCBehaviourConditional::DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour)
{
	URCBehaviourConditional* InBehaviourConditional = Cast<URCBehaviourConditional>(InBehaviour);
	if (!ensureMsgf(InBehaviourConditional, TEXT("Expected Behaviour of the same type (Conditional) For CopyAction operation!")))
	{
		return nullptr;
	}

	URCAction* NewAction = Super::DuplicateAction(InAction, InBehaviour);
	if (ensure(NewAction))
	{
		// Copy Action specific data residing in Conditional Behaviour:
		if (FRCBehaviourCondition* ConditionData = this->Conditions.Find(InAction))
		{
			// Virtual Property for the Comparand
			Comparand = NewObject<URCVirtualPropertySelfContainer>(InBehaviourConditional);
			Comparand->DuplicatePropertyWithCopy(ConditionData->Comparand);

			InBehaviourConditional->OnActionAdded(NewAction, ConditionData->ConditionType, Comparand);
		}
	}

	return NewAction;
}

void URCBehaviourConditional::OnActionAdded(URCAction* Action, const ERCBehaviourConditionType InConditionType, const TObjectPtr<URCVirtualPropertySelfContainer> InComparand)
{
	FRCBehaviourCondition Condition(InConditionType, InComparand);

	Conditions.Add(Action, MoveTemp(Condition));
}

URCAction* URCBehaviourConditional::AddConditionalAction(const TSharedRef<const FRemoteControlField> InRemoteControlField, const ERCBehaviourConditionType InConditionType, const TObjectPtr<URCVirtualPropertySelfContainer> InComparand)
{
	TRCActionUniquenessTest UniquenessTest = [this, InRemoteControlField, InConditionType, InComparand](const TSet<TObjectPtr<URCAction>>& Actions)
	{
		const FGuid FieldId = InRemoteControlField->GetId();

		for (const URCAction* Action : Actions)
		{
			if (const FRCBehaviourCondition* Condition = this->Conditions.Find(Action))
			{
				const ERCBehaviourConditionType ActionCondition = Condition->ConditionType;

				TObjectPtr<URCVirtualPropertySelfContainer> ActionComparand = Condition->Comparand;

				if (Action->ExposedFieldId == FieldId && ActionCondition == InConditionType && ActionComparand->IsValueEqual(InComparand))
				{
					return false; // Not Unique!
				}
			}
		}

		return true;
	};

	return ActionContainer->AddAction(UniquenessTest, InRemoteControlField);
}

bool URCBehaviourConditional::CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const
{
	if (InRemoteControlField->FieldType == EExposedFieldType::Property)
	{
		if (const TSharedPtr<FRemoteControlProperty>& RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(InRemoteControlField))
		{
			if (const FProperty* Property = RCProperty->GetProperty())
			{
				const FPropertyBagPropertyDesc PropertyBagDesc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
				return RCProperty->IsEditable() && PropertyBagDesc.ValueType != EPropertyBagPropertyType::None;
			}
		}
	}

	return true;
}

FText URCBehaviourConditional::GetConditionTypeAsText(ERCBehaviourConditionType ConditionType) const
{
	FText ConditionDisplayText;

	switch (ConditionType)
	{
		case ERCBehaviourConditionType::IsEqual:
		{
			ConditionDisplayText = FText::FromName("=");
			break;
		}
		case ERCBehaviourConditionType::IsGreaterThan:
		{
			ConditionDisplayText = FText::FromName(">");
			break;
		}
		case ERCBehaviourConditionType::IsLesserThan:
		{
			ConditionDisplayText = FText::FromName("<");
			break;
		}
		case ERCBehaviourConditionType::IsGreaterThanOrEqualTo:
		{
			ConditionDisplayText = FText::FromName(">=");
			break;
		}
		case ERCBehaviourConditionType::IsLesserThanOrEqualTo:
		{
			ConditionDisplayText = FText::FromName("<=");
			break;
		}
		case ERCBehaviourConditionType::Else:
		{
			ConditionDisplayText = FText::FromName("Else");
			break;
		}
		default:
		{
			ensureMsgf(false, TEXT("Unknown condition type"));
		}
	}

	return ConditionDisplayText;
}

void URCBehaviourConditional::NotifyActionValueChanged(URCAction* InChangedAction)
{
	if (InChangedAction)
	{
		ExecuteSingleAction(InChangedAction);
	}
}

void URCBehaviourConditional::ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute)
{
	const URCBehaviourNode* BehaviourNode = GetBehaviourNode();
	check(BehaviourNode);

	if (BehaviourNode->GetClass() != URCBehaviourConditionalNode::StaticClass())
	{
		return; // Allow custom Blueprints to drive their own behaviour entirely
	}

	// Execute before the logic
	BehaviourNode->PreExecute(this);

	bool bConditionPass = false;

	/* Consider the following sequence of conditions for a hypothetical Controller "Tricode"
	*  For Tricode with current value "Tri2"
	* 
	*   =Tri1    <FALSE>  (Action 1)           ...skip...
	* 
	*   =Tri2   <TRUE>    (Action 2a)      ...execute...
	*   =Tri2   <TRUE>    (Action 2b)      ...execute...
	* 
	*   =Tri3   <FALSE>  (Action 3)          ...skip...
	* 
	*    Else                    (Action 4a)        ...skip...
	*    Else                    (Action 4b)        ...skip...
	*    Else                    (Action 4c)        ...skip...
	* 
	* For such multiple equality rows we want to know if at least one of them succeeded. 
	* The flag bHasEqualitySuccess is used for determining whether Else should be executed 
	*/
	bool bHasEqualitySuccess = false;
	bool bPreviousConditionPass = false;

	ERCBehaviourConditionType PreviousConditionType = ERCBehaviourConditionType::None;

	// execute all the action if the given action is null otherwise only execute the given action
	for (const TObjectPtr<URCAction>& Action : InActionsToExecute)
	{
		FRCBehaviourCondition* Condition = Conditions.Find(Action);
		if (!Condition)
		{
			ensureMsgf(false, TEXT("Unable to find condition for Action"));
			continue;
		}

		const ERCBehaviourConditionType ConditionType = Condition->ConditionType;

		if (ConditionType != PreviousConditionType) // New block
		{
			if (ConditionType != ERCBehaviourConditionType::Else)
			{
				bHasEqualitySuccess = false; // Reset flag for starting a new block of equality checks
			}
		}

		URCController* RCController = ControllerWeakPtr.Get();
		if (RCController)
		{
			switch (ConditionType)
			{
			case ERCBehaviourConditionType::IsEqual:
				bConditionPass = RCController->IsValueEqual(Condition->Comparand);
				bHasEqualitySuccess |= bConditionPass;
				break;			

			case ERCBehaviourConditionType::IsGreaterThan:
				bConditionPass = RCController->IsValueGreaterThan(Condition->Comparand);
				break;

			case ERCBehaviourConditionType::IsGreaterThanOrEqualTo:
				bConditionPass = RCController->IsValueGreaterThanOrEqualTo(Condition->Comparand);
				break;

			case ERCBehaviourConditionType::IsLesserThan:
				bConditionPass = RCController->IsValueLesserThan(Condition->Comparand);
				break;

			case ERCBehaviourConditionType::IsLesserThanOrEqualTo:
				bConditionPass = RCController->IsValueLesserThanOrEqualTo(Condition->Comparand);
				break;

			case ERCBehaviourConditionType::Else:
				// If the previous condition failed and no prior equality condition succeeded (among multiple =rows above) then execute Else!
				bConditionPass = !bPreviousConditionPass && !bHasEqualitySuccess;
				break;

			default:				
				ensureAlwaysMsgf(false, TEXT("Unimplemented comparator!"));
			}

			PreviousConditionType = ConditionType;

			if (ConditionType != ERCBehaviourConditionType::Else)
			{
				bPreviousConditionPass = bConditionPass; // Else should not contribute to the previous state flag to support multiple Else Actions (created via Add All Action, etc)
			}
		}

		if (bConditionPass)
		{
			Action->Execute();
			BehaviourNode->OnPassed(this);
		}
	}
}
