// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Conditional/RCBehaviourConditionalNode.h"

#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "Controller/RCStatelessEventController.h"
#include "RCVirtualProperty.h"

#define LOCTEXT_NAMESPACE "RemoteControlBehaviours"

URCBehaviourConditionalNode::URCBehaviourConditionalNode()
{
	DisplayName = LOCTEXT("BehaviorNameConditional", "Conditional");
	BehaviorDescription = LOCTEXT("BehaviorDescConditional", "If value matches a set condition, apply action list.");
}

bool URCBehaviourConditionalNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	if (GetClass() != URCBehaviourConditionalNode::GetClass())
	{
		return true; // This is a custom blueprint node
	}

	// Unlike other Behaviours, Conditional Behaviour needs to evaluate each Action individually
	// This logic is currently housed inside URCBehaviourConditional for convenience and flow should never come here
	ensureAlwaysMsgf(false, TEXT("Unexpected flow for Conditional Behaviour! Control should never come here."));

	return false;
}

bool URCBehaviourConditionalNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	// Conditional is supported in general for all types that are configured in BaseRemoteControl.ini
	// Additional filtering per condition type (eg: numeric comparators for numeric fields only) is performed at UI level based on the actively selected Comparator type
	
	if (const URCController* RCController = InBehaviour->ControllerWeakPtr.Get())
	{
		return !FRCStatelessEventController::IsStatelessEventController(RCController);
	}

	return false;
}

UClass* URCBehaviourConditionalNode::GetBehaviourClass() const
{
	return URCBehaviourConditional::StaticClass();
}

void URCBehaviourConditionalNode::OnPassed_Implementation(URCBehaviour* InBehaviour) const
{
}

#undef LOCTEXT_NAMESPACE