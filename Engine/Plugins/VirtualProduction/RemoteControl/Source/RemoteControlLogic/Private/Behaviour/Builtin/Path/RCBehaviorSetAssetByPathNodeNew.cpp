// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Path/RCBehaviorSetAssetByPathNodeNew.h"

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"

#define LOCTEXT_NAMESPACE "RCBehaviorSetAssetByPathNodeNew"

URCBehaviorSetAssetByPathNodeNew::URCBehaviorSetAssetByPathNodeNew()
{
	DisplayName = LOCTEXT("Behaviour Name - Set Asset By Path", "Path");
	BehaviorDescription = LOCTEXT("Behaviour Desc - Set Asset By Path", "Triggers an event which sets objects by building a path to an asset.");
}

bool URCBehaviorSetAssetByPathNodeNew::Execute_Implementation(URCBehaviour* InBehavior) const
{
	return true;
}

bool URCBehaviorSetAssetByPathNodeNew::IsSupported_Implementation(URCBehaviour* InBehavior) const
{
	static const TArray<TPair<EPropertyBagPropertyType, UObject*>> SupportedPropertyBagTypes =
	{
		{EPropertyBagPropertyType::String, nullptr}
	};
	
	return SupportedPropertyBagTypes.ContainsByPredicate(GetIsSupportedCallback(InBehavior));
}

UClass* URCBehaviorSetAssetByPathNodeNew::GetBehaviourClass() const
{
	return URCSetAssetByPathBehaviorNew::StaticClass();
}

#undef LOCTEXT_NAMESPACE
