// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Path/RCBehaviourSetAssetByPathNode.h"

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"

URCBehaviourSetAssetByPathNode::URCBehaviourSetAssetByPathNode()
{
	DisplayName = NSLOCTEXT("Remote Control Behaviour", "Behaviour Name - Set Asset By Path", "Path");
	BehaviorDescription = NSLOCTEXT("Remote Control Behaviour", "Behaviour Desc - Set Asset By Path", "Triggers an event which sets an object based on the selected Exposed Entity.");
}

bool URCBehaviourSetAssetByPathNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	if (!ensure(InBehaviour))
	{
		return false;
	}

	URCController* RCController = InBehaviour->ControllerWeakPtr.Get();
	if (!RCController)
	{
		return false;
	}

	URCSetAssetByPathBehaviour* SetAssetByPathBehaviour = Cast<URCSetAssetByPathBehaviour>(InBehaviour);
	if (!SetAssetByPathBehaviour)
	{
		return false;
	}

	FString DefaultString;
	SetAssetByPathBehaviour->PropertyInContainer->GetVirtualProperty(SetAssetByPathBehaviourHelpers::DefaultInput)->GetValueString(DefaultString);

	const FString ConcatPath = SetAssetByPathBehaviour->GetCurrentPath();
	
	return SetAssetByPathBehaviour->SetAssetByPath(ConcatPath, DefaultString);
}

bool URCBehaviourSetAssetByPathNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	static TArray<TPair<EPropertyBagPropertyType, UObject*>> SupportedPropertyBagTypes =
	{
		{ EPropertyBagPropertyType::String, nullptr }
	};
	
	return SupportedPropertyBagTypes.ContainsByPredicate(GetIsSupportedCallback(InBehaviour));
}

UClass* URCBehaviourSetAssetByPathNode::GetBehaviourClass() const
{
	return URCSetAssetByPathBehaviour::StaticClass();
}
