// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"

#include "Action/Path/RCSetAssetByPathActionNew.h"
#include "Controller/RCController.h"
#include "IRemoteControlModule.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "RCVirtualProperty.h"

#if WITH_EDITOR
#include "PropertyPath.h"
#endif

URCSetAssetByPathActionNew* URCSetAssetByPathBehaviorNew::AddPropertyBindAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty)
{
	if (!ensure(ControllerWeakPtr.IsValid() && ActionContainer))
	{
		return nullptr;
	}

	URCSetAssetByPathActionNew* BindAction = NewObject<URCSetAssetByPathActionNew>(ActionContainer);
	BindAction->PresetWeakPtr = ActionContainer->PresetWeakPtr;
	BindAction->ExposedFieldId = InRemoteControlProperty->GetId();
	BindAction->Id = FGuid::NewGuid();

	// Add action to array
	ActionContainer->AddAction(BindAction);

	return BindAction;
}

void URCSetAssetByPathBehaviorNew::Initialize()
{
	// Path already creates itself an empty entry. Let's add one bound to the owning controller. At least in name.

	if (URCController* Controller = Cast<URCController>(GetOuter()))
	{
		FRCAssetPathElementNew& ControllerPathElement = PathStruct.AssetPath.AddDefaulted_GetRef();
		ControllerPathElement.bIsInput = true;
		ControllerPathElement.Path = Controller->DisplayName.ToString();
	}
}

URCAction* URCSetAssetByPathBehaviorNew::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	const TSharedRef<const FRemoteControlProperty> RemoteControlProperty = StaticCastSharedRef<const FRemoteControlProperty>(InRemoteControlField);

	URCAction* SetAssetByPathAction = AddPropertyBindAction(RemoteControlProperty);

	if (ensure(SetAssetByPathAction))
	{
		SetAssetByPathAction->Execute();
	}

	return SetAssetByPathAction;
}

#if WITH_EDITOR
bool URCSetAssetByPathBehaviorNew::CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const
{
	if (!InRemoteControlField.IsValid())
	{
		return false;
	}

	if (!Super::CanHaveActionForField(InRemoteControlField))
	{
		return false;
	}

	if (!InRemoteControlField->FieldPathInfo.IsResolved())
	{
		UObject* BoundObject = InRemoteControlField->GetBoundObject();

		if (!BoundObject)
		{
			return false;
		}

		if (!InRemoteControlField->FieldPathInfo.Resolve(BoundObject))
		{
			return false;
		}		
	}

	TSharedPtr<FPropertyPath> PropertyPath = InRemoteControlField->FieldPathInfo.ToPropertyPath();

	if (!PropertyPath.IsValid() || PropertyPath->GetNumProperties() <= 0)
	{
		return false;
	}

	FProperty* Property = PropertyPath->GetPropertyInfo(PropertyPath->GetNumProperties() - 1).Property.Get();

	if (!Property)
	{
		return false;
	}

	if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
	{
		if (ArrayProperty->Inner && ArrayProperty->IsA<FObjectPropertyBase>())
		{
			return true;
		}
	}
	else if (Property->IsA<FObjectPropertyBase>())
	{
		return true;
	}

	return false;
}
#endif

FString URCSetAssetByPathBehaviorNew::GetCurrentPath() const
{
	using namespace UE::RemoteControl::Logic;

	FString CurrentPath;
	URCController* RCController = ControllerWeakPtr.Get();
	if (!RCController)
	{
		ensureMsgf(false, TEXT("Controller is invalid."));
		return TEXT("Path is invalid!");
	}

	CurrentPath.Empty();
	FString ControllerString;
	RCController->GetValueString(ControllerString);

	// Add Path String Concat
	CurrentPath = bInternal ? SetAssetByPathNew::ContentFolder : FString("");
	
	for (FRCAssetPathElementNew PathPart : PathStruct.AssetPath)
	{
		if (PathPart.bIsInput)
		{
			if (PathPart.Path.IsEmpty())
			{
				URCController* Controller = ControllerWeakPtr.Get();
				if (!Controller)
				{
					ensureMsgf(false, TEXT("Cannot find parent controller"));
					continue;
				}

				FString ControllerValue;
				if (!Controller->GetValueString(ControllerValue))
				{
					ensureMsgf(false, TEXT("Unable to get controller value as string"));
					continue;
				}

				PathPart.Path = ControllerValue;
			}
			else
			{
				const URemoteControlPreset* RemoteControlPreset = RCController->PresetWeakPtr.Get();
				if (!RemoteControlPreset)
				{
					ensureMsgf(false, TEXT("Remote Control Preset is invalid"));
					continue;
				}

				const URCVirtualPropertyBase* TokenController = RemoteControlPreset->GetControllerByDisplayName(FName(PathPart.Path));
				if (!TokenController)
				{
					PathPart.Path = TEXT("InvalidControllerName");
				}
				else
				{
					PathPart.Path = TokenController->GetDisplayValueAsString();
				}
			}
		}
		else if (PathPart.Path.IsEmpty())
		{
			continue;
		}

		PathPart.Path.ReplaceCharInline(TCHAR('\\'), TCHAR('/'), ESearchCase::IgnoreCase);
		
		CurrentPath += PathPart.Path;
	}

	return CurrentPath.Replace(TEXT("//"), TEXT("/"));
}

TArray<FRCSetAssetByPathBehaviorNewPathElement> URCSetAssetByPathBehaviorNew::GetPathElements() const
{
	TArray<FRCSetAssetByPathBehaviorNewPathElement> PathElements;

	FString CurrentPath;
	URCController* OwningController = ControllerWeakPtr.Get();
	if (!OwningController)
	{
		UE_LOGF(LogRemoteControl, Error, "Owning controller is invalid.");
		return PathElements;
	}

	FString ControllerString;
	OwningController->GetValueString(ControllerString);

	using namespace UE::RemoteControl::Logic;

	// Add Path String Concat
	CurrentPath = bInternal ? SetAssetByPathNew::ContentFolder : FString("");

	for (const FRCAssetPathElementNew& PathPart : PathStruct.AssetPath)
	{
		FRCSetAssetByPathBehaviorNewPathElement PathElement;

		if (PathPart.bIsInput)
		{
			PathElement.Controller = nullptr;
			PathElement.ControllerName = *PathPart.Path;

			if (PathPart.Path.IsEmpty())
			{
				PathElement.Controller = OwningController;
			}
			else if (const URemoteControlPreset* RemoteControlPreset = OwningController->PresetWeakPtr.Get())
			{
				PathElement.Controller = RemoteControlPreset->GetControllerByDisplayName(*PathPart.Path);

				if (!PathElement.Controller)
				{
					UE_LOGF(LogRemoteControl, Error, "Unable to find controller. [%ls]", *PathPart.Path);
					PathElement.Path = TEXT("InvalidControllerName");
					PathElement.bIsError = true;
				}
			}
			else
			{
				UE_LOGF(LogRemoteControl, Error, "Unable to find remote control preset.");
				PathElement.Path = TEXT("InvalidPreset");
				PathElement.bIsError = true;
			}

			// All control paths without a controller already have a path and error set.
			if (PathElement.Controller)
			{
				FString ControllerValue;

				if (PathElement.Controller->GetValueString(ControllerValue))
				{
					PathElement.Path = ControllerValue;
				}
				else
				{
					if (PathPart.Path.IsEmpty())
					{
						UE_LOGF(LogRemoteControl, Error, "Failed to get owning controller value as string.");
					}
					else
					{
						UE_LOGF(LogRemoteControl, Error, "Failed to get controller value as string. [%ls]", *PathPart.Path);
					}

					PathElement.Path = TEXT("UnableToGetControllerValue");
					PathElement.bIsError = true;
				}
			}
		}
		else
		{
			PathElement.Path = PathPart.Path;
			PathElement.Controller = nullptr;
		}

		PathElement.Path.ReplaceCharInline(TCHAR('\\'), TCHAR('/'), ESearchCase::IgnoreCase);
		PathElement.Path.ReplaceInline(TEXT("//"), TEXT("/"), ESearchCase::IgnoreCase);
		
		CurrentPath += PathElement.Path;

		PathElements.Add(PathElement);
	}

	return PathElements;
}

void URCSetAssetByPathBehaviorNew::RefreshPathArray()
{
	if (PathStruct.AssetPath.Num() < 1)
	{
		PathStruct.AssetPath.AddDefaulted();
	}
}
