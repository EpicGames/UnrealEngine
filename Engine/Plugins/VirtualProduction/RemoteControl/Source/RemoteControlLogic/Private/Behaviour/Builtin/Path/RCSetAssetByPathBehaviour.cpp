// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"

#include "Backends/CborStructDeserializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Components/MeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Controller/RCController.h"
#include "Engine/Blueprint.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DDynamic.h"
#include "Engine/World.h"
#include "HttpManager.h"
#include "HttpModule.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlPropertyHandle.h"
#include "Interfaces/IHttpResponse.h"
#include "Kismet/KismetRenderingLibrary.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstance.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlLogger.h"
#include "RemoteControlPreset.h"

URCSetAssetByPathBehaviour::URCSetAssetByPathBehaviour()
{
	PropertyInContainer = CreateDefaultSubobject<URCVirtualPropertyContainerBase>(FName("VirtualPropertyInContainer"));
}

void URCSetAssetByPathBehaviour::Initialize()
{
	PropertyInContainer->AddProperty(SetAssetByPathBehaviourHelpers::DefaultInput, URCController::StaticClass(), EPropertyBagPropertyType::String);
	UpdateTargetEntity();
	
	Super::Initialize();
}

void URCSetAssetByPathBehaviour::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	if (const FGuid* FoundId = InEntityIdMap.Find(TargetEntityId))
	{
		TargetEntityId = *FoundId;
	}

	Super::UpdateEntityIds(InEntityIdMap);
}

void URCSetAssetByPathBehaviour::PostLoad()
{
	Super::PostLoad();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// support for older version, loading the data of the deprecated property into the new one
	if (!PathStruct.PathArray_DEPRECATED.IsEmpty())
	{
		PathStruct.AssetPath.Empty();
	}
	for (const FString& Path : PathStruct.PathArray_DEPRECATED)
	{
		bool bIsInput = false;
		FString PathToCopy = Path;
		if (PathToCopy.Contains(SetAssetByPathBehaviourHelpers::InputToken)
			&& PathToCopy.Find(SetAssetByPathBehaviourHelpers::InputToken) == 0)
		{
			bIsInput = true;
			PathToCopy.RemoveFromStart(SetAssetByPathBehaviourHelpers::InputToken);
			int32 CharIndexStart;
			int32 CharIndexEnd;
			PathToCopy.FindChar('(', CharIndexStart);
			PathToCopy.FindChar(')', CharIndexEnd);
			if (CharIndexStart != INDEX_NONE)
			{
				PathToCopy.RemoveAt(CharIndexStart);
			}
			if (CharIndexEnd != INDEX_NONE)
			{
				PathToCopy.RemoveAt(CharIndexEnd-1);
			}
		}
		PathStruct.AssetPath.Add(FRCAssetPathElement(bIsInput, PathToCopy));
	}
	PathStruct.PathArray_DEPRECATED.Empty();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

bool URCSetAssetByPathBehaviour::SetAssetByPath(const FString& AssetPath, const FString& DefaultString)
{
	const URCController* Controller = ControllerWeakPtr.Get();
	
	if (AssetPath.IsEmpty() || !Controller)
	{
		return false;
	}
	
	FString ControllerString;
	if (!Controller->GetValueString(ControllerString))
	{
		return false;
	}
	
	URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get();
	if (!Preset)
	{
		return false;
	}
	
	const TSharedPtr<FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(TargetEntityId).Pin();
	if (!RemoteControlProperty)
	{
		return false;
	}

	FProperty* Property = RemoteControlProperty->GetProperty();
	if (!Property)
	{
		return false;
	}

	if (!bInternal)
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [AssetPath, ControllerString]
		{
			return FText::FromString(FString("Path Behaviour attempts to set Asset to %s") + *AssetPath + *ControllerString);
		});

#if WITH_EDITOR
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);
		RemoteControlProperty->GetBoundObject()->PreEditChange(PropertyChain);
#endif
		if (SetExternalAsset(AssetPath + ControllerString))
		{
#if WITH_EDITOR
			FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
			RemoteControlProperty->GetBoundObject()->PostEditChangeProperty(ChangedEvent);
#endif
			
			return true;
		}

		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [AssetPath, DefaultString]
		{
			return FText::FromString(FString("Path Behaviour attempts to set Asset to %s") + *AssetPath + *DefaultString);
		});
		if (!DefaultString.IsEmpty() && SetExternalAsset(AssetPath + DefaultString))
		{
			return true;
		}

		return false;
	}

	FSoftObjectPath MainObjectRef(AssetPath + ControllerString);
	
	if (UObject* Object = MainObjectRef.TryLoad())
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [MainObjectRef]
		{
			return FText::FromString(FString("Path Behaviour attempts to set Asset to %s") + *MainObjectRef.GetAssetPathString());
		});

#if WITH_EDITOR
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);
		RemoteControlProperty->GetBoundObject()->PreEditChange(PropertyChain);
#endif
		if (SetInternalAsset(Object))
		{
#if WITH_EDITOR
			FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
			RemoteControlProperty->GetBoundObject()->PostEditChangeProperty(ChangedEvent);
#endif

			return true;
		}

		return false;
	}
	
	if (!DefaultString.IsEmpty())
	{
		FSoftObjectPath DefaultObjectRef(AssetPath + DefaultString);
		
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [DefaultObjectRef]
		{
			return FText::FromString(FString("Path Behaviour attempts to set Asset to %s") + *DefaultObjectRef.GetAssetPathString());
		});

#if WITH_EDITOR
		FEditPropertyChain PropertyChain;
		PropertyChain.AddHead(Property);
		RemoteControlProperty->GetBoundObject()->PreEditChange(PropertyChain);
#endif
		if (SetInternalAsset(DefaultObjectRef.TryLoad()))
		{
#if WITH_EDITOR
			FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
			RemoteControlProperty->GetBoundObject()->PostEditChangeProperty(ChangedEvent);
#endif
			
			return true;
		}
	}
	
	return false;
}

bool URCSetAssetByPathBehaviour::SetInternalAsset(UObject* SetterObject)
{
	const URCController* Controller = ControllerWeakPtr.Get();

	if (!TargetEntity || TargetEntity->GetId() != TargetEntityId)
	{
		UpdateTargetEntity();
	}

	if (!SetterObject || !Controller || !TargetEntity)
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [SetterObject]
		{
			return FText::FromString(FString("Path Behaviour fails to set Asset: %s") + *SetterObject->GetName());
		}, EVerbosityLevel::Error);
		return false;
	}
	
	const TWeakObjectPtr<URemoteControlPreset> PresetPtr = Controller->PresetWeakPtr;
	
	TArray<TWeakPtr<FRemoteControlProperty>> ExposedProperties = PresetPtr->GetExposedEntities<FRemoteControlProperty>();

	if (const TWeakPtr<FRemoteControlProperty>* ExposedPropertyPtr = ExposedProperties.FindByPredicate([this](const TWeakPtr<FRemoteControlProperty>& Property){ return Property.Pin()->GetLabel() == TargetEntity->GetLabel(); }))
	{
		const TSharedPtr<FRemoteControlProperty> ExposedProperty = (*ExposedPropertyPtr).Pin();
		if (!ExposedProperty)
		{
			ensureMsgf(false, TEXT("Exposed Property is not valid."));
			return false;
		}
		
		if (AssetClass == UStaticMesh::StaticClass() && SetterObject->IsA(UStaticMesh::StaticClass()))
		{
			UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ExposedProperty->GetBoundObject());
			StaticMeshComponent->SetStaticMesh(Cast<UStaticMesh>(SetterObject));
			
			UE_LOGF(LogRemoteControl, Verbose, "Path Behaviour sets Static Mesh Asset %ls", *SetterObject->GetName());
			return true;
		}
		
		if (AssetClass == UMaterial::StaticClass() && SetterObject->IsA(UMaterial::StaticClass()))
		{
			if (UMeshComponent* MaterialOwnerComponent = Cast<UMeshComponent>(ExposedProperty->GetBoundObject()))
			{
				MaterialOwnerComponent->SetMaterial(0, Cast<UMaterial>(SetterObject));
			}
			
			UE_LOGF(LogRemoteControl, Verbose, "Path Behaviour sets Material Asset %ls", *SetterObject->GetName());
			return true;
		}

		// Only Texture runs through here
		if (GetSupportedClasses().Contains(AssetClass) && SetterObject->IsA(UTexture::StaticClass()))
		{
			if (const TSharedPtr<FRemoteControlProperty> RemoteControlProperty = ExposedProperty)
			{
				SetTextureAsset(RemoteControlProperty, Cast<UTexture>(SetterObject));
			}
		}
		
		return false;
	}
	
	if (TargetEntity)
	{
		const TWeakPtr<const FRemoteControlEntity> ExposedEntity = TargetEntity;
		if (AssetClass == UBlueprint::StaticClass() && SetterObject->IsA(UBlueprint::StaticClass()))
		{
			AActor* OldActor = Cast<AActor>(ExposedEntity.Pin()->GetBoundObject());
			UWorld* World = OldActor->GetWorld();
			
			OldActor->UnregisterAllComponents();
			
			const FVector OldLocation = OldActor->GetActorLocation();
			const FRotator OldRotation = OldActor->GetActorRotation();
			const FName OldActorName = OldActor->GetFName();
			
			FActorSpawnParameters SpawnParams;
			SpawnParams.Name = OldActorName;
			
			const FName OldActorReplacedNamed = MakeUniqueObjectName(OldActor->GetOuter(), OldActor->GetClass(), *FString::Printf(TEXT("%s_REPLACED"), *OldActor->GetFName().ToString()));
			OldActor->Rename(*OldActorReplacedNamed.ToString(), OldActor->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);

			if (AActor* NewActor = World->SpawnActor(Cast<UBlueprint>(SetterObject)->GeneratedClass, &OldLocation, &OldRotation, SpawnParams))
			{
				World->DestroyActor(OldActor);
				
				UE_LOGF(LogRemoteControl, Verbose, "Path Behaviour sets Blueprint Asset %ls", *SetterObject->GetName());
				return true;
			}

			UE_LOGF(LogRemoteControl, Error, "Path Behaviour unable to delete old Actor %ls", *OldActor->GetName());
			OldActor->UObject::Rename(*OldActorName.ToString(), OldActor->GetOuter(), REN_DoNotDirty | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			OldActor->RegisterAllComponents();
		}
	}
	
	UE_LOGF(LogRemoteControl, Error, "Path Behaviour fails to set Asset %ls", *SetterObject->GetName());
	return false;
}

TArray<UClass*> URCSetAssetByPathBehaviour::GetSupportedClasses() const
{
	static TArray<UClass*> SupportedClasses =
	{
		UStaticMesh::StaticClass(),
		UMaterial::StaticClass(),
		UTexture::StaticClass(),
		UBlueprint::StaticClass(),
	};

	return SupportedClasses;
}

FString URCSetAssetByPathBehaviour::GetCurrentPath()
{
	FString CurrentPath;
	URCController* RCController = ControllerWeakPtr.Get();
	if (!RCController)
	{
		ensureMsgf(false, TEXT("Controller is invalid."));
		return TEXT("Path is invalid!");
	}

	CurrentPath.Empty();
	FString ControllerString;
	RCController->GetValueString(ControllerString);

	// Add Path String Concat
	CurrentPath = bInternal ? SetAssetByPathBehaviourHelpers::ContentFolder : FString("");
	for (FRCAssetPathElement PathPart : PathStruct.AssetPath)
	{
		if (PathPart.Path.IsEmpty())
		{
			continue;
		}

		if (PathPart.bIsInput)
		{
			const URemoteControlPreset* RemoteControlPreset = RCController->PresetWeakPtr.Get();
			if (!RemoteControlPreset)
			{
				ensureMsgf(false, TEXT("Remote Control Preset is invalid"));
				continue;
			}

			const URCVirtualPropertyBase* TokenController = RemoteControlPreset->GetControllerByDisplayName(FName(PathPart.Path));
			if (!TokenController)
			{
				PathPart.Path = TEXT("InvalidControllerName");
			}
			else
			{
				PathPart.Path = TokenController->GetDisplayValueAsString();
			}
		}

		// Check for certain chars.
		int32 IndexFound;
		if (PathPart.Path.FindLastChar('_', IndexFound) && IndexFound == PathPart.Path.Len() - 1)
		{
			// In the case there's underscore char in the at the end of one of the Path Strings, do nothing to facilitate more complex pathing behaviours.
		}
		else if (!PathPart.Path.FindLastChar('/', IndexFound) || IndexFound < PathPart.Path.Len() - 1)
		{
			PathPart.Path = PathPart.Path.AppendChar('/');
		}

		PathPart.Path.ReplaceCharInline(TCHAR('\\'), TCHAR('/'), ESearchCase::IgnoreCase);
		
		CurrentPath += PathPart.Path;
	}

	return CurrentPath;
}

void URCSetAssetByPathBehaviour::SetTargetEntity(const TSharedPtr<const FRemoteControlEntity>& InEntity)
{
	TargetEntityId = InEntity->GetId();
	TargetEntity = InEntity;
}

TWeakPtr<const FRemoteControlEntity> URCSetAssetByPathBehaviour::GetTargetEntity() const
{
	return TargetEntity;
}

const FGuid& URCSetAssetByPathBehaviour::GetTargetEntityId() const
{
	return TargetEntityId;
}

bool URCSetAssetByPathBehaviour::SetExternalAsset(FString InExternalPath)
{
	URCController* Controller = ControllerWeakPtr.Get();

	if (!TargetEntity || TargetEntity->GetId() != TargetEntityId)
	{
		UpdateTargetEntity();
	}

	if (!Controller || !TargetEntity)
	{
		return false;
	}
	
	URemoteControlPreset* PresetPtr = Controller->PresetWeakPtr.Get();
	if (!PresetPtr)
	{
		return false;
	}
	
	TArray<TWeakPtr<FRemoteControlProperty>> ExposedProperties = PresetPtr->GetExposedEntities<FRemoteControlProperty>();
	const TWeakPtr<FRemoteControlProperty>* ExposedPropertyPtr = ExposedProperties.FindByPredicate([this](const TWeakPtr<FRemoteControlProperty>& Property)
	{
		if (const TSharedPtr<FRemoteControlProperty> PinnedProperty = Property.Pin())
		{
			return PinnedProperty->GetLabel() == TargetEntity->GetLabel();
		}
		
		return false;
	});

	const TSharedPtr<FRemoteControlProperty> RemoteControlProperty = ExposedPropertyPtr->Pin();
	if (!RemoteControlProperty)
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, []
		{
			return FText::FromString(TEXT("Path Behaviour Exposed Property not found"));
		});
		return false;
	}

	if (!SetTextureFromPath(RemoteControlProperty, InExternalPath))
	{
		FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, [InExternalPath]
		{
			return FText::FromString(FString("Path Behaviour Set external Asset: ") + *InExternalPath);
		});
		return true;
	}

	FRemoteControlLogger::Get().Log(SetAssetByPathBehaviourHelpers::SetAssetByPathBehaviour, []
	{
		return FText::FromString(TEXT("Path Behaviour Set external Asset failed"));
	});
	return false;
}

bool URCSetAssetByPathBehaviour::SetTextureAsset(TSharedPtr<FRemoteControlProperty> InRemoteControlPropertyPtr, UTexture* InObject)
{
	if (!InObject)
	{
		return false;
	}
	
	FRCObjectReference ObjectRef;
	ObjectRef.Property = InRemoteControlPropertyPtr->GetProperty();
	ObjectRef.Access = InRemoteControlPropertyPtr->GetPropertyHandle()->ShouldGenerateTransaction() ? ERCAccess::WRITE_TRANSACTION_ACCESS : ERCAccess::WRITE_ACCESS;
	ObjectRef.PropertyPathInfo = InRemoteControlPropertyPtr->FieldPathInfo.ToString();

	for (UObject* Object : InRemoteControlPropertyPtr->GetBoundObjects())
	{
		if (IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, ObjectRef.PropertyPathInfo, ObjectRef))
		{
			/** Setting with Pointer and using Serialization of itself afterwards as a workaround with the Texture Asset not updating in the world. */
			FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(InRemoteControlPropertyPtr->GetProperty());
			if (!ObjectProperty)
			{
				return false;
			}

			FProperty* Property = InRemoteControlPropertyPtr->GetProperty();
			uint8* ValueAddress = Property->ContainerPtrToValuePtr<uint8>(ObjectRef.ContainerAdress);
			ObjectProperty->SetObjectPropertyValue(ValueAddress, InObject);

			TArray<uint8> Buffer;
			FMemoryReader Reader(Buffer);
			FCborStructDeserializerBackend DeserializerBackend(Reader);

			IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Cbor, Buffer);
			return true;
		}
	}
	return false;	
}

void URCSetAssetByPathBehaviour::UpdateTargetEntity()
{
	const URCController* Controller = ControllerWeakPtr.Get();
	if (!Controller)
	{
		return;
	}
	
	const URemoteControlPreset* PresetPtr = Controller->PresetWeakPtr.Get();
	if (!PresetPtr)
	{
		return;
	}

	TWeakPtr<const FRemoteControlEntity> ExposedEntity = PresetPtr->GetExposedEntity<FRemoteControlEntity>(TargetEntityId);
	if (!ExposedEntity.Pin())
	{
		return;
	}
	
	TargetEntity = ExposedEntity.Pin();
}

void URCSetAssetByPathBehaviour::RefreshPathArray()
{
	if (PathStruct.AssetPath.Num() < 1)
	{
		PathStruct.AssetPath.AddDefaulted();
	}
}

bool URCSetAssetByPathBehaviour::SetTextureFromPath(TSharedPtr<FRemoteControlProperty> RCPropertyToSet, FString& FileName)
{
	// Local and Network Drives
	if (FPaths::FileExists(FileName))
	{
		if (UTexture2D* LocalTexturePtr = UKismetRenderingLibrary::ImportFileAsTexture2D(this, FileName))
		{
			SetTextureAsset(RCPropertyToSet, LocalTexturePtr);
			return true;
		}
	}
	
	// Http Link
	TSharedRef<class IHttpRequest> HttpRequest = FHttpModule::Get().CreateRequest();
	HttpRequest->OnProcessRequestComplete().BindUObject(this, &URCSetAssetByPathBehaviour::ReadFileHttpHandler, RCPropertyToSet);
	HttpRequest->SetURL(FileName);
	HttpRequest->SetVerb(TEXT("GET"));
	return HttpRequest->ProcessRequest();
}

void URCSetAssetByPathBehaviour::ReadFileHttpHandler(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, TSharedPtr<FRemoteControlProperty> InRCPropertyToSet)
{
	if (!HttpRequest.IsValid())
	{
		return;
	}

	if (bSucceeded && HttpResponse.IsValid())
	{
		UE_LOGF(LogTemp, Verbose, "ReadFile request complete, Url=%ls Code=%d", *HttpRequest->GetURL(), HttpResponse->GetResponseCode());

		const TArray<uint8> ResponseBuffer = HttpResponse->GetContent();
		if (UTexture2D* HttpTexture = UKismetRenderingLibrary::ImportBufferAsTexture2D(this, ResponseBuffer))
		{
			SetTextureAsset(InRCPropertyToSet, HttpTexture);
		}
	}
}


