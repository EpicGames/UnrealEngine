// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/RCBehaviourOnValueChangedNode.h"

#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "StructUtils/PropertyBag.h"

URCBehaviourOnValueChangedNode::URCBehaviourOnValueChangedNode()
{
	DisplayName = NSLOCTEXT("Remote Control Behaviour", "Behavior Name - On Value Changed", "On Modify");
	BehaviorDescription = NSLOCTEXT("Remote Control Behaviour", "Behavior Desc - On Value Changed", "Triggers an event when the associated property is modified");
}


bool URCBehaviourOnValueChangedNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	return true;
}

bool URCBehaviourOnValueChangedNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	static TArray<EPropertyBagPropertyType> SupportedPropertyBagTypes =
	{
		EPropertyBagPropertyType::Bool,
		EPropertyBagPropertyType::Byte,
		EPropertyBagPropertyType::Int32,
		EPropertyBagPropertyType::Float,
		EPropertyBagPropertyType::Double,
		EPropertyBagPropertyType::Name,
		EPropertyBagPropertyType::String,
		EPropertyBagPropertyType::Text,
		EPropertyBagPropertyType::Struct,
		EPropertyBagPropertyType::Enum
	};
	
	if (const URCController* RCController = InBehaviour->ControllerWeakPtr.Get())
	{
		return SupportedPropertyBagTypes.Contains(RCController->GetValueType());
	}
	
	return false;
}
