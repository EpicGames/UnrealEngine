// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/RangeMap/RCBehaviourRangeMapNode.h"

#include "Behaviour/Builtin/RangeMap/RCRangeMapBehaviour.h"
#include "Controller/RCController.h"
#include "RCVirtualProperty.h"

URCBehaviourRangeMapNode::URCBehaviourRangeMapNode()
{
	DisplayName = NSLOCTEXT("Remote Control Behaviour", "Behavior Name - Range Mapping", "Range");
	BehaviorDescription = NSLOCTEXT("Remote Control Behaviour", "Behavior Desc - Range Mapping", "This Behaviours can take a list of Actions, sort them using normalized values between 0..1, and interpolates numerical values whilst executing non-numerical whose distance are less than 0.05 from a given action.");
}

bool URCBehaviourRangeMapNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	if (!ensure(InBehaviour))
	{
		return false;
	}
	
	URCRangeMapBehaviour* RangeMapBehaviour = Cast<URCRangeMapBehaviour>(InBehaviour);
	if (!RangeMapBehaviour)
	{
		return false;
	}

	RangeMapBehaviour->Refresh();
	return true;
}

bool URCBehaviourRangeMapNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	static TArray<TPair<EPropertyBagPropertyType, UObject*>> SupportedPropertyBagTypes =
	{
		{ EPropertyBagPropertyType::Float, nullptr},
	};
	
	return SupportedPropertyBagTypes.ContainsByPredicate(GetIsSupportedCallback(InBehaviour));
}

UClass* URCBehaviourRangeMapNode::GetBehaviourClass() const
{
	return URCRangeMapBehaviour::StaticClass();
}

void URCBehaviourRangeMapNode::OnPassed_Implementation(URCBehaviour* InBehaviour) const
{
}
