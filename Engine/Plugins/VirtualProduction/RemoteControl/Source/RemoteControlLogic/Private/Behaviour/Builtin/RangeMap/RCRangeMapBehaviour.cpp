// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/Builtin/RangeMap/RCRangeMapBehaviour.h"

#include "IRemoteControlModule.h"
#include "Action/RCAction.h"
#include "Action/RCActionContainer.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/Builtin/RangeMap/RCBehaviourRangeMapNode.h"
#include "Behaviour/RCBehaviourNode.h"
#include "Containers/Set.h"
#include "Controller/RCController.h"
#include "IRemoteControlPropertyHandle.h"
#include "Kismet/KismetMathLibrary.h"
#include "StructUtils/PropertyBag.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"

namespace UE::RCRangeMapBehaviour
{
	const FName InputMin = TEXT("InputMin");
	const FName InputMax = TEXT("InputMax");
	const FName Input = TEXT("Input");
	
	constexpr double Threshold = 0.05;
}

URCRangeMapBehaviour::URCRangeMapBehaviour()
{
	PropertyContainer = CreateDefaultSubobject<URCVirtualPropertyContainerBase>(FName("VirtualPropertyContainer"));

	bExecuteBehavioursDuringPreChange = true;
}

void URCRangeMapBehaviour::Initialize()
{
	const URCController* RCController = ControllerWeakPtr.Get();
	if (!RCController)
	{
		return;
	}

	const TWeakObjectPtr<URCVirtualPropertyContainerBase> ContainerPtr = PropertyContainer;

	ContainerPtr->AddProperty(UE::RCRangeMapBehaviour::InputMin, URCController::StaticClass(), EPropertyBagPropertyType::Double);
	ContainerPtr->AddProperty(UE::RCRangeMapBehaviour::InputMax, URCController::StaticClass(), EPropertyBagPropertyType::Double);
	ContainerPtr->AddProperty(UE::RCRangeMapBehaviour::Input, URCController::StaticClass(), EPropertyBagPropertyType::Double);

	Super::Initialize();
}

bool URCRangeMapBehaviour::IsSupportedActionLerpType(TObjectPtr<URCAction> InAction) const
{
	if (TObjectPtr<URCFunctionAction> InFunctionAction = Cast<URCFunctionAction>(InAction))
	{
		// Early false if function. Functions are always non-lerp.
		return false;
	}
	
	const TObjectPtr<URCPropertyAction> InPropertyAction = Cast<URCPropertyAction>(InAction);
	if (!InPropertyAction)
	{
		return false;
	}
	
	const bool bIsNumeric = InPropertyAction->PropertySelfContainer->IsNumericType();
	const bool bIsVector  = InPropertyAction->PropertySelfContainer->IsVectorType();
	const bool bIsRotator = InPropertyAction->PropertySelfContainer->IsRotatorType();

	return bIsNumeric || bIsVector || bIsRotator;
}

void URCRangeMapBehaviour::Refresh()
{
	URCController* Controller = ControllerWeakPtr.Get();
	if (!Controller)
	{
		return;
	}

	if (!PropertyContainer->GetVirtualProperty(UE::RCRangeMapBehaviour::InputMin) || !PropertyContainer->GetVirtualProperty(UE::RCRangeMapBehaviour::InputMax))
	{
		return;
	}
	// Step 0: Get All required Values, getting the current values set within the DetailPanel setting them up for our header.
	PropertyContainer->GetVirtualProperty(UE::RCRangeMapBehaviour::InputMin)->GetValueDouble(InputMin);
	PropertyContainer->GetVirtualProperty(UE::RCRangeMapBehaviour::InputMax)->GetValueDouble(InputMax);

	Controller->GetValueFloat(ControllerFloatValue);
	
	// Step 1: Clamp the Value between Max/Min if necessary
	if (ControllerFloatValue < InputMin || InputMax < ControllerFloatValue)
	{
		ControllerFloatValue = FMath::Clamp(ControllerFloatValue, InputMin, InputMax);
		Controller->SetValueFloat(ControllerFloatValue);
	}
}

bool URCRangeMapBehaviour::GetNearestActionByThreshold(TTuple<URCAction*, bool>& OutTuple)
{
	// Variables
	TMap<double, URCAction*> NonLerpActions = GetNonLerpActions();
	
	TArray<double> InputActionArray;
	NonLerpActions.GenerateKeyArray(InputActionArray);

	// Step 1: Convert to double for Kismet Operation and Normalize.
	const double NormalizedControllerValue = UKismetMathLibrary::NormalizeToRange(ControllerFloatValue, InputMin, InputMax);

	// Step 2: Go through each of the Action Input and look for the shortest distance
	for (int InputIndex = 0; InputIndex < InputActionArray.Num(); InputIndex++)
	{
		const double InputValue = InputActionArray[InputIndex];
		const double NormalizedInputValue = UKismetMathLibrary::NormalizeToRange(InputValue, InputMin, InputMax);
		const double ValueDifference = FMath::Abs(NormalizedInputValue - NormalizedControllerValue);
		
		if (InputActionArray.Num() > 1 && InputIndex > 0)
		{
			const double PrevNormalizedInputValue = UKismetMathLibrary::NormalizeToRange(InputActionArray[InputIndex-1], InputMin, InputMax);
			const double PrevValueDifference = FMath::Abs(PrevNormalizedInputValue - NormalizedControllerValue);
			
			// Break out early in case previous Input had a lesser difference
			if (PrevValueDifference <= ValueDifference)
			{
				break;
			}
		}
		OutTuple = TTuple<URCAction*, bool>(NonLerpActions[InputValue], ValueDifference <= UE::RCRangeMapBehaviour::Threshold);
	}
	
	return true;
}

bool FRCRangeMapInput::SetInputValue(double InValue) const
{
	if (InputProperty)
	{
		return InputProperty->SetValueDouble(InValue);
	}
	return false;
}

bool FRCRangeMapInput::GetInputValue(double& OutValue) const
{
	if (InputProperty)
	{
		return InputProperty->GetValueDouble(OutValue);
	}

	return false;
}

bool URCRangeMapBehaviour::GetValueForAction(const URCAction* InAction, double& OutValue)
{
	if (FRCRangeMapInput* StepData = RangeMapActionContainer.Find(InAction))
	{
		return StepData->GetInputValue(OutValue);
	}

	return false;
}

void URCRangeMapBehaviour::NotifyActionValueChanged(URCAction* InChangedAction)
{
	if (InChangedAction)
	{
		ExecuteSingleAction(InChangedAction);
	}
}

void URCRangeMapBehaviour::ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute)
{
	Refresh();
	const URCBehaviourNode* BehaviourNode = GetBehaviourNode();
	check(BehaviourNode);

	if (BehaviourNode->GetClass() != URCBehaviourRangeMapNode::StaticClass())
	{
		return; // Allow custom Blueprints to drive their own behaviour entirely
	}

	if (InputMax <= InputMin)
	{
		UE_LOGF(LogRemoteControl, Warning, "Mapping Behaviour Input Max cannot be equal or smaller than the Minimum.")
		return;
	}

	// Do this beforehand.
	BehaviourNode->PreExecute(this);
	
	const URCController* RCController = ControllerWeakPtr.Get();
	if (!RCController)
	{
		ensureMsgf(false, TEXT("Remote Control Controller is not available/nullptr."));
		return;
	}

	RCController->GetValueFloat(ControllerFloatValue);
	ControllerFloatValue = FMath::Clamp(ControllerFloatValue, InputMin, InputMax);

	TTuple<URCAction*, bool> NearestAction;
	if (GetNearestActionByThreshold(NearestAction) && NearestAction.Value)
	{
		// Execute nearest Action in case its under/passes the Threshold in distance.
		NearestAction.Key->Execute();
	}

	// Apply Lerp if possible
	TMap<FGuid, TTuple<URCAction*, URCAction*>> LerpActions;
	TSet<TObjectPtr<URCAction>> RangeMapActionToExecute;

	// Do this only when the action to execute is 1 otherwise pass the entire array
	if (InActionsToExecute.Num() == 1)
	{
		// Get the single action
		const TObjectPtr<URCAction> ActionToExecute = InActionsToExecute.Array()[0];

		// Get all actions that are based on the same exposed property
		for (const TObjectPtr<URCAction>& Action : ActionContainer->GetActions())
		{
			if (Action->ExposedFieldId == ActionToExecute->ExposedFieldId)
			{
				RangeMapActionToExecute.Add(Action);
			}
		}
	}
	else
	{
		RangeMapActionToExecute = InActionsToExecute;
	}

	if (!GetRangeValuePairsForLerp(LerpActions, RangeMapActionToExecute))
	{
		return;
	}

	for (const TTuple<FGuid, TTuple<URCAction*, URCAction*>>& LerpAction : LerpActions)
	{
		URemoteControlPreset* Preset = RCController->PresetWeakPtr.Get();
		if (!Preset)
		{
			ensureMsgf(false, TEXT("Preset is invalid or unavailable."));
			return;
		}
		FGuid CurrentFieldId = LerpAction.Key;

		// Get the InputValue given for the FGuid
		const URCAction* MinActionOfPair = LerpAction.Value.Key;
		const URCAction* MaxActionOfPair = LerpAction.Value.Value;

		if (!MinActionOfPair || !MaxActionOfPair)
		{
			return;
		}

		const FRCRangeMapInput* MinRangeMapInput = RangeMapActionContainer.Find(MinActionOfPair);
		const FRCRangeMapInput* MaxRangeMapInput = RangeMapActionContainer.Find(MaxActionOfPair);

		if (!(MinRangeMapInput && MaxRangeMapInput))
		{
			ensureMsgf(false, TEXT("%s is an invalid pointer."), MinRangeMapInput ? TEXT("MaxRangeMapInput") : TEXT("MinRangeMapInput"));
			return;
		}

		double MinRangeValue, MaxRangeValue;
		bool bRes1 = MinRangeMapInput->GetInputValue(MinRangeValue);
		bool bRes2 = MaxRangeMapInput->GetInputValue(MaxRangeValue);
		if (!(bRes1 && bRes2))
		{
			continue; // ideally this should never happen
		}

		// Denormalize and Map them based on our Min and Max Value
		const double MappedMinInput = UKismetMathLibrary::Lerp(InputMin, InputMax, UKismetMathLibrary::NormalizeToRange(MinRangeValue, InputMin, InputMax));
		const double MappedMaxInput = UKismetMathLibrary::Lerp(InputMin, InputMax, UKismetMathLibrary::NormalizeToRange(MaxRangeValue, InputMin, InputMax));

		// Normalize our Controller based on the new MappedMinInput and MappedMaxInput Range.
		double CustomNormalizedInputValue = UKismetMathLibrary::NormalizeToRange(ControllerFloatValue, MappedMinInput, MappedMaxInput);

		// Apply value based on type
		switch(MinRangeMapInput->PropertyValue->GetValueType())
		{
			case EPropertyBagPropertyType::Double:
			{
				double MinRangeDouble;
				double MaxRangeDouble;

				MinRangeMapInput->PropertyValue->GetValueDouble(MinRangeDouble);
				MaxRangeMapInput->PropertyValue->GetValueDouble(MaxRangeDouble);

				const double ResultDouble = UKismetMathLibrary::Lerp(MinRangeDouble, MaxRangeDouble, CustomNormalizedInputValue);
				if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(CurrentFieldId).Pin())
				{
					RCProperty->GetPropertyHandle()->SetValue(ResultDouble);
				}
				
				break;
			}
			case EPropertyBagPropertyType::Float:
			{
				float MinRangeFloat;
				float MaxRangeFloat;

				MinRangeMapInput->PropertyValue->GetValueFloat(MinRangeFloat);
				MaxRangeMapInput->PropertyValue->GetValueFloat(MaxRangeFloat);

				const float ResultFloat = UKismetMathLibrary::Lerp(MinRangeFloat, MaxRangeFloat, CustomNormalizedInputValue);
				if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(CurrentFieldId).Pin())
				{
					RCProperty->GetPropertyHandle()->SetValue(ResultFloat);
				}
				
				break;
			}
			case EPropertyBagPropertyType::Struct:
			{
				ApplyLerpOnStruct(MinRangeMapInput, MaxRangeMapInput, CustomNormalizedInputValue, CurrentFieldId);
				
				break;
			}
			default:
			// Shouldn't happen.
			ensureMsgf(false, TEXT("Unsupported Lerp Type."));
		}
	}
}

void URCRangeMapBehaviour::GetLerpActions(TMap<FGuid, TArray<URCAction*>>& OutNumericActionsByField, const TSet<TObjectPtr<URCAction>>& InActionsToExecute)
{
	for (URCAction* Action : InActionsToExecute)
	{
		TArray<URCAction*>& LerpActionArray = OutNumericActionsByField.FindOrAdd(Action->ExposedFieldId);
		
		// Step 01: Find Action
		if (IsSupportedActionLerpType(Action))
		{
			// Step 02: Add Action if it's numerical
			LerpActionArray.Add(Action);
		}
	}

	// Step 03: Sort actions using their InputValue
	for (TTuple<FGuid, TArray<URCAction*>>& NumericActionTuple : OutNumericActionsByField)
	{
		TArray<URCAction*>& ArrayToSort = NumericActionTuple.Value;

		Algo::Sort(ArrayToSort, [this](const URCAction* ActionA, const URCAction* ActionB)
		{
			double A, B;
			bool bRes1 = GetValueForAction(ActionA, A);
			bool bRes2 = GetValueForAction(ActionB, B);

			if (bRes1 && bRes2)
			{
				return A < B;
			}

			return false;
		});
	}
}

URCAction* URCRangeMapBehaviour::DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour)
{
	URCRangeMapBehaviour* InBehaviourRangeMap = Cast< URCRangeMapBehaviour>(InBehaviour);
	if (!ensureMsgf(InBehaviourRangeMap, TEXT("Expected Behaviour of the same type (Range Map) For CopyAction operation!")))
	{
		return nullptr;
	}

	URCAction* NewAction = Super::DuplicateAction(InAction, InBehaviour);
	if (ensure(NewAction))
	{
		// Copy action specific data residing in Range Map Behaviour:
		if (const FRCRangeMapInput* RangeMapInputData = RangeMapActionContainer.Find(InAction))
		{
			URCVirtualPropertySelfContainer* ActionValueProperty = nullptr;
			if (const URCPropertyAction* PropertyAction = Cast<URCPropertyAction>(NewAction))
			{
				ActionValueProperty = PropertyAction->PropertySelfContainer;
			}

			URCVirtualPropertySelfContainer* ActionInputProperty = NewObject<URCVirtualPropertySelfContainer>(InBehaviour);
			ActionInputProperty->DuplicatePropertyWithCopy(RangeMapInputData->InputProperty);

			if (ensureAlways(ActionValueProperty))
			{
				const FRCRangeMapInput NewData(ActionInputProperty, ActionValueProperty);
				InBehaviourRangeMap->RangeMapActionContainer.Add(NewAction, NewData);
			}
		}
	}

	return NewAction;
}

URCAction* URCRangeMapBehaviour::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	// The new workflow for Rangemap involves the user creating the Step action first (with default step value) and then clicking on Edit to set the value
	// For this reason no uniqueness test can be performed at this point
	const TRCActionUniquenessTest PassthroughUniquenessTest = [](const TSet<TObjectPtr<URCAction>>& Actions) { return true;	};
	
	return ActionContainer->AddAction(PassthroughUniquenessTest, InRemoteControlField);
}

void URCRangeMapBehaviour::OnActionAdded(URCAction* Action, URCVirtualPropertySelfContainer* InPropertyValue)
{
	// Virtual Property for Step Input
	URCVirtualPropertySelfContainer* InputProperty = NewObject<URCVirtualPropertySelfContainer>(this);
	InputProperty->AddProperty(UE::RCRangeMapBehaviour::Input, EPropertyBagPropertyType::Double);
	
	FRCRangeMapInput RangeMapInput = FRCRangeMapInput(InputProperty, InPropertyValue);

	RangeMapActionContainer.Add(Action, MoveTemp(RangeMapInput));
}

bool URCRangeMapBehaviour::GetRangeValuePairsForLerp(TMap<FGuid, TTuple<URCAction*, URCAction*>>& OutPairs, const TSet<TObjectPtr<URCAction>>& InActionsToExecute)
{
	const double NormalizedControllerValue = UKismetMathLibrary::NormalizeToRange(ControllerFloatValue, InputMin, InputMax);

	TMap<FGuid, TArray<URCAction*>> LerpActionMap;
	GetLerpActions(LerpActionMap, InActionsToExecute);
	
	for (TTuple<FGuid, TArray<URCAction*>>& NumericActionTuple : LerpActionMap)
	{
		if (NumericActionTuple.Value.Num() < 2)
		{
			// Skip and continue early if the Array has less Numeric Values needed to actually Lerp
			continue;
		}

		// Intermediary calculations of Lerp for in-between Lerp.
		FRCRangeMapInput* MaxRangeMap = nullptr;
		FRCRangeMapInput* MinRangeMap = nullptr;
		URCAction* MaxAction = nullptr;
		URCAction* MinAction = nullptr;
		
		for (URCAction* CurrentAction : NumericActionTuple.Value)
		{
			FRCRangeMapInput* RangeMap = RangeMapActionContainer.Find(CurrentAction);
			if (!RangeMap)
			{
				// Shouldn't happen, but skip if it does
				continue;
			}

			double Value;
			bool bRes = RangeMap->GetInputValue(Value);
			if (!bRes)
			{
				continue;
			}

			const double NormalizedValue = UKismetMathLibrary::NormalizeToRange(Value, InputMin, InputMax);

			// Deal with the nullptr
			if (NormalizedValue <= NormalizedControllerValue && NumericActionTuple.Value.Last() != CurrentAction)
			{
				MinAction = CurrentAction;
				MinRangeMap = RangeMap;
				
				continue;
			}

			if (NormalizedControllerValue <= NormalizedValue)
			{
				MaxAction = CurrentAction;
				MaxRangeMap = RangeMap;

				if (MinRangeMap)
				{
					break;
				}
				
				continue;
			}

			// Ensure there are no nullptr
			if (!MinRangeMap || !MaxRangeMap)
			{
				return false;
			}
		}

		OutPairs.Add(NumericActionTuple.Key, TTuple<URCAction*, URCAction*>(MinAction, MaxAction));
	}
	
	return true;
}

TMap<double, URCAction*> URCRangeMapBehaviour::GetNonLerpActions()
{
	TMap<double, URCAction*> ActionMap;
	
	for (URCAction* Action : ActionContainer->GetActions())
	{
		if (!IsSupportedActionLerpType(Action))
		{
			if (const FRCRangeMapInput* RangeMapInput = RangeMapActionContainer.Find(Action))
			{
				double InputValue;
				if (RangeMapInput->GetInputValue(InputValue))
				{
					ActionMap.Add(InputValue, Action);
				}
			}
		}
	}

	return ActionMap;
}

bool URCRangeMapBehaviour::CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const
{
	if (InRemoteControlField->FieldType == EExposedFieldType::Property)
	{
		if (const TSharedPtr<FRemoteControlProperty>& RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(InRemoteControlField))
		{
			if (const FProperty* Property = RCProperty->GetProperty())
			{
				const FPropertyBagPropertyDesc PropertyBagDesc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
				return RCProperty->IsEditable() && PropertyBagDesc.ValueType != EPropertyBagPropertyType::None;
			}
		}
	}

	return false;
}

void URCRangeMapBehaviour::ApplyLerpOnStruct(const FRCRangeMapInput* MinRangeInput, const FRCRangeMapInput* MaxRangeInput, const double& InputAlpha, const FGuid& FieldId)
{
	const URCController* RCController = ControllerWeakPtr.Get();
	if (!RCController)
	{
		ensureMsgf(false, TEXT("Remote Controller is invalid or unavailable."));
		return;
	}
	
	URemoteControlPreset* Preset = RCController->PresetWeakPtr.Get();
	if (!Preset)
	{
		ensureMsgf(false, TEXT("Preset is invalid or unavailable."));
		return;
	}
	
	// Check whether its a Rotator or FVector based on the MinRangeInput
	if (MinRangeInput->PropertyValue->IsVectorType())
	{
		FVector MinRangeVector;
		FVector MaxRangeVector;

		MinRangeInput->PropertyValue->GetValueVector(MinRangeVector);
		MaxRangeInput->PropertyValue->GetValueVector(MaxRangeVector);

		const FVector ResultVector = UKismetMathLibrary::VLerp(MinRangeVector, MaxRangeVector, InputAlpha);
		if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(FieldId).Pin())
		{
			RCProperty->GetPropertyHandle()->SetValue(ResultVector);
		}
	}

	if (MinRangeInput->PropertyValue->IsRotatorType())
	{
		FRotator MinRangeRotator;
		FRotator MaxRangeRotator;

		MinRangeInput->PropertyValue->GetValueRotator(MinRangeRotator);
		MaxRangeInput->PropertyValue->GetValueRotator(MaxRangeRotator);

		const FRotator ResultVector = UKismetMathLibrary::RLerp(MinRangeRotator, MaxRangeRotator, InputAlpha, true);
		if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(FieldId).Pin())
		{
			RCProperty->GetPropertyHandle()->SetValue(ResultVector);
		}
	}
}

bool URCRangeMapBehaviour::IsActionUnique(const TSharedRef<const FRemoteControlField> InRemoteControlField, const double& InValue, const TSet<TObjectPtr<URCAction>>& InActions)
{
	const FGuid FieldId = InRemoteControlField->GetId();

	for (const URCAction* Action : InActions)
	{
		// Check whether or not we have a RangeInput.
		if (const FRCRangeMapInput* RangeMapInput = this->RangeMapActionContainer.Find(Action))
		{
			// Check if the Step itself is already used
			double ActionStepValue;
			if(RangeMapInput->GetInputValue(ActionStepValue))
			{
				// Only Important bit is StepValue and FieldId
				if (Action->ExposedFieldId == FieldId && FMath::IsNearlyEqual(ActionStepValue, InValue))
				{
					return false; // Not Unique.
				}
			}
		}
	}
	
	return true;
}
