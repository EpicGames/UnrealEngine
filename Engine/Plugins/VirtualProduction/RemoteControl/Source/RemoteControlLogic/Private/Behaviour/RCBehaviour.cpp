// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/RCBehaviour.h"

#include "Action/RCAction.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/RCBehaviourNode.h"
#include "Controller/RCController.h"
#include "Engine/Blueprint.h"
#include "StructUtils/PropertyBag.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"

URCBehaviour::URCBehaviour()
{
	ActionContainer = CreateDefaultSubobject<URCActionContainer>(FName("ActionContainer"));
}

void URCBehaviour::Execute()
{
	ExecuteInternal(ActionContainer->GetActions());
}

void URCBehaviour::ExecuteSingleAction(URCAction* InAction)
{
	if (InAction)
	{
		ExecuteInternal({ InAction });
	}
}

URCAction* URCBehaviour::AddAction()
{
#if WITH_EDITOR
	ActionContainer->Modify();
#endif // WITH_EDITOR

	return ActionContainer->AddAction();
}

URCAction* URCBehaviour::AddAction(FName InFieldId)
{
#if WITH_EDITOR
	ActionContainer->Modify();
#endif // WITH_EDITOR

	return ActionContainer->AddAction(InFieldId);
}

URCAction* URCBehaviour::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
#if WITH_EDITOR
	ActionContainer->Modify();
#endif

	return ActionContainer->AddAction(InRemoteControlField);
}

URCAction* URCBehaviour::DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour)
{
	URCAction* NewAction = DuplicateObject<URCAction>(InAction, InBehaviour->ActionContainer);
	if(ensure(NewAction))
	{
		InBehaviour->ActionContainer->AddAction(NewAction);

		return NewAction;
	}

	return nullptr;
}

int32 URCBehaviour::GetNumActions() const
{
	return ActionContainer->GetActions().Num();
}

bool URCBehaviour::HasAction(const URCAction* InAction) const
{
	return InAction && ActionContainer->GetActions().Contains(InAction);
}

bool URCBehaviour::CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const
{
	if (ActionContainer->FindActionByFieldId(InRemoteControlField->GetId()))
	{
		return false; // already exists!
	}

	if (InRemoteControlField->FieldType == EExposedFieldType::Property)
	{
		if (const TSharedPtr<FRemoteControlProperty>& RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(InRemoteControlField))
		{
			if (const FProperty* Property = RCProperty->GetProperty())
			{
				const FPropertyBagPropertyDesc PropertyBagDesc = FPropertyBagPropertyDesc(Property->GetFName(), Property);
				return RCProperty->IsEditable() && PropertyBagDesc.ValueType != EPropertyBagPropertyType::None;
			}
		}
	}

	return true;
}

UClass* URCBehaviour::GetOverrideBehaviourBlueprintClass() const
{
	// Blueprint extensions stores as a separate blueprints files that is why it should check can be path resolved as asset 
	if (!OverrideBehaviourBlueprintClassPath.IsValid() || !ensure(OverrideBehaviourBlueprintClassPath.IsAsset()))
	{
		return nullptr;
	}

	// If class was loaded before then just resolve class, that is faster
	if (UClass* ResolveClass = OverrideBehaviourBlueprintClassPath.ResolveClass())
	{
		return ResolveClass;
	}

	// Load class from the UObject path
	return OverrideBehaviourBlueprintClassPath.TryLoadClass<URCBehaviourNode>();
}

#if WITH_EDITORONLY_DATA
UBlueprint* URCBehaviour::GetBlueprint() const
{
	UClass* OverrideBehaviourBlueprintClass = GetOverrideBehaviourBlueprintClass();
	return OverrideBehaviourBlueprintClass ? Cast<UBlueprint>(OverrideBehaviourBlueprintClass->ClassGeneratedBy) : nullptr;
}
#endif

void URCBehaviour::SetOverrideBehaviourBlueprintClass(UBlueprint* InBlueprint)
{
	if (InBlueprint)
	{
		OverrideBehaviourBlueprintClassPath = InBlueprint->GeneratedClass.Get();
	}
}

URCBehaviourNode* URCBehaviour::GetBehaviourNode()
{
	UClass* OverrideBehaviourBlueprintClass = GetOverrideBehaviourBlueprintClass();
	UClass* FinalBehaviourNodeClass = OverrideBehaviourBlueprintClass ? OverrideBehaviourBlueprintClass : BehaviourNodeClass.Get();

	if (!CachedBehaviourNode || CachedBehaviourNode->GetClass() != FinalBehaviourNodeClass)
	{
		CachedBehaviourNode = NewObject<URCBehaviourNode>(this, FinalBehaviourNodeClass);
	}

	return CachedBehaviourNode;
}

void URCBehaviour::UpdateEntityUsage(const URCController* InController, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const
{
	URCController* Controller = ControllerWeakPtr.Get();

	if (!Controller || Controller != InController || !Controller->HasBehavior(this))
	{
		return;
	}

	if (ActionContainer)
	{
		ActionContainer->UpdateEntityUsage(this, InEntityToBehavior);
	}	
}

void URCBehaviour::ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute)
{
	const URCBehaviourNode* BehaviourNode = GetBehaviourNode();

	// Execute before the logic
	BehaviourNode->PreExecute(this);

	if (BehaviourNode->Execute(this))
	{
		for (const TObjectPtr<URCAction>& Action : InActionsToExecute)
		{
			Action->Execute();
		}
		BehaviourNode->OnPassed(this);
	}	
}

#if WITH_EDITOR
const FText& URCBehaviour::GetDisplayName()
{
	if (!ensure(BehaviourNodeClass))
	{
		return FText::GetEmpty();
	}

	return GetDefault<URCBehaviourNode>(BehaviourNodeClass)->DisplayName;
}

const FText& URCBehaviour::GetBehaviorDescription()
{
	if (!ensure(BehaviourNodeClass))
	{
		return FText::GetEmpty();
	}

	return GetDefault<URCBehaviourNode>(BehaviourNodeClass)->BehaviorDescription;
}
#endif

void URCBehaviour::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	if (ActionContainer)
	{
		ActionContainer->ForEachAction([&InEntityIdMap](URCAction* InAction)
		{
			InAction->UpdateEntityIds(InEntityIdMap);
		}, /*bInRecursive*/ true);
	}
}
