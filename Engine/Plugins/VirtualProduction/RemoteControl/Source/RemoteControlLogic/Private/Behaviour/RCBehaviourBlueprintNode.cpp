// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/RCBehaviourBlueprintNode.h"

bool URCBehaviourBlueprintNode::Execute_Implementation(URCBehaviour* InBehaviour) const
{
	return false;
}

void URCBehaviourBlueprintNode::PreExecute_Implementation(URCBehaviour* InBehaviour) const
{
}

bool URCBehaviourBlueprintNode::IsSupported_Implementation(URCBehaviour* InBehaviour) const
{
	return false;
}

void URCBehaviourBlueprintNode::OnPassed_Implementation(URCBehaviour* InBehaviour) const
{
}
