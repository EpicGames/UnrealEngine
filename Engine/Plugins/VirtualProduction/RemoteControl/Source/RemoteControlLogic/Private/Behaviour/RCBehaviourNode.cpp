// Copyright Epic Games, Inc. All Rights Reserved.

#include "Behaviour/RCBehaviourNode.h"
#include "RCVirtualProperty.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"

void URCBehaviourNode::PreExecute(URCBehaviour* InBehaviour) const
{
}

bool URCBehaviourNode::Execute(URCBehaviour* InBehaviour) const
{
	return false;
}

bool URCBehaviourNode::IsSupported(URCBehaviour* InBehaviour) const
{
	return false;
}

void URCBehaviourNode::OnPassed(URCBehaviour* InBehaviour) const
{
}

UClass* URCBehaviourNode::GetBehaviourClass() const
{
	return URCBehaviour::StaticClass();
}

TFunction<bool(const TPair<EPropertyBagPropertyType, UObject*>)> URCBehaviourNode::GetIsSupportedCallback(URCBehaviour* InBehaviour)
{
	return[InBehaviour](const TPair<EPropertyBagPropertyType, UObject*> InPair)
	{
		if (const URCController* RCController = InBehaviour->ControllerWeakPtr.Get())
		{
			EPropertyBagPropertyType ValueType = RCController->GetValueType();

			if (ValueType == EPropertyBagPropertyType::Enum && InPair.Key == EPropertyBagPropertyType::Enum)
			{
				return true;
			}

			return ValueType == InPair.Key && RCController->GetValueTypeObjectWeakPtr() == InPair.Value;
		}
		
		return false;
	};
}
