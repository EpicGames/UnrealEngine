// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controller/RCController.h"

#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "Action/RCActionContainer.h"
#include "Behaviour/RCBehaviour.h"
#include "Behaviour/RCBehaviourNode.h"

#define LOCTEXT_NAMESPACE "RCController"

void URCController::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	for (URCBehaviour* Behaviour : Behaviors)
	{
		if (Behaviour)
		{
			Behaviour->UpdateEntityIds(InEntityIdMap);
		}
	}
	
	Super::UpdateEntityIds(InEntityIdMap);
}

#if WITH_EDITOR
void URCController::PostEditUndo()
{
	Super::PostEditUndo();

	OnBehaviourListModified.Broadcast();
}
#endif

URCBehaviour* URCController::AddBehaviour(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass)
{
	URCBehaviour* NewBehaviour = CreateBehaviour(InBehaviourNodeClass);
	if (!ensure(NewBehaviour))
	{
		return nullptr;
	}

	// This is the only time this is called - when the behavior is first created and added.
	NewBehaviour->Initialize();
	
	Behaviors.Add(NewBehaviour);

	return NewBehaviour;
}

URCBehaviour* URCController::CreateBehaviour(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass)
{
	const URCBehaviourNode* DefaultBehaviourNode = Cast<URCBehaviourNode>(InBehaviourNodeClass->GetDefaultObject());
	
	URCBehaviour* NewBehaviour = NewObject<URCBehaviour>(this, DefaultBehaviourNode->GetBehaviourClass(), NAME_None, RF_Transactional);
	NewBehaviour->BehaviourNodeClass = InBehaviourNodeClass;
	NewBehaviour->Id = FGuid::NewGuid();
	NewBehaviour->ActionContainer->PresetWeakPtr = PresetWeakPtr;
	NewBehaviour->ControllerWeakPtr = this;
	
	if (!DefaultBehaviourNode->IsSupported(NewBehaviour))
	{
		return nullptr;
	}
	
	return NewBehaviour;
}

URCBehaviour* URCController::CreateBehaviourWithoutCheck(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass)
{
	const URCBehaviourNode* DefaultBehaviourNode = Cast<URCBehaviourNode>(InBehaviourNodeClass->GetDefaultObject());
	
	URCBehaviour* NewBehaviour = NewObject<URCBehaviour>(this, DefaultBehaviourNode->GetBehaviourClass(), NAME_None, RF_Transactional);
	NewBehaviour->BehaviourNodeClass = InBehaviourNodeClass;
	NewBehaviour->Id = FGuid::NewGuid();
	NewBehaviour->ActionContainer->PresetWeakPtr = PresetWeakPtr;
	NewBehaviour->ControllerWeakPtr = this;
	
	return NewBehaviour;
}

int32 URCController::RemoveBehaviour(URCBehaviour* InBehaviour)
{
	return Behaviors.Remove(InBehaviour);
}

int32 URCController::RemoveBehaviour(const FGuid InBehaviourId)
{
	int32 RemovedCount = 0;

	for (TArray<TObjectPtr<URCBehaviour>>::TIterator BehaviourIt(Behaviors); BehaviourIt; ++BehaviourIt)
	{
		const URCBehaviour* Behaviour = *BehaviourIt;
		if (Behaviour && Behaviour->Id == InBehaviourId)
		{
			BehaviourIt.RemoveCurrent();
			RemovedCount++;
		}
	}

	return RemovedCount;
}

bool URCController::HasBehavior(const URCBehaviour* InBehavior) const
{
	return InBehavior && Behaviors.Contains(InBehavior);
}

void URCController::EmptyBehaviours()
{
	Behaviors.Empty();
}

void URCController::ExecuteBehaviours(const bool bIsPreChange/* = false*/)
{
	for (URCBehaviour* Behaviour : Behaviors)
	{
		if (!Behaviour || !Behaviour->bIsEnabled)
		{
			continue;
		}

		if (bIsPreChange && !Behaviour->bExecuteBehavioursDuringPreChange)
		{
			continue;
		}

		Behaviour->Execute();

	}
}

void URCController::OnPreChangePropertyValue()
{
	constexpr bool bIsPreChange = true;
	ExecuteBehaviours(bIsPreChange);
}

void URCController::OnModifyPropertyValue()
{
	ExecuteBehaviours();
}

URCBehaviour* URCController::DuplicateBehaviour(URCController* InController, URCBehaviour* InBehaviour)
{
	URCBehaviour* NewBehaviour = DuplicateObject<URCBehaviour>(InBehaviour, InController);

	NewBehaviour->ControllerWeakPtr = InController;

	InController->Behaviors.Add(NewBehaviour);

	return NewBehaviour;
}

bool URCController::ReorderBehaviorItems(URCBehaviour* InDroppedOnBehavior, bool bInBelowItem, TArray<URCBehaviour*> InDroppedBehaviors)
{
	if (!Behaviors.Contains(InDroppedOnBehavior) || InDroppedBehaviors.IsEmpty())
	{
		return false;
	}

	TArray<URCBehaviour*> NewBehaviors;

	auto AddDroppedItems = [this, &InDroppedBehaviors, &NewBehaviors]
		{
			for (URCBehaviour* Behavior : InDroppedBehaviors)
			{
				if (Behavior)
				{
					if (Behaviors.Contains(Behavior))
					{
						NewBehaviors.Add(Behavior);
					}
				}
			}
		};

	for (URCBehaviour* Behavior : Behaviors)
	{
		if (Behavior == InDroppedOnBehavior && !bInBelowItem)
		{
			AddDroppedItems();
		}

		if (!InDroppedBehaviors.Contains(Behavior))
		{
			NewBehaviors.Add(Behavior);
		}

		if (Behavior == InDroppedOnBehavior && bInBelowItem)
		{
			AddDroppedItems();
		}
	}

	Behaviors = NewBehaviors;

	OnBehaviourListModified.Broadcast();

	return true;
}

void URCController::UpdateEntityUsage(const URemoteControlPreset* InPreset, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const
{
	URemoteControlPreset* Preset = PresetWeakPtr.Get();

	if (!Preset || Preset != InPreset || !Preset->HasController(this))
	{
		return;
	}

	for (URCBehaviour* Behaviour : Behaviors)
	{
		if (Behaviour)
		{
			Behaviour->UpdateEntityUsage(this, InEntityToBehavior);
		}
	}
}

void URCController::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	if (!Behaviours.IsEmpty())
	{
		Behaviors = Behaviours.Array();
		Behaviours.Empty();
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef LOCTEXT_NAMESPACE /* RCController */ 
