// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controller/RCControllerContainer.h"

#include "Action/RCAction.h"
#include "Action/RCActionContainer.h"
#include "Controller/RCController.h"
#include "RCVirtualProperty.h"
#include "Templates/SubclassOf.h"

void URCControllerContainer::UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap)
{
	for (URCActionContainer* ActionContainer : SharedActionContainers)
	{
		if (ActionContainer)
		{
			ActionContainer->ForEachAction([&InEntityIdMap](URCAction* InAction)
			{
				InAction->UpdateEntityIds(InEntityIdMap);
			}, /*bInRecursive*/ true);
		}
	}

	Super::UpdateEntityIds(InEntityIdMap);
}

URCVirtualPropertyInContainer* URCControllerContainer::AddProperty(const FName& InPropertyName, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject /*= nullptr*/, TArray<FPropertyBagPropertyDescMetaData> MetaData /*= TArray<FPropertyBagPropertyDescMetaData>()*/)
{
	if (InValueType == EPropertyBagPropertyType::String)
	{
		// Add multiline capabilities to string controllers.
		// TODO: a meta-data editing interface to customize meta-data like this per-controller.
		MetaData.Add(FPropertyBagPropertyDescMetaData(FName("MultiLine"), TEXT("true")));
	}
	else if (InValueType == EPropertyBagPropertyType::Struct)
	{
		// Color Controllers
		if (InValueTypeObject == TBaseStructure<FColor>::Get())
		{
			MetaData.Add(FPropertyBagPropertyDescMetaData(FName("OnlyUpdateOnInteractionEnd"), TEXT("true")));
		}
	}

	return Super::AddProperty(InPropertyName, InPropertyClass, InValueType, InValueTypeObject, MetaData);
}

#if WITH_EDITOR
URCController* URCControllerContainer::GetControllerFromChangeEvent(const FPropertyChangedEvent& Event)
{
	if (const FProperty* FinalProperty = (Event.Property == Event.MemberProperty) ? Event.Property : Event.MemberProperty)
	{
		const FName PropertyName = FinalProperty->GetFName();

		URCVirtualPropertyBase* VirtualProperty = GetVirtualProperty(PropertyName);

		return Cast<URCController>(VirtualProperty);
	}

	return nullptr;
}

void URCControllerContainer::OnPreChangePropertyValue(const FPropertyChangedEvent& Event)
{
	if (URCController* Controller = GetControllerFromChangeEvent(Event))
	{
		Controller->OnPreChangePropertyValue();
	}

	Super::OnPreChangePropertyValue(Event);
}

void URCControllerContainer::OnModifyPropertyValue(const FPropertyChangedEvent& Event)
{
	if (URCController* Controller = GetControllerFromChangeEvent(Event))
	{
		Controller->OnModifyPropertyValue();
	}

	Super::OnModifyPropertyValue(Event);
}
#endif
