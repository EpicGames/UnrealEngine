// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controller/RCControllerUtilities.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInterface.h"
#include "RemoteControlField.h"
#include "StructUtils/PropertyBag.h"
#include "UObject/Class.h"

namespace UE::RCControllers
{
	namespace Private
	{
		/**
		 * Determines whether a class of a specific type is supported explicitly. 
		 * Explicit here means that checking against a derived type of a supported type will return false as the checks done here do not include child/derived classes.
		 */
		bool CanCreateControllerFromStruct(const UScriptStruct* InStruct)
		{
			return InStruct == TBaseStructure<FVector>::Get()
				|| InStruct == TBaseStructure<FVector2D>::Get()
				|| InStruct == TBaseStructure<FRotator>::Get()
				|| InStruct == TBaseStructure<FColor>::Get()
				|| InStruct == TBaseStructure<FLinearColor>::Get(); // todo: unify with supported controller structs in RC Config
		}

		/**
		 * Determines whether a class of a specific type is supported explicitly. 
		 * Explicit here means that checking against UTexture2D or UMaterial will return false as the checks done here do not include child/derived classes.
		 */
		bool CanCreateControllerFromClass(const UClass* InClass)
		{
			return InClass == UTexture::StaticClass()
				|| InClass == UStaticMesh::StaticClass()
				|| InClass == UMaterialInterface::StaticClass();
		}
	} // UE::RCControllers::Private

	bool CanCreateControllerFromPropertyDesc(const FPropertyBagPropertyDesc& InPropertyDesc)
	{
		switch (InPropertyDesc.ValueType)
		{
		case EPropertyBagPropertyType::Enum:
			if (const UEnum* Enum = Cast<UEnum>(InPropertyDesc.ValueTypeObject))
			{
				const int64 MaxEnumValue = Enum->GetMaxEnumValue();
				const uint32 NeededBits = FMath::RoundUpToPowerOfTwo(MaxEnumValue);

				// 8 bits enums only
				return NeededBits <= 256;
			}
			break;

		case EPropertyBagPropertyType::Struct:
			if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(InPropertyDesc.ValueTypeObject))
			{
				return Private::CanCreateControllerFromStruct(ScriptStruct);
			}
			break;

		case EPropertyBagPropertyType::Object:
		case EPropertyBagPropertyType::SoftObject:
		case EPropertyBagPropertyType::Class:
		case EPropertyBagPropertyType::SoftClass:
			if (const UClass* Class = Cast<UClass>(InPropertyDesc.ValueTypeObject))
			{
				return Private::CanCreateControllerFromClass(Class);
			}
			break;

		default:
			// All other types are allowed
			return true;
		}

		return false;
	}

	bool CanCreateControllerFromEntity(const TSharedPtr<const FRemoteControlProperty>& InPropertyEntity)
	{
		if (!InPropertyEntity.IsValid() || !InPropertyEntity->IsEditable() || InPropertyEntity->FieldType != EExposedFieldType::Property)
		{
			// Property with error(s)
			return false;
		}
		return UE::RCControllers::CanCreateControllerFromProperty(InPropertyEntity->GetProperty());
	}

	bool CanCreateControllerFromProperty(const FProperty* InProperty)
	{
		return InProperty && CanCreateControllerFromPropertyDesc(FPropertyBagPropertyDesc(NAME_None, InProperty));
	}
} // UE::RCControllers
