// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controller/RCCustomControllerUtilities.h"
#include "Action/RCAction.h"
#include "Action/RCPropertyIdAction.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlCommon.h"
#include "RemoteControlPreset.h"
#include "RemoteControlPropertyIdRegistry.h"

namespace UE::RCCustomControllers
{
	// A Map to associate a custom controller name to its underlying type
	inline static TMap<FString, EPropertyBagPropertyType> CustomControllerTypes =
		{
			{CustomTextureControllerName, EPropertyBagPropertyType::String}
		};
}

bool UE::RCCustomControllers::IsCustomController(const URCVirtualPropertyBase* InController)
{
	return !InController->GetMetadataValue(CustomControllerNameKey).IsEmpty();
}

FString UE::RCCustomControllers::GetCustomControllerTypeName(const URCVirtualPropertyBase* InController)
{
	return InController->GetMetadataValue(CustomControllerNameKey);
}

bool UE::RCCustomControllers::IsValidCustomController(const FName& InCustomControllerTypeName)
{
	return CustomControllerTypes.Contains(InCustomControllerTypeName.ToString());
}

EPropertyBagPropertyType UE::RCCustomControllers::GetCustomControllerType(const FName& InCustomControllerTypeName)
{
	if (IsValidCustomController(InCustomControllerTypeName))
	{
		return CustomControllerTypes[InCustomControllerTypeName.ToString()];
	}

	return EPropertyBagPropertyType::None;
}

EPropertyBagPropertyType UE::RCCustomControllers::GetCustomControllerType(const FString& InCustomControllerTypeName)
{
	return GetCustomControllerType(FName(InCustomControllerTypeName));
}

TMap<FName, FString> UE::RCCustomControllers::GetCustomControllerMetaData(const FString& InCustomControllerTypeName)
{
	TMap<FName, FString> OutMetaData;
	OutMetaData.Emplace(UE::RCCustomControllers::CustomControllerNameKey, InCustomControllerTypeName);

	// Note: this is a good place to additionally edit OutMetaData by manually adding elements here if custom logic is needed
	
	return OutMetaData;
}

FName UE::RCCustomControllers::GetUniqueNameForController(const URCVirtualPropertyInContainer* InController)
{
	const FString& CustomTypeName = UE::RCCustomControllers::GetCustomControllerTypeName(InController);
	if (!CustomTypeName.IsEmpty())
	{
		if (InController->ContainerWeakPtr.IsValid())
		{
			URCVirtualPropertyContainerBase* Container = InController->ContainerWeakPtr.Get();
								
			int32 Index = 0;
			const FString InitialName = CustomTypeName;
			FString FinalName = InitialName;

			for (const TObjectPtr<URCVirtualPropertyBase>& VirtualProperty : Container->VirtualProperties)
			{
				if (VirtualProperty->DisplayName == FinalName)
				{
					Index++;
					if (Index > 0)
					{
						FinalName = InitialName + TEXT("_") + FString::FromInt(Index++);
					}
				}
			}
			
			return FName(FinalName);				
		}
	}

	// Worst case - we return the plain type name.
	return FName(CustomTypeName);
}

bool UE::RCCustomControllers::GetEntitiesControlledByController(const URemoteControlPreset* InRemoteControlPreset, const URCVirtualPropertyBase* InVirtualProperty, TSet<FGuid>& OutEntityIds)
{
	if (!InRemoteControlPreset)
	{
		return false;
	}

	const URCController* Controller = Cast<URCController>(InVirtualProperty);

	if (!IsValid(Controller))
	{
		return false;
	}

	for (const URCBehaviour* Behavior : Controller->GetBehaviors())
	{
		if (!Behavior)
		{
			continue;
		}

		// Special case: SetAssetByPath behaviors have an extra 'TargetEntity' that these behaviors control outside of RC Actions.
		if (const URCSetAssetByPathBehaviour* SetAssetByPathBehavior = Cast<URCSetAssetByPathBehaviour>(Behavior))
		{
			OutEntityIds.Add(SetAssetByPathBehavior->GetTargetEntityId());
		}

		for (const URCAction* Action : Behavior->ActionContainer->GetActions())
		{
			if (!Action)
			{
				continue;
			}

			if (const URCPropertyIdAction* IdentityAction = Cast<URCPropertyIdAction>(Action))
			{
				if (const URemoteControlPropertyIdRegistry* IdentityRegistry = InRemoteControlPreset->GetPropertyIdRegistry())
				{
					OutEntityIds.Append(IdentityRegistry->GetEntityIdsForPropertyId(IdentityAction->PropertyId));

					// Explicit support for Sub-PropertyId
					for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainer : IdentityAction->PropertySelfContainer)
					{
						OutEntityIds.Append(IdentityRegistry->GetEntityIdsForPropertyId(PropertyContainer.Key.PropertyId));
					}
				}
			}
			else
			{
				OutEntityIds.Add(Action->ExposedFieldId);
			}
		}
	}
	return true;
}

bool UE::RCCustomControllers::GetControllersOfEntity(const URemoteControlPreset* InRemoteControlPreset, const FGuid& InEntity, TSet<const URCVirtualPropertyBase*>& OutControllers)
{
	if (!InRemoteControlPreset)
	{
		return false;
	}

	if (URCVirtualPropertyContainerBase* ControllerContainer = InRemoteControlPreset->GetControllerContainer())
	{
		for (const URCVirtualPropertyBase* VirtualProperty : ControllerContainer->VirtualProperties)
		{
			if (const URCController* Controller = Cast<URCController>(VirtualProperty))
			{
				bool bFoundEntity = false;

				for (const URCBehaviour* Behavior : Controller->GetBehaviors())
				{
					if (!Behavior)
					{
						continue;
					}

					// Special case: SetAssetByPath behaviors have an extra 'TargetEntity' that these behaviors control outside of RC Actions.
					if (const URCSetAssetByPathBehaviour* SetAssetByPathBehavior = Cast<URCSetAssetByPathBehaviour>(Behavior))
					{
						if (SetAssetByPathBehavior->GetTargetEntityId() == InEntity)
						{
							bFoundEntity = true;
							break;
						}
					}

					if (!bFoundEntity)
					{
						for (const URCAction* Action : Behavior->ActionContainer->GetActions())
						{
							if (!Action)
							{
								continue;
							}

							if (const URCPropertyIdAction* IdentityAction = Cast<URCPropertyIdAction>(Action))
							{
								if (const URemoteControlPropertyIdRegistry* IdentityRegistry = InRemoteControlPreset->GetPropertyIdRegistry())
								{
									if (IdentityRegistry->GetEntityIdsForPropertyId(IdentityAction->PropertyId).Contains(InEntity))
									{
										bFoundEntity = true;
										break;
									}

									// Explicit support for Sub-PropertyId
									for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainer : IdentityAction->PropertySelfContainer)
									{
										if (IdentityRegistry->GetEntityIdsForPropertyId(PropertyContainer.Key.PropertyId).Contains(InEntity))
										{
											bFoundEntity = true;
											break;
										}
									}
								}
							}
							else if (Action->ExposedFieldId == InEntity)
							{
								bFoundEntity = true;
								break;
							}
						}
					}

					if (bFoundEntity)
					{
						break;
					}
				}

				if (bFoundEntity)
				{
					OutControllers.Add(Controller);
				}
			}
		}
	}

	return true;
}
