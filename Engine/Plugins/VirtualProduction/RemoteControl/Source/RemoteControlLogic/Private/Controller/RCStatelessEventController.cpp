// Copyright Epic Games, Inc. All Rights Reserved.

#include "Controller/RCStatelessEventController.h"

#include "UObject/UnrealType.h"
#include "RCVirtualProperty.h"

bool FRCStatelessEventController::IsStatelessEventController(const URCVirtualPropertyBase* InController)
{
	if (InController->GetValueType() != EPropertyBagPropertyType::Struct)
	{
		return false;
	}

	const FStructProperty* StructProperty = CastField<FStructProperty>(InController->GetProperty());

	if (!StructProperty)
	{
		return false;
	}

	return StructProperty->Struct == FRCStatelessEventController::StaticStruct();
}
