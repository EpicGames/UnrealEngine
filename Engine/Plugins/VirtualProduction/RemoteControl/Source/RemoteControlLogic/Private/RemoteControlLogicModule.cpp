// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "RemoteControlPreset.h"
#include "Controller/RCControllerContainer.h"

/**
 * Remote Control Logic Module Class
 */
class FRemoteControlLogicModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	virtual void StartupModule() override
	{
		URemoteControlPreset::OnPostInitPropertiesRemoteControlPreset.AddRaw(this, &FRemoteControlLogicModule::OnPostInitPropertiesRemoteControlPreset);
	}

	virtual void ShutdownModule() override
	{
		URemoteControlPreset::OnPostInitPropertiesRemoteControlPreset.RemoveAll(this);
	}
	//~ End IModuleInterface

private:
	void OnPostInitPropertiesRemoteControlPreset(URemoteControlPreset* InPreset) const
	{
		// Create a new controller for preset if that does not exists
		if (InPreset && !InPreset->IsControllerContainerValid())
		{
			URCControllerContainer* ControllerContainer = NewObject<URCControllerContainer>(InPreset, NAME_None, RF_Transactional);
			ControllerContainer->PresetWeakPtr = InPreset;

			InPreset->SetControllerContainer(ControllerContainer);
		}
	}
};

IMPLEMENT_MODULE(FRemoteControlLogicModule, RemoteControlLogic);