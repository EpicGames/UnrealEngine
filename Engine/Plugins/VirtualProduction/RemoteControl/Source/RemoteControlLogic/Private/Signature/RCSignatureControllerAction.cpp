// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureControllerAction.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "RCSignature.h"
#include "RemoteControlPreset.h"

void FRCSignatureControllerAction::Initialize(const FRCSignatureField& InField)
{
	ControllerName = InField.FieldPath.GetFieldName();
}

bool FRCSignatureControllerAction::Execute(const FRCSignatureActionContext& InContext) const
{
	bool bControllerAdded;
	URCController* Controller = FindOrAddController(InContext, &bControllerAdded);
	if (!Controller)
	{
		return false;
	}

	bool bControllerModified = ExecuteControllerAction(InContext, Controller);

	// if controller was added or modified, trigger refresh
	if (bControllerAdded || bControllerModified)
	{
		InContext.Preset->OnVirtualPropertyContainerModified().Broadcast();
		return true;
	}

	return false;
}

#if WITH_EDITOR
FRCSignatureActionIcon FRCSignatureControllerAction::GetIcon() const
{
	FRCSignatureActionIcon Icon;
	Icon.StyleSetName = TEXT("EditorStyle");
	Icon.StyleName = TEXT("GraphEditor.StateMachine_16x");
	return Icon;
}
#endif

URCController* FRCSignatureControllerAction::FindOrAddController(const FRCSignatureActionContext& InContext, bool* bOutControllerAdded) const
{
	bool bControllerAdded = false;

	FName ControllerNameToUse = ControllerName;

	if (ControllerNameToUse.IsNone())
	{
		ControllerNameToUse = InContext.Property->FieldName;
	}

	URCController* Controller =  Cast<URCController>(InContext.Preset->GetController(ControllerNameToUse));

	if (Controller && !IsControllerCompatible(InContext, Controller))
	{
		// Unset controller if it's not compatible
		Controller = nullptr;
	}

	// Create a controller if it wasn't found or one was found but deemed incompatible
	if (!Controller && bCreateControllerIfNotFound)
	{
		FPropertyBagPropertyDesc Desc;

		if (MakeControllerDesc(InContext, Desc))
		{
			Controller = CastChecked<URCController>(InContext.Preset->AddController(URCController::StaticClass(), Desc.ValueType, const_cast<UObject*>(Desc.ValueTypeObject.Get()), ControllerNameToUse));
			Controller->DisplayIndex = InContext.Preset->GetNextControllerDisplayIndex();

#if WITH_EDITOR
			for (const FPropertyBagPropertyDescMetaData& MetaData : Desc.MetaData)
			{
				Controller->SetMetadataValue(MetaData.Key, MetaData.Value);
			}
#endif

			bControllerAdded = true;
			OnControllerCreated(InContext, Controller);
		}
	}

	if (bOutControllerAdded)
	{
		*bOutControllerAdded = bControllerAdded;
	}

	return Controller;
}

URCAction* FRCSignatureControllerAction::FindActionInBehavior(const FRCSignatureActionContext& InContext, const URCBehaviour& InBehavior) const
{
	if (InBehavior.ActionContainer)
	{
		return InBehavior.ActionContainer->FindActionByFieldId(InContext.Property->GetId());
	}
	return nullptr;
}
