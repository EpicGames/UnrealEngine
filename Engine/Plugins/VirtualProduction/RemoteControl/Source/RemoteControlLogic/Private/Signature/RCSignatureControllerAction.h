// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureAction.h"
#include "RCSignatureControllerAction.generated.h"

class URCAction;
class URCBehaviour;
class URCController;
struct FPropertyBagPropertyDesc;

USTRUCT(meta=(Hidden))
struct FRCSignatureControllerAction : public FRCSignatureAction
{
	GENERATED_BODY()

	//~ Begin FRCSignatureAction
	virtual void Initialize(const FRCSignatureField& InField) override;
	virtual bool Execute(const FRCSignatureActionContext& InContext) const override;
#if WITH_EDITOR
	virtual FRCSignatureActionIcon GetIcon() const override;
#endif
	//~ End FRCSignatureAction

	/**
	 * Determines whether a found Controller is compatible for re-use
	 * @param InContext context of the action (preset, exposed property, etc)
	 * @param InController the existing controller to check
	 * @return true if compatible
	 */
	virtual bool IsControllerCompatible(const FRCSignatureActionContext& InContext, URCController* InController) const
	{
		return false;
	}

	/**
	 * Retrieves the Property Desc to use for creating a new controller
	 * @param InContext context of the action (preset, exposed property, etc)
	 * @param OutDesc the property description to use for creating the controller
	 * @return true if the property desc was filled in
	 */
	virtual bool MakeControllerDesc(const FRCSignatureActionContext& InContext, FPropertyBagPropertyDesc& OutDesc) const
	{
		return false;
	}

	/**
	 * Called after a controller has been created
	 * @param InContext context of the action (preset, exposed property, etc)
	 * @param InController the controller created
	 */
	virtual void OnControllerCreated(const FRCSignatureActionContext& InContext, URCController* InController) const
	{
	}

	/**
	 * The main logic of the action
	 * @param InContext context of the action (preset, exposed property, etc)
	 * @param InController the controller to use for the action
	 * @return true if the action modified the controller or some other object; false if nothing happened
	 */
	virtual bool ExecuteControllerAction(const FRCSignatureActionContext& InContext, URCController* InController) const
	{
		return false;
	}

	URCController* FindOrAddController(const FRCSignatureActionContext& InContext, bool* bOutControllerAdded = nullptr) const;

	URCAction* FindActionInBehavior(const FRCSignatureActionContext& InContext, const URCBehaviour& InBehavior) const;

	UPROPERTY(EditAnywhere, Category = "Controller")
	FName ControllerName;

	UPROPERTY(EditAnywhere, Category = "Controller")
	bool bCreateControllerIfNotFound = true;
};
