// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureControllerBindAction.h"
#include "Action/Bind/RCPropertyBindAction.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBindNode.h"
#include "Controller/RCController.h"
#include "Controller/RCControllerUtilities.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "RCSignature.h"
#include "RemoteControlField.h"

#if WITH_EDITOR
#include "Engine/Texture2D.h"
#endif

bool FRCSignatureControllerBindAction::IsSupported(const FRCSignatureField& InField) const
{
	return UE::RCControllers::CanCreateControllerFromPropertyDesc(InField.PropertyDesc);
}

bool FRCSignatureControllerBindAction::IsControllerCompatible(const FRCSignatureActionContext& InContext, URCController* InController) const
{
	return URCBehaviourBind::CanHaveActionForField(InController, InContext.Property.ToSharedRef(), /*bAllowNumericInputAsStrings*/true);
}

bool FRCSignatureControllerBindAction::MakeControllerDesc(const FRCSignatureActionContext& InContext, FPropertyBagPropertyDesc& OutDesc) const
{
	EPropertyBagPropertyType ValueType = EPropertyBagPropertyType::None;
	UObject* ValueTypeObject = nullptr;

	if (!URCBehaviourBind::GetPropertyBagTypeFromFieldProperty(InContext.Property->GetProperty(), ValueType, ValueTypeObject))
	{
		return false;
	}

#if WITH_EDITOR
	FString CustomControllerName;

	// Custom controller.This might require some interfacing for more customizations.
	// This was taken from SRCControllerPanel::OnAddControllerClicked
	if (ValueTypeObject == UTexture::StaticClass() || ValueTypeObject == UTexture2D::StaticClass())
	{
		if (ValueType == EPropertyBagPropertyType::String)
		{
			ValueTypeObject = nullptr;
			CustomControllerName = UE::RCCustomControllers::CustomTextureControllerName;
		}
	}
#endif

	OutDesc.ValueType = ValueType;
	OutDesc.ValueTypeObject = ValueTypeObject;

#if WITH_EDITOR
	// Add metadata to this controller, if this is a custom controller
	if (!CustomControllerName.IsEmpty())
	{
		const TMap<FName, FString>& CustomControllerMetaData = UE::RCCustomControllers::GetCustomControllerMetaData(CustomControllerName);
		OutDesc.MetaData.Reserve(CustomControllerMetaData.Num());

		for (const TPair<FName, FString>& MetaData : CustomControllerMetaData)
		{
			OutDesc.MetaData.Emplace(MetaData.Key, MetaData.Value);
		}
	}
#endif

	return true;
}

void FRCSignatureControllerBindAction::OnControllerCreated(const FRCSignatureActionContext& InContext, URCController* InController) const
{
	// Transfer property value from Exposed Property to the New Controller.
	// The goal is to keep values synced for a Controller newly created via "Auto Bind"
	URCBehaviourBind::CopyPropertyValueToController(InController, InContext.Property.ToSharedRef());
}

bool FRCSignatureControllerBindAction::ExecuteControllerAction(const FRCSignatureActionContext& InContext, URCController* InController) const
{
	TSharedRef<FRemoteControlProperty> RCProperty = InContext.Property.ToSharedRef();

	// Controller is already compatible with URCBehaviourBind::CanHaveActionForField when allowing string numeric input
	// (see FRCSignatureControllerBindAction::IsControllerCompatible)
	// Numeric conversion is required if a numeric action can't be added 
	const bool bRequiresNumericConversion = !URCBehaviourBind::CanHaveActionForField(InController, RCProperty, /*bAllowNumericInputAsStrings*/false);

	URCBehaviourBind* BindBehavior = nullptr;

	for (URCBehaviour* Behavior : InController->GetBehaviors())
	{
		URCBehaviourBind* CurrentBindBehavior = Cast<URCBehaviourBind>(Behavior);

		// In case numeric conversion is required we might have multiple Bind behaviours with different settings,
		// so we do not break in case there is at least one Bind behavior with a matching clause.
		// If not, we will create a new Bind behavior with the required setting (see below)
		if (CurrentBindBehavior && (!bRequiresNumericConversion || CurrentBindBehavior->AreNumericInputsAllowedAsStrings()))
		{
			BindBehavior = CurrentBindBehavior;
			break;
		}
	}

	if (!BindBehavior)
	{
		BindBehavior = Cast<URCBehaviourBind>(InController->AddBehaviour(URCBehaviourBindNode::StaticClass()));

		// If this is a new Bind Behavior and we are attempting to link unrelated (but compatible types), via numeric conversion, then we apply the bAllowNumericInputAsStrings flag
		BindBehavior->SetAllowNumericInputAsStrings(bRequiresNumericConversion);
	}

	if (!ensure(BindBehavior))
	{
		return false;
	}

	URCAction* BindAction = FindActionInBehavior(InContext, *BindBehavior);
	if (!BindAction)
	{
		BindAction = BindBehavior->AddPropertyBindAction(RCProperty);
	}

	check(BindAction);
	BindAction->Execute();

	return true;
}
