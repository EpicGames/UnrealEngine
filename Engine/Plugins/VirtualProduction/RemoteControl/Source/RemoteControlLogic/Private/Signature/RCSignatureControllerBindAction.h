// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureControllerAction.h"
#include "RCSignatureControllerBindAction.generated.h"

USTRUCT(DisplayName="Controller Bind Action")
struct FRCSignatureControllerBindAction : public FRCSignatureControllerAction
{
	GENERATED_BODY()

	//~ Begin FRCSignatureAction
	virtual bool IsSupported(const FRCSignatureField& InField) const override;
	//~ End FRCSignatureAction

	//~ Begin FRCSignatureControllerAction
	virtual bool IsControllerCompatible(const FRCSignatureActionContext& InContext, URCController* InController) const override;
	virtual bool MakeControllerDesc(const FRCSignatureActionContext& InContext, FPropertyBagPropertyDesc& OutDesc) const override;
	virtual void OnControllerCreated(const FRCSignatureActionContext& InContext, URCController* InController) const override;
	virtual bool ExecuteControllerAction(const FRCSignatureActionContext& InContext, URCController* InController) const override;
	//~ End FRCSignatureControllerAction
};
