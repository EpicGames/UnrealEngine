// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RemoteControlLogicTestData.generated.h"

UCLASS()
class URemoteControlLogicTestData : public UObject
{
public:
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RemoteControlTest)
	FColor Color = FColor(1,2,3,4 );

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = RemoteControlTest)
	int32 TestInt = 0;
	
	UFUNCTION(BlueprintCallable, Category = RemoteControlTest)
	void TestIntFunction()
	{
		TestInt++;
	}
};
