// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "CoreMinimal.h"
#include "IRemoteControlModule.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlLogicTestData.h"
#include "RemoteControlPreset.h"
#include "Action/RCAction.h"
#include "Action/RCActionContainer.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/RCBehaviour.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBindNode.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditionalNode.h"
#include "Behaviour/Builtin/RCBehaviourOnValueChangedNode.h"
#include "Controller/RCController.h"
#include "Misc/AutomationTest.h"
#include "UObject/StrongObjectPtr.h"

#define PROP_NAME(Class, Name) GET_MEMBER_NAME_CHECKED(Class, Name)
#define GET_TEST_PROP(PropName) URemoteControlLogicTestData::StaticClass()->FindPropertyByName(PROP_NAME(URemoteControlLogicTestData, PropName))
 
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FRemoteControlLogicTest, "Plugins.RemoteControl.Logic.Runtime", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FRemoteControlLogicTest::RunTest(const FString& Parameters)
{
	// 1. Create Preset
	TStrongObjectPtr<URemoteControlPreset> Preset{ NewObject<URemoteControlPreset>() };
	TStrongObjectPtr<URemoteControlLogicTestData> TestObject{ NewObject<URemoteControlLogicTestData>() };
 
	// 1.1 Copy Int test property value
	const int32 TestIntValue = TestObject->TestInt;
	
	// 2. Expose Fields
	// 2.1 Expose Properties
	const TSharedRef<FRemoteControlProperty> RCProp1 = Preset->ExposeProperty(TestObject.Get(), FRCFieldPathInfo{GET_TEST_PROP(Color)->GetName()}).Pin().ToSharedRef();
 
	// 2.2 Expose Functions
	// UObject* Object, UFunction* Function, FRemoteControlPresetExposeArgs Args
	UFunction* TestIntFunction = TestObject->GetClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(URemoteControlLogicTestData, TestIntFunction));
	const TSharedRef<FRemoteControlFunction> RCFunc1 = Preset->ExposeFunction(TestObject.Get(), TestIntFunction).Pin().ToSharedRef();
 
	// 3. Add Controllers
	// 3.1.1 Create Float Property
	URCController* FloatController =  Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Float, nullptr, TEXT("FloatController")) );
	// 3.1.2 Create Second Float Property
	URCController* FloatController1 = Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Float, nullptr, TEXT("FloatController1")) );
	// 3.1.3 Create Bool Property
	URCController* BoolController = Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Bool, nullptr, TEXT("BoolController")));
	// 3.1.4 Create Int Controller
	URCController* IntController =  Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Int32, nullptr, TEXT("IntController")));
	// 3.1.5 Create String Controller
	URCController* StrController =  Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::String, nullptr, TEXT("StrController")));
	// 3.1.6 Create Vector Struct Controller
	URCController* VectorStructController =  Cast<URCController>(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Struct, TBaseStructure<FVector>::Get(), TEXT("VectorProperty")));
 
	// 3.2 Set Float Property Value to controller
	constexpr float FloatValue = 0.65f;
	TestTrue(TEXT("Should Set Float"), FloatController->SetValueFloat(FloatValue) == true);

	// 3.3 Set Vector Property Value to container
	const FVector VectorValue = FVector(5.f, 6.f, 7.f);
	TestTrue(TEXT("Should Set Float"), VectorStructController->SetValueVector(VectorValue) == true);

	FVector OutVectorValue = FVector(0.f);
	TestTrue(TEXT("Should Set Float"), VectorStructController->GetValueVector(OutVectorValue) == true);
	TestEqual(TEXT("Vectors should be the same"), VectorValue, OutVectorValue);
	
	// 4. Add Behaviour To Properties
	// 4.1 Add Conditional Behaviour
	URCBehaviour* FloatControllerBehaviour = FloatController->AddBehaviour(URCBehaviourConditionalNode::StaticClass());
	URCBehaviourConditional* ConditionalBehaviour = Cast<URCBehaviourConditional>(FloatControllerBehaviour);
	float IsEqualBehaviourFloatValue = 0.f;
 
	// @todo: Add Conditional Behaviour Unit Tests here.
	// ...
 
	// 4.2 Add Other behaviours
	const URCBehaviour* IntControllerBehaviour = FloatController->AddBehaviour(URCBehaviourOnValueChangedNode::StaticClass());
	const URCBehaviour* StrControllerBehaviour = StrController->AddBehaviour(URCBehaviourBindNode::StaticClass());
 
	// 5 Add actions
	// 5.1 Add Float Controller Actions
	URCPropertyAction* FloatControllerBehaviourAction = Cast<URCPropertyAction> (FloatControllerBehaviour->ActionContainer->AddAction(RCProp1));

	FColor OutColorValue = FColor();
	TestTrue(TEXT("Should Get Color"), FloatControllerBehaviourAction->PropertySelfContainer->GetValueColor(OutColorValue) == true);
	TestEqual(TEXT("Action value should be equal expose property value"), OutColorValue, TestObject->Color);
	
	// 5.3 Set new Action Value
	constexpr FColor SecColorValue(5,4,3,2);
	TestTrue(TEXT("Should Set Color"), FloatControllerBehaviourAction->PropertySelfContainer->SetValueColor(SecColorValue));
	
	// 5.4 Add Function Controller Actions
	URCFunctionAction* FloatControllerBehaviourAction1 = Cast<URCFunctionAction>(FloatControllerBehaviour->ActionContainer->AddAction(RCFunc1));
	URCPropertyAction* StrControllerBehaviourAction = Cast<URCPropertyAction> (StrControllerBehaviour->ActionContainer->AddAction(RCProp1));
	FColor StringControllerColorValue(7,8,9,10);
	TestTrue(TEXT("Should Set Color"), StrControllerBehaviourAction->PropertySelfContainer->SetValueColor(StringControllerColorValue));

	// 5.41 Add Condition to behaviour
	FRCBehaviourCondition ConditionProp;
	ConditionProp.ConditionType = ERCBehaviourConditionType::IsEqual;
	ConditionProp.Comparand = NewObject<URCVirtualPropertySelfContainer>();
	ConditionProp.Comparand->AddProperty(FName(TEXT("Prop_Condition")), EPropertyBagPropertyType::Float);
	ConditionProp.Comparand->SetValueFloat(0.65f);
	ConditionalBehaviour->Conditions.Add(FloatControllerBehaviourAction, ConditionProp);
	FRCBehaviourCondition ConditionFunc;
	ConditionFunc.ConditionType = ERCBehaviourConditionType::IsEqual;
	ConditionFunc.Comparand = NewObject<URCVirtualPropertySelfContainer>();
	ConditionFunc.Comparand->AddProperty(FName(TEXT("Func_Condition")), EPropertyBagPropertyType::Float);
	ConditionFunc.Comparand->SetValueFloat(0.65f);
	ConditionalBehaviour->Conditions.Add(FloatControllerBehaviourAction1, ConditionFunc);

	// 5.5 It should not add same action for second time
	URCAction* StrControllerBehaviourAction1 = StrControllerBehaviour->ActionContainer->AddAction(RCProp1);
	
	// 6. Execute Behaviours
 
	// 6.1 Check Exposed Property Value before execute
	FRCObjectReference ObjectRef;
	IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, RCProp1->GetBoundObjects()[0], RCProp1->FieldPathInfo.ToString(), ObjectRef);
	const uint8* RCProp1ValuePtr = RCProp1->GetProperty()->ContainerPtrToValuePtr<uint8>(ObjectRef.ContainerAdress);
	FMemory::Memcpy(&OutColorValue, RCProp1ValuePtr, RCProp1->GetProperty()->GetSize());
	TestNotEqual(TEXT("The exposed property should not be updated before behaviour"), OutColorValue, SecColorValue);
 
	// 6.2 Execute behaviours
	FloatController->ExecuteBehaviours();
	
	// 6.3 Check Exposed Property Value after execute
	FMemory::Memcpy(&OutColorValue, RCProp1ValuePtr, RCProp1->GetProperty()->GetSize());
	TestEqual(TEXT("The exposed property should be updated after behaviour"), OutColorValue, SecColorValue);
	
	// 6.4 Check test function, it should call the function with default arguments and increase int test value
	TestEqual(TEXT("The after calling test function TestInt should be =+ 1"), TestIntValue + 1, TestObject->TestInt);
 
	// 6.5 Execute and test string controller 
	StrController->ExecuteBehaviours();
	FMemory::Memcpy(&OutColorValue, RCProp1ValuePtr, RCProp1->GetProperty()->GetSize());
	TestEqual(TEXT("The exposed property should be updated after behaviour"), OutColorValue, StringControllerColorValue);
	
	// 7. Remove Actions
	int32 ActionNum = FloatControllerBehaviour->GetNumActions();
	FloatControllerBehaviour->ActionContainer->RemoveAction(FloatControllerBehaviourAction1);
	FloatControllerBehaviour->ActionContainer->RemoveAction(RCProp1->GetId());
	TestEqual(TEXT("After remove 2 action the count should be =-2"), FloatControllerBehaviour->GetNumActions(), ActionNum - 2);
	FloatControllerBehaviour->ActionContainer->EmptyActions();
	TestEqual(TEXT("After exmpty actions the count should be = 0"), FloatControllerBehaviour->GetNumActions(), 0);
 
	// 7. Remove Actions
	ActionNum = StrControllerBehaviour->GetNumActions();
	StrControllerBehaviour->ActionContainer->RemoveAction(StrControllerBehaviourAction);
	TestEqual(TEXT("After empty actions the count should be = 0"), StrControllerBehaviour->GetNumActions(), 0);
 
	// 8. Remove Behaviour by ID and by Pointer
	const int32 BehaviourNum = FloatController->GetBehaviors().Num();
	FloatController->RemoveBehaviour(FloatControllerBehaviour);
	FloatController->RemoveBehaviour(IntControllerBehaviour->Id);
	TestEqual(TEXT("After empty 2 behaviours the count should be =-2"), FloatController->GetBehaviors().Num(), BehaviourNum - 2);
	FloatController->EmptyBehaviours();
	TestEqual(TEXT("After empty actions the count should be = 0"), FloatController->GetBehaviors().Num(), 0);
 
	// 9. Remove Controllers
	const int32 NumPropertiesBeforeRemove = Preset->GetNumControllers();
	float OutFloatValue = 0.f;
	float OutFloatValue1 = 0.f;
	FloatController->GetValueFloat(OutFloatValue);
	Preset->RemoveController(FloatController1->PropertyName);
	FloatController->GetValueFloat(OutFloatValue1);
	TestEqual(TEXT("After remove properties the old value should be the same"), OutFloatValue1, OutFloatValue);
	Preset->RemoveController(FloatController->PropertyName);
	Preset->RemoveController(BoolController->PropertyName);
	Preset->RemoveController(IntController->PropertyName);
	TestEqual(TEXT("After remove 4 properties the count should be =-4"), Preset->GetNumControllers(), NumPropertiesBeforeRemove - 4);
	Preset->ResetControllers();
	TestEqual(TEXT("After empty controllers could should be equal 0"), Preset->GetNumControllers(), 0);
 
	return true;
} 