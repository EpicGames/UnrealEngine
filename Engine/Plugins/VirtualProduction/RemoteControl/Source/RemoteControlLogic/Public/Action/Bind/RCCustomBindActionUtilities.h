// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Texture2D.h"
#include "Templates/SharedPointer.h"

struct FRemoteControlProperty;

namespace UE::RCCustomBindActionUtilities
{
	REMOTECONTROLLOGIC_API UTexture2D* LoadTextureFromPath(const FString& InPath);
	
	/**
	 * Loads the texture found at the specified path, and uses it to set a Texture value in the specified RC Property
	 * Useful e.g. for Custom External Texture Controller
	 */
	REMOTECONTROLLOGIC_API void SetTexturePropertyFromPath(const TSharedRef<FRemoteControlProperty>& InRemoteControlEntityAsProperty, const FString& InPath);
};
