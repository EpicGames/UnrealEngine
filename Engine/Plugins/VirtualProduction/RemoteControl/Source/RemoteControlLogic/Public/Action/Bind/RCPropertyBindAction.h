// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCPropertyAction.h"
#include "RCPropertyBindAction.generated.h"

class URCController;

/**
 * Property Bind Action specifically for Bind Behaviour
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCPropertyBindAction : public URCPropertyAction
{
	GENERATED_BODY()

public:
	URCPropertyBindAction() = default;

	//~ Begin URCAction interface
	virtual void Execute() const override;
	//~ End URCAction interface

	/** The Controller that drives us */
	UPROPERTY()
	TObjectPtr<URCController> Controller;
};
