// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCPropertyAction.h"
#include "RCSetAssetByPathActionNew.generated.h"

class FString;
class URCSetAssetByPathBehaviorNew;
class UTexture;
struct FRemoteControlProperty;

/**
 * Property Set Asset By Path Action (New) specifically for Set Asset By Path Behaviour (New)
 */
UCLASS(MinimalAPI)
class URCSetAssetByPathActionNew : public URCPropertyAction
{
	GENERATED_BODY()

public:	
	//~ Begin URCAction interface
	REMOTECONTROLLOGIC_API virtual void Execute() const override;
	//~ End URCAction interface

	REMOTECONTROLLOGIC_API URCSetAssetByPathBehaviorNew* GetSetAssetByPathBehavior() const;

	/** Internal -> Asset. External -> File. External is only supported by textures. */
	REMOTECONTROLLOGIC_API bool IsInternal() const;

private:
	/** Gets the path to the asset. */
	FString GetPath() const;

	/** Updates property with a existing asset. */
	bool UpdateInternal();

	/** Updates property with an imported, external texture. */
	bool UpdateExternal();

	/** Auxiliary Function which sets the given InSetterObject Asset onto the Exposed Property. */
	bool SetInternalAsset(UObject* InSetterObject);

	/** Given an External Path towards a external location, loads the asset associated and places it onto an object */
	bool SetExternalAsset(const FString& InExternalPath);

	/** Auxiliary Function to apply a Texture onto a given Property */
	bool SetTextureFromPath(TSharedPtr<FRemoteControlProperty> InRCPropertyToSet, const FString& InFileName);
};
