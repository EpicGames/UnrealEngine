// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RCAction.generated.h"

class URCBehaviour;
class URemoteControlPreset;

/**
 * Base class for remote control action 
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCAction : public UObject
{
	GENERATED_BODY()

public:
	/** Execute action */
	virtual void Execute() const {}

	FName GetExposedFieldLabel() const;

	virtual void CopyTo(URCAction* Action) {}

	/** Returns the parent Behaviour associated with this Action */
	URCBehaviour* GetParentBehaviour() const;

	/** Invoked when this Action item value change */
	void NotifyActionValueChanged();

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap);

	virtual void UpdateEntityUsage(const URCBehaviour* InBehavior, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const;

public:
	/** Exposed Property or Function field Id*/
	UPROPERTY()
	FGuid ExposedFieldId;

	/** Action Id */
	UPROPERTY()
	FGuid Id;

	/** Reference to preset */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> PresetWeakPtr;
};