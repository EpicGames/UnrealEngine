// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"

#include "RCActionContainer.generated.h"

class URemoteControlPreset;
class URCAction;
class URCBehaviour;
class URCController;
class URCFunctionAction;
class URCPropertyIdAction;
class URCPropertyAction;
class URCPropertyBindAction;

struct FRemoteControlField;
struct FRemoteControlFunction;
struct FRemoteControlProperty;

typedef TFunction<bool(const TSet<TObjectPtr<URCAction>>& Actions)> TRCActionUniquenessTest;

DECLARE_MULTICAST_DELEGATE(FOnActionsListModified);

/**
 * Container for created actions
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCActionContainer : public UObject
{
	GENERATED_BODY()

public:
	/** Execute container actions */
	void ExecuteActions();

	TRCActionUniquenessTest GetDefaultActionUniquenessTest(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Add remote control identity action. */
	URCAction* AddAction();

	/** Add remote control identity action. */
	URCAction* AddAction(FName InFieldId);

	/** Add remote control property action  */
	URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Add remote control property action  */
	URCAction* AddAction(TRCActionUniquenessTest InUniquenessTest, const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Find Action by given exposed filed id */
	URCAction* FindActionByFieldId(const FGuid InFieldId) const;

	/** Find Action by given remote control field */
	URCAction* FindActionByField(const TSharedRef<const FRemoteControlField> InRemoteControlField) const;
	
	/** Remove action by exposed field Id */
	virtual int32 RemoveAction(const FGuid InExposedFieldId);

	/** Remove Action by given action UObject */
	virtual int32 RemoveAction(URCAction* InAction);

	/** Empty action set */
	void EmptyActions();

	/** Register a newly created Action with the Action container, provides Undo/Redo support. */
	void AddAction(URCAction* NewAction);

	/** Retrieves the actions present in this container */
	const TSet<TObjectPtr<URCAction>>& GetActions() const { return Actions; }

	/** Retrieves all the Property Actions present in this container */
	TArray<const URCPropertyAction*> GetPropertyActions() const;

	//~ Begin UObject
#if WITH_EDITOR
	/** Called after applying a transaction to the object. Used to broadcast Undo related container changes to UI */
	virtual void PostEditUndo() override;
#endif
	//~ End UObject

	/** Set of child action container */
	UPROPERTY()
	TSet<TObjectPtr<URCActionContainer>> ActionContainers;

	/** Reference to Preset */
	UPROPERTY()
	TWeakObjectPtr<URemoteControlPreset> PresetWeakPtr;

	/** Delegate that notifies changes to the list of actions to a single listener*/
	FOnActionsListModified OnActionsListModified;

	/** Derive the parent Behaviour holding this Action Container */
	URCBehaviour* GetParentBehaviour();

	/** Call the given function on all actions of the container. */
	void ForEachAction(TFunctionRef<void(URCAction*)> InActionFunction, bool bInRecursive);

	void UpdateEntityUsage(const URCBehaviour* InBehavior, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const;

private:
	/** Add remote control property action  */
	URCPropertyAction* AddPropertyAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty);

	/** Add remote control property function */
	URCFunctionAction* AddFunctionAction(const TSharedRef<const FRemoteControlFunction> InRemoteControlFunction);
	
	/** The list of Actions present in this container */
	UPROPERTY()
	TSet<TObjectPtr<URCAction>> Actions;
};
