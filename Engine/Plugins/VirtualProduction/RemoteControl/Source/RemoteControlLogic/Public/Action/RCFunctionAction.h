// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCAction.h"
#include "RCFunctionAction.generated.h"

/**
 * Function Action which is executes bound exposed function 
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCFunctionAction : public URCAction
{
	GENERATED_BODY()

public:
	//~ Begin URCAction Interface
	virtual void Execute() const override;
	//~ End URCAction Interface
};
