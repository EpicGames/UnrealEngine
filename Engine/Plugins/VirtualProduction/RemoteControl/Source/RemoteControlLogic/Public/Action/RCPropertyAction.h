// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCAction.h"
#include "RCPropertyAction.generated.h"

struct FRemoteControlProperty;
class URCController;
class URCVirtualPropertySelfContainer;

/**
 * Property Action which is set property value from the action to exposed property
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCPropertyAction : public URCAction
{
	GENERATED_BODY()

public:
	URCPropertyAction();
	
	//~ Begin URCAction interface
	virtual void Execute() const override;
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) override;
	//~ End URCAction interface

	/* Returns the FProperty underlying this Action */
	FProperty* GetProperty() const;

	/* Update the value of this Action with the current value of the RC property it is bound to */
	void UpdateValueBasedOnRCProperty() const;

public:
	/** Virtual Property Container */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertySelfContainer> PropertySelfContainer = nullptr;

	TSharedPtr<FRemoteControlProperty> GetRemoteControlProperty() const;
};