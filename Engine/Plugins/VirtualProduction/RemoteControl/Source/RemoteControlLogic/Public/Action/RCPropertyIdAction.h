// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCAction.h"
#include "RemoteControlFieldPath.h"
#include "RCPropertyIdAction.generated.h"

class URCVirtualPropertySelfContainer;

/**
 * Key for the PropertyContainer of the PropertyId Action
 */
USTRUCT()
struct FPropertyIdContainerKey
{
	GENERATED_BODY()

	FPropertyIdContainerKey() {}

	FPropertyIdContainerKey(FName InPropertyId, FName InContainerName)
		: PropertyId(InPropertyId)
		, ContainerName(InContainerName)
	{}

	/** PropertyId */
	UPROPERTY()
	FName PropertyId;

	/** Name of the container */
	UPROPERTY()
	FName ContainerName;

	/** Return hash value for this object, used when using this object as a key inside hashing containers. */
	friend uint32 GetTypeHash(const FPropertyIdContainerKey& InKey)
	{
		return HashCombine(GetTypeHash(InKey.PropertyId), GetTypeHash(InKey.ContainerName));
	}

	/** Comparison operator, used by hashing containers. */
	bool operator==(const FPropertyIdContainerKey& InOtherKey) const
	{
		return PropertyId == InOtherKey.PropertyId && ContainerName == InOtherKey.ContainerName;
	}

	bool operator<(const FPropertyIdContainerKey& InOtherKey) const
	{
		return PropertyId.Compare(InOtherKey.PropertyId) < 0;
	}
};

/**
 * Action for PropertyId
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCPropertyIdAction : public URCAction
{
	GENERATED_BODY()
	virtual ~URCPropertyIdAction() override;

public:
	//~ BEGIN : URCAction Interface
	virtual void Execute() const override;
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) override;
	//~ END : URCAction Interface
	
	//~ BEGIN : UObject Interface
	virtual void PostLoad() override;
#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR
	//~ END : URCAction Interface

	void UpdatePropertyId();

	void Initialize();

	void OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InGuid);

public:
	/** Holds the field identifier associated with this. */
	UPROPERTY()
	FName PropertyId = NAME_None;

	/** Virtual Property Container */
	UPROPERTY()
	TMap<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>> PropertySelfContainer;

	/** Cached Virtual Property Container */
	UPROPERTY()
	TMap<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>> CachedPropertySelfContainer;

	/** Contains the real property container */
	UPROPERTY()
	TMap<FGuid, TObjectPtr<URCVirtualPropertySelfContainer>> RealPropertySelfContainer;

private:
	static TSet<FName> AllowedStructNameToCopy;
};
