// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCActionContainer.h"
#include "Behaviour/RCBehaviour.h"

#include "RCBehaviourBind.generated.h"

enum class EPropertyBagPropertyType : uint8;
class URCVirtualPropertySelfContainer;
class URCAction;
class URCController;

/**
 * [Bind Behaviour]
 * 
 * Binds a given Controller with multiple exposed properties.
 * Any changes to the value of the controller are directly propagated to the linked properties by Bind Behaviour. 
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourBind : public URCBehaviour
{
	GENERATED_BODY()

public:
	URCBehaviourBind();
	
	//~ Begin URCBehaviour interface
	virtual void Initialize() override;

	/** Add a Logic action using a remote control field as input */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField) override;

	/** Whether we can create an action pertaining to a given remote control field for the current behaviour */
	virtual bool CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const override;

	/** Whether the behaviour support property id */
	virtual bool SupportPropertyId() const override { return false; }
	//~ End URCBehaviour interface

	/** Contains behaviour-independent validation logic for determining whether a Remote Controller property can be bound to a given Controller*/
	static bool CanHaveActionForField(URCController* InController, TSharedRef<const FRemoteControlField> InRemoteControlField, const bool bInAllowNumericInputAsStrings);

	/** Transfers the value of a given Remote Control Property to a Controller.
	* Used for Auto Bind workflow where matching Controller types for bind are automatically created from a given Remote Control Property */
	static bool CopyPropertyValueToController(URCController* InController, TSharedRef<const FRemoteControlProperty> InRemoteControlProperty);

	/** Add an action specifically for Bind Behaviour */
	URCPropertyBindAction* AddPropertyBindAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty);

	/** GetPropertyBagTypeFromFieldProperty
	*
	* Given a Remote Control Exposed Property type this function determines the appropriate EPropertyBagType and struct object required (if any) 
	* In the Logic realm we use a single type like (eg: String / Int) to represent various related types (String/Name/Text, Int32, Int64, etc)
	* For this reason explicit mapping conversion is required between a given FProperty type and the desired Controller (Property Bag) type */
	static bool GetPropertyBagTypeFromFieldProperty(const FProperty* InProperty, EPropertyBagPropertyType& OutPropertyBagType, UObject*& OutStructObject);

	/** Indicates whether we support binding of strings to numeric fields and vice versa.
	* This flag determines the list of bindable properties the user sees in the "Add Action" menu*/
	bool AreNumericInputsAllowedAsStrings() const { return bAllowNumericInputAsStrings; }
	void SetAllowNumericInputAsStrings(bool Value) { bAllowNumericInputAsStrings = Value; }
private:
	bool bAllowNumericInputAsStrings = false;
};
