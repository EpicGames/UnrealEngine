// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourBindNode.generated.h"

/**
 * Behaviour Node class for Bind Behaviour
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourBindNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	
	URCBehaviourBindNode();

	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	void OnPassed(URCBehaviour* InBehaviour) const;

	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode interface
};
