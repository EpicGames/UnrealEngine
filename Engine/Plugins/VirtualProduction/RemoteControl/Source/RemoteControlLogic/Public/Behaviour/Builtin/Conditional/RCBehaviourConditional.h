// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCActionContainer.h"
#include "Behaviour/RCBehaviour.h"

#include "RCBehaviourConditional.generated.h"

class URCVirtualPropertySelfContainer;
class URCAction;

/**
 * Condition Typess
 */
UENUM()
enum class ERCBehaviourConditionType : uint8
{
	IsEqual,
	IsGreaterThan,
	IsLesserThan,
	IsGreaterThanOrEqualTo,
	IsLesserThanOrEqualTo,
	Else,
	None
};

/**
 * Struct representing a single Condition
 * 
 * Serialization and GC support are automatically provided by standard container types, as all relevant fields have been marked as UPROPERTY()
 */
USTRUCT()
struct FRCBehaviourCondition
{
	GENERATED_BODY()

	FRCBehaviourCondition() {}
	FRCBehaviourCondition(const ERCBehaviourConditionType InConditionType, const TObjectPtr<URCVirtualPropertySelfContainer> InComparand)
		: ConditionType(InConditionType), Comparand(InComparand) {}

	/** The type of condition to be used for comparison*/
	UPROPERTY()
	ERCBehaviourConditionType ConditionType = ERCBehaviourConditionType::None;

	/** The value with which to compare */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertySelfContainer> Comparand;
};

/**
 * [Conditional Behaviour]
 * 
 * Supports unique conditions per action that allow powerful Logic & Rigging of multiple usecases
 * including Tricode based multi-mapping (selector behaviour), simulation of Hide On Empty behaviours, etc.
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourConditional : public URCBehaviour
{
	GENERATED_BODY()

public:
	URCBehaviourConditional();
	
	//~ Begin URCBehaviour interface
	virtual void Initialize() override;

	/** Duplicates an Action belonging to us into a given target behaviour*/
	virtual URCAction* DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour) override;

	/** Called when an action value changed */
	virtual void NotifyActionValueChanged(URCAction* InChangedAction) override;
	//~ End URCBehaviour interface

	/** ~ OnActionAdded ~
	* Invoked after a new condition action has just been created.
	* Used to link the newly created action with its condition data */
	void OnActionAdded(URCAction* Action, const ERCBehaviourConditionType InConditionType, const TObjectPtr<URCVirtualPropertySelfContainer> InComparand);

	/** Add a Logic action using a remote control field as input */
	URCAction* AddConditionalAction(const TSharedRef<const FRemoteControlField> InRemoteControlField, const ERCBehaviourConditionType InConditionType, const TObjectPtr<URCVirtualPropertySelfContainer> InComparand);

	/** Whether we can create an action pertaining to a given remote control field for the current behaviour */
	virtual bool CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const override;

	/** Whether the behaviour support property id */
	virtual bool SupportPropertyId() const override { return true; }

	/** User-friendly text representation of a condition enum. Used to display comparator info in the Actions table */
	FText GetConditionTypeAsText(ERCBehaviourConditionType ConditionType) const;

protected:
	//~ Begin URCBehaviour interface
	/** Execute all the action if not provided a valid Action otherwise will only execute the given action */
	virtual void ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute) override;
	//~ End URCBehaviour interface

public:
	/** Data storage for Actions and related Conditions; stored as a mapping of Action object and associated condition data
	*  Each Action is associated with a unique condition (for the Conditional Behaviour) */
	UPROPERTY()
	TMap<TObjectPtr<URCAction>, FRCBehaviourCondition> Conditions;

	/** Virtual property used to build the Comparand - i.e. the property with which the Controller will be compared for a given condition*/
	UPROPERTY()
	TObjectPtr<URCVirtualPropertySelfContainer> Comparand;
};
