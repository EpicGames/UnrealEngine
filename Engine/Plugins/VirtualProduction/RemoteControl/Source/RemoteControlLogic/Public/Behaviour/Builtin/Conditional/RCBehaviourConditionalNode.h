// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourConditionalNode.generated.h"

/**
 * Compares given property value with controller property value
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourConditionalNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	
	URCBehaviourConditionalNode();

	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	void OnPassed(URCBehaviour* InBehaviour) const;

	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode interface
};
