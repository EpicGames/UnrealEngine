// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviorSetAssetByPathNodeNew.generated.h"

/**
 * Takes the string as a path and goes on to search for the Asset it is connected to, setting it on the list of bound entities.
 */
UCLASS(MinimalAPI)
class URCBehaviorSetAssetByPathNodeNew : public URCBehaviourNode
{
	GENERATED_BODY()
	
public:
	REMOTECONTROLLOGIC_API URCBehaviorSetAssetByPathNodeNew();
	
	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	REMOTECONTROLLOGIC_API bool Execute(URCBehaviour* InBehavior) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	REMOTECONTROLLOGIC_API bool IsSupported(URCBehaviour* InBehavior) const override;

	REMOTECONTROLLOGIC_API UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode interface
};
