// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourSetAssetByPathNode.generated.h"

/**
 * Takes the string as a path and goes on to search for the Asset it is connected to, setting it to a Target Exposed Property.
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourSetAssetByPathNode : public URCBehaviourNode
{
	GENERATED_BODY()
	
public:
	URCBehaviourSetAssetByPathNode();
	
	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;

	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode interface
};
