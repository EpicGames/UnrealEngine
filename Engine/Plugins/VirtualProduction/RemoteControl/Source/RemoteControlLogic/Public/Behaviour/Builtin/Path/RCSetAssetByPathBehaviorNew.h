// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviour.h"
#include "Containers/UnrealString.h"
#include "HttpModule.h"
#include "RCVirtualProperty.h"
#include "RemoteControlEntity.h"

#include "RCSetAssetByPathBehaviorNew.generated.h"

class URCSetAssetByPathActionNew;
class URCUserDefinedStruct;
class URCVirtualPropertyBase;
class URCVirtualPropertyContainerBase;
class UTexture;
struct FRemoteControlProperty;

namespace UE::RemoteControl::Logic
{
	namespace SetAssetByPathNew
	{
		const FString ContentFolder = FString(TEXT("/Game/"));
		const FName SetAssetByPathBehaviour = FName(TEXT("Set Asset By Path"));
	}
}

USTRUCT()
struct FRCAssetPathElementNew
{
	GENERATED_BODY()

	FRCAssetPathElementNew()
		: bIsInput(false)
		, Path(TEXT(""))
	{}

	FRCAssetPathElementNew(bool bInIsInput, const FString& InPath)
		: bIsInput(bInIsInput)
		, Path(InPath)
	{}

	UPROPERTY(EditAnywhere, Category="Path")
	bool bIsInput;

	UPROPERTY(EditAnywhere, Category="Path")
	FString Path;
};

/** Struct to help generate Widgets for the DetailsPanel of the Behaviour */
USTRUCT()
struct FRCSetAssetPathNew
{
	GENERATED_BODY()

	FRCSetAssetPathNew()
	{
		// Add Empty index 0 when creating
		AssetPath.AddDefaulted();
	}
	~FRCSetAssetPathNew() = default;
	FRCSetAssetPathNew(const FRCSetAssetPathNew&) = default;
	FRCSetAssetPathNew(FRCSetAssetPathNew&&) = default;
	FRCSetAssetPathNew& operator=(const FRCSetAssetPathNew&) = default;
	FRCSetAssetPathNew& operator=(FRCSetAssetPathNew&&) = default;

	/** An Array of Strings holding the Path of an Asset, seperated in several String. Will concatenated back together later. */
	UPROPERTY(EditAnywhere, Category="Path")
	TArray<FRCAssetPathElementNew> AssetPath;
};

struct FRCSetAssetByPathBehaviorNewPathElement
{
	FString Path;
	URCVirtualPropertyBase* Controller = nullptr;
	FName ControllerName = NAME_None;
	bool bIsError = false;
};

/**
 * Custom behaviour for Set Asset By Path
 */
UCLASS(MinimalAPI)
class URCSetAssetByPathBehaviorNew : public URCBehaviour
{
	GENERATED_BODY()

public:
	//~ Begin URCBehaviour interface
	/** Add default entries to the path. */
	virtual void Initialize() override;
	/** Add a Logic action using a remote control field as input */
	REMOTECONTROLLOGIC_API virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField) override;
	REMOTECONTROLLOGIC_API virtual bool SupportPropertyId() const override { return false; }

#if WITH_EDITOR
	REMOTECONTROLLOGIC_API virtual bool CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const override;
#endif
	//~ End URCBehaviour interface

	/** Returns the Path as the completed version, including Token Inputs. */
	REMOTECONTROLLOGIC_API FString GetCurrentPath() const;

	/** 
	 * Returns an array of path elements, indicating whether the element was sourced from a controller and 
	 * the controller itself, if it is still valid.
	 */
	REMOTECONTROLLOGIC_API TArray<FRCSetAssetByPathBehaviorNewPathElement> GetPathElements() const;

	/** Called whenever a change has occured in the Slate */
	REMOTECONTROLLOGIC_API void RefreshPathArray();

	/** Add an action specifically for Set Asset By Path Behaviour */
	REMOTECONTROLLOGIC_API URCSetAssetByPathActionNew* AddPropertyBindAction(const TSharedRef<const FRemoteControlProperty> InRemoteControlProperty);
	
public:
	/** Struct holding the Path Information to help load Assets. */
	UPROPERTY()
	FRCSetAssetPathNew PathStruct;

	/** Bool used to help tell if the path given is an internal or external one. */
	UPROPERTY() 
	bool bInternal = true;
};
