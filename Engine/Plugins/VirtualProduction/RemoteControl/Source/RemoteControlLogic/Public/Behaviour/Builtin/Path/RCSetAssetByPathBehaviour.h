// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviour.h"
#include "HttpModule.h"
#include "RCVirtualProperty.h"
#include "RemoteControlEntity.h"

#include "RCSetAssetByPathBehaviour.generated.h"

class URCVirtualPropertyContainerBase;
class URCUserDefinedStruct;
class UTexture;

namespace SetAssetByPathBehaviourHelpers
{
	const FString ContentFolder = FString(TEXT("/Game/"));
	const FString InputToken = FString(TEXT("{INPUT}"));
	const FName TargetProperty = FName(TEXT("TargetProperty"));
	const FName DefaultInput = FName(TEXT("DefaultInput"));
	const FName SetAssetByPathBehaviour = FName(TEXT("Set Asset By Path"));
}

/** Info used to send request for a file */
struct FPendingFileRequest
{
	/**
	 * Constructor
	 */
	FPendingFileRequest(const FString& InFileName=FString(TEXT("")))
		:  FileName(InFileName)
	{
	}

	/**
	 * Equality op
	 */
	inline bool operator==(const FPendingFileRequest& Other) const
	{
		return FileName == Other.FileName;
	}

	/** File being operated on by the pending request */
	FString FileName;
};

USTRUCT()
struct FRCAssetPathElement
{
	GENERATED_BODY()

	FRCAssetPathElement()
		: bIsInput(false)
		, Path(TEXT(""))
	{}

	FRCAssetPathElement(bool bInIsInput, const FString& InPath)
		: bIsInput(bInIsInput)
		, Path(InPath)
	{}

	UPROPERTY(EditAnywhere, Category="Path")
	bool bIsInput;

	UPROPERTY(EditAnywhere, Category="Path")
	FString Path;
};

/** Struct to help generate Widgets for the DetailsPanel of the Behaviour */
USTRUCT()
struct FRCSetAssetPath
{
	GENERATED_BODY()

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// TODO [UE_DEPRECATED(5.4)] this rule of 5 is obligatory to avoid compile-time warning, delete them when the property is removed
	FRCSetAssetPath()
	{
		// Add Empty index 0 when creating
		AssetPath.AddDefaulted();
	}
	~FRCSetAssetPath() = default;
	FRCSetAssetPath(const FRCSetAssetPath&) = default;
	FRCSetAssetPath(FRCSetAssetPath&&) = default;
	FRCSetAssetPath& operator=(const FRCSetAssetPath&) = default;
	FRCSetAssetPath& operator=(FRCSetAssetPath&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** An Array of Strings holding the Path of an Asset, seperated in several String. Will concatenated back together later. */
	UPROPERTY(EditAnywhere, Category="Path")
	TArray<FRCAssetPathElement> AssetPath;

	/** An Array of Strings holding the Path of an Asset, seperated in several String. Will concatenated back together later. */
	UE_DEPRECATED(5.4, "This property is deprecated please use AssetPathArray instead")
	UPROPERTY()
	TArray<FString> PathArray_DEPRECATED;
};

/**
 * Custom behaviour for Set Asset By Path
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCSetAssetByPathBehaviour : public URCBehaviour
{
	GENERATED_BODY()
public:
	URCSetAssetByPathBehaviour();

	//~ Begin URCBehaviour interface
	virtual void Initialize() override;
	virtual bool SupportPropertyId() const override { return false; }
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) override;
	//~ End URCBehaviour interface

	virtual void PostLoad() override;

	/** Given an Input Path, sets the Target Exposed Property to the Asset. */
	bool SetAssetByPath(const FString& AssetPath, const FString& DefaultString);

	/** List of Supported Assets SetAssetByPath Logic Behaviour can set and change. */
	TArray<UClass*> GetSupportedClasses() const;

	/** Returns the Path as the completed version, including Token Inputs. */
	FString GetCurrentPath();

	/** Sets the target entity upon which the asset will be set upon. */
	void SetTargetEntity(const TSharedPtr<const FRemoteControlEntity>& InEntity);

	/** Returns a Pointer to the current Target Entity */
	TWeakPtr<const FRemoteControlEntity> GetTargetEntity() const;

	/** Returns the current Target Entity Id */
	const FGuid& GetTargetEntityId() const;

	/** Auxiliary Function to apply to update the Target Texture */
	void UpdateTargetEntity();

	/** Called whenever a change has occured in the Slate */
	void RefreshPathArray();
	
public:
	/** Pointer to property container */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertyContainerBase> PropertyInContainer;

	/** Pointer to the current Class the Asset will use to set the Targeted Exposed Object. */
	UPROPERTY()
	TObjectPtr<UClass> AssetClass = GetSupportedClasses()[0];

	/** Struct holding the Path Information to help load Assets. */
	UPROPERTY()
	FRCSetAssetPath PathStruct;

	/** Bool used to help tell if the path given is an internal or external one. */
	UPROPERTY()
	bool bInternal = true;

private:
	/** Targeted Property Id */
	UPROPERTY()
	FGuid TargetEntityId;

	/** Internal Targeted Property, used for any of the setter operations */
	TSharedPtr<const FRemoteControlEntity> TargetEntity;
	
private:
	/** Auxiliary Function which sets the given SetterObject Asset onto the Exposed Asset with the given PropertyString name. */
	bool SetInternalAsset(UObject* SetterObject);

	/** Given an External Path towards a external location, loads the asset associated and places it onto an object */
	bool SetExternalAsset(FString InExternalPath);

	/** Auxiliary Function to apply a Texture onto a given Property */
	bool SetTextureAsset(TSharedPtr<FRemoteControlProperty> InRemoteControlPropertyPtr, UTexture* InObject);

	/** Auxiliary Function to apply a Texture onto a given Property */
	bool SetTextureFromPath(TSharedPtr<FRemoteControlProperty> TexturePtr, FString& FileName);

	/** Http Request Process Handler */
	void ReadFileHttpHandler(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded, TSharedPtr<FRemoteControlProperty> InRCPropertyToSet);
};
