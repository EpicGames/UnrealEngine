// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourOnValueChangedNode.generated.h"

/**
 * On Value Changed Behaviour Node
 *
 * This behaviour always returns true and thus executes all actions any time the Controller value is modified
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourOnValueChangedNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:

	URCBehaviourOnValueChangedNode();

	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;
	//~ End URCBehaviourNode interface
};
