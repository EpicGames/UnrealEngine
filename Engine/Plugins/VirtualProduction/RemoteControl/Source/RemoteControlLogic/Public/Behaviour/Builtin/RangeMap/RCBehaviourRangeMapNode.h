// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourRangeMapNode.generated.h"

/**
 * Uses the given properties and values to limit the values of the Controller and perform lerp operations for the values.
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCBehaviourRangeMapNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	
	URCBehaviourRangeMapNode();

	//~ Begin URCBehaviourNode interface
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	void OnPassed(URCBehaviour* InBehaviour) const;

	UClass* GetBehaviourClass() const override;
	//~ End URCBehaviourNode interface
};
