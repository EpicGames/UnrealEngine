// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviour.h"
#include "RCVirtualProperty.h"
#include "RCRangeMapBehaviour.generated.h"

class URCAction;
class URCVirtualPropertyContainerBase;

USTRUCT()
struct REMOTECONTROLLOGIC_API FRCRangeMapInput
{
	GENERATED_BODY()

	FRCRangeMapInput() {}
	FRCRangeMapInput(URCVirtualPropertySelfContainer* InSelfContainer, URCVirtualPropertySelfContainer* InPropertyValue)
		: InputProperty(InSelfContainer), PropertyValue(InPropertyValue)
	{
	}

	/** The Value which we use represent the action based on a normalized step. */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertySelfContainer> InputProperty;

	/** The Property this Action holds and will be used for calculations for the lerp. */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertySelfContainer> PropertyValue;

	/** Returns the Step value from the virtual property, safely*/
	bool GetInputValue(double& OutValue) const;

	/** Set the Step value for the virtual property, return true if successful */
	bool SetInputValue(double InValue) const;
};

/**
 * Custom behaviour for Set Asset By Path Logic Behaviour 
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCRangeMapBehaviour : public URCBehaviour
{
	GENERATED_BODY()

public:
	/** Pointer to property container */
	UPROPERTY()
	TObjectPtr<URCVirtualPropertyContainerBase> PropertyContainer;

	/** Container holding all FRCRangeMapInputs correlating to each Action in the ActionContainer. */
	UPROPERTY()
	TMap<TObjectPtr<URCAction>, FRCRangeMapInput> RangeMapActionContainer;

public:
	URCRangeMapBehaviour();
	
	//~ Begin URCBehaviour interface
	virtual void Initialize() override;

	/** Duplicates an Action belonging to us into a given target behaviour*/
	virtual URCAction* DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour) override;

	/** Called when an action value changed */
	virtual void NotifyActionValueChanged(URCAction* InChangedAction) override;
	//~ End URCBehaviour interface

	/** Refresh function being called whenever either the Controller or the Properties of the Behaviour Details Panel change */
	void Refresh();
	
	/** Called whenever a new action is added into the ActionContainer. Will add a FRCRangeMapInput into RangeMapActionContainer corresponding to the Action. */
	void OnActionAdded(URCAction* Action, URCVirtualPropertySelfContainer* InPropertyValue);

	/** Adds an Action to the ActionContainer, whilst making sure that it is unique. */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField) override;

	/** Checks whether or not one of the following fields can be added to the AddAction Menu. Makes sure they're unique. */
	virtual bool CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const override;

	/** Whether the behaviour support property id */
	virtual bool SupportPropertyId() const override { return false; }

	/** Returns the Step value associated with a given Action*/
	bool GetValueForAction(const URCAction* InAction, double& OutValue);

protected:
	//~ Begin URCBehaviour interface
	/** Execute all the action if not provided a valid Action otherwise will only execute the given action */
	virtual void ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute) override;
	//~ End URCBehaviour interface

private:
	/** Minimum Value which the Range has */
	double InputMin;

	/** Maximum Value which the Range has */
	double InputMax;

	/** Controller Value of Type Float, which is used for the purpose of readability. */
	float ControllerFloatValue;
	
private:
	/** Boolean Function to help differentiate Custom Actions */
	bool IsSupportedActionLerpType(TObjectPtr<URCAction> InAction) const;

	/** Return the nearest Action and whether or not the distance between is less than the threshold.  */
	bool GetNearestActionByThreshold(TTuple<URCAction*, bool>& OutTuple);

	/** Returns a Map of Numeric Actions, mapped by a double value, bound to a given FieldId */
	void GetLerpActions(TMap<FGuid, TArray<URCAction*>>& OutNumericActionsByField, const TSet<TObjectPtr<URCAction>>& InActionsToExecute);
	
	/** Gives out all pairs per Unique Exposed Field, which are applicable for Lerp. */
	bool GetRangeValuePairsForLerp(TMap<FGuid, TTuple<URCAction*, URCAction*>>& OutPairs, const TSet<TObjectPtr<URCAction>>& InActionsToExecute);

	/** Gives out all Actions which do not fall under as being able to Lerp. Used for Executing them based on Threshold and Distance */
	TMap<double, URCAction*> GetNonLerpActions();

	/** Auxiliary Function used to calculate and apply the Lerp on Structs, like FVectors or FRotators. */
	void ApplyLerpOnStruct(const FRCRangeMapInput* MinRangeInput, const FRCRangeMapInput* MaxRangeInput, const double& InputAlpha, const FGuid& FieldId);

	/** Returns whether or not an Action created with a given RemoteControlField can be considered unique. */
	bool IsActionUnique(const TSharedRef<const FRemoteControlField> InRemoteControlField, const double& InValue, const TSet<TObjectPtr<URCAction>>& InActions);
};
