// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCActionContainer.h"
#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "RCBehaviour.generated.h"

struct FRemoteControlField;
class URCBehaviourNode;
class URCController;
class URCFunctionAction;
class URCPropertyIdAction;
class URCPropertyAction;
class URemoteControlPreset;
class URCAction;
class URCActionContainer;
class URCVirtualProperty;

/**
 * Base class for remote control behaviour.
 *
 * Behaviour is container for:
 * - Set of behaviour conditions
 * - And associated actions which should be executed if that is passed the behaviour
 *
 * Behaviour can be extended in Blueprints or CPP classes
 */
UCLASS(BlueprintType)
class REMOTECONTROLLOGIC_API URCBehaviour : public UObject
{
	GENERATED_BODY()

public:
	URCBehaviour();

	/** Initialize behaviour functionality. Called just once when created by a controller. */
	virtual void Initialize() {}

	/** Execute the behaviour */
	void Execute();

	/** Add a Logic action as an identity action. */
	virtual URCAction* AddAction();

	/** Add a Logic action as an identity action. */
	virtual URCAction* AddAction(FName InFieldId);

	/** Add a Logic action using a remote control field as input */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Duplicates an Action belonging to us into a given target behaviour*/
	virtual URCAction* DuplicateAction(URCAction* InAction, URCBehaviour* InBehaviour);

	/** Get number of action associated with behaviour */
	int32 GetNumActions() const;

	bool HasAction(const URCAction* InAction) const;

	/** Whether we can create an action pertaining to a given remote control field for the current behaviour */
	virtual bool CanHaveActionForField(const TSharedPtr<FRemoteControlField> InRemoteControlField) const;

	/** Whether the behaviour support property id, by default true because of the OnModify Behaviour */
	virtual bool SupportPropertyId() const { return true; }

	/**
	 * Return blueprint class associated with behaviour if exists
	 */
	UClass* GetOverrideBehaviourBlueprintClass() const;

#if WITH_EDITORONLY_DATA
	/**
	 * Return blueprint instance associated with behaviour if exists
	 */
	UBlueprint* GetBlueprint() const;
#endif

	/**
	 * Set blueprint class for this behaviour
	 */
	void SetOverrideBehaviourBlueprintClass(UBlueprint* InBlueprint);

#if WITH_EDITOR
	/** Get Display Name for this Behaviour */
	const FText& GetDisplayName();

	/** Get Description for this Behaviour */
	const FText& GetBehaviorDescription();
#endif

	/**
	 * @brief Called internally when entity Ids are renewed.
	 * @param InEntityIdMap Map of old Id to new Id.
	 */
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap);

	/** Called when an action value changed */
	virtual void NotifyActionValueChanged(URCAction* InChangedAction) {}

	/**
	 * It created the node if it called first time
	 * If BehaviourNodeClass changes it creates new instance
	 * Or just return cached one
	 */
	URCBehaviourNode* GetBehaviourNode();
	
	void UpdateEntityUsage(const URCController* InController, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const;

protected:
	/** Execute all the action if not provided a valid Action otherwise will only execute the given action */
	virtual void ExecuteInternal(const TSet<TObjectPtr<URCAction>>& InActionsToExecute);

	/** Execute the given Action */
    void ExecuteSingleAction(URCAction* InAction);

public:
	/** Associated cpp behaviour */
	UPROPERTY()
	TSubclassOf<URCBehaviourNode> BehaviourNodeClass;

	/** Class path to associated blueprint class behaviour */
	UPROPERTY()
	FSoftClassPath OverrideBehaviourBlueprintClassPath;

	/** Behaviour Id */
	UPROPERTY()
	FGuid Id;

	/** Action container which is associated with current behaviour */
	UPROPERTY()
	TObjectPtr<URCActionContainer> ActionContainer;

	/** Reference to controller virtual property with this behaviour */
	UPROPERTY()
	TWeakObjectPtr<URCController> ControllerWeakPtr;

private:
	/** Cached behaviour node */
	UPROPERTY(Instanced)
	TObjectPtr<URCBehaviourNode> CachedBehaviourNode;

public:
	/** Whether this Behaviour is currently enabled. 
	* If disabled, it will be not evaluated when the associated Controller changes */
	UPROPERTY()
	bool bIsEnabled = true;

	/** Indicates whether we want a behavour to trigger during live scrubbing of values.
	* For example, for a light intensity value controlled via Bind behavour we want the bound properties to update live even while the float widget is being scrubbed by the user. */
	UPROPERTY()
	bool bExecuteBehavioursDuringPreChange = false;
};
