// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "RCBehaviourBlueprintNode.generated.h"

/**
 * Base blueprint class for behaviour node which holds the logic to execute behaviour 
 */
UCLASS(Abstract, Blueprintable)
class REMOTECONTROLLOGIC_API URCBehaviourBlueprintNode : public URCBehaviourNode
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool IsSupported(URCBehaviour* InBehaviour) const override;
	
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	bool Execute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	void PreExecute(URCBehaviour* InBehaviour) const override;

	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Remote Control Behaviour")
	void OnPassed(URCBehaviour* InBehaviour) const override;
};
