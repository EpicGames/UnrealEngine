// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "RCBehaviourNode.generated.h"

class URCBehaviour;
enum class EPropertyBagPropertyType : uint8;

/**
 * Base class for behaviour node which holds the logic to execute behaviour 
 */
UCLASS(Abstract, BlueprintType, Blueprintable, EditInlineNew)
class REMOTECONTROLLOGIC_API URCBehaviourNode : public UObject
{
	GENERATED_BODY()

public:
	/**
	 * Whether node supported based on input handle
	 * 
	 * @param InBehaviour Remote Control Behaviour
	 */
	virtual bool IsSupported(URCBehaviour* InBehaviour) const;
	
	/**
	 * Called before behaviour event
	 * 
	 * @param InBehaviour Remote Control Behaviour
	 */
	virtual void PreExecute(URCBehaviour* InBehaviour) const;

	/**
	 * Execute behaviour event
	 * 
	 * @param InBehaviour Remote Control Behaviour
	 */
	virtual bool Execute(URCBehaviour* InBehaviour) const;

	/**
	 * Called if executed with true
	 * 
	 * @param InBehaviour Remote Control Behaviour 
	 */
	virtual void OnPassed(URCBehaviour* InBehaviour) const;
	
	/** Get the Behaviour class for this Behaviour node */
	virtual UClass* GetBehaviourClass() const;

	/**
	 * User-friendly display name for this Behavior, displayed in Action Panel's Header.
	 * Custom behavior blueprints can set this value in defaults to update the display name
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Remote Control Behavior")
	FText DisplayName;

	/**
	 * Detailed description of what this behavior does, displayed in Action Panel's Header.
	 * Custom behavior blueprints can set this value in defaults to update the display name
	 */
	UPROPERTY(EditDefaultsOnly, Category = "Remote Control Behavior")
	FText BehaviorDescription;

protected:
	/** Return supported check callback */
	static TFunction<bool(const TPair<EPropertyBagPropertyType, UObject*>)> GetIsSupportedCallback(URCBehaviour* InBehaviour);
};
