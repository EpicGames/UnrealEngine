// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCVirtualProperty.h"
#include "RCController.generated.h"

class URCBehaviour;
class URCBehaviourNode;

DECLARE_MULTICAST_DELEGATE(FOnBehaviourListModified);

/**
 * Remote Control Controller. Container for Behaviours and Actions
 */
UCLASS(BlueprintType)
class REMOTECONTROLLOGIC_API URCController : public URCVirtualPropertyInContainer
{
	GENERATED_BODY()

public:

	//~ Begin URCVirtualPropertyBase
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) override;
	//~ End URCVirtualPropertyBase
	
#if WITH_EDITOR
	/** Called after applying a transaction to the object. Used to broadcast Undo related container changes to UI Used to broadcast Undo related container changes to UI */
	virtual void PostEditUndo();
#endif

	/** Create and add behaviour to behaviour set */
	virtual URCBehaviour* AddBehaviour(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass);

	/** Create new behaviour */
	virtual URCBehaviour* CreateBehaviour(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass);

	/** Create new behaviour without checking if supported */
	virtual URCBehaviour* CreateBehaviourWithoutCheck(TSubclassOf<URCBehaviourNode> InBehaviourNodeClass);

	/** Remove the behaviour by behaviour UObject pointer */
	virtual int32 RemoveBehaviour(URCBehaviour* InBehaviour);

	/** Remove the behaviour by behaviour id */
	virtual int32 RemoveBehaviour(const FGuid InBehaviourId);

	/** Returns true if the behaviour exist in this controller. */
	bool HasBehavior(const URCBehaviour* InBehavior) const;

	/** Removes all behaviours. */
	virtual void EmptyBehaviours();

	/** Execute all behaviours for this controller. */
	virtual void ExecuteBehaviours(const bool bIsPreChange = false);

	/** Pre-change notification for Controllers. Triggered while the user is scrubbing a float or vector slider in the UI*/
	virtual void OnPreChangePropertyValue() override;

	/** Handles modifications to controller value; evaluates all behaviours */
	virtual void OnModifyPropertyValue() override;

	TConstArrayView<TObjectPtr<URCBehaviour>> GetBehaviors() const
	{
		return Behaviors;
	}

	/** Duplicates an existing behaviour and adds it to the behaviour set of a given Controller */
	static URCBehaviour* DuplicateBehaviour(URCController* InController, URCBehaviour* InBehaviour);

	/** Rearranges the behaviors based on the dropped behaviours. */
	bool ReorderBehaviorItems(URCBehaviour* InDroppedOnBehavior, bool bInBelowItem, TArray<URCBehaviour*> InDroppedBehaviors);

	/** Delegate that notifies changes to the list of behaviours*/
	FOnBehaviourListModified OnBehaviourListModified;

	//~ Begin URCVirtualPropertyBase
	virtual void UpdateEntityUsage(const URemoteControlPreset* InPreset, TMap<FGuid, TSet<UObject*>>& InEntityToBehavior) const override;
	//~ End URCVirtualPropertyBase

	//~ Begin UObject
	virtual void PostLoad() override;
	//~ End UObject

public:
	/** Deprecated set of behaviors */
	UE_DEPRECATED(5.7, "Please use the array Behaviors instead.")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Please use the array Behaviors instead."))
	TSet<TObjectPtr<URCBehaviour>> Behaviours;

protected:
	/** Array of the behaviors */
	UPROPERTY()
	TArray<TObjectPtr<URCBehaviour>> Behaviors;
};
