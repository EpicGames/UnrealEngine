// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCVirtualPropertyContainer.h"
#include "RCControllerContainer.generated.h"

class URCActionContainer;
class URCController;

/**
 * Controller Container which holds all virtual controller properties 
 */
UCLASS()
class REMOTECONTROLLOGIC_API URCControllerContainer : public URCVirtualPropertyContainerBase
{
	GENERATED_BODY()

public:
	//~ Begin URCVirtualPropertyContainerBase
	virtual void UpdateEntityIds(const TMap<FGuid, FGuid>& InEntityIdMap) override;
	//~ End URCVirtualPropertyContainerBase
	
	/** Adds a new Controller to this container */
	virtual URCVirtualPropertyInContainer* AddProperty(const FName& InPropertyName, TSubclassOf<URCVirtualPropertyInContainer> InPropertyClass, const EPropertyBagPropertyType InValueType, UObject* InValueTypeObject = nullptr, TArray<FPropertyBagPropertyDescMetaData> MetaData = TArray<FPropertyBagPropertyDescMetaData>()) override;
	
private:
#if WITH_EDITOR
	/** Fetches the Controller underlying a given a Property Changed Event*/
	URCController* GetControllerFromChangeEvent(const FPropertyChangedEvent& Event);

	/** Delegate when the value is being scrubbed in UI*/
	virtual void OnPreChangePropertyValue(const FPropertyChangedEvent& PropertyChangedEvent) override;

	/** Delegate when object changed */
	virtual void OnModifyPropertyValue(const FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

protected:
	/** Controller Container could holds the set of shared Action containers which is holds some action but independent from the behaviour */
	UPROPERTY()
	TSet<TObjectPtr<URCActionContainer>> SharedActionContainers;
};
