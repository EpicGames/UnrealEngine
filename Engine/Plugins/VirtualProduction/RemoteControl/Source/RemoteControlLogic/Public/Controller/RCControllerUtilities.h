// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Templates/SharedPointerFwd.h"

class FProperty;
struct FPropertyBagPropertyDesc;
struct FRemoteControlProperty;

namespace UE::RCControllers
{
	/**
	 * Determines whether a controller can be created from a given property desc explicitly
	 * Explicit meaning that it won't use IsChildOf or recurse up its Super Types for Object/Struct properties
	 */
	bool CanCreateControllerFromPropertyDesc(const FPropertyBagPropertyDesc& InPropertyDesc);

	/**
	 * Determines whether a controller can be created from a given property entity explicitly
	 * Explicit meaning that it won't use IsChildOf or recurse up its Super Types for Object/Struct properties
	 */
	REMOTECONTROLLOGIC_API bool CanCreateControllerFromEntity(const TSharedPtr<const FRemoteControlProperty>& InPropertyEntity);

	/**
	 * Determines whether a controller can be created from a given property type explicitly
	 * Explicit meaning that it won't use IsChildOf or recurse up its Super Types for Object/Struct properties
	 */
	REMOTECONTROLLOGIC_API bool CanCreateControllerFromProperty(const FProperty* InProperty);
};
