// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "StructUtils/PropertyBag.h"

class URemoteControlPreset;
class URCVirtualPropertyBase;
class URCVirtualPropertyInContainer;

namespace UE::RCCustomControllers
{
	/** Checks if the passed virtual property has Custom Controller metadata available */
	REMOTECONTROLLOGIC_API bool IsCustomController(const URCVirtualPropertyBase* InController);

	/** Checks if the specified Custom Controller Type Name exists (e.g. is not a made up type) */
	REMOTECONTROLLOGIC_API bool IsValidCustomController(const FName& InCustomControllerTypeName);
	
	/** Returns the Custom Controller Type Name (e.g. Asset Path). Empty string if Controller is not a custom one. */
	REMOTECONTROLLOGIC_API FString GetCustomControllerTypeName(const URCVirtualPropertyBase* InController);
	
	UE_DEPRECATED(5.4, "Custom Controllers execution on load is deprecated. Function will always return false.")
	REMOTECONTROLLOGIC_API inline bool CustomControllerExecutesOnLoad(const URCVirtualPropertyBase* InController) { return false; }

	UE_DEPRECATED(5.4, "Custom Controllers execution on load is deprecated. Function will always return false.")
	REMOTECONTROLLOGIC_API inline bool CustomControllerExecutesOnLoad(const FName& InCustomControllerTypeName) { return false; }
	
	/** Returns the type associated to the specified Custom Controller Type Name */
	REMOTECONTROLLOGIC_API EPropertyBagPropertyType GetCustomControllerType(const FName& InCustomControllerTypeName);

	/** Returns the type associated to the specified Custom Controller Type Name */
	REMOTECONTROLLOGIC_API EPropertyBagPropertyType GetCustomControllerType(const FString& InCustomControllerTypeName);

	REMOTECONTROLLOGIC_API TMap<FName, FString> GetCustomControllerMetaData(const FString& InCustomControllerTypeName);

	/**
	 * Generates a unique name for a Custom Controller, based on its Custom Type Name
	 * Useful when creating a new controller.
	 */
	REMOTECONTROLLOGIC_API FName GetUniqueNameForController(const URCVirtualPropertyInContainer* InController);

	REMOTECONTROLLOGIC_API bool GetEntitiesControlledByController(const URemoteControlPreset* InRemoteControlPreset, const URCVirtualPropertyBase* InVirtualProperty, TSet<FGuid>& OutEntityIds);

	REMOTECONTROLLOGIC_API bool GetControllersOfEntity(const URemoteControlPreset* InRemoteControlPreset, const FGuid& InEntity, TSet<const URCVirtualPropertyBase*>& OutControllers);
	
	inline static FName CustomControllerNameKey = "CustomControllerTypeName";
	inline static FName CustomControllerExecuteOnLoadNameKey = "CustomControllerExecuteOnLoad";

	inline static FString CustomTextureControllerName = TEXT("Texture");
	inline static FString CustomStringListControllerName = TEXT("StringList");
	inline static FString CustomStatelessEventControllerName = TEXT("StatelessEvent");

	inline static FName StringListControllerEntriesName = "StringOptions";
};
