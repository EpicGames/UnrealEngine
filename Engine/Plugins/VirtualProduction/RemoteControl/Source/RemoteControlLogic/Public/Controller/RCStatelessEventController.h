// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "UObject/Class.h"

#include "RCStatelessEventController.generated.h"

class URCVirtualPropertyBase;

USTRUCT()
struct FRCStatelessEventController
{
	GENERATED_BODY()

	REMOTECONTROLLOGIC_API static bool IsStatelessEventController(const URCVirtualPropertyBase* InController);
};
