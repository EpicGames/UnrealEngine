// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"

#include "RemoteControlLogicConfig.generated.h"

/** Config class for Remote Control Logic related settings.
* Supports configuration of a valid list of Controller types.
*/
UCLASS(config=RemoteControl)
class REMOTECONTROLLOGIC_API URemoteControlLogicConfig: public UObject
{
	GENERATED_BODY()

public:
	/** PropertyBag types that are supported for use as Controllers */
	UPROPERTY(config)
	TArray<EPropertyBagPropertyType> SupportedControllerTypes;

	/** Certain types such as FVector / FRotator / FColor are specializations of UStructProperty
	* This config represents the list of such types that may be used as Controllers*/
	UPROPERTY(config)
	TArray<FName> SupportedControllerStructTypes;

	/** Add specific Object types Class Paths to this list in order to provide support for them */
	UPROPERTY(config)
	TArray<FName> SupportedControllerObjectClassPaths;

	/** Custom Controllers list. The elements in this list are used to generate customized versions of available controllers
	 * e.g. External Texture controller generates a specialized version of a String Controller.
	 * Support is introduced by adding MetaData to each newly created controller.
	 */
	UPROPERTY(config)
	TArray<FName> SupportedControllerCustomTypes;
};
