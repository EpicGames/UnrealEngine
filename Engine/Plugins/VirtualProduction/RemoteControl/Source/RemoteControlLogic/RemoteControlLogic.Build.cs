// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlLogic : ModuleRules
{
	public RemoteControlLogic(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"Cbor",
				"Engine",
				"HTTP",
				"ImageCore",
				"RemoteControl",
				"RemoteControlCommon",
				"RenderCore",
				"RHI",
				"Serialization",
			}
		);
		
		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"UnrealEd",
				}
			);
		}
	}
}
