// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlMultiUserModule.h"

#include "IConcertClientTransactionBridge.h"
#include "IConcertSyncClientModule.h"
#include "RemoteControlBinding.h"
#include "RemoteControlPreset.h"


namespace RemoteControlMultiUserUtils
{
	ETransactionFilterResult HandleTransactionFiltering(const FConcertTransactionFilterArgs& FilterArgs)
	{
		//Always allow RemoteControlPresets and RemoteControlExposeRegistry
		static const FName ExposeRegistryName = "RemoteControlExposeRegistry";
		UObject* ObjectToFilter = FilterArgs.ObjectToFilter;
		if (Cast<URemoteControlPreset>(ObjectToFilter) != nullptr
			|| (ObjectToFilter && ObjectToFilter->GetClass()->GetFName() == ExposeRegistryName)
			|| (ObjectToFilter && ObjectToFilter->IsA<URemoteControlBinding>()))
		{
			return ETransactionFilterResult::IncludeObject;
		}

		return ETransactionFilterResult::UseDefault;
	}
}

void FRemoteControlMultiUserModule::StartupModule()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
		TransactionBridge.RegisterTransactionFilter( TEXT("RemoteControlTransactionFilter"), FOnFilterTransactionDelegate::CreateStatic(&RemoteControlMultiUserUtils::HandleTransactionFiltering));
	}
}

void FRemoteControlMultiUserModule::ShutdownModule()
{
	if (IConcertSyncClientModule::IsAvailable())
	{
		IConcertClientTransactionBridge& TransactionBridge = IConcertSyncClientModule::Get().GetTransactionBridge();
		TransactionBridge.UnregisterTransactionFilter( TEXT("RemoteControlTransactionFilter"));
	}	
}

IMPLEMENT_MODULE(FRemoteControlMultiUserModule, RemoteControlMultiUser);
