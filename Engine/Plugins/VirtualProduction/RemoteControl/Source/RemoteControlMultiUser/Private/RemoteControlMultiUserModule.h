// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

/**
 * Implements a simple Remote Control Multiuser module to register our own transaction filter.
 */
class FRemoteControlMultiUserModule : public IModuleInterface
{
public:

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
};
