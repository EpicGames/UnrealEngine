// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class RemoteControlMultiUser : ModuleRules
	{
		public RemoteControlMultiUser(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
			new string[]
			{
			});

			PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ConcertSyncClient", // Dependancy to register our transaction filters. Needs to enable that plugin. Would be better to have modular feature interface functionality
				"Core",
				"CoreUObject",
				"RemoteControl",
			});
		}
	}
}
