// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocol.h"

#include "GameFramework/Actor.h"
#include "Misc/CoreDelegates.h"
#include "RCModifyOperationFlags.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolBinding.h"
#include "RemoteControlProtocolEntityProcessor.h"
#include "RemoteControlProtocolModule.h"
#include "RemoteControlSettings.h"
#include "UObject/StructOnScope.h"

FRemoteControlProtocol::FRemoteControlProtocol(const FName InProtocolName)
	: ProtocolName(InProtocolName)
{
	FCoreDelegates::OnBeginFrame.AddRaw(this, &FRemoteControlProtocol::OnBeginFrame);
	FCoreDelegates::OnEndFrame.AddRaw(this, &FRemoteControlProtocol::OnEndFrame);
}

FRemoteControlProtocol::~FRemoteControlProtocol()
{
	FCoreDelegates::OnBeginFrame.RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);
}

void FRemoteControlProtocol::Init()
{
#if WITH_EDITOR

	RegisterColumns();

#endif // WITH_EDITOR
}

FRemoteControlProtocolEntityPtr FRemoteControlProtocol::CreateNewProtocolEntity(FProperty* InProperty, URemoteControlPreset* InOwner, FGuid InPropertyId) const
{
	FRemoteControlProtocolEntityPtr NewDataPtr = MakeShared<TStructOnScope<FRemoteControlProtocolEntity>>();
	NewDataPtr->InitializeFrom(FStructOnScope(GetProtocolScriptStruct()));
	(*NewDataPtr)->Init(InOwner, InPropertyId);

	return NewDataPtr;
}

void FRemoteControlProtocol::QueueValue(const FRemoteControlProtocolEntityPtr InProtocolEntity, const double InProtocolValue)
{
	EntityValuesToApply.Add(InProtocolEntity, InProtocolValue);
}

void FRemoteControlProtocol::OnBeginFrame()
{
	using namespace UE::RemoteControl;
	ProtocolEntityProcessor::ProcessEntities(EntityValuesToApply);

	EntityValuesToApply.Reset();
}

#if WITH_EDITOR

FProtocolColumnPtr FRemoteControlProtocol::GetRegisteredColumn(const FName& ByColumnName) const
{
	for (const FProtocolColumnPtr& ProtocolColumn : RegisteredColumns)
	{
		if (ProtocolColumn->ColumnName == ByColumnName)
		{
			return ProtocolColumn.ToSharedRef();
		}
	}

	return nullptr;
}

void FRemoteControlProtocol::GetRegisteredColumns(TSet<FName>& OutColumns)
{
	for (const FProtocolColumnPtr& ProtocolColumn : RegisteredColumns)
	{
		OutColumns.Add(ProtocolColumn->ColumnName);
	}
}

#endif // WITH_EDITOR

TFunction<bool(FRemoteControlProtocolEntityWeakPtr InProtocolEntityWeakPtr)> FRemoteControlProtocol::CreateProtocolComparator(FGuid InPropertyId)
{
	return [InPropertyId](FRemoteControlProtocolEntityWeakPtr InProtocolEntityWeakPtr) -> bool
	{
		if (FRemoteControlProtocolEntityPtr ProtocolEntityPtr = InProtocolEntityWeakPtr.Pin())
		{
			return InPropertyId == (*ProtocolEntityPtr)->GetPropertyId();
		}

		return false;
	};
}

FProperty* IRemoteControlProtocol::GetRangeInputTemplateProperty() const
{
	FProperty* RangeInputTemplateProperty = nullptr;
	if(UScriptStruct* ProtocolScriptStruct = GetProtocolScriptStruct())
	{
		RangeInputTemplateProperty = ProtocolScriptStruct->FindPropertyByName("RangeInputTemplate");
	}

	if(!ensure(RangeInputTemplateProperty))
	{
		UE_LOGF(LogRemoteControlProtocol, Warning, "Could not find RangeInputTemplate Property for this Protocol. Please either add this named property to the ProtocolScriptStruct implementation, or override IRemoteControlProtocol::GetRangeTemplateType.");
	}
	
	return RangeInputTemplateProperty;
}
