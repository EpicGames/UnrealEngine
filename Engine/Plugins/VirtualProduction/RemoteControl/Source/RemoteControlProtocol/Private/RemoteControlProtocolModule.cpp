// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolModule.h"

#include "CoreGlobals.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocol.h"
#include "Misc/CommandLine.h"

#if WITH_EDITOR
#include "AnalyticsEventAttribute.h"
#include "EngineAnalytics.h"
#endif

DEFINE_LOG_CATEGORY(LogRemoteControlProtocol);

#define LOCTEXT_NAMESPACE "RemoteControlProtocol"

void FRemoteControlProtocolModule::StartupModule()
{
	bRCProtocolsDisable = FParse::Param(FCommandLine::Get(), TEXT("RCProtocolsDisable"));

	if (!bRCProtocolsDisable)
	{
		// Called when remote control preset loaded
		URemoteControlPreset::OnPostLoadRemoteControlPreset.AddRaw(this, &FRemoteControlProtocolModule::OnPostLoadRemoteControlPreset);
	}
}

void FRemoteControlProtocolModule::ShutdownModule()
{
	if (!bRCProtocolsDisable)
	{
		EmptyProtocols();
		URemoteControlPreset::OnPostLoadRemoteControlPreset.RemoveAll(this);
	}
}

TArray<FName> FRemoteControlProtocolModule::GetProtocolNames() const
{
	TArray<FName> ProtocolNames;
	Protocols.GenerateKeyArray(ProtocolNames);
	ProtocolNames.Sort(FNameLexicalLess());
	return ProtocolNames;
}

TSharedPtr<IRemoteControlProtocol> FRemoteControlProtocolModule::GetProtocolByName(FName InProtocolName) const
{
	if (const TSharedRef<IRemoteControlProtocol>* ProtocolPtr = Protocols.Find(InProtocolName))
	{
		return *ProtocolPtr;
	}

	return nullptr;
}

bool FRemoteControlProtocolModule::AddProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol)
{
	if (bRCProtocolsDisable)
	{
		return false;
	}
	
#if WITH_EDITOR
	if (FEngineAnalytics::IsAvailable())
	{
		TArray<FAnalyticsEventAttribute> EventAttributes;
		if (UScriptStruct* ScriptStruct = InProtocol->GetProtocolScriptStruct())
		{
			EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ProtocolType"), ScriptStruct->GetName()));
		}
		EventAttributes.Add(FAnalyticsEventAttribute(TEXT("ProtocolName"), InProtocolName.ToString()));
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("RemoteControl.AddProtocol"), EventAttributes);	
	}
#endif
	
	InProtocol->Init();

	// Check for presence of required RangeInputTemplate property
	checkf(InProtocol->GetRangeInputTemplateProperty(), TEXT("ScriptStruct for Protocol %s did not have the required RangeInputTemplate Property."), *InProtocolName.ToString());
	
	Protocols.Add(InProtocolName, MoveTemp(InProtocol));

	return true;
}

void FRemoteControlProtocolModule::RemoveProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol)
{
	if (TSharedRef<IRemoteControlProtocol>* Protocol = Protocols.Find(InProtocolName))
	{
		(*Protocol)->UnbindAll();
	}

	Protocols.Remove(InProtocolName);
}

void FRemoteControlProtocolModule::EmptyProtocols()
{
	for (TPair<FName, TSharedRef<IRemoteControlProtocol>>& ProtocolPair : Protocols)
	{
		ProtocolPair.Value->UnbindAll();
	}

	Protocols.Empty();
}

void FRemoteControlProtocolModule::ApplyProtocolBindings(URemoteControlPreset* InPreset) const
{
	if (!InPreset)
	{
		return;
	}

	for (TWeakPtr<FRemoteControlProperty> ExposedPropertyWeakPtr : InPreset->GetExposedEntities<FRemoteControlProperty>())
	{
		if (TSharedPtr<FRemoteControlProperty> ExposedPropertyPtr = ExposedPropertyWeakPtr.Pin())
		{
			for (const FRemoteControlProtocolBinding& Binding : ExposedPropertyPtr->ProtocolBindings)
			{
				// Supporting plugin needs to be loaded/protocol available.
				if (const TSharedPtr<IRemoteControlProtocol> Protocol = GetProtocolByName(Binding.GetProtocolName()))
				{
					if (TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity = Binding.GetRemoteControlProtocolEntityPtr())
					{
						Protocol->Unbind(ProtocolEntity);
						Protocol->Bind(ProtocolEntity);
					}
				}
			}
		}
	}
}

void FRemoteControlProtocolModule::UnapplyProtocolBindings(URemoteControlPreset* InPreset) const
{
	if (!InPreset)
	{
		return;
	}

	for (TWeakPtr<FRemoteControlProperty> ExposedPropertyWeakPtr : InPreset->GetExposedEntities<FRemoteControlProperty>())
	{
		if (TSharedPtr<FRemoteControlProperty> ExposedPropertyPtr = ExposedPropertyWeakPtr.Pin())
		{
			for (const FRemoteControlProtocolBinding& Binding : ExposedPropertyPtr->ProtocolBindings)
			{
				// Supporting plugin needs to be loaded/protocol available.
				if (const TSharedPtr<IRemoteControlProtocol> Protocol = GetProtocolByName(Binding.GetProtocolName()))
				{
					Protocol->Unbind(Binding.GetRemoteControlProtocolEntityPtr());
				}
			}
		}
	}
}

void FRemoteControlProtocolModule::OnPostLoadRemoteControlPreset(URemoteControlPreset* InPreset) const
{
	// Avoid protocol binding if the preset is not valid or while cooking.
	if (!ensure(InPreset) || GIsCookerLoadingPackage)
	{
		return;
	}

	ApplyProtocolBindings(InPreset);
}

IMPLEMENT_MODULE(FRemoteControlProtocolModule, RemoteControlProtocol);

#undef LOCTEXT_NAMESPACE /*RemoteControlProtocol*/
