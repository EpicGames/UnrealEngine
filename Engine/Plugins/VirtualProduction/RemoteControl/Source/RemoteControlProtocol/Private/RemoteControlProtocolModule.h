// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlProtocolModule.h"

class URemoteControlPreset;

REMOTECONTROLPROTOCOL_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteControlProtocol, Log, All);

class FRemoteControlProtocolModule : public IRemoteControlProtocolModule
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface
	
	//~ Begin IRemoteControlProtocolModule interface
	virtual int32 GetProtocolNum() const override { return Protocols.Num(); }
	virtual TArray<FName> GetProtocolNames() const override;
	virtual TSharedPtr<IRemoteControlProtocol> GetProtocolByName(FName InProtocolName) const override;
	virtual bool AddProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol) override;
	virtual void RemoveProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol) override;
	virtual void EmptyProtocols() override;
	virtual bool IsRCProtocolsDisable() const override { return bRCProtocolsDisable; };
	virtual void ApplyProtocolBindings(URemoteControlPreset* InPreset) const override;
	virtual void UnapplyProtocolBindings(URemoteControlPreset* InPreset) const override;
	//~ End IRemoteControlProtocolModule interface

private:
	/** Called when remote control preset loaded */
	void OnPostLoadRemoteControlPreset(URemoteControlPreset* InPreset) const;

private:
	/** Map with all protocols */
	TMap<FName, TSharedRef<IRemoteControlProtocol>> Protocols;

	/** Prevents the Remote Control protocols from starting. Can be set from command line with -RCProtocolsDisable */
	bool bRCProtocolsDisable = false;
};
