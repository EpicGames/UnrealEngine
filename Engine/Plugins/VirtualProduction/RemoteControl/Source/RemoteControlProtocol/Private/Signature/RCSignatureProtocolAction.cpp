// Copyright Epic Games, Inc. All Rights Reserved.

#include "Signature/RCSignatureProtocolAction.h"
#include "IRemoteControlPropertyHandle.h"
#include "IRemoteControlProtocolModule.h"
#include "RCSignature.h"
#include "RemoteControlField.h"
#include "RemoteControlProtocolBinding.h"

namespace UE::RemoteControlProtocol::Private
{
	uint8 GetPropertyDimension(const UScriptStruct* InPropertyStruct)
	{
		if (!InPropertyStruct)
		{
			return 1;
		}

		uint8 Dimensions = 0;
		const FProperty* Property = InPropertyStruct->PropertyLink;

		while (Property)
		{
			++Dimensions;
			Property = Property->PropertyLinkNext;
		}
		return Dimensions;
	}

	// Logic taken from FProtocolBindingViewModel::AddDefaultRangeMappings.
	// TODO: Move the logic for re-use
	void SetMappingRange(const FRemoteControlProtocolEntity& InEntity
		, const IRemoteControlProtocol& InProtocol
		, FRemoteControlProtocolMapping& InOutMinMapping
		, FRemoteControlProtocolMapping& InOutMaxMapping)
	{
		const FNumericProperty* NumericProperty = CastField<FNumericProperty>(InProtocol.GetRangeInputTemplateProperty());
		if (!NumericProperty)
		{
			return;
		}

		FName RangePropertyTypeName = InEntity.GetRangePropertyName();
		const uint8 RangePropertySize = InEntity.GetRangePropertySize();

		if (NumericProperty->IsInteger())
		{
			int64 IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<int64>(NumericProperty);
			int64 IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<int64>(NumericProperty);
			// fixup typename according to typesize, ie. it can be a UInt32 but a typesize of 2 makes its a UInt16
			if (RangePropertyTypeName == NAME_UInt32Property && RangePropertySize > 0)
			{
				if (RangePropertySize == sizeof(uint8))
				{
					IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint8>(NumericProperty);
					IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint8>(NumericProperty);
					RangePropertyTypeName = NAME_ByteProperty;
				}
				else if (RangePropertySize == sizeof(uint16))
				{
					IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint16>(NumericProperty);
					IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint16>(NumericProperty);
					RangePropertyTypeName = NAME_UInt16Property; 
				}
				else if (RangePropertySize == sizeof(uint64))
				{
					IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint64>(NumericProperty);
					IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint64>(NumericProperty);
					RangePropertyTypeName = NAME_UInt64Property;
				}
			}
			if (RangePropertyTypeName == NAME_ByteProperty)
			{
				InOutMinMapping.SetRangeValue<uint8>(IntMin);
				InOutMaxMapping.SetRangeValue<uint8>(IntMax);
			}
			else if (RangePropertyTypeName == NAME_UInt16Property)
			{
				InOutMinMapping.SetRangeValue<uint16>(IntMin);
				InOutMaxMapping.SetRangeValue<uint16>(IntMax);
			}
			else if (RangePropertyTypeName == NAME_UInt32Property)
			{
				InOutMinMapping.SetRangeValue<uint32>(IntMin);
				InOutMaxMapping.SetRangeValue<uint32>(IntMax);
			}
			else if (RangePropertyTypeName == NAME_UInt64Property)
			{
				InOutMinMapping.SetRangeValue<uint64>(IntMin);
				InOutMaxMapping.SetRangeValue<uint64>(IntMax);
			}
		}
		else if(NumericProperty->IsFloatingPoint())
		{
			const float FloatMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<float>(NumericProperty);
			const float FloatMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<float>(NumericProperty);
			if (RangePropertyTypeName == NAME_FloatProperty)
			{
				InOutMinMapping.SetRangeValue<float>(FloatMin);
				InOutMaxMapping.SetRangeValue<float>(FloatMax);
			}
		}
	}

	template<typename InCppType>
	struct FRelativeOperation
	{
		static InCppType CalculateAdditive(const InCppType& InPropertyValue, const InCppType& InMappingValue)
		{
			return InPropertyValue + InMappingValue;
		}

		static InCppType CalculateScalar(const InCppType& InPropertyValue, const InCppType& InMappingValue)
		{
			return InPropertyValue * InMappingValue;
		}
	};

	template<>
	struct FRelativeOperation<FRotator>
	{
		static FRotator CalculateAdditive(const FRotator& InPropertyValue, const FRotator& InMappingValue)
		{
			return InPropertyValue + InMappingValue;
		}

		// FRotators have no Scalar support. Treat this as Absolute
		static FRotator CalculateScalar(const FRotator& InPropertyValue, const FRotator& InMappingValue)
		{
			return InMappingValue;
		}
	};

	template<>
	struct FRelativeOperation<FColor>
	{
		static FColor CalculateAdditive(const FColor& InPropertyValue, const FColor& InMappingValue)
		{
			return FRelativeOperation<FLinearColor>::CalculateAdditive(InPropertyValue, InMappingValue)
				.ToFColor(/*bSRGB*/true);
		}

		static FColor CalculateScalar(const FColor& InPropertyValue, const FColor& InMappingValue)
		{
			return FRelativeOperation<FLinearColor>::CalculateScalar(InPropertyValue, InMappingValue)
				.ToFColor(/*bSRGB*/true);
		}
	};

	struct FMappingTypeHelper
	{
		FMappingTypeHelper(const FRCSignatureActionContext& InContext, ERCSignatureProtocolActionMappingSpace InMappingType)
			: PropertyHandle(InContext.Property.IsValid() ? InContext.Property->GetPropertyHandle() : nullptr)
			, MappingType(InMappingType)
		{
		}

		bool TryApply(FRemoteControlProtocolMapping& InOutMapping)
		{
			if (!PropertyHandle.IsValid())
			{
				return false;
			}

			// Nothing to do with an Absolute Mapping
			if (MappingType == ERCSignatureProtocolActionMappingSpace::Absolute)
			{
				return false;
			}

			return TryApply<double>(InOutMapping)
				|| TryApply<float>(InOutMapping)
				|| TryApply<FVector>(InOutMapping)
				|| TryApply<FVector4>(InOutMapping)
				|| TryApply<FRotator>(InOutMapping)
				|| TryApply<FColor>(InOutMapping)
				|| TryApply<FLinearColor>(InOutMapping)
				|| TryApply<int8>(InOutMapping)
				|| TryApply<uint8>(InOutMapping)
				|| TryApply<int32>(InOutMapping)
				|| TryApply<uint32>(InOutMapping)
				|| TryApply<int16>(InOutMapping)
				|| TryApply<uint16>(InOutMapping)
				|| TryApply<int64>(InOutMapping)
				|| TryApply<uint64>(InOutMapping);
		}

	private:
		template<typename InCppType>
		bool TryApply(FRemoteControlProtocolMapping& InOutMapping) const
		{
			InCppType PropertyValue;
			if (!PropertyHandle->GetValue(PropertyValue) ||
				!InOutMapping.CanGetMappingValueAsPrimitive<InCppType>())
			{
				return false;
			}

			const InCppType MappingValue = InOutMapping.GetMappingValueAsPrimitive<InCppType>();

			switch (MappingType)
			{
			case ERCSignatureProtocolActionMappingSpace::Additive:
				InOutMapping.SetMappingValueAsPrimitive(FRelativeOperation<InCppType>::CalculateAdditive(PropertyValue, MappingValue));
				break;

			case ERCSignatureProtocolActionMappingSpace::Multiply:
				InOutMapping.SetMappingValueAsPrimitive(FRelativeOperation<InCppType>::CalculateScalar(PropertyValue, MappingValue));
				break;

			default:
				checkNoEntry();
			}

			return true;
		}

		TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle;
		ERCSignatureProtocolActionMappingSpace MappingType;
	};
}

void FRCSignatureProtocolAction::Initialize(const FRCSignatureField& InField)
{
	MinMappingDesc = InField.PropertyDesc;
	MaxMappingDesc = InField.PropertyDesc;

	MinMappingDesc.Name = TEXT("Min");
	MaxMappingDesc.Name = TEXT("Max");

#if WITH_EDITORONLY_DATA
	// Remove Meta-data like DisplayName, Tooltip
	// Other meta-data like Min/Max clamping, whether to hide Alpha Channel should be retained
	static const TSet<FName, DefaultKeyFuncs<FName>, TFixedSetAllocator<2>> MetaDataToRemove =
	{
		TEXT("DisplayName"),
		TEXT("Tooltip"),
	};

	MinMappingDesc.MetaData.RemoveAll(
		[](const FPropertyBagPropertyDescMetaData& InMetaData)
		{
			return MetaDataToRemove.Contains(InMetaData.Key);
		});

	// Both Min/Max refer to the same property so should have the same meta-data
	MaxMappingDesc.MetaData = MinMappingDesc.MetaData;
#endif

	PropertyDimension = UE::RemoteControlProtocol::Private::GetPropertyDimension(GetPropertyStruct());

	TArray<FName> ProtocolNames = IRemoteControlProtocolModule::Get().GetProtocolNames();
	if (!ProtocolNames.IsEmpty())
	{
		if (ProtocolName.IsNone())
		{
			ProtocolName = ProtocolNames[0];
		}
		UpdateProtocolEntity();
	}

	UpdateMappingType();
}

bool FRCSignatureProtocolAction::IsSupported(const FRCSignatureField& InField) const
{
	return InField.PropertyDesc.ValueType == EPropertyBagPropertyType::Struct
		|| InField.PropertyDesc.IsNumericType();
}

bool FRCSignatureProtocolAction::Execute(const FRCSignatureActionContext& InContext) const
{
	TSharedPtr<IRemoteControlProtocol> Protocol = GetProtocol();
	if (!Protocol.IsValid())
	{
		return false;
	}

	if (PropertyDimension == 1)
	{
		InContext.Property->ProtocolBindings.Empty(1);
		CreateProtocolEntity(InContext, *Protocol, 0xFF);
		return true;
	}

	if (bSingleProtocolChannel)
	{
		InContext.Property->ProtocolBindings.Empty(1);
		CreateProtocolEntity(InContext, *Protocol, OverrideMask);
	}
	else
	{
		InContext.Property->ProtocolBindings.Empty(PropertyDimension);

		// Add Mask Single Channel per Entity instead of all Channels
		for (uint8 Dimension = 0; Dimension < PropertyDimension; ++Dimension)
		{
			const uint8 Mask = 1 << Dimension;
			if (Mask & OverrideMask)
			{
				CreateProtocolEntity(InContext, *Protocol, Mask);
			}
		}
	}

	return true;
}

#if WITH_EDITOR
void FRCSignatureProtocolAction::PostEditChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (InPropertyChangedEvent.GetMemberPropertyName() == GET_MEMBER_NAME_CHECKED(FRCSignatureProtocolAction, ProtocolName))
	{
		UpdateProtocolEntity();
	}
	UpdateMappingType();
}

FRCSignatureActionIcon FRCSignatureProtocolAction::GetIcon() const
{
	FRCSignatureActionIcon Icon;
	Icon.StyleSetName = TEXT("EditorStyle");
	Icon.StyleName = TEXT("LevelEditor.Tabs.StatsViewer");
	return Icon;
}
#endif

void FRCSignatureProtocolAction::UpdateProtocolEntity()
{
	const TSharedPtr<IRemoteControlProtocol> Protocol = GetProtocol();
	if (!Protocol.IsValid())
	{
		return;
	}

	const UScriptStruct* ProtocolStruct = Protocol->GetProtocolScriptStruct();

	check(ProtocolStruct && ProtocolStruct->IsChildOf(FRemoteControlProtocolEntity::StaticStruct()));
	if (ProtocolStruct != ProtocolEntity.GetScriptStruct())
	{
		ProtocolEntity.InitializeAsScriptStruct(ProtocolStruct, /*StructMemory*/nullptr);
	}
}

void FRCSignatureProtocolAction::UpdateMappingType()
{
	if (!Mappings.FindPropertyDescByName(MinMappingDesc.Name) || !Mappings.FindPropertyDescByName(MaxMappingDesc.Name))
	{
		Mappings.Reset();
		Mappings.AddProperties({ MinMappingDesc, MaxMappingDesc });
	}
}

TSharedPtr<IRemoteControlProtocol> FRCSignatureProtocolAction::GetProtocol() const
{
	return IRemoteControlProtocolModule::Get().GetProtocolByName(ProtocolName);
}

void FRCSignatureProtocolAction::CreateProtocolEntity(const FRCSignatureActionContext& InContext, IRemoteControlProtocol& InProtocol, uint8 InMask) const
{
	const FGuid EntityId = InContext.Property->GetId();
	FProperty* Property = InContext.Property->GetProperty();

	TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> NewEntity = InProtocol.CreateNewProtocolEntity(Property
		, InContext.Preset
		, EntityId);

	if (const UScriptStruct* ScriptStruct = Cast<UScriptStruct>(NewEntity->GetStruct()))
	{
		check(ProtocolEntity.GetScriptStruct() == ScriptStruct);
		ScriptStruct->CopyScriptStruct(NewEntity->Get(), ProtocolEntity.GetPtr());
	}
	NewEntity->CastChecked<FRemoteControlProtocolEntity>()->Init(InContext.Preset, EntityId);

	FRemoteControlProtocolBinding ProtocolBinding(ProtocolName, EntityId, NewEntity, FGuid::NewGuid());
	AddMappings(InContext, InProtocol, /*InOut*/ProtocolBinding);

	(*NewEntity)->ClearMask(RC_AllMasks);
	(*NewEntity)->EnableMask(static_cast<ERCMask>(InMask));

	InProtocol.Bind(NewEntity);
	InContext.Property->ProtocolBindings.Emplace(MoveTemp(ProtocolBinding));
}

void FRCSignatureProtocolAction::AddMappings(const FRCSignatureActionContext& InContext
	, const IRemoteControlProtocol& InProtocol
	, FRemoteControlProtocolBinding& InOutBinding) const
{
	const FRemoteControlProtocolEntity& ProtocolEntityRef = ProtocolEntity.Get();

	FProperty* Property = InContext.Property->GetProperty();

	const uint8 RangePropertySize = ProtocolEntityRef.GetRangePropertySize();

	FRemoteControlProtocolMapping MinMapping(Property, RangePropertySize);
	FRemoteControlProtocolMapping MaxMapping(Property, RangePropertySize);

	UE::RemoteControlProtocol::Private::SetMappingRange(ProtocolEntity.Get(), InProtocol, MinMapping, MaxMapping);

	const uint8* MappingValue = Mappings.GetValue().GetMemory();

	UE::RemoteControlProtocol::Private::FMappingTypeHelper MappingHelper(InContext, MappingSpace);

	if (const FPropertyBagPropertyDesc* Desc = Mappings.FindPropertyDescByName(MinMappingDesc.Name))
	{
		const uint8* MinValue = MappingValue + Desc->CachedProperty->GetOffset_ForInternal();
		MinMapping.SetRawMappingData(InContext.Preset, Property, MinValue);
		MappingHelper.TryApply(MinMapping);
	}

	if (const FPropertyBagPropertyDesc* Desc = Mappings.FindPropertyDescByName(MaxMappingDesc.Name))
	{
		const uint8* MaxValue = MappingValue + Desc->CachedProperty->GetOffset_ForInternal();
		MaxMapping.SetRawMappingData(InContext.Preset, Property, MaxValue);
		MappingHelper.TryApply(MaxMapping);
	}

	InOutBinding.AddMapping(MinMapping);
	InOutBinding.AddMapping(MaxMapping);
}

const UScriptStruct* FRCSignatureProtocolAction::GetPropertyStruct() const
{
	return Cast<UScriptStruct>(MinMappingDesc.ValueTypeObject);
}
