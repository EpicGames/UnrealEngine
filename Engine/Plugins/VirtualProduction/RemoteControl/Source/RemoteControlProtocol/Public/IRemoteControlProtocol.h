// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RCModifyOperationFlags.h"

#include "UObject/StructOnScope.h"

class URemoteControlPreset;
struct FRemoteControlProtocolEntity;

using FRemoteControlProtocolEntityPtr = TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>;
using FRemoteControlProtocolEntityWeakPtr = TWeakPtr<TStructOnScope<FRemoteControlProtocolEntity>>;

#if WITH_EDITOR

namespace ProtocolColumnConstants
{
	static float ColumnSizeMicro = 0.075f;
	static float ColumnSizeMini = 0.15f;
	static float ColumnSizeSmall = 0.21f;
	static float ColumnSizeNormal = 0.3f;
	static float ColumnSizeMedium = 0.5f;
	static float ColumnSizeLarge = 0.8f;
	static float ColumnSizeExtraLarge = 1.f;
}

/**
 * Simple utility struct to represent a protocol column.
 */
struct FProtocolColumn
{
	FProtocolColumn() = default;

	FProtocolColumn(const FName& InColumnName, const FText& InDisplayText, float InColumnSize)
		: ColumnName(InColumnName)
		, DisplayText(InDisplayText)
		, ColumnSize(InColumnSize)
	{
	}

	friend uint32 GetTypeHash(const FProtocolColumn& OtherColumn)
	{
		return GetTypeHash(OtherColumn.ColumnName);
	}

	bool operator==(const FProtocolColumn& OtherColumn)
	{
		return ColumnName == OtherColumn.ColumnName;
	}
	
	bool operator==(const FProtocolColumn& OtherColumn) const
	{
		return ColumnName == OtherColumn.ColumnName;
	}

	/** Holds the name of the column. */
	FName ColumnName;

	/** Holds the display text of the column. */
	FText DisplayText;

	/** Holds the size of the column. */
	float ColumnSize;
};

using FProtocolColumnPtr = TSharedPtr<FProtocolColumn>;
using FProtocolColumnRef = TSharedRef<FProtocolColumn>;

#endif // WITH_EDITOR

/**
 * Interface for remote control protocol
 */
class REMOTECONTROLPROTOCOL_API IRemoteControlProtocol : public TSharedFromThis<IRemoteControlProtocol>
{
public:
	/** Virtual destructor */
	virtual ~IRemoteControlProtocol() {}

	/** Add custom initializing of the protocol */
	virtual void Init() = 0;

	/**
	 * Add the new protocol entity. That is created based on protocol specific script struct
	 * @param InProperty exposed property
	 * @param InOwner The preset that owns this protocol entity.
	 * @param InPresetName unique id of exposed property
	 * @return A shared pointer protocol entity struct on scope
	 */
	virtual FRemoteControlProtocolEntityPtr CreateNewProtocolEntity(FProperty* InProperty, URemoteControlPreset* InOwner, FGuid InPropertyId) const = 0;

	/** Get protocol specific Script Struct class */
	virtual UScriptStruct* GetProtocolScriptStruct() const = 0;

	/** Get range input template Property */
	virtual FProperty* GetRangeInputTemplateProperty() const;

	/**
	 * Bind the protocol entity to the protocol
	 * @param InEntityPtr protocol entity struct on scope pointer
	 */
	virtual void Bind(FRemoteControlProtocolEntityPtr InEntityPtr) = 0;

	/**
	 * Unbind the protocol entity from the protocol
	 * @param InEntityPtr protocol entity struct on scope pointer
	 */
	virtual void Unbind(FRemoteControlProtocolEntityPtr InEntityPtr) = 0;

	/** Unbind all protocol entities from the protocol */
	virtual void UnbindAll() = 0;

	/** Build to core engine delegated and called at the beginning of a frame */
	virtual void OnBeginFrame() {};

	/** Build to core engine delegated and called at the end of a frame */
	virtual void OnEndFrame() {};

	/**
	 * Queue protocol entity and value to apply for the protocol
	 * It stores only unique tick entities which should be apply next frame.
	 * Prevents from applying more then one entity for frame.
	 * @param InProtocolEntity Protocol Entity
	 * @param InProtocolValue Protocol Value
	 */
	virtual void QueueValue(const FRemoteControlProtocolEntityPtr InProtocolEntity, const double InProtocolValue) = 0;

#if WITH_EDITOR

	/** Retrieves the column reference by its name. */
	virtual FProtocolColumnPtr GetRegisteredColumn(const FName& ByColumnName) const = 0;

	/** Retrieves the set of columns registered by this protocol. */
	virtual void GetRegisteredColumns(TSet<FName>& OutColumns) = 0;
	
#endif // WITH_EDITOR
};
