// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "IRemoteControlProtocol.h"

class URemoteControlPreset;

/**
 * A Remote Control Protocol module. That is base for all remote control protcols
 */
class IRemoteControlProtocolModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IRemoteControlProtocolModule& Get()
	{
		static const FName ModuleName = "RemoteControlProtocol";
		return FModuleManager::LoadModuleChecked<IRemoteControlProtocolModule>(ModuleName);
	}

	/** Get number of remote control protocols */
	virtual int32 GetProtocolNum() const = 0;

	/** Get array of all remote control protocol names */
	virtual TArray<FName> GetProtocolNames() const = 0;

	/** Get specific remote control protocol by given name */
	virtual TSharedPtr<IRemoteControlProtocol> GetProtocolByName(FName InProtocolName) const = 0;

	/**
	 * Register the protocol
	 * @param InProtocolName protocol name
	 * @param InProtocol protocol shared reference
	 * @return true if protocol added
	 */
	virtual bool AddProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol) = 0;

	/**
	 * Unregister the protocol
	 * @param InProtocolName protocol name
	 * @param InProtocol protocol instance pointer
	 */
	virtual void RemoveProtocol(FName InProtocolName, TSharedRef<IRemoteControlProtocol> InProtocol) = 0;

	/** Applies all the protocol bindings found within the given preset */
	virtual void ApplyProtocolBindings(URemoteControlPreset* InPreset) const = 0;

	/** Unapplies all the protocol bindings found within the given preset */
	virtual void UnapplyProtocolBindings(URemoteControlPreset* InPreset) const = 0;

	/** Unregister all protocols */
	virtual void EmptyProtocols() = 0;

	/** Whether protocol disabled */
	virtual bool IsRCProtocolsDisable() const = 0;
};
