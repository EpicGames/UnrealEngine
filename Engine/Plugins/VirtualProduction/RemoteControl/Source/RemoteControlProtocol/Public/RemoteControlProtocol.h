// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlProtocol.h"

#if WITH_EDITOR

#define REGISTER_COLUMN(ColumnName, DisplayText, ColumnSize) RegisteredColumns.Add(MakeShared<FProtocolColumn>(ColumnName, DisplayText, ColumnSize));

#endif // WITH_EDITOR

/**
 * Base class implementation for remote control protocol
 */
class REMOTECONTROLPROTOCOL_API FRemoteControlProtocol : public IRemoteControlProtocol
{
public:
	FRemoteControlProtocol(FName InProtocolName);
	~FRemoteControlProtocol();
	
	//~ Begin IRemoteControlProtocol interface
	virtual void Init() override;
	virtual FRemoteControlProtocolEntityPtr CreateNewProtocolEntity(FProperty* InProperty, URemoteControlPreset* InOwner, FGuid InPropertyId) const override;
	virtual void QueueValue(const FRemoteControlProtocolEntityPtr InProtocolEntity, const double InProtocolValue) override;
	virtual void OnBeginFrame() override;

#if WITH_EDITOR

	virtual FProtocolColumnPtr GetRegisteredColumn(const FName& ByColumnName) const override;

	virtual void GetRegisteredColumns(TSet<FName>& OutColumns) override;

#endif // WITH_EDITOR

	//~ End IRemoteControlProtocol interface

	/**
	 * Helper function for comparing the Protocol Entity with given Property Id inside returned lambda
	 * @param InPropertyId Property unique Id
	 * @return Lambda function with ProtocolEntityWeakPtr as argument
	 */
	static TFunction<bool(FRemoteControlProtocolEntityWeakPtr InProtocolEntityWeakPtr)> CreateProtocolComparator(FGuid InPropertyId);

#if WITH_EDITOR

protected:

	/** Populates protocol specific columns. */
	virtual void RegisterColumns() {};

#endif // WITH_EDITOR

protected:
	/** Current Protocol Name */
	FName ProtocolName;

#if WITH_EDITOR

	/** Holds a set of protocol specific columns. */
	TSet<FProtocolColumnPtr> RegisteredColumns;

#endif // WITH_EDITOR

private:
	/** Map of the entities and protocol values about to apply */
	TMap<FRemoteControlProtocolEntityPtr, double> EntityValuesToApply;
};
