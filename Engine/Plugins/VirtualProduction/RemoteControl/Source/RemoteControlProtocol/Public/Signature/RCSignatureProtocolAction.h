// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureAction.h"
#include "RemoteControlProtocolBinding.h"
#include "StructUtils/InstancedStruct.h"
#include "StructUtils/PropertyBag.h"
#include "RCSignatureProtocolAction.generated.h"

class IRemoteControlProtocol;
struct FRemoteControlProtocolEntity;

UENUM()
enum class ERCSignatureProtocolActionMappingSpace : uint8
{
	Additive,
	Multiply,
	Absolute
};

USTRUCT(DisplayName="Protocol Action")
struct FRCSignatureProtocolAction : public FRCSignatureAction
{
	GENERATED_BODY()

	//~ Begin FRCSignatureAction
	virtual void Initialize(const FRCSignatureField& InField) override;
	virtual bool IsSupported(const FRCSignatureField& InField) const override;
	virtual bool Execute(const FRCSignatureActionContext& InContext) const override;
#if WITH_EDITOR
	virtual void PostEditChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged) override;
	virtual FRCSignatureActionIcon GetIcon() const override;
#endif
	//~ End FRCSignatureAction

	void UpdateProtocolEntity();

	void UpdateMappingType();

	TSharedPtr<IRemoteControlProtocol> GetProtocol() const;

	void CreateProtocolEntity(const FRCSignatureActionContext& InContext, IRemoteControlProtocol& InProtocol, uint8 InMask) const;

	void AddMappings(const FRCSignatureActionContext& InContext, const IRemoteControlProtocol& InProtocol, FRemoteControlProtocolBinding& InOutBinding) const;

	REMOTECONTROLPROTOCOL_API const UScriptStruct* GetPropertyStruct() const;

	UPROPERTY(EditAnywhere, Category="Protocol")
	FName ProtocolName;

	UPROPERTY(EditAnywhere, Category="Protocol", meta = (StructTypeConst))
	TInstancedStruct<FRemoteControlProtocolEntity> ProtocolEntity;

	UPROPERTY(EditAnywhere, Category="Protocol", meta=(EditCondition="PropertyDimension > 1", EditConditionHides))
	uint8 OverrideMask = 0xFF;

	/** Whether to combine all the Masks into a Single Protocol Channel rather than each Mask bit assigned to their own channel */
	UPROPERTY(EditAnywhere, Category="Protocol", meta=(EditCondition="PropertyDimension > 1", EditConditionHides))
	bool bSingleProtocolChannel = false;

	UPROPERTY(EditAnywhere, Category="Protocol")
	ERCSignatureProtocolActionMappingSpace MappingSpace = ERCSignatureProtocolActionMappingSpace::Additive;

	UPROPERTY(EditAnywhere, Category="Protocol", meta = (FixedLayout))
	FInstancedPropertyBag Mappings;

	UPROPERTY()
	FPropertyBagPropertyDesc MinMappingDesc;

	UPROPERTY()
	FPropertyBagPropertyDesc MaxMappingDesc;

	UPROPERTY()
	uint8 PropertyDimension = 1;
};
