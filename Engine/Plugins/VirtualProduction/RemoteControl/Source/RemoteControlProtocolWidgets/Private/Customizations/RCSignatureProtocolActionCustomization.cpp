// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureProtocolActionCustomization.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IRemoteControlProtocolModule.h"
#include "Signature/RCSignatureProtocolAction.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Masks/SRCSignatureProtocolMaskProperty.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::RemoteControlProtocolWidgets::Private
{
	TSharedRef<SWidget> CreateProtocolPicker(const TSharedRef<IPropertyHandle>& InProtocolNameHandle)
	{
		const TSharedRef<TArray<FName>> ProtocolOptions = MakeShared<TArray<FName>>();

		FName CurrentValue;
		InProtocolNameHandle->GetValue(CurrentValue);

		return SNew(SComboBox<FName>)
			.OptionsSource(&*ProtocolOptions) // ProtocolOptions is captured by copy in a lambda below, keeping it valid
			.InitiallySelectedItem(CurrentValue)
			.OnComboBoxOpening_Lambda([ProtocolOptions]()
			{
				*ProtocolOptions = IRemoteControlProtocolModule::Get().GetProtocolNames();
			})
			.OnGenerateWidget_Lambda([](FName InName)
			{
				return SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFont())
					.Text(FText::FromName(InName));
			})
			.OnSelectionChanged_Lambda([InProtocolNameHandle](FName InName, ESelectInfo::Type)
			{
				InProtocolNameHandle->SetValue(InName);
			})
			.Content()
			[
				SNew(STextBlock)
				.Font(IDetailLayoutBuilder::GetDetailFont())
				.Text_Lambda([InProtocolNameHandle]()->FText
				{
						FName CurrentValue;
						InProtocolNameHandle->GetValue(CurrentValue);
						return FText::FromName(CurrentValue);
				})
			];
	}
}

TSharedRef<IDetailCustomization> FRCSignatureProtocolActionCustomization::MakeInstance()
{
	return MakeShared<FRCSignatureProtocolActionCustomization>();
}

void FRCSignatureProtocolActionCustomization::CustomizeDetails(IDetailLayoutBuilder& InDetailBuilder)
{
	const TSharedRef<IPropertyHandle> ProtocolNameHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FRCSignatureProtocolAction, ProtocolName));
	const TSharedRef<IPropertyHandle> ProtocolEntityHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FRCSignatureProtocolAction, ProtocolEntity));
	const TSharedRef<IPropertyHandle> OverrideMaskHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FRCSignatureProtocolAction, OverrideMask));
	const TSharedRef<IPropertyHandle> SingleProtocolChannelHandle = InDetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FRCSignatureProtocolAction, bSingleProtocolChannel));

	InDetailBuilder.HideProperty(ProtocolNameHandle);
	InDetailBuilder.HideProperty(ProtocolEntityHandle);
	InDetailBuilder.HideProperty(OverrideMaskHandle);
	InDetailBuilder.HideProperty(SingleProtocolChannelHandle);

	IDetailCategoryBuilder& ProtocolCategory = InDetailBuilder.EditCategory(TEXT("Protocol"));

	ProtocolCategory.AddProperty(ProtocolNameHandle)
		.CustomWidget()
		.NameContent()
		[
			ProtocolNameHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			UE::RemoteControlProtocolWidgets::Private::CreateProtocolPicker(ProtocolNameHandle)
		];

	ProtocolCategory.AddProperty(ProtocolEntityHandle);
	ProtocolCategory.AddProperty(OverrideMaskHandle)
		.CustomWidget()
		.NameContent()
		[
			OverrideMaskHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SRCSignatureProtocolMaskProperty, OverrideMaskHandle, GetProtocolActions(InDetailBuilder))
		];
	ProtocolCategory.AddProperty(SingleProtocolChannelHandle);
}

TArray<const FRCSignatureProtocolAction*, TInlineAllocator<1>> FRCSignatureProtocolActionCustomization::GetProtocolActions(IDetailLayoutBuilder& InDetailBuilder) const
{
	TArray<const FRCSignatureProtocolAction*, TInlineAllocator<1>> ProtocolActions;

	const UScriptStruct* ProtocolActionStruct = FRCSignatureProtocolAction::StaticStruct();

	TArray<TSharedPtr<FStructOnScope>> StructOnScopes;
	InDetailBuilder.GetStructsBeingCustomized(StructOnScopes);

	if (!StructOnScopes.IsEmpty())
	{
		ProtocolActions.Reserve(StructOnScopes.Num());
		for (const TSharedPtr<FStructOnScope>& StructOnScope : StructOnScopes)
		{
			if (StructOnScope.IsValid())
			{
				const UStruct* ScriptStruct = StructOnScope->GetStruct();
				check(ScriptStruct == ProtocolActionStruct);
				ProtocolActions.Add(reinterpret_cast<const FRCSignatureProtocolAction*>(StructOnScope->GetStructMemory()));
			}
		}
	}

	return ProtocolActions;
}
