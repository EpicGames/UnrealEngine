// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "RemoteControlPreset.h"
#include "Misc/Change.h"

template <typename ChangeArgsType>
class TRemoteControlProtocolCommandChange final : public FCommandChange
{
public:
	DECLARE_DELEGATE_TwoParams(FOnUndoRedoDelegate, URemoteControlPreset* /* InPreset */, const ChangeArgsType& /* InChangeArgs */);

	TRemoteControlProtocolCommandChange(URemoteControlPreset* InPreset, ChangeArgsType&& InChangeArgs, FOnUndoRedoDelegate&& InOnApply, FOnUndoRedoDelegate&& InOnRevert);

	virtual void Apply(UObject* InObject) override;
	virtual void Revert(UObject* InObject) override;
	virtual FString ToString() const override { return TEXT(""); }

private:
	TWeakObjectPtr<URemoteControlPreset> PresetPtr;
	ChangeArgsType ChangeArgs;
	FOnUndoRedoDelegate OnApply;
	FOnUndoRedoDelegate OnRevert;
};

template <typename ChangeArgsType>
TRemoteControlProtocolCommandChange<ChangeArgsType>::TRemoteControlProtocolCommandChange(URemoteControlPreset* InPreset, ChangeArgsType&& InChangeArgs, FOnUndoRedoDelegate&& InOnApply, FOnUndoRedoDelegate&& InOnRevert)
	: PresetPtr(InPreset)
	, ChangeArgs(MoveTemp(InChangeArgs))
	, OnApply(MoveTemp(InOnApply))
	, OnRevert(MoveTemp(InOnRevert))
{}

template <typename ChangeArgsType>
void TRemoteControlProtocolCommandChange<ChangeArgsType>::Apply(UObject* InObject)
{
	if (URemoteControlPreset* Preset = PresetPtr.Get())
	{
		OnApply.ExecuteIfBound(Preset, ChangeArgs);
	}
}

template <typename ChangeArgsType>
void TRemoteControlProtocolCommandChange<ChangeArgsType>::Revert(UObject* InObject)
{
	if (URemoteControlPreset* Preset = PresetPtr.Get())
	{
		OnRevert.ExecuteIfBound(Preset, ChangeArgs);
	}
}
