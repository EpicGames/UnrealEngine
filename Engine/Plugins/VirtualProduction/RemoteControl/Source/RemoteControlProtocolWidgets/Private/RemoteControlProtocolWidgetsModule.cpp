// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlProtocolWidgetsModule.h"
#include "Customizations/RCSignatureProtocolActionCustomization.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlProtocol.h"
#include "IRemoteControlProtocolModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolWidgetsSettings.h"
#include "Signature/RCSignatureProtocolAction.h"
#include "Styling/ProtocolPanelStyle.h"
#include "ViewModels/ProtocolEntityViewModel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SRCProtocolBindingList.h"

DEFINE_LOG_CATEGORY(LogRemoteControlProtocolWidgets);

void FRemoteControlProtocolWidgetsModule::StartupModule()
{
	FProtocolPanelStyle::Initialize();

	RegisterPropertyEditorCustomizations();

	OnActiveProtocolChanged().AddRaw(this, &FRemoteControlProtocolWidgetsModule::SetActiveProtocolName);
}

void FRemoteControlProtocolWidgetsModule::ShutdownModule()
{
	FProtocolPanelStyle::Shutdown();

	UnregisterPropertyEditorCustomizations();

	OnActiveProtocolChanged().RemoveAll(this);
}

void FRemoteControlProtocolWidgetsModule::AddProtocolBinding(const FName InProtocolName)
{
	if (!RCProtocolBindingList.IsValid())
	{
		return;
	}

	RCProtocolBindingList->AddProtocolBinding(InProtocolName);
}

TSharedRef<SWidget> FRemoteControlProtocolWidgetsModule::GenerateDetailsForEntity(URemoteControlPreset* InPreset, const FGuid& InFieldId, const EExposedFieldType& InFieldType)
{
	check(InPreset);

	ResetProtocolBindingList();

	if (!InFieldId.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// Currently only supports Properties
	if (const UScriptStruct* PropertyStruct = InPreset->GetExposedEntityType(InFieldId))
	{
		if (PropertyStruct->IsChildOf(FRemoteControlProperty::StaticStruct()))
		{
			TSharedRef<FProtocolEntityViewModel> EntityViewModel = FProtocolEntityViewModel::Create(InPreset, InFieldId);

			EntityViewModel->OnBindingAdded().AddRaw(this, &FRemoteControlProtocolWidgetsModule::OnBindingAdded);

			EntityViewModel->OnBindingRemoved().AddRaw(this, &FRemoteControlProtocolWidgetsModule::OnBindingRemoved);

			return SAssignNew(RCProtocolBindingList, SRCProtocolBindingList, EntityViewModel);
		}
	}

	return SNullWidget::NullWidget;
}

void FRemoteControlProtocolWidgetsModule::ResetProtocolBindingList()
{
	RCProtocolBindingList = nullptr;
}

TSharedPtr<IRCProtocolBindingList> FRemoteControlProtocolWidgetsModule::GetProtocolBindingList() const
{
	return RCProtocolBindingList;
}

const FName FRemoteControlProtocolWidgetsModule::GetSelectedProtocolName() const
{
	return ActiveProtocolName;
}

FOnProtocolBindingAddedOrRemoved& FRemoteControlProtocolWidgetsModule::OnProtocolBindingAddedOrRemoved()
{
	return OnProtocolBindingAddedOrRemovedDelegate;
}

FOnActiveProtocolChanged& FRemoteControlProtocolWidgetsModule::OnActiveProtocolChanged()
{
	return OnActiveProtocolChangedDelegate;
}

void FRemoteControlProtocolWidgetsModule::OnBindingAdded(TSharedRef<FProtocolBindingViewModel> InBindingViewModel)
{
	OnProtocolBindingAddedOrRemovedDelegate.Broadcast(ERCProtocolBinding::Added);
}

void FRemoteControlProtocolWidgetsModule::OnBindingRemoved(FGuid InBindingId)
{
	OnProtocolBindingAddedOrRemovedDelegate.Broadcast(ERCProtocolBinding::Removed);
}

void FRemoteControlProtocolWidgetsModule::SetActiveProtocolName(const FName InProtocolName)
{
	if (ActiveProtocolName == InProtocolName)
	{
		return;
	}

	ActiveProtocolName = InProtocolName;

	URemoteControlProtocolWidgetsSettings* Settings = GetMutableDefault<URemoteControlProtocolWidgetsSettings>();

	Settings->PreferredProtocol = ActiveProtocolName;

	Settings->PostEditChange();

	Settings->SaveConfig();

	if (RCProtocolBindingList.IsValid())
	{
		RCProtocolBindingList->Refresh();
	}
}

void FRemoteControlProtocolWidgetsModule::RegisterPropertyEditorCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyEditorModule.RegisterCustomClassLayout(FRCSignatureProtocolAction::StaticStruct()->GetFName()
		, FOnGetDetailCustomizationInstance::CreateStatic(&FRCSignatureProtocolActionCustomization::MakeInstance));
}

void FRemoteControlProtocolWidgetsModule::UnregisterPropertyEditorCustomizations()
{
	if (!UObjectInitialized())
	{
		return;
	}

	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyEditorModule->UnregisterCustomClassLayout(FRCSignatureProtocolAction::StaticStruct()->GetFName());
	}
}

IMPLEMENT_MODULE(FRemoteControlProtocolWidgetsModule, RemoteControlProtocolWidgets);
