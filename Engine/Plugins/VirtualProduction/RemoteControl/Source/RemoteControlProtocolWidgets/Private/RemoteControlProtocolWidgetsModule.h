// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlProtocolWidgetsModule.h"

REMOTECONTROLPROTOCOLWIDGETS_API DECLARE_LOG_CATEGORY_EXTERN(LogRemoteControlProtocolWidgets, Log, All);

class FProtocolBindingViewModel;
class FProtocolEntityViewModel;
class IRCProtocolBindingList;
class URemoteControlPreset;

class FRemoteControlProtocolWidgetsModule : public IRemoteControlProtocolWidgetsModule
{
public:

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IRemoteControlProtocolWidgetsModule Interface
	virtual void AddProtocolBinding(const FName InProtocolName) override;
	virtual TSharedRef<SWidget> GenerateDetailsForEntity(URemoteControlPreset* InPreset, const FGuid& InFieldId, const EExposedFieldType& InFieldType) override;
	virtual void ResetProtocolBindingList() override;
	virtual TSharedPtr<IRCProtocolBindingList> GetProtocolBindingList() const override;
	virtual const FName GetSelectedProtocolName() const override;
	virtual FOnProtocolBindingAddedOrRemoved& OnProtocolBindingAddedOrRemoved() override;
	virtual FOnActiveProtocolChanged& OnActiveProtocolChanged() override;
	//~ End IRemoteControlProtocolWidgetsModule Interface

private:

	/** Called when binding is added. */
	void OnBindingAdded(TSharedRef<FProtocolBindingViewModel> InBindingViewModel);
	
	/** Called when binding is removed. */
	void OnBindingRemoved(FGuid InBindingId);

	/** Sets the selected protocol in the list and user setting. */
	void SetActiveProtocolName(const FName InProtocolName);

	void RegisterPropertyEditorCustomizations();
	void UnregisterPropertyEditorCustomizations();

private:
	/** Binding list public interface instance */
	TSharedPtr<IRCProtocolBindingList> RCProtocolBindingList;

	/** Called when binding is added or removed. */
	FOnProtocolBindingAddedOrRemoved OnProtocolBindingAddedOrRemovedDelegate;

	/** Called when active protocol selection changed. */
	FOnActiveProtocolChanged OnActiveProtocolChangedDelegate;

	/** Holds the active protocol name. */
	FName ActiveProtocolName;
};
