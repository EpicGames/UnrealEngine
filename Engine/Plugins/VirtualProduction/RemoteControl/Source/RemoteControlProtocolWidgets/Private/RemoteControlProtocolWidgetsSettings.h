// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "UObject/Object.h"

#include "RemoteControlProtocolWidgetsSettings.generated.h"

/**
 * Remote Control Protocol Widget Settings
 */
UCLASS(Config = RemoteControlProtocolWidgets)
class REMOTECONTROLPROTOCOLWIDGETS_API URemoteControlProtocolWidgetsSettings : public UObject
{
	GENERATED_BODY()

public:
	/** Protocol types to be hidden in the list view. */
	UPROPERTY(Config, EditAnywhere, Category = Widgets)
	TSet<FName> HiddenProtocolTypeNames;

	/** Last protocol added. Used as default in the binding list. */
	UPROPERTY(Config)
	FName PreferredProtocol;
};
