// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ProtocolPanelStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ProtocolStyles.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

#include "Interfaces/IPluginManager.h"

#define BOX_PLUGIN_BRUSH( RelativePath, ... ) FSlateBoxBrush(FProtocolPanelStyle::InContent( RelativePath, ".png" ), __VA_ARGS__)
#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FProtocolPanelStyle::StyleSet;

void FProtocolPanelStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	// Protocol Widget Styles
	SetupWidgetStyles(StyleSet.ToSharedRef());

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FProtocolPanelStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

TSharedPtr<ISlateStyle> FProtocolPanelStyle::Get()
{
	return StyleSet;
}

FName FProtocolPanelStyle::GetStyleSetName()
{
	static const FName ProtocolPanelStyleName(TEXT("ProtocolPanelStyle"));
	return ProtocolPanelStyleName;
}

FString FProtocolPanelStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("RemoteControl"))->GetBaseDir() + TEXT("/Resources");
	return (ContentDir / RelativePath) + Extension;
}

void FProtocolPanelStyle::SetupWidgetStyles(TSharedRef<FSlateStyleSet> InStyle)
{
	const ISlateStyle& AppStyle = FAppStyle::Get();

	// Mask Widget Styles
	FSlateBrush ContentAreaBrushDark = BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.f / 16.f), FLinearColor(0.5f, 0.5f, 0.5f, 1.f));
	FSlateBrush ContentAreaBrushLight = BOX_BRUSH("Common/LightGroupBorder", FMargin(4.f / 16.f));

	FCheckBoxStyle MaskButtonStyle = AppStyle.GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	MaskButtonStyle.SetCheckBoxType(ESlateCheckBoxType::ToggleButton);
	MaskButtonStyle.SetPadding(0.f);

	// Text Styles
	const FStyleFonts& StyleFonts = FStyleFonts::Get();

	const FTextBlockStyle& PlainTextStyle = AppStyle.GetWidgetStyle<FTextBlockStyle>("NormalText");

	const FTextBlockStyle BoldTextStyle = FTextBlockStyle(PlainTextStyle).SetFont(StyleFonts.NormalBold);

	FProtocolWidgetStyle ProtocolMaskWidgetStyle = FProtocolWidgetStyle()
		.SetContentAreaBrush(FSlateColorBrush(FStyleColors::Panel))
		.SetContentAreaBrushDark(ContentAreaBrushDark)
		.SetContentAreaBrushLight(ContentAreaBrushLight)
		
		.SetMaskButtonStyle(MaskButtonStyle)

		.SetBoldTextStyle(BoldTextStyle)
		.SetPlainTextStyle(PlainTextStyle);

	StyleSet->Set("ProtocolsPanel.Widgets.Mask", ProtocolMaskWidgetStyle);
}

#undef BOX_PLUGIN_BRUSH
#undef RootToContentDir
