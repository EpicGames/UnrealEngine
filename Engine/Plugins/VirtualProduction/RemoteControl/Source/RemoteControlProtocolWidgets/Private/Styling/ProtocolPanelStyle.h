// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

/**
 * 
 */
class FProtocolPanelStyle
{
public:
	/**
	 * Initialize the style set.
	 */
	static void Initialize();

	/**
	 * Shutdown the style set.
	 */
	static void Shutdown();

	/**
	 * Static getter for the style set.
	 */
	static TSharedPtr<ISlateStyle> Get();

	/**
	 * Static getter for the style set name.
	 */
	static FName GetStyleSetName();

private:

	/** Get a path to the module's resources folder. */
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

	/** Initializes Protocol Widget Styles. */
	static void SetupWidgetStyles(TSharedRef<FSlateStyleSet> InStyle);

private:

	/** Holds the style set. */
	static TSharedPtr<FSlateStyleSet> StyleSet;
};