// Copyright Epic Games, Inc. All Rights Reserved.

#include "Styling/ProtocolStyles.h"

FProtocolWidgetStyle::FProtocolWidgetStyle()
	: BoldTextStyle(FTextBlockStyle::GetDefault())
	, PlainTextStyle(FTextBlockStyle::GetDefault())
{
}

const FName FProtocolWidgetStyle::TypeName(TEXT("FProtocolWidgetStyle"));

const FProtocolWidgetStyle& FProtocolWidgetStyle::GetDefault()
{
	static FProtocolWidgetStyle Default;
	return Default;
}

void FProtocolWidgetStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	MaskButtonStyle.GetResources(OutBrushes);
	OutBrushes.Add(&ContentAreaBrush);
	OutBrushes.Add(&ContentAreaBrushDark);
	OutBrushes.Add(&ContentAreaBrushLight);
}

