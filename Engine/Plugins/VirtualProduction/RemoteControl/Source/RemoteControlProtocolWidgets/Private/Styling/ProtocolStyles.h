// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyle.h"
#include "Styling/SlateWidgetStyleContainerBase.h"

#include "ProtocolStyles.generated.h"

/**
 * Represents all custom protocol widgets.
 */
USTRUCT()
struct FProtocolWidgetStyle : public FSlateWidgetStyle
{
	GENERATED_USTRUCT_BODY()

	FProtocolWidgetStyle();
	virtual ~FProtocolWidgetStyle() {}

	//~ Begin : FSlateWidgetStyle
	virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;
	
	static const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	
	static const FProtocolWidgetStyle& GetDefault();
	//~ End : FSlateWidgetStyle

	/** Mask Button Bold Text Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTextBlockStyle BoldTextStyle;
	FProtocolWidgetStyle& SetBoldTextStyle(const FTextBlockStyle& InBoldTextStyle) { BoldTextStyle = InBoldTextStyle; return *this; }

	/** Brush used by the RC Panels to draw the content associated with this panel. */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrush;
	FProtocolWidgetStyle& SetContentAreaBrush(const FSlateBrush& InContentAreaBrush) { ContentAreaBrush = InContentAreaBrush; return *this; }

	/** Brush used by the RC Panels to draw the content associated with this panel (Dark). */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrushDark;
	FProtocolWidgetStyle& SetContentAreaBrushDark(const FSlateBrush& InContentAreaBrushDark) { ContentAreaBrushDark = InContentAreaBrushDark; return *this; }

	/** Brush used by the RC Panels to draw the content associated with this panel (Light). */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrushLight;
	FProtocolWidgetStyle& SetContentAreaBrushLight(const FSlateBrush& InContentAreaBrushLight) { ContentAreaBrushLight = InContentAreaBrushLight; return *this; }

	/** Style used for the mask button */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FCheckBoxStyle MaskButtonStyle;
	FProtocolWidgetStyle& SetMaskButtonStyle(const FCheckBoxStyle& InMaskButtonStyle) { MaskButtonStyle = InMaskButtonStyle; return *this; }

	/** Mask Button Plain Text Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTextBlockStyle PlainTextStyle;
	FProtocolWidgetStyle& SetPlainTextStyle(const FTextBlockStyle& InPlainTextStyle) { PlainTextStyle = InPlainTextStyle; return *this; }
};
