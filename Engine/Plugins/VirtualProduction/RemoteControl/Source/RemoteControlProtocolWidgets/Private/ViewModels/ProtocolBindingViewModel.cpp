// Copyright Epic Games, Inc. All Rights Reserved.

#include "ViewModels/ProtocolBindingViewModel.h"

#include "Editor.h"
#include "IRemoteControlProtocolModule.h"
#include "ProtocolCommandChange.h"
#include "ProtocolEntityViewModel.h"
#include "ProtocolRangeViewModel.h"
#include "RCTypeTraits.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolWidgetsModule.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"
#include "Algo/AnyOf.h"

#define LOCTEXT_NAMESPACE "ProtocolBindingViewModel"

// Used by FChange (for Undo/Redo). Keep independent of ViewModel (can be out of scope when called).
namespace Commands
{
	struct FAddRemoveRangeArgs
	{
		FGuid EntityId;
		FGuid BindingId;
		FGuid RangeId;
		TArray<FGuid> RangeIds; // Specify if there are multiple to add or remove
	};

	static FGuid AddRangeMappingInternal(URemoteControlPreset* InPreset, const FAddRemoveRangeArgs& InArgs);
	static bool RemoveRangeMappingInternal(URemoteControlPreset* InPreset, const FAddRemoveRangeArgs& InArgs);
	
	static FGuid AddRangeMappingInternal(URemoteControlPreset* InPreset, const FAddRemoveRangeArgs& InArgs)
	{
		TArray<FGuid> NewRangeIds;
		NewRangeIds.Reserve(1);
		
		if(const TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(InArgs.EntityId).Pin())
		{
			FRemoteControlProtocolBinding* ProtocolBinding = RCProperty->ProtocolBindings.FindByHash(GetTypeHash(InArgs.BindingId), InArgs.BindingId);
			
			// If undone, this can be nullptr
			if(!ProtocolBinding)
			{
				return FGuid();
			}

			const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> EntityPtr = ProtocolBinding->GetRemoteControlProtocolEntityPtr();

			// Add multiple rather than single
			if (InArgs.RangeIds.Num() > 0)
			{
				// @note: This only adds the same number of range mappings back, excluding their parameters
				for (int32 Idx = 0; Idx < InArgs.RangeIds.Num(); ++Idx)
				{
					FGuid RangeId = InArgs.RangeIds[Idx];
					RangeId = RangeId.IsValid() ? RangeId : FGuid::NewGuid();
					
					const FRemoteControlProtocolMapping RangesData(RCProperty->GetProperty(), (*EntityPtr)->GetRangePropertySize(), RangeId);
					ProtocolBinding->AddMapping(RangesData);
					NewRangeIds.Add(RangesData.GetId());
				}
			}
			else
			{
				const FRemoteControlProtocolMapping RangesData(RCProperty->GetProperty(), (*EntityPtr)->GetRangePropertySize(), InArgs.RangeId.IsValid() ? InArgs.RangeId : FGuid::NewGuid());
				ProtocolBinding->AddMapping(RangesData);

				NewRangeIds.Add(RangesData.GetId());
			}

			using FCommandChange = TRemoteControlProtocolCommandChange<Commands::FAddRemoveRangeArgs>;
			using FOnChange = FCommandChange::FOnUndoRedoDelegate;
			if(GUndo)
			{
				// Set or re-set the RangeId' (will be same if it was already set), so undos below remove or add the correct ranges
				FAddRemoveRangeArgs Args = InArgs;
				Args.RangeId = NewRangeIds[0];
				Args.RangeIds = NewRangeIds;
				
				// What to do on undo and redo
				GUndo->StoreUndo(InPreset,
					MakeUnique<FCommandChange>(
						InPreset,
						MoveTemp(Args),
						FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveRangeArgs& InChangeArgs)
						{
							Commands::AddRangeMappingInternal(InPreset, InChangeArgs);
						}),
						FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveRangeArgs& InChangeArgs)
						{
							Commands::RemoveRangeMappingInternal(InPreset, InChangeArgs);
						})
					)
				);
			}
		}

		return NewRangeIds[0]; // If multiple added, only return first as caller only needs a valid guid
	}

	static bool RemoveRangeMappingInternal(URemoteControlPreset* InPreset, const FAddRemoveRangeArgs& InArgs)
	{
		if(const TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(InArgs.EntityId).Pin())
		{
			FRemoteControlProtocolBinding* ProtocolBinding = RCProperty->ProtocolBindings.FindByHash(GetTypeHash(InArgs.BindingId), InArgs.BindingId);

			// If undone, this can be nullptr
			if(!ProtocolBinding)
			{
				return false;
			}

			int32 NumRemoved = 0;
			if (InArgs.RangeId.IsValid())
			{
				NumRemoved = ProtocolBinding->RemoveMapping(InArgs.RangeId);
			}
			else if (InArgs.RangeIds.Num() > 0)
			{
				ProtocolBinding->ClearMappings();
				NumRemoved = 1; // ClearMapping doesn't return a num, so set to 1 to indicate success
			}

			using FCommandChange = TRemoteControlProtocolCommandChange<Commands::FAddRemoveRangeArgs>;
			using FOnChange = FCommandChange::FOnUndoRedoDelegate;
			if(GUndo)
			{
				// What to do on undo and redo
				GUndo->StoreUndo(InPreset,
					MakeUnique<FCommandChange>(
						InPreset,
						CopyTemp(InArgs),
						FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveRangeArgs& InChangeArgs)
						{
							Commands::RemoveRangeMappingInternal(InPreset, InChangeArgs);
						}),
						FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveRangeArgs& InChangeArgs)
						{
							Commands::AddRangeMappingInternal(InPreset, InChangeArgs);
						})
					)
				);
			}

			return NumRemoved > 0;
		}
		
		return false;
	}
}

TMap<FProtocolBindingViewModel::EValidity, FText> FProtocolBindingViewModel::ValidityMessages =
{
	{FProtocolBindingViewModel::EValidity::Unchecked, FText::GetEmpty()},
	{FProtocolBindingViewModel::EValidity::Ok, FText::GetEmpty()},
	{FProtocolBindingViewModel::EValidity::InvalidChild, LOCTEXT("InvalidChild", "There are one or more errors for this binding. Binding is disabled until these are fixed.")},
	{FProtocolBindingViewModel::EValidity::DuplicateInput, LOCTEXT("DuplicateInput", "One or more other protocol bindings have the same input parameters:")},
	{FProtocolBindingViewModel::EValidity::LessThanTwoRanges, LOCTEXT("LessThanTwoRanges", "Range Mapping requires two or more items.")},
};

TSharedRef<FProtocolBindingViewModel> FProtocolBindingViewModel::Create(const TSharedRef<FProtocolEntityViewModel>& InParentViewModel, const TSharedRef<FRemoteControlProtocolBinding>& InBinding)
{
	TSharedRef<FProtocolBindingViewModel> ViewModel = MakeShared<FProtocolBindingViewModel>(FPrivateToken{}, InParentViewModel, InBinding);
	ViewModel->Initialize();

	return ViewModel;
}

FProtocolBindingViewModel::FProtocolBindingViewModel(FPrivateToken, const TSharedRef<FProtocolEntityViewModel>& InParentViewModel, const TSharedRef<FRemoteControlProtocolBinding>& InBinding)
	: Preset(InParentViewModel->Preset)
	, ParentViewModel(InParentViewModel)
	, Property(InParentViewModel->Property)
	, PropertyId(InParentViewModel->PropertyId)
	, BindingId(InBinding->GetId())
{
	GEditor->RegisterForUndo(this);
}

FProtocolBindingViewModel::~FProtocolBindingViewModel()
{
	ParentViewModel.Reset();
	Ranges.Reset();
	GEditor->UnregisterForUndo(this);
}

void FProtocolBindingViewModel::Initialize()
{
	const FRemoteControlProtocolBinding* Binding = GetBinding();
	// May be stale as a result of an undo deleting it.
	if(Binding)
	{
		const FProperty* MappingProperty = GetProperty().Get();
		if (!MappingProperty || !RemoteControlTypeUtilities::IsSupportedMappingType(MappingProperty))
		{
			return;
		}

		Ranges.Empty(Ranges.Num());
		GetBinding()->ForEachMapping([&](FRemoteControlProtocolMapping& InMapping)
        { 
            Ranges.Emplace(FProtocolRangeViewModel::Create(SharedThis(this), InMapping.GetId()));
        });
	}
}

FGuid FProtocolBindingViewModel::AddRangeMapping()
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("AddRangeMapping", "Add protocol binding range mapping"));
	GetPreset()->MarkPackageDirty();

	TSharedPtr<FProtocolRangeViewModel>& NewRangeViewModel = AddRangeMappingInternal();

	OnRangeMappingAddedDelegate.Broadcast(NewRangeViewModel.ToSharedRef());	

	return NewRangeViewModel->GetId();
}

// Can call elsewhere without triggering a transaction or event
TSharedPtr<FProtocolRangeViewModel>& FProtocolBindingViewModel::AddRangeMappingInternal()
{
	const FGuid RangeId = Commands::AddRangeMappingInternal(Preset.Get(), {ParentViewModel.Pin()->GetId(), GetId(), FGuid()});
	TSharedPtr<FProtocolRangeViewModel>& NewRangeViewModel = Ranges.Add_GetRef(FProtocolRangeViewModel::Create(AsShared(), RangeId));
	return NewRangeViewModel;
}

void FProtocolBindingViewModel::PostUndo(bool bSuccess)
{
	// Rebuild range ViewModels
	Initialize();
	OnChangedDelegate.Broadcast();
}

void FProtocolBindingViewModel::RemoveRangeMapping(const FGuid& InId)
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("RemoveRangeMapping", "Remove protocol binding range mapping"));
	check(GUndo != nullptr);
	GetPreset()->MarkPackageDirty();

	if(Commands::RemoveRangeMappingInternal(Preset.Get(), Commands::FAddRemoveRangeArgs{ParentViewModel.Pin()->GetId(), GetId(), InId}))
	{
		const int32 NumItemsRemoved = Ranges.RemoveAll([&](const TSharedPtr<FProtocolRangeViewModel>& InRange)
        {
            return InRange->GetId() == InId;
        });

		if(NumItemsRemoved <= 0)
		{
			return;
		}

		OnRangeMappingRemovedDelegate.Broadcast(InId);
	}
}

void FProtocolBindingViewModel::RemoveAllRangeMappings()
{
	check(IsValid());

	// Early-out if there are no range mappings
	if (Ranges.Num() == 0)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveAllRangeMappings", "Remove all protocol binding range mappings"));
	check(GUndo != nullptr);
	GetPreset()->MarkPackageDirty();

	TArray<FGuid> AllRangeIds;
	AllRangeIds.Reserve(Ranges.Num());
	Algo::Transform(Ranges, AllRangeIds, [](const TSharedPtr<FProtocolRangeViewModel>& InRange)
	{
		return InRange->GetId();
	});

	if (Commands::RemoveRangeMappingInternal(Preset.Get(), Commands::FAddRemoveRangeArgs{ParentViewModel.Pin()->GetId(), GetId(), {}, AllRangeIds}))
	{
		const int32 NumItemsRemoved = Ranges.Num();
		Ranges.Empty();

		if (NumItemsRemoved <= 0)
		{
			return;
		}

		OnRangeMappingsRemovedDelegate.Broadcast();
	}
}

void FProtocolBindingViewModel::AddDefaultRangeMappings()
{
	check(IsValid());

	const TSharedPtr<FProtocolRangeViewModel> MinItem = AddRangeMappingInternal();
	const TSharedPtr<FProtocolRangeViewModel> MaxItem = AddRangeMappingInternal();

	FName RangePropertyTypeName = GetBinding()->GetRemoteControlProtocolEntityPtr()->Get()->GetRangePropertyName();
	FProperty* RangeProperty = GetProtocol()->GetRangeInputTemplateProperty();
	const uint8 RangePropertySize = GetBinding()->GetRemoteControlProtocolEntityPtr()->Get()->GetRangePropertySize();

	// Range (input)
	{
		if (const FNumericProperty* NumericProperty = CastField<FNumericProperty>(RangeProperty))
		{
			if (NumericProperty->IsInteger())
			{
				int64 IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<int64>(NumericProperty);
				int64 IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<int64>(NumericProperty);

				// fixup typename according to typesize, ie. it can be a UInt32 but a typesize of 2 makes its a UInt16
				if(RangePropertyTypeName == NAME_UInt32Property && RangePropertySize > 0)
				{
					if(RangePropertySize == sizeof(uint8))
					{
						IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint8>(NumericProperty);
						IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint8>(NumericProperty);
						RangePropertyTypeName = NAME_ByteProperty;
					}
					else if(RangePropertySize == sizeof(uint16))
					{
						IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint16>(NumericProperty);
						IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint16>(NumericProperty);
						RangePropertyTypeName = NAME_UInt16Property; 
					}
					else if(RangePropertySize == sizeof(uint64))
					{
						IntMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<uint64>(NumericProperty);
						IntMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<uint64>(NumericProperty);
						RangePropertyTypeName = NAME_UInt64Property;
					}
				}

				if(RangePropertyTypeName == NAME_ByteProperty)
				{
					MinItem->SetInputValue<uint8>(IntMin);
					MaxItem->SetInputValue<uint8>(IntMax);
				}
				else if(RangePropertyTypeName == NAME_UInt16Property)
				{
					MinItem->SetInputValue<uint16>(IntMin);
					MaxItem->SetInputValue<uint16>(IntMax);
				}
				else if(RangePropertyTypeName == NAME_UInt32Property)
				{
					MinItem->SetInputValue<uint32>(IntMin);
					MaxItem->SetInputValue<uint32>(IntMax);
				}
				else if(RangePropertyTypeName == NAME_UInt64Property)
				{
					MinItem->SetInputValue<uint64>(IntMin);
					MaxItem->SetInputValue<uint64>(IntMax);
				}
			}
			else if(NumericProperty->IsFloatingPoint())
			{
				const float FloatMin = RemoteControlTypeUtilities::GetDefaultRangeValueMin<float>(RangeProperty);
				const float FloatMax = RemoteControlTypeUtilities::GetDefaultRangeValueMax<float>(RangeProperty);

				if(RangePropertyTypeName == NAME_FloatProperty)
				{
					MinItem->SetInputValue<float>(FloatMin);
					MaxItem->SetInputValue<float>(FloatMax);
				}
			}
		}
	}

	const FProperty* MappingProperty = GetProperty().Get();
	if (!MappingProperty)
	{
		return;
	}
	
	FName MappingPropertyTypeName = MappingProperty->GetClass()->GetFName();
	if(const FStructProperty* StructProperty = CastField<FStructProperty>(MappingProperty))
	{
		MappingPropertyTypeName = StructProperty->Struct->GetFName();
	}

	// Mapping (output)
	{
		if(const FNumericProperty* NumericProperty = CastField<FNumericProperty>(MappingProperty))
		{
			if(NumericProperty->IsInteger())
			{
				const int64 IntMin = RemoteControlTypeUtilities::GetDefaultMappingValueMin<int64>(NumericProperty);
				const int64 IntMax = RemoteControlTypeUtilities::GetDefaultMappingValueMax<int64>(NumericProperty);

				if (MappingPropertyTypeName == NAME_ByteProperty)
				{
					MinItem->SetOutputValue<uint8>(IntMin);
					MaxItem->SetOutputValue<uint8>(IntMax);
				}
				else if(MappingPropertyTypeName == NAME_Int8Property)
				{
					MinItem->SetOutputValue<int8>(IntMin);
					MaxItem->SetOutputValue<int8>(IntMax);
				}
				else if(MappingPropertyTypeName == NAME_Int16Property)
				{
					MinItem->SetOutputValue<int16>(IntMin);
					MaxItem->SetOutputValue<int16>(IntMax);
				}
				else if(RangePropertyTypeName == NAME_UInt16Property)
				{
					MinItem->SetOutputValue<uint16>(IntMin);
					MaxItem->SetOutputValue<uint16>(IntMax);
				}
				else if(MappingPropertyTypeName == NAME_Int32Property)
				{
					MinItem->SetOutputValue<int32>(IntMin);
					MaxItem->SetOutputValue<int32>(IntMax);
				}
				else if(RangePropertyTypeName == NAME_UInt32Property)
				{
					MinItem->SetOutputValue<uint32>(IntMin);
					MaxItem->SetOutputValue<uint32>(IntMax);
				}
				else if(MappingPropertyTypeName == NAME_Int64Property)
				{
					MinItem->SetOutputValue<int64>(IntMin);
					MaxItem->SetOutputValue<int64>(IntMax);
				}
				else if (MappingPropertyTypeName == NAME_UInt64Property)
				{
					MinItem->SetOutputValue<uint64>(IntMin);
					MaxItem->SetOutputValue<uint64>(IntMax);
				}
			}
			else if(NumericProperty->IsFloatingPoint())
			{
				const double FloatMin = RemoteControlTypeUtilities::GetDefaultMappingValueMin<double>(MappingProperty);
				const double FloatMax = RemoteControlTypeUtilities::GetDefaultMappingValueMax<double>(MappingProperty);

				if (MappingPropertyTypeName == NAME_FloatProperty)
				{
					MinItem->SetOutputValue<float>(FloatMin);
					MaxItem->SetOutputValue<float>(FloatMax);
				}
				else if (MappingPropertyTypeName == NAME_DoubleProperty)
				{
					MinItem->SetOutputValue<double>(FloatMin);
					MaxItem->SetOutputValue<double>(FloatMax);
				}
			}
		}
		else
		{
			if (MappingPropertyTypeName == NAME_BoolProperty)
            {
				MinItem->SetOutputValue<bool>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<bool>(MappingProperty));
				MaxItem->SetOutputValue<bool>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<bool>(MappingProperty));
            }

			else if (MappingPropertyTypeName == NAME_StrProperty)
			{
				using Type = TRCTypeNameToType<NAME_StrProperty>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_NameProperty)
			{
				using Type = TRCTypeNameToType<NAME_NameProperty>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_TextProperty)
            {
				using Type = TRCTypeNameToType<NAME_TextProperty>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
            }

			else if (MappingPropertyTypeName == NAME_Vector)
			{
				MinItem->SetOutputValue<FVector>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<FVector>(MappingProperty));
				MaxItem->SetOutputValue<FVector>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<FVector>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Vector2D)
			{
				using Type = TRCTypeNameToType<NAME_Vector2D>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Vector4)
			{
				using Type = TRCTypeNameToType<NAME_Vector4>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Rotator)
			{
				using Type = TRCTypeNameToType<NAME_Rotator>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Color)
			{
				using Type = TRCTypeNameToType<NAME_Color>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_LinearColor)
			{
				using Type = TRCTypeNameToType<NAME_LinearColor>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Transform)
			{
				using Type = TRCTypeNameToType<NAME_Transform>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_Quat)
			{
				using Type = TRCTypeNameToType<NAME_Quat>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else if (MappingPropertyTypeName == NAME_IntPoint)
			{
				using Type = TRCTypeNameToType<NAME_IntPoint>::Type;
				MinItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMin<Type>(MappingProperty));
				MaxItem->SetOutputValue<Type>(RemoteControlTypeUtilities::GetDefaultMappingValueMax<Type>(MappingProperty));
			}
			else
			{
				UE_LOGF(LogRemoteControlProtocolWidgets, Warning, "AddDefaultRangeMappings type: %ls for default value resolution.", *MappingPropertyTypeName.ToString());
			}
		}
		
		// (Overrides above behavior) - get initial "min" range item value from current property value
		MinItem->CopyFromCurrentPropertyValue();
	}
}

FRemoteControlProtocolBinding* FProtocolBindingViewModel::GetBinding() const
{
	check(IsValid());

	if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin())
	{
		return RCProperty->ProtocolBindings.FindByHash(GetTypeHash(BindingId), BindingId);
	}

	// Undo may have deleted the exposed entity
	return nullptr;
}

TSharedPtr<IRemoteControlProtocol> FProtocolBindingViewModel::GetProtocol() const
{
	// If undo removes the binding, this will occur
	if(!GetBinding())
	{
		return nullptr;		
	}
	
	return IRemoteControlProtocolModule::Get().GetProtocolByName(GetBinding()->GetProtocolName());
}

FRemoteControlProtocolMapping* FProtocolBindingViewModel::GetRangesMapping(const FGuid& InRangeId) const
{
	check(IsValid());

	if (FRemoteControlProtocolBinding* Binding = GetBinding())
	{
		FRemoteControlProtocolMapping* Mapping = Binding->FindMapping(InRangeId);
		ensure(Mapping && InRangeId == Mapping->GetId());
		return Mapping;
	}

	return nullptr;
}

void FProtocolBindingViewModel::Remove() const
{
	ParentViewModel.Pin()->RemoveBinding(GetId());
}

bool FProtocolBindingViewModel::GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren)
{
	const TArray<TSharedPtr<FProtocolRangeViewModel>> Children = GetRanges();
	OutChildren.Append(Children);
	return Children.Num() > 0;
}

bool FProtocolBindingViewModel::IsValid(FText& OutMessage)
{
	// Check regular validity
	check(IsValid());

	EValidity Result = EValidity::Ok;
	TUniquePtr<FString> AdditionalMessage = MakeUnique<FString>();

	if (Ranges.Num() <= 1)
	{
		Result = EValidity::LessThanTwoRanges;
	}

	// Continue if ok
	if (Result == EValidity::Ok)
	{
		if (Algo::AnyOf(Ranges, [](const TSharedPtr<FProtocolRangeViewModel>& InRangeViewModel)
		{
			return !InRangeViewModel->GetCurrentValidity();
		}))
		{
			Result = EValidity::InvalidChild;
		}
	}

	// Continue if ok
	if (Result == EValidity::Ok)
	{
		const FRemoteControlProtocolBinding* ProtocolBinding = GetBinding();
		// Was probably deleted
		if(ProtocolBinding == nullptr)
		{
			return false;
		}
		
		const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity = ProtocolBinding->GetRemoteControlProtocolEntityPtr();

		FString DuplicateBindings;

		// Iterate over each exposed field
		for (TWeakPtr<FRemoteControlField> ExposedFieldWeakPtr : GetPreset()->GetExposedEntities<FRemoteControlField>())
		{
			const TSharedPtr<FRemoteControlField> ExposedField = ExposedFieldWeakPtr.Pin();

			// Skip self
			if (ExposedField->GetId() == PropertyId)
			{
				continue;
			}

			for (FRemoteControlProtocolBinding& Binding : ExposedField->ProtocolBindings)
			{
				// Only care about same protocol
				if (Binding.GetProtocolName() == GetBinding()->GetProtocolName())
				{
					const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> OtherProtocolEntity = Binding.GetRemoteControlProtocolEntityPtr();
					if (OtherProtocolEntity->Get()->IsSame(ProtocolEntity->Get()))
					{
						Result = EValidity::DuplicateInput;
						DuplicateBindings.Appendf(TEXT("•	%s"), *ExposedField->FieldPathInfo.ToString());
						DuplicateBindings.Append(TEXT("\n"));
					}
				}
			}
		}

		if (DuplicateBindings.Len() > 0)
		{
			AdditionalMessage.Reset(new FString(MoveTemp(DuplicateBindings)));
		}
	}

	CurrentValidity = Result;
	OutMessage = ValidityMessages[CurrentValidity];

	if (AdditionalMessage)
	{
		FTextBuilder TextBuilder;
		TextBuilder.AppendLine(OutMessage);
		TextBuilder.AppendLine(FText::FromString(*AdditionalMessage));

		OutMessage = TextBuilder.ToText();
	}

	return CurrentValidity == EValidity::Ok;
}

bool FProtocolBindingViewModel::IsValid() const
{
	return Preset.IsValid()
			&& PropertyId.IsValid()
			&& BindingId.IsValid();
}

void FProtocolBindingViewModel::NotifyChanged() const
{
	check(IsValid());
	OnChangedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE
