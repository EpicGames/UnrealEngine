// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "EditorUndoClient.h"
#include "IRemoteControlProtocol.h"
#include "RCViewModelCommon.h"
#include "RemoteControlProtocolBinding.h"
#include "UObject/WeakFieldPtr.h"

class FProtocolEntityViewModel;
class FProtocolRangeViewModel;
class URemoteControlPreset;
struct FRemoteControlProtocolBinding;

namespace ProtocolBindingViewModel
{
	/** Describes all possible validity states */
	enum class EValidity : uint8
	{
		Unchecked = 0, /** Not yet checked */
		Ok = 1, /** Valid */
		InvalidChild = 2, /** There are one or more errors in child viewmodels */
		DuplicateInput = 3, /** Input event/channel, etc. is the same as another protocol binding */
		LessThanTwoRanges = 4, /** Two or more ranges are required */
	};
}

/** Represents a single protocol binding for an entity */
class REMOTECONTROLPROTOCOLWIDGETS_API FProtocolBindingViewModel
	: public TSharedFromThis<FProtocolBindingViewModel>
	, public FEditorUndoClient
	, public TRCValidatableViewModel<ProtocolBindingViewModel::EValidity>
	, public IRCTreeNodeViewModel
{
public:
	/** Create a new ViewModel for the given Entity and Protocol Binding. */
	static TSharedRef<FProtocolBindingViewModel> Create(const TSharedRef<FProtocolEntityViewModel>& InParentViewModel, const TSharedRef<FRemoteControlProtocolBinding>& InBinding);
	virtual ~FProtocolBindingViewModel() override;

	/** Add a new RangeMapping. */
	FGuid AddRangeMapping();

	/** Remove a RangeMapping by Id. */
	void RemoveRangeMapping(const FGuid& InId);

	/** Removes all RangeMappings. */
	void RemoveAllRangeMappings();

	/** Creates default range mappings depending on the input and output types. */
	void AddDefaultRangeMappings();

	/** Get the Binding Id. */
	const FGuid& GetId() const { return BindingId; }

	/** Get the owning Preset. */
	TWeakObjectPtr<URemoteControlPreset> GetPreset() const { return Preset; }

	/** Get the bound FProperty. Should always be valid so long as the VM itself is valid. */
	TWeakFieldPtr<FProperty> GetProperty() const { return Property; }

	/** Get the underlying protocol binding instance. */
	FRemoteControlProtocolBinding* GetBinding() const;

	/** Get the bound protocol. */
	TSharedPtr<IRemoteControlProtocol> GetProtocol() const;

	/** Get the bound protocol name as Text. */
	FText GetProtocolName() const { return FText::FromName(GetBinding()->GetProtocolName()); }

	/** Get the Range ViewModels. */
	const TArray<TSharedPtr<FProtocolRangeViewModel>>& GetRanges() const { return Ranges; }

	/** Get the underlying RangeMapping. */
	FRemoteControlProtocolMapping* GetRangesMapping(const FGuid& InRangeId) const;

	/** Removes this Protocol Binding. */
	void Remove() const;

	/** Get logical children of this ViewModel */
	virtual bool GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren) override;

	/** Checks (non-critical, fixable) validity */
	virtual bool IsValid(FText& OutMessage) override;

	/** Checks validity of this ViewModel */
	bool IsValid() const;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRangeMappingAdded, TSharedRef<FProtocolRangeViewModel> /* InRangeViewModel */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnRangeMappingRemoved, FGuid /* InRangeId */);
	DECLARE_MULTICAST_DELEGATE(FOnRangeMappingsRemoved);
	DECLARE_MULTICAST_DELEGATE(FOnChanged);

	/** When single range mapping added. */
	FOnRangeMappingAdded& OnRangeMappingAdded() { return OnRangeMappingAddedDelegate; }

	/** When single range mapping removed. */
	FOnRangeMappingRemoved& OnRangeMappingRemoved() { return OnRangeMappingRemovedDelegate; }

	/** When all range mappings removed. */
	FOnRangeMappingsRemoved& OnRangeMappingsRemoved() { return OnRangeMappingsRemovedDelegate; }

	/** Something has changed within the ViewModel. */
	FOnChanged& OnChanged() { return OnChangedDelegate; }

	/** Allows the view to notify the ViewModel of a change (ie. from SDetailsView) */
	void NotifyChanged() const;

private:
	friend class FProtocolEntityViewModel;
	
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

public:
	FProtocolBindingViewModel(FPrivateToken) {}
	FProtocolBindingViewModel(FPrivateToken, const TSharedRef<FProtocolEntityViewModel>& InParentViewModel, const TSharedRef<FRemoteControlProtocolBinding>& InBinding);

private:
	void Initialize();

	TSharedPtr<FProtocolRangeViewModel>& AddRangeMappingInternal();

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	/** Message for each Validity state */
	static TMap<EValidity, FText> ValidityMessages;

private:
	friend class FProtocolRangeViewModel;

	/** Owning Preset. */
	TWeakObjectPtr<URemoteControlPreset> Preset;

	/** Owning Entity (Property within a Preset). */
	TWeakPtr<FProtocolEntityViewModel> ParentViewModel;

	/** Bound property. */
	TWeakFieldPtr<FProperty> Property;

	/** Unique Id of the bound Property. */
	FGuid PropertyId;

	/** Unique Id of this protocol binding. */
	FGuid BindingId;

	/** Range ViewModels for this protocol binding. */
	TArray<TSharedPtr<FProtocolRangeViewModel>> Ranges;
	
	/** When single range mapping added. */
	FOnRangeMappingAdded OnRangeMappingAddedDelegate;

	/** When single range mapping removed. */
	FOnRangeMappingRemoved OnRangeMappingRemovedDelegate;

	/** When all range mappings removed. */
	FOnRangeMappingsRemoved OnRangeMappingsRemovedDelegate;

	/** Something has changed within the ViewModel. */
	FOnChanged OnChangedDelegate;
};
