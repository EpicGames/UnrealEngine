// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProtocolEntityViewModel.h"

#include "Editor.h"
#include "IRCProtocolBindingList.h"
#include "IRemoteControlProtocol.h"
#include "IRemoteControlProtocolModule.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "ProtocolBindingViewModel.h"
#include "ProtocolCommandChange.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolWidgetsSettings.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"

#define LOCTEXT_NAMESPACE "ProtocolEntityViewModel"

// Used by FChange (for Undo/Redo). Keep independent of ViewModel (can be out of scope when called).
namespace Commands
{
	struct FAddRemoveProtocolArgs
	{
		FGuid EntityId;
		FName ProtocolName;
		FGuid BindingId;
	};

	static TSharedPtr<FRemoteControlProtocolBinding> AddProtocolInternal(URemoteControlPreset* InPreset, const FAddRemoveProtocolArgs& InArgs);
	static FName RemoveProtocolInternal(URemoteControlPreset* InPreset, const FAddRemoveProtocolArgs& InArgs);

	static TSharedPtr<FRemoteControlProtocolBinding> AddProtocolInternal(URemoteControlPreset* InPreset, const FAddRemoveProtocolArgs& InArgs)
	{
		const TSharedPtr<IRemoteControlProtocol> Protocol = IRemoteControlProtocolModule::Get().GetProtocolByName(InArgs.ProtocolName);
		if (Protocol.IsValid())
		{
			if (const TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(InArgs.EntityId).Pin())
			{
				const FRemoteControlProtocolEntityPtr RemoteControlProtocolEntityPtr = Protocol->CreateNewProtocolEntity(RCProperty->GetProperty(), InPreset, InArgs.EntityId);

				FRemoteControlProtocolBinding ProtocolBinding(InArgs.ProtocolName, InArgs.EntityId, RemoteControlProtocolEntityPtr, InArgs.BindingId.IsValid() ? InArgs.BindingId : FGuid::NewGuid());
				Protocol->Bind(ProtocolBinding.GetRemoteControlProtocolEntityPtr());
				RCProperty->ProtocolBindings.Emplace(MoveTemp(ProtocolBinding));

				TSharedPtr<FRemoteControlProtocolBinding> NewBinding = MakeShared<FRemoteControlProtocolBinding>(ProtocolBinding);

				using FCommandChange = TRemoteControlProtocolCommandChange<Commands::FAddRemoveProtocolArgs>;
				using FOnChange = FCommandChange::FOnUndoRedoDelegate;
				if (GUndo)
				{
					FAddRemoveProtocolArgs Args = InArgs;
					// Set or re-set the BindingId (will be same if it was already set), so undos below remove or add the correct binding
					Args.BindingId = NewBinding->GetId();
					
					GUndo->StoreUndo(InPreset,
						MakeUnique<FCommandChange>(
							InPreset,
							MoveTemp(Args),
							FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveProtocolArgs& InChangeArgs)
							{
								Commands::AddProtocolInternal(InPreset, InChangeArgs);
							}),
							FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveProtocolArgs& InChangeArgs)
							{
								Commands::RemoveProtocolInternal(InPreset, InChangeArgs);
							})
						)
					);
				}

				return NewBinding;
			}
		}

		return nullptr;
	}

	static FName RemoveProtocolInternal(URemoteControlPreset* InPreset, const FAddRemoveProtocolArgs& InArgs)
	{
		if (const TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(InArgs.EntityId).Pin())
		{
			const FRemoteControlProtocolBinding* FoundElement = RCProperty->ProtocolBindings.FindByHash(GetTypeHash(InArgs.BindingId), InArgs.BindingId);
			if (FoundElement)
			{
				// Unbind the protocol from the property explicitly while removing.
				if (const TSharedPtr<IRemoteControlProtocol> Protocol = IRemoteControlProtocolModule::Get().GetProtocolByName(FoundElement->GetProtocolName()))
				{
					if (TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> RCProtocolEntityPtr = FoundElement->GetRemoteControlProtocolEntityPtr())
					{
						const ERCBindingStatus BindingStatus = (*RCProtocolEntityPtr)->GetBindingStatus();

						IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
						
						if (BindingStatus == ERCBindingStatus::Bound)
						{
							if (TSharedPtr<IRCProtocolBindingList> RCProtocolBindingList = RCProtocolWidgetsModule.GetProtocolBindingList())
							{
								RCProtocolBindingList->OnStopRecording(RCProtocolEntityPtr);
							}
						}

						(*RCProtocolEntityPtr)->ResetDefaultBindingState();

						Protocol->Unbind(RCProtocolEntityPtr);
					}
				}

				RCProperty->ProtocolBindings.RemoveByHash(GetTypeHash(InArgs.BindingId), InArgs.BindingId);

				const FName RemovedProtocolName = FoundElement->GetProtocolName();
				if (RemovedProtocolName != NAME_None)
				{
					using FCommandChange = TRemoteControlProtocolCommandChange<Commands::FAddRemoveProtocolArgs>;
					using FOnChange = FCommandChange::FOnUndoRedoDelegate;
					if (GUndo)
					{
						FAddRemoveProtocolArgs Args = InArgs;
						Args.ProtocolName = RemovedProtocolName;
						
						GUndo->StoreUndo(InPreset,
							MakeUnique<FCommandChange>(
								InPreset,
								MoveTemp(Args),
								FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveProtocolArgs& InChangeArgs)
								{
									Commands::RemoveProtocolInternal(InPreset, InChangeArgs);
								}),
								FOnChange::CreateLambda([](URemoteControlPreset* InPreset, const Commands::FAddRemoveProtocolArgs& InChangeArgs)
								{
									Commands::AddProtocolInternal(InPreset, InChangeArgs);
								})
							)
						);
					}
				}
				
				return RemovedProtocolName;
			}
		}

		return NAME_None;
	}
}

TMap<FProtocolEntityViewModel::EValidity, FText> FProtocolEntityViewModel::ValidityMessages =
{
	{FProtocolEntityViewModel::EValidity::Unchecked, FText::GetEmpty()},
	{FProtocolEntityViewModel::EValidity::Ok, FText::GetEmpty()},
	{FProtocolEntityViewModel::EValidity::InvalidChild, LOCTEXT("InvalidChild", "There are one or more errors for this binding.")},
	{FProtocolEntityViewModel::EValidity::UnsupportedType, LOCTEXT("UnsupportedType", "The input or output types are unsupported.")},
	{FProtocolEntityViewModel::EValidity::Unbound, LOCTEXT("Unbound", "The entity is unbound. Rebind to re-enable.")}
};

TSharedRef<FProtocolEntityViewModel> FProtocolEntityViewModel::Create(URemoteControlPreset* InPreset, const FGuid& InEntityId)
{
	TSharedRef<FProtocolEntityViewModel> ViewModel = MakeShared<FProtocolEntityViewModel>(FPrivateToken{}, InPreset, InEntityId);
	ViewModel->Initialize();

	return ViewModel;
}

FProtocolEntityViewModel::FProtocolEntityViewModel(FPrivateToken, URemoteControlPreset* InPreset, const FGuid& InEntityId)
	: Preset(InPreset)
	, PropertyId(InEntityId)
{
	check(Preset != nullptr);
	check(Preset->IsExposed(InEntityId));

	GEditor->RegisterForUndo(this);
}

FProtocolEntityViewModel::~FProtocolEntityViewModel()
{
	Bindings.Reset();
	GEditor->UnregisterForUndo(this);
}

void FProtocolEntityViewModel::Initialize()
{
	Preset->OnEntityUnexposed().AddSP(this, &FProtocolEntityViewModel::OnEntityUnexposed);

	Bindings.Empty(Bindings.Num());
	if (const TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin())
	{
		Property = RCProperty->GetProperty();
		for(FRemoteControlProtocolBinding& Binding : RCProperty->ProtocolBindings)
		{
			const TSharedPtr<IRemoteControlProtocol> Protocol = IRemoteControlProtocolModule::Get().GetProtocolByName(Binding.GetProtocolName());
			// Supporting plugin needs to be loaded/protocol available
			if (Protocol.IsValid())
			{
				const TSharedPtr<FProtocolBindingViewModel>& BindingViewModel = Bindings.Add_GetRef(FProtocolBindingViewModel::Create(SharedThis(this), MakeShared<FRemoteControlProtocolBinding>(Binding)));
				// Probably result of "redo" action
				if(BindingViewModel->GetRanges().IsEmpty())
				{
					BindingViewModel->AddDefaultRangeMappings();
				}

				Protocol->Bind(Binding.GetRemoteControlProtocolEntityPtr());
			}
		}
	}
}

void FProtocolEntityViewModel::OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InEntityId)
{
	// It's possible these aren't valid at this point if the owning preset is deleted as well.
	if (InPreset && InEntityId.IsValid())
	{
		if (GetId() == InEntityId)
		{
			for (const TSharedPtr<FProtocolBindingViewModel>& Binding : Bindings)
			{
				if (Binding.IsValid())
				{
					RemoveProtocolInternal(InPreset, Commands::FAddRemoveProtocolArgs{GetId(), NAME_None, Binding->GetId()});
				}
			}

			Bindings.Empty();
			OnChanged().Broadcast();
		}
	}
}

// @note: This should closely match RemoteControlProtocolBinding.h
// InProtocolName can be NAME_All which only checks output type support
bool FProtocolEntityViewModel::CanAddBinding(const FName& InProtocolName, FText& OutMessage)
{
	check(IsValid());

	// If no protocols registered
	if (InProtocolName == NAME_None)
	{
		return false;
	}

	if (RemoteControlTypeUtilities::IsSupportedMappingType(GetProperty().Get()))
	{
		return true;
	}

	// Remaining should be strings, enums
	OutMessage = FText::Format(LOCTEXT("UnsupportedTypeForBinding", "Unsupported Type \"{0}\" for Protocol Binding"), GetProperty()->GetClass()->GetDisplayNameText());
	return false;
}

TSharedPtr<FProtocolBindingViewModel> FProtocolEntityViewModel::AddBinding(const FName& InProtocolName)
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("AddProtocolBinding", "Add Protocol Binding"));
	Preset->MarkPackageDirty();

	const FGuid CommandId = FGuid::NewGuid();
	const TSharedPtr<FRemoteControlProtocolBinding> ProtocolBinding = Commands::AddProtocolInternal(Preset.Get(), Commands::FAddRemoveProtocolArgs{GetId(), InProtocolName});
	if (ProtocolBinding.IsValid())
	{
		TSharedPtr<FProtocolBindingViewModel>& NewBindingViewModel = Bindings.Add_GetRef(FProtocolBindingViewModel::Create(AsShared(), ProtocolBinding.ToSharedRef()));
		NewBindingViewModel->AddDefaultRangeMappings();
		
		OnBindingAddedDelegate.Broadcast(NewBindingViewModel.ToSharedRef());

		return NewBindingViewModel;
	}

	return nullptr;
}

void FProtocolEntityViewModel::RemoveBinding(const FGuid& InBindingId)
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("RemoveProtocolBinding", "Remove Protocol Binding"));
	Preset->MarkPackageDirty();

	Commands::RemoveProtocolInternal(Preset.Get(), Commands::FAddRemoveProtocolArgs{GetId(), NAME_None, InBindingId});

	const int32 NumItemsRemoved = Bindings.RemoveAll([&](const TSharedPtr<FProtocolBindingViewModel>& InBinding)
    {
        return InBinding->GetId() == InBindingId;
    });

	if (NumItemsRemoved <= 0)
	{
		return;
	}

	OnBindingRemovedDelegate.Broadcast(InBindingId);
}

TWeakFieldPtr<FProperty> FProtocolEntityViewModel::GetProperty()
{
	if (!Property.IsValid())
	{
		Initialize();
	}

	return Property;
}

TArray<TSharedPtr<FProtocolBindingViewModel>> FProtocolEntityViewModel::GetFilteredBindings(const TSet<FName>& InHiddenProtocolTypeNames)
{
	check(IsValid());

	if (InHiddenProtocolTypeNames.Num() == 0)
	{
		return GetBindings();
	}

	TArray<TSharedPtr<FProtocolBindingViewModel>> FilteredBindings;
	for(const TSharedPtr<FProtocolBindingViewModel>& Binding : Bindings)
	{
		// Make sure this bindings protocol name is NOT in the Hidden list, then add it to FilteredBindings
		if (Algo::NoneOf(InHiddenProtocolTypeNames, [&Binding](const FName& InHiddenTypeName)
		{
			return InHiddenTypeName == Binding->GetBinding()->GetProtocolName();
		}))
		{
			FilteredBindings.Add(Binding);
		}
	}
	
	return FilteredBindings;
}

bool FProtocolEntityViewModel::IsBound() const
{
	return Preset.IsValid()
			&& Preset->GetExposedEntity<FRemoteControlField>(PropertyId).IsValid()
			&& Preset->GetExposedEntity<FRemoteControlField>(PropertyId).Pin()->IsBound();
}

bool FProtocolEntityViewModel::GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren)
{
	const URemoteControlProtocolWidgetsSettings* Settings = GetDefault<URemoteControlProtocolWidgetsSettings>();

	const TArray<TSharedPtr<FProtocolBindingViewModel>> Children = GetFilteredBindings(Settings->HiddenProtocolTypeNames);
	OutChildren.Append(Children);
	return Children.Num() > 0;
}

bool FProtocolEntityViewModel::IsValid(FText& OutMessage)
{
	// Check regular validity
	check(IsValid());

	EValidity Result = EValidity::Ok;

	// 1. Check if bound
	if (!IsBound())
	{
		Result = EValidity::Unbound;
	}

	// 2. Check if input or output containers are invalid, and therefore type is unsupported
	if (Result == EValidity::Ok)
	{
		FText Unused;
		if (!CanAddBinding(NAME_All, Unused))
		{
			Result = EValidity::UnsupportedType;
		}
	}

	// 3. Check if any children aren't valid
	if (Result == EValidity::Ok)
	{
		if (Algo::AnyOf(Bindings, [](const TSharedPtr<FProtocolBindingViewModel>& InBindingViewModel)
		{
			return !InBindingViewModel->GetCurrentValidity();
		}))
		{
			Result = EValidity::InvalidChild;
		}
	}

	OutMessage = ValidityMessages[Result];
	CurrentValidity = Result;
	return Result == EValidity::Ok;
}

bool FProtocolEntityViewModel::IsValid() const
{
	return Preset.IsValid()
			&& PropertyId.IsValid();
}

URemoteControlPreset* FProtocolEntityViewModel::GetPreset() const
{
	return Preset.Get();
}

void FProtocolEntityViewModel::PostUndo(bool bSuccess)
{
	check(IsValid());

	// Rebuild range ViewModels
	Initialize();
	OnChangedDelegate.Broadcast();
}

#undef LOCTEXT_NAMESPACE