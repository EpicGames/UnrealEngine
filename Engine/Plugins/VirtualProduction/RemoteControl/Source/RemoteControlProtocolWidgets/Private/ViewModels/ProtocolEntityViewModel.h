// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorUndoClient.h"
#include "RCViewModelCommon.h"
#include "UObject/WeakFieldPtr.h"

class FProtocolBindingViewModel;
class URemoteControlPreset;

namespace ProtocolEntityViewModel
{
	/** Describes all possible validity states */
	enum class EValidity : uint8
	{
		Unchecked = 0, /** Not yet checked */
		Ok = 1, /** Valid */
		InvalidChild = 2, /** There are one or more errors in child viewmodels */
		UnsupportedType = 3, /** The input or output property types aren't supported */
		Unbound = 4, /** The entity needs to be re-bound */
	};
}

/** Contains all bindings for a given Entity (ie. Property) */
class FProtocolEntityViewModel
	: public TSharedFromThis<FProtocolEntityViewModel>
	, public FEditorUndoClient
	, public TRCValidatableViewModel<ProtocolEntityViewModel::EValidity>
	, public IRCTreeNodeViewModel
{
public:
	/** Create a new ViewModel for the given Preset and EntityId */
	REMOTECONTROLPROTOCOLWIDGETS_API static TSharedRef<FProtocolEntityViewModel> Create(URemoteControlPreset* InPreset, const FGuid& InEntityId);
	REMOTECONTROLPROTOCOLWIDGETS_API virtual ~FProtocolEntityViewModel() override;

	/** Check if the bound entity type is supported by Protocol Binding */
	REMOTECONTROLPROTOCOLWIDGETS_API bool CanAddBinding(const FName& InProtocolName, FText& OutMessage);

	/** Add a new Protocol Binding */
	REMOTECONTROLPROTOCOLWIDGETS_API TSharedPtr<FProtocolBindingViewModel> AddBinding(const FName& InProtocolName);

	/** Remove a Protocol Binding by Id */
	REMOTECONTROLPROTOCOLWIDGETS_API void RemoveBinding(const FGuid& InBindingId);

	/** Get the Entity Id */
	const FGuid& GetId() const { return PropertyId; }

	/** Get the bound FProperty */
	REMOTECONTROLPROTOCOLWIDGETS_API TWeakFieldPtr<FProperty> GetProperty();

	/** Get the Protocol Binding ViewModels */
	const TArray<TSharedPtr<FProtocolBindingViewModel>>& GetBindings() const { return Bindings; }

	/** Get the Protocol Binding ViewModels, filtered by the provided list of hidden protocol type names */
	REMOTECONTROLPROTOCOLWIDGETS_API TArray<TSharedPtr<FProtocolBindingViewModel>> GetFilteredBindings(const TSet<FName>& InHiddenProtocolTypeNames);
	
	/** Checks if this entity is bound to one or more objects. */
	REMOTECONTROLPROTOCOLWIDGETS_API bool IsBound() const;

	/** Get logical children of this ViewModel */
	REMOTECONTROLPROTOCOLWIDGETS_API virtual bool GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren) override;

	/** Checks (non-critical, fixable) validity */
	REMOTECONTROLPROTOCOLWIDGETS_API virtual bool IsValid(FText& OutMessage) override;

	/** Checks validity of this ViewModel */
	REMOTECONTROLPROTOCOLWIDGETS_API bool IsValid() const;

	/** Returns the preset of this view model or nullptr if there is no outer preset */
	URemoteControlPreset* GetPreset() const;

public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingAdded, TSharedRef<FProtocolBindingViewModel> /* InBindingViewModel */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBindingRemoved, FGuid /* InBindingId */);
	DECLARE_MULTICAST_DELEGATE(FOnChanged);

	FOnBindingAdded& OnBindingAdded() { return OnBindingAddedDelegate; }
	FOnBindingRemoved& OnBindingRemoved() { return OnBindingRemovedDelegate; }

	/** Something has changed within the ViewModel */
	FOnChanged& OnChanged() { return OnChangedDelegate; }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };
	
public:
	FProtocolEntityViewModel(FPrivateToken) {}
	REMOTECONTROLPROTOCOLWIDGETS_API FProtocolEntityViewModel(FPrivateToken, URemoteControlPreset* InPreset, const FGuid& InEntityId);

protected:
	void Initialize();

	/** Respond when entity is unexposed */
	void OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InEntityId);

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

private:
	/** Message for each Validity state */
	static TMap<EValidity, FText> ValidityMessages;

private:
	friend class FProtocolBindingViewModel;

	/** Owning Preset */
	TWeakObjectPtr<URemoteControlPreset> Preset;

	/** Bound property */
	TWeakFieldPtr<FProperty> Property;

	/** Unique Id of the bound Property */
	FGuid PropertyId;

	/** Protocol Binding ViewModels for this Entity */
	TArray<TSharedPtr<FProtocolBindingViewModel>> Bindings;

	FOnBindingAdded OnBindingAddedDelegate;
	FOnBindingRemoved OnBindingRemovedDelegate;
	FOnChanged OnChangedDelegate;
};
