// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProtocolRangeViewModel.h"

#include "Editor.h"
#include "IRemoteControlModule.h"
#include "PropertyHandle.h"
#include "Misc/DefaultValueHelper.h"

#define LOCTEXT_NAMESPACE "ProtocolRangeViewModel"

TMap<FProtocolRangeViewModel::EValidity, FText> FProtocolRangeViewModel::ValidityMessages =
{
	{FProtocolRangeViewModel::EValidity::Unchecked, FText::GetEmpty()},
	{FProtocolRangeViewModel::EValidity::Ok, FText::GetEmpty()},
	{FProtocolRangeViewModel::EValidity::DuplicateInput, LOCTEXT("DuplicateInput", "This range contains the same input value as another.")},
	{FProtocolRangeViewModel::EValidity::UnsupportedType, LOCTEXT("UnsupportedType", "The input or output types are unsupported.")}
};

TSharedRef<FProtocolRangeViewModel> FProtocolRangeViewModel::Create(const TSharedRef<FProtocolBindingViewModel>& InParentViewModel, const FGuid& InRangeId)
{
	TSharedRef<FProtocolRangeViewModel> ViewModel = MakeShared<FProtocolRangeViewModel>(FPrivateToken{}, InParentViewModel, InRangeId);
	ViewModel->Initialize();

	return ViewModel;
}

FProtocolRangeViewModel::FProtocolRangeViewModel(FPrivateToken, const TSharedRef<FProtocolBindingViewModel>& InParentViewModel, const FGuid& InRangeId)
	: Preset(InParentViewModel->Preset)
	, ParentViewModel(InParentViewModel)
	, RangeId(InRangeId)
{
	GEditor->RegisterForUndo(this);
	InParentViewModel->OnChanged().AddRaw(this, &FProtocolRangeViewModel::OnParentChanged);
}

FProtocolRangeViewModel::~FProtocolRangeViewModel()
{
	if(ParentViewModel.IsValid())
	{
		ParentViewModel.Pin()->OnChanged().RemoveAll(this);
	}
	
	GEditor->UnregisterForUndo(this);
}

void FProtocolRangeViewModel::Initialize()
{
	// Creates input property container
	if (const FProperty* Property = GetInputProperty())
	{
		URCPropertyContainerBase* PropertyContainer = PropertyContainers::CreateContainerForProperty(GetTransientPackage(), Property);
		if (PropertyContainer)
		{			
			InputProxyPropertyContainer.Reset(PropertyContainer);
			UpdateInputValueRange();
		}
	}

	// Creates output property container
	if (const FProperty* Property = GetProperty().Get())
	{
		URCPropertyContainerBase* PropertyContainer = PropertyContainers::CreateContainerForProperty(GetTransientPackage(), Property);
		if (PropertyContainer)
		{
			OutputProxyPropertyContainer.Reset(PropertyContainer);
		}
	}
}

void FProtocolRangeViewModel::UpdateInputValueRange()
{
	check(IsValid());

	// Early out if not yet initialized or stale
	if(!InputProxyPropertyContainer.IsValid() || !ParentViewModel.IsValid())
	{
		return;	
	}

	if (const FProperty* Property = GetInputProperty())
	{
		// If there's a mismatch between the RangePropertySize and Property, clamp to RangePropertySize
		const uint8 NewRangeTypeSize = GetBinding()->GetRemoteControlProtocolEntityPtr()->Get()->GetRangePropertySize();
		FProperty* PropertyInContainer = InputProxyPropertyContainer->GetValueProperty();
		int64 NewRangeMaxValue = 0;

		// Set this metadata to indicate we set the ClampMax flag, not the user
		static const FName RCSetKey = TEXT("RCSetClampMax");
		{
			if(const FNumericProperty* NumericProperty = CastField<FNumericProperty>(Property))
			{
				// @note: Only works with ints, up to uint32
				if(NumericProperty->IsInteger())
				{
					const FString RangeMaxValueStr = GetBinding()->GetRemoteControlProtocolEntityPtr()->Get()->GetRangePropertyMaxValue();
					if(!RangeMaxValueStr.IsEmpty())
					{
						FDefaultValueHelper::ParseInt64(RangeMaxValueStr, NewRangeMaxValue);
						NewRangeMaxValue = FCString::Atoi64(*RangeMaxValueStr);
					}
					// Infer from RangeTypeSize
					else if (NewRangeTypeSize < 4)
					{
						NewRangeMaxValue = (static_cast<int64>(1) << (NewRangeTypeSize * 8)) - 1;
					}
					else
					{
						NewRangeMaxValue = TNumericLimits<uint32>::Max();
					}
					
					if(PreviousRangeMaxValue == 0)
					{
						PreviousRangeMaxValue = NewRangeMaxValue;
					}

					if(PreviousRangeMaxValue == NewRangeMaxValue)
					{
						return;
					}

					PreviousRangeMaxValue = RemoteControlTypeUtilities::GetMetadataValue(
						NumericProperty,
						RemoteControlTypeUtilities::ClampMaxKey,
						RemoteControlTypeUtilities::GetMetadataValue(NumericProperty, RCSetKey, PreviousRangeMaxValue));

					// Remaps values (eg. 0,64,255 for 8 bit remaps to 0, 16384, 65535 for 16 bit)
					const uint32* PreviousValue = InputProxyPropertyContainer->GetValue<uint32>();
					const float PreviousValueNormalized = static_cast<float>(*PreviousValue) / static_cast<float>(PreviousRangeMaxValue);

					const uint32 NewValue = static_cast<uint32>(static_cast<double>(NewRangeMaxValue) * PreviousValueNormalized);

					FScopedTransaction Transaction(LOCTEXT("SetPresetProtocolInputData", "Set Preset protocol binding input data"));
					InputProxyPropertyContainer->SetValue(NewValue);

					// 16777216 = 24bit, metadata breaks beyond >8 digits so remove
					if(NewRangeMaxValue > 16777216)
					{
						PropertyInContainer->RemoveMetaData(RemoteControlTypeUtilities::ClampMaxKey);
						PropertyInContainer->RemoveMetaData(RCSetKey);
					}
					else
					{
						FString ClampMaxStr;
						ClampMaxStr.AppendInt(NewRangeMaxValue);
						PropertyInContainer->SetMetaData(RemoteControlTypeUtilities::ClampMaxKey, *ClampMaxStr);
						PropertyInContainer->SetMetaData(RCSetKey, *ClampMaxStr);
					}
				}
			}
		}

		PreviousRangeMaxValue = NewRangeMaxValue;
	}
}

void FProtocolRangeViewModel::CopyInputValue(const TSharedPtr<IPropertyHandle>& InDstHandle) const
{
	check(IsValid());

	if (FRemoteControlProtocolMapping* RangesData = GetRangesData())
	{
		RangesData->CopyRawRangeData<FProperty>(InDstHandle);
	}
}

void FProtocolRangeViewModel::SetInputData(const TSharedPtr<IPropertyHandle>& InSrcHandle) const
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("SetPresetProtocolInputData", "Set Preset protocol binding input data"));
	Preset->Modify();

	GetRangesData()->SetRawRangeData<FProperty>(InSrcHandle);
}

void FProtocolRangeViewModel::CopyOutputValue(const TSharedPtr<IPropertyHandle>& InDstHandle) const
{
	check(IsValid());

	GetRangesData()->CopyRawMappingData<FProperty>(InDstHandle);
}

void FProtocolRangeViewModel::SetOutputData(const TSharedPtr<IPropertyHandle>& InSrcHandle) const
{
	check(IsValid());

	FScopedTransaction Transaction(LOCTEXT("SetPresetProtocolOutputData", "Set Preset protocol binding output data"));
	Preset->Modify();

	GetRangesData()->SetRawMappingData<FProperty>(InSrcHandle);
}

void FProtocolRangeViewModel::SetOutputData(const FRCFieldResolvedData& InResolvedData) const
{
	check(IsValid());
	check(InResolvedData.IsValid());

	if (InputProxyPropertyContainer.IsValid() && OutputProxyPropertyContainer.IsValid())
	{
		const bool bShouldActuallyTransact = !GIsTransacting;
		FScopedTransaction Transaction(LOCTEXT("SetPresetProtocolOutputData", "Set Preset protocol binding output data"), bShouldActuallyTransact);
		OutputProxyPropertyContainer.Get()->Modify();

		void* SrcPropertyData = InResolvedData.Field->ContainerPtrToValuePtr<void>(InResolvedData.ContainerAddress);
		OutputProxyPropertyContainer->SetValue((uint8*)SrcPropertyData, RemoteControlTypeUtilities::GetPropertySize(InResolvedData.Field, SrcPropertyData));

		Preset->Modify();

		FProperty* DstProperty = OutputProxyPropertyContainer->GetValueProperty();
		void* DstPropertyData = DstProperty->ContainerPtrToValuePtr<void>(OutputProxyPropertyContainer.Get());

		GetRangesData()->SetRawMappingData(Preset.Get(), DstProperty, DstPropertyData);
	}
}

FProperty* FProtocolRangeViewModel::GetInputProperty() const
{
	check(ParentViewModel.IsValid());
	
	return ParentViewModel.Pin()->GetProtocol()->GetRangeInputTemplateProperty();
}

UObject* FProtocolRangeViewModel::GetInputContainer() const
{
	return InputProxyPropertyContainer.IsValid() ? InputProxyPropertyContainer.Get() : nullptr;
}

FName FProtocolRangeViewModel::GetInputTypeName() const
{
	check(IsValid());

	return GetBinding()->GetRemoteControlProtocolEntityPtr()->Get()->GetRangePropertyName();
}

UObject* FProtocolRangeViewModel::GetOutputContainer() const
{
	return OutputProxyPropertyContainer.IsValid() ? OutputProxyPropertyContainer.Get() : nullptr;
}

void FProtocolRangeViewModel::CopyFromCurrentPropertyValue() const
{
	check(IsValid());

	// @note: if this fails, ammend IsValid above to account for it
	const TSharedPtr<const FRemoteControlField> ExposedEntity = GetPreset()->GetExposedEntity<FRemoteControlField>(GetBinding()->GetPropertyId()).Pin();
	FRCFieldPathInfo FieldPathInfo = ExposedEntity->FieldPathInfo;

	// Why would there be more than one?
	for (UObject* BoundObject : ExposedEntity->GetBoundObjects())
	{
		if (BoundObject)
		{
			if (FieldPathInfo.Resolve(BoundObject))
			{
				const FRCFieldResolvedData ResolvedData = FieldPathInfo.GetResolvedData();
				if (ResolvedData.IsValid())
				{
					SetOutputData(ResolvedData);
				}
			}
			break;
		}
	}
}

void FProtocolRangeViewModel::Remove() const
{
	ParentViewModel.Pin()->RemoveRangeMapping(GetId());
}

bool FProtocolRangeViewModel::IsValid(FText& OutMessage)
{
	// Check regular validity
	check(IsValid());

	EValidity Result = EValidity::Ok;

	// 1. Check if input or output containers are invalid, and therefore type is unsupported
	if (!InputProxyPropertyContainer.IsValid() || !OutputProxyPropertyContainer.IsValid())
	{
		Result = EValidity::UnsupportedType;
	}

	// Continue if ok
	if (Result == EValidity::Ok)
	{
		// 2. Check if input is same as another
		for (const TSharedPtr<FProtocolRangeViewModel>& RangeViewModel : ParentViewModel.Pin()->GetRanges())
		{
			// skip self
			if (RangeViewModel->GetId() == this->GetId())
			{
				continue;
			}

			if (RangeViewModel->IsInputSame(this))
			{
				// input is the same as another
				Result = EValidity::DuplicateInput;
				break;
			}
		}
	}

	OutMessage = ValidityMessages[Result];
	CurrentValidity = Result;
	return Result == EValidity::Ok;
}

bool FProtocolRangeViewModel::IsValid() const
{
	return ParentViewModel.IsValid()
			&& Preset.IsValid()
			&& RangeId.IsValid();
}

void FProtocolRangeViewModel::PostUndo(bool bSuccess)
{
	OnChangedDelegate.Broadcast();
}

void FProtocolRangeViewModel::OnParentChanged()
{
	check(IsValid());
	
	// Undo deleted the parent binding
	if(!GetBinding())
	{
		return;
	}
	
	UpdateInputValueRange();
	OnChangedDelegate.Broadcast();
}

bool FProtocolRangeViewModel::IsInputSame(const FProtocolRangeViewModel* InOther) const
{
	check(InOther);
	check(IsValid());

	if (InputProxyPropertyContainer.IsValid() && InOther->InputProxyPropertyContainer.IsValid())
	{
		FProperty* InputProperty = InputProxyPropertyContainer->GetValueProperty();
		TArray<uint8> ValueData;
		
		InputProxyPropertyContainer->GetValue(ValueData);
		TArray<uint8> OtherValueData;
		
		InOther->InputProxyPropertyContainer->GetValue(OtherValueData);

		return InputProperty->Identical(ValueData.GetData(), OtherValueData.GetData());
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
