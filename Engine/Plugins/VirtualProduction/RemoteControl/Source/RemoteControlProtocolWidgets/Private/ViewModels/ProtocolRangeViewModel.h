// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ProtocolBindingViewModel.h"
#include "RCPropertyContainer.h"
#include "RCViewModelCommon.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "Logging/TokenizedMessage.h"
#include "UObject/StrongObjectPtr.h"

class FProtocolBindingViewModel;
class IPropertyHandle;

#define LOCTEXT_NAMESPACE "ProtocolBindingViewModel"

namespace ProtocolRangeViewModel
{
	/** Describes all possible validity states */
	enum class EValidity : uint8
	{
		Unchecked = 0,
		/** Not yet checked */
		Ok = 1,
		/** Valid */
		DuplicateInput = 2,
		/** The input value is the same as another range input */
		UnsupportedType = 3,
		/** The input or output property types aren't supported */
	};
}

/** Represents a single range mapping for a given protocol binding  */
class REMOTECONTROLPROTOCOLWIDGETS_API FProtocolRangeViewModel
	: public TSharedFromThis<FProtocolRangeViewModel>
	, public FEditorUndoClient
	, public TRCValidatableViewModel<ProtocolRangeViewModel::EValidity>
	, public IRCTreeNodeViewModel
{
public:
	/** Create a new ViewModel for the given Protocol Binding and Range Id */
	static TSharedRef<FProtocolRangeViewModel> Create(const TSharedRef<FProtocolBindingViewModel>& InParentViewModel, const FGuid& InRangeId);
	virtual ~FProtocolRangeViewModel() override;

	/** Copy the input to the given PropertyHandle */
	void CopyInputValue(const TSharedPtr<IPropertyHandle>& InDstHandle) const;

	/** Set input from raw data */
	void SetInputData(const TSharedPtr<IPropertyHandle>& InSrcHandle) const;

	/** Set input */
	template <typename ValueType>
    void SetInputValue(ValueType InValue, bool bWithTransaction = false)
	{
		check(IsValid());

		if(bWithTransaction)
		{
			FScopedTransaction Transaction(LOCTEXT("SetInputValue", "Set Input Value"));
			Preset->Modify();	
		}
		
		GetRangesData()->SetRangeValue(InValue);
	}

	/** Copy the output to the given PropertyHandle */
	void CopyOutputValue(const TSharedPtr<IPropertyHandle>& InDstHandle) const;

	/** Set underlying output from raw data */
	void SetOutputData(const TSharedPtr<IPropertyHandle>& InSrcHandle) const;

	/** Set underlying output from resolved data */
	void SetOutputData(const FRCFieldResolvedData& InResolvedData) const;

	/** Set output */
	template <typename ValueType>
	void SetOutputValue(ValueType InValue, bool bWithTransaction = false)
	{
		check(IsValid());

		if(bWithTransaction)
		{
			FScopedTransaction Transaction(LOCTEXT("SetOutputValue", "Set Output Value"));
        	Preset->Modify();	
		}

		GetRangesData()->SetMappingValueAsPrimitive(InValue);
	}

	/** Get the Range Id */
	const FGuid& GetId() const { return RangeId; }

	/** Get the owning ProtocolBinding Id */
	const FGuid& GetBindingId() const { return GetBinding()->GetId(); }

	/** Get the owning Preset */
	const URemoteControlPreset* GetPreset() const { return Preset.Get(); }

	/** Get the input TypeName */
	FName GetInputTypeName() const;

	/** Get the range input FProperty */
	FProperty* GetInputProperty() const;

	/** Get the containing object of the input property, or nullptr if it doesn't exist */
	UObject* GetInputContainer() const;

	/** Get the bound FProperty */
	TWeakFieldPtr<FProperty> GetProperty() const { return ParentViewModel.Pin()->GetProperty(); }

	/** Get the containing object of the output property, or nullptr if it doesn't exist */
	UObject* GetOutputContainer() const;

	/** Copies current property value to this range's output value. */
	void CopyFromCurrentPropertyValue() const;

	/** Removes this Range Mapping */
	void Remove() const;

	/** Get logical children of this ViewModel */
	virtual bool GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren) override { return false; }

	/** Checks (non-critical, fixable) validity */
	virtual bool IsValid(FText& OutMessage) override;

	/** Checks validity of this ViewModel */
	bool IsValid() const;

public:
	DECLARE_MULTICAST_DELEGATE(FOnChanged);
	
	/** Something has changed within the ViewModel */
	FOnChanged& OnChanged() { return OnChangedDelegate; }

private:
	// Private token only allows members or friends to call MakeShared
	struct FPrivateToken { explicit FPrivateToken() = default; };

	friend class FProtocolBindingViewModel;
	
public:
	FProtocolRangeViewModel(FPrivateToken) {}
	FProtocolRangeViewModel(FPrivateToken, const TSharedRef<FProtocolBindingViewModel>& InParentViewModel, const FGuid& InRangeId);

protected:
	void Initialize();

	/** Updates the input range value clamping (in case the type/precision has changed). */
	void UpdateInputValueRange();

	FRemoteControlProtocolMapping* GetRangesData() const { return ParentViewModel.Pin()->GetRangesMapping(RangeId); }
	FRemoteControlProtocolBinding* GetBinding() const { return ParentViewModel.Pin()->GetBinding(); }

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); }
	// End of FEditorUndoClient

	/** Parent ViewModel changed. */
	void OnParentChanged();

private:
	/** Checks if the input value is the same as that of the other provided ViewModel. */
	bool IsInputSame(const FProtocolRangeViewModel* InOther) const;

	/** Message for each Validity state */
	static TMap<EValidity, FText> ValidityMessages;

	/** Last checked validity state. */
	EValidity CurrentValidity = EValidity::Unchecked;

private:
	/** Owning Preset */
	TWeakObjectPtr<URemoteControlPreset> Preset;
	
	/** Owning ProtocolBinding */
	TWeakPtr<FProtocolBindingViewModel> ParentViewModel;

	/** Unique Id of this Range */
	FGuid RangeId;

	FOnChanged OnChangedDelegate;

	/** Previous value of RangeTypeSize for comparison when changed. */
	uint64 PreviousRangeMaxValue = 0;

	/** A UObject container/owner for the input property */
	TStrongObjectPtr<URCPropertyContainerBase> InputProxyPropertyContainer;

	/** A UObject container/owner for the output property */
	TStrongObjectPtr<URCPropertyContainerBase> OutputProxyPropertyContainer;
};

#undef LOCTEXT_NAMESPACE
