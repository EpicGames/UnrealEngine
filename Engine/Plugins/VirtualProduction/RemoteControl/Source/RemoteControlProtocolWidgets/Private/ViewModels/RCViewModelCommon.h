// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * A base type of viewmodels with validation for display
 * ValidityStateType must have an "Unchecked" and "Ok" member
 */
template <typename ValidityStateType>
class TRCValidatableViewModel
{
public:
	typedef ValidityStateType EValidity;

	virtual ~TRCValidatableViewModel() = default;

	/** Checks (non-critical, fixable) validity */
	virtual bool IsValid(FText& OutMessage) = 0;

	/** Get last-checked validity. Note that this does not perform any validation itself. */
	virtual bool GetCurrentValidity() const { return CurrentValidity <= EValidity::Ok; }

protected:
	/** Last checked validity state. */
	EValidity CurrentValidity = EValidity::Unchecked;
};

/** Common interface for treeview nodes. */
class IRCTreeNodeViewModel
{
protected:
	~IRCTreeNodeViewModel() = default;

public:
	/** See STreeView::OnGetChildren, return false if no children. */
	virtual bool GetChildren(TArray<TSharedPtr<IRCTreeNodeViewModel>>& OutChildren) = 0;
};
