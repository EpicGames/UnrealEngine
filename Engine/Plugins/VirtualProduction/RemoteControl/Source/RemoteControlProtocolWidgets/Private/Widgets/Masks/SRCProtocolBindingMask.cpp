// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolBindingMask.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolBinding.h"
#include "ViewModels/ProtocolBindingViewModel.h"

#define LOCTEXT_NAMESPACE "SRCProtocolBindingMask"

void SRCProtocolBindingMask::Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel)
{
	ViewModel = InViewModel;

	const bool bHasOptionalMask = HasOptionalMask();

	const bool bCanBeMasked = CanBeMasked();

	SRCProtocolMaskTriplet::Construct(SRCProtocolMaskTriplet::FArguments()
		.MaskA(ERCMask::MaskA)
		.MaskB(ERCMask::MaskB)
		.MaskC(ERCMask::MaskC)
		.OptionalMask(ERCMask::MaskD)
		.MaskingType(GetMaskingType())
		.CanBeMasked(bCanBeMasked)
		.EnableOptionalMask(bHasOptionalMask)
	);
}

bool SRCProtocolBindingMask::CanBeMasked() const
{
	if (ViewModel.IsValid())
	{
		if (const FProperty* Property = ViewModel->GetProperty().Get())
		{
			return IRemoteControlModule::Get().SupportsMasking(Property);
		}
	}

	return false;
}

EMaskingType SRCProtocolBindingMask::GetMaskingType()
{
	if (CanBeMasked())
	{
		if (const FProperty* Property = ViewModel->GetProperty().Get())
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				if (const EMaskingType* StructToMaskingType = FRemoteControlProtocolMasking::GetStructsToMaskingTypes().Find(StructProperty->Struct))
				{
					return *StructToMaskingType;
				}
			}
		}
	}

	return EMaskingType::Unsupported;
}

bool SRCProtocolBindingMask::HasOptionalMask() const
{
	if (ViewModel.IsValid())
	{
		if (const FProperty* Property = ViewModel->GetProperty().Get())
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				return FRemoteControlProtocolMasking::GetOptionalMaskStructs().Contains(StructProperty->Struct);
			}
		}
	}

	return false;
}

ECheckBoxState SRCProtocolBindingMask::IsMaskEnabled(ERCMask InMaskBit) const
{
	if (ViewModel.IsValid())
	{
		if (const FRemoteControlProtocolBinding* RCProtocolBinding = ViewModel->GetBinding())
		{
			if (TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> RCProtocolEntity = RCProtocolBinding->GetRemoteControlProtocolEntityPtr())
			{
				return (*RCProtocolEntity)->HasMask((ERCMask)InMaskBit) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			}
		}
	}

	return SRCProtocolMaskTriplet::IsMaskEnabled(InMaskBit);
}

void SRCProtocolBindingMask::SetMaskEnabled(ECheckBoxState NewState, ERCMask NewMaskBit)
{
	if (ViewModel.IsValid())
	{
		if (URemoteControlPreset* Preset = ViewModel->GetPreset().Get())
		{
			Preset->Modify();
		}

		if (const FRemoteControlProtocolBinding* RCProtocolBinding = ViewModel->GetBinding())
		{
			if (TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> RCProtocolEntity = RCProtocolBinding->GetRemoteControlProtocolEntityPtr())
			{
				if (NewState == ECheckBoxState::Checked)
				{
					(*RCProtocolEntity)->EnableMask((ERCMask)NewMaskBit);
				}
				else
				{
					(*RCProtocolEntity)->ClearMask((ERCMask)NewMaskBit);
				}
			}
		}
		else
		{
			SRCProtocolMaskTriplet::SetMaskEnabled(NewState, NewMaskBit);
		}
	}
}

#undef LOCTEXT_NAMESPACE
