// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Masks/SRCProtocolMaskTriplet.h"

enum class ERCMask : uint8;
class FProtocolBindingViewModel;

/**
 * A widget that represents a mask (specific to protocol bindings).
 */
class SRCProtocolBindingMask : public SRCProtocolMaskTriplet, public IHasMaskExtensibility
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolBindingMask)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel);

protected:

	//~ Begin : IHasMaskExtensibility Interface
	virtual bool CanBeMasked() const override;
	virtual EMaskingType GetMaskingType() override;
	virtual bool HasOptionalMask() const override;
	//~ End : IHasMaskExtensibility Interface

	//~ Begin : SRCProtocolMaskTriplet Interface
	virtual ECheckBoxState IsMaskEnabled(ERCMask InMaskBit) const override;
	virtual void SetMaskEnabled(ECheckBoxState NewState, ERCMask NewMaskBit) override;
	//~ End : SRCProtocolMaskTriplet Interface

private:

	/** ViewModel for the Protocol Binding. */
	TSharedPtr<FProtocolBindingViewModel> ViewModel;
};
