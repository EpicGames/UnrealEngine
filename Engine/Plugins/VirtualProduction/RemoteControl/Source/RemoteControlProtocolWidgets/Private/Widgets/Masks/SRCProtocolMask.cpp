// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Masks/SRCProtocolMask.h"

#include "RemoteControlField.h"

#define LOCTEXT_NAMESPACE "SRCProtocolMask"

void SRCProtocolMask::Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> InField)
{
	WeakField = MoveTemp(InField);

	const bool bHasOptionalMask = HasOptionalMask();

	const bool bCanBeMasked = CanBeMasked();

	SRCProtocolMaskTriplet::Construct(SRCProtocolMaskTriplet::FArguments()
		.MaskA(ERCMask::MaskA)
		.MaskB(ERCMask::MaskB)
		.MaskC(ERCMask::MaskC)
		.OptionalMask(ERCMask::MaskD)
		.MaskingType(GetMaskingType())
		.CanBeMasked(bCanBeMasked)
		.EnableOptionalMask(bHasOptionalMask)
	);
}

bool SRCProtocolMask::CanBeMasked() const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		return RCField->SupportsMasking();
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return false;
}

EMaskingType SRCProtocolMask::GetMaskingType()
{
	if (CanBeMasked())
	{
		if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
		{
			if (TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(RCField))
			{
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(RCProperty->GetProperty()))
				{
					if (const EMaskingType* StructToMaskingType = FRemoteControlProtocolMasking::GetStructsToMaskingTypes().Find(StructProperty->Struct))
					{
						return *StructToMaskingType;
					}
				}
			}
		}
	}

	return EMaskingType::Unsupported;
}

bool SRCProtocolMask::HasOptionalMask() const
{
	if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		if (TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(RCField))
		{
			if (const FStructProperty* StructProperty = CastField<FStructProperty>(RCProperty->GetProperty()))
			{
				return FRemoteControlProtocolMasking::GetOptionalMaskStructs().Contains(StructProperty->Struct);
			}
		}
	}

	return false;
}

ECheckBoxState SRCProtocolMask::IsMaskEnabled(ERCMask InMaskBit) const
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		return RCField->HasMask((ERCMask)InMaskBit) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return SRCProtocolMaskTriplet::IsMaskEnabled(InMaskBit);
}

void SRCProtocolMask::SetMaskEnabled(ECheckBoxState NewState, ERCMask NewMaskBit)
{
	if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (NewState == ECheckBoxState::Checked)
		{
			RCField->EnableMask((ERCMask)NewMaskBit);
		}
		else
		{
			RCField->ClearMask((ERCMask)NewMaskBit);
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		SRCProtocolMaskTriplet::SetMaskEnabled(NewState, NewMaskBit);
	}
}

#undef LOCTEXT_NAMESPACE
