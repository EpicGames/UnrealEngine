// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolMaskButton.h"

#include "SlateOptMacros.h"

#include "Styling/ProtocolPanelStyle.h"
#include "Styling/ProtocolStyles.h"

#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRCProtocolMaskButton"

FText SRCProtocolMaskButton::MaskText = LOCTEXT("MaskLabel", "Mask");
FText SRCProtocolMaskButton::UnmaskText = LOCTEXT("UnmaskLabel", "Unmask");

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCProtocolMaskButton::Construct(const FArguments& InArgs)
{
	WidgetStyle = &FProtocolPanelStyle::Get()->GetWidgetStyle<FProtocolWidgetStyle>("ProtocolsPanel.Widgets.Mask");

	MaskBit = InArgs._MaskBit;

	MaskColor = InArgs._MaskColor;

	MaskedState = ECheckBoxState::Unchecked;

	IsInMaskedState = InArgs._IsMasked;

	OnMaskStateChanged = InArgs._OnMasked;

	DefaultLabel = FText::Format(LOCTEXT("DefaultMaskLabel", "{0}"), { InArgs._DefaultLabel.IsSet() ? InArgs._DefaultLabel.Get() : FText::FromString(TEXT("[LabelMissing]")) });

	OptionalToolTipText = InArgs._ToolTip.IsSet() ? TAttribute<FText>() :
		InArgs._OptionalTooltip.IsSet() ? InArgs._OptionalTooltip.Get() : TAttribute<FText>();

	SBorder::Construct(SBorder::FArguments()
		.Padding(0.f)
		.BorderImage(&WidgetStyle->ContentAreaBrushDark)
		.Content()
		[
			SAssignNew(Mask, SCheckBox)
			.Style(&WidgetStyle->MaskButtonStyle)
			.IsChecked(this, &SRCProtocolMaskButton::IsMaskEnabled)
			.OnCheckStateChanged(this, &SRCProtocolMaskButton::SetMaskEnabled)
			.HAlign(HAlign_Center)
			.ToolTip(InArgs._ToolTip)
			.ToolTipText(this, &SRCProtocolMaskButton::HandleMaskTooltip)
			[
				SNew(SOverlay)

				+SOverlay::Slot()
				.HAlign(HAlign_Fill)
				.Padding(0.f, 2.f, 0.f, 0.f)
				[
					SNew(SBorder)
					.BorderImage(&WidgetStyle->ContentAreaBrushLight)
					.BorderBackgroundColor(MaskColor)
				]

				+SOverlay::Slot()
				.HAlign(HAlign_Center)
				.Padding(0.f, 0.f, 0.f, 2.f)
				[
					SAssignNew(MaskBorder, SBorder)
					.BorderImage(&WidgetStyle->ContentAreaBrushDark)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(FMargin(8.f, 2.f))
					[
						SAssignNew(MaskLabel, STextBlock)
						.Text(DefaultLabel.Get())
						.TextStyle(&WidgetStyle->PlainTextStyle)
					]
				]
			]
		]
	);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SRCProtocolMaskButton::IsMasked() const
{
	if (Mask.IsValid())
	{
		return Mask->IsChecked();
	}

	return false;
}

void SRCProtocolMaskButton::ToggleMaskedState(bool bSoftToggle)
{
	if (bSoftToggle && MaskLabel.IsValid())
	{
		MaskLabel->SetTextStyle(&WidgetStyle->BoldTextStyle);
		MaskLabel->SetColorAndOpacity(FLinearColor::White);
		MaskBorder->SetBorderBackgroundColor(MaskColor);
		MaskBorder->SetBorderImage(&WidgetStyle->ContentAreaBrushLight);
	}
	else if (Mask.IsValid())
	{
		Mask->ToggleCheckedState();
	}
}

FText SRCProtocolMaskButton::HandleMaskTooltip() const
{
	if (OptionalToolTipText.Get().IsEmpty())
	{
		return FText::Format(LOCTEXT("DefaultMaskTooltip", "{0} {1}"), {  IsMasked() ? UnmaskText : MaskText,  DefaultLabel.Get() });
	}

	return OptionalToolTipText.Get();
}

ECheckBoxState SRCProtocolMaskButton::IsMaskEnabled() const
{
	if (IsInMaskedState.IsBound())
	{
		return IsInMaskedState.Execute(MaskBit);
	}

	return MaskedState.Get(ECheckBoxState::Unchecked);
}

void SRCProtocolMaskButton::SetMaskEnabled(ECheckBoxState NewState)
{
	const bool bIsChecked = NewState == ECheckBoxState::Checked && MaskLabel.IsValid();

	MaskedState = NewState;

	if (bIsChecked)
	{
		MaskLabel->SetTextStyle(&WidgetStyle->BoldTextStyle);
		MaskLabel->SetColorAndOpacity(FLinearColor::White);
		MaskBorder->SetBorderBackgroundColor(MaskColor);
		MaskBorder->SetBorderImage(&WidgetStyle->ContentAreaBrushLight);
	}
	else
	{
		MaskLabel->SetTextStyle(&WidgetStyle->PlainTextStyle);
		MaskLabel->SetColorAndOpacity(FSlateColor::UseForeground());
		MaskBorder->SetBorderBackgroundColor(FLinearColor::White);
		MaskBorder->SetBorderImage(&WidgetStyle->ContentAreaBrushDark);
	}

	OnMaskStateChanged.ExecuteIfBound(NewState, MaskBit);
}

#undef LOCTEXT_NAMESPACE
