// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SBorder.h"

enum class ERCMask : uint8;
struct FProtocolWidgetStyle;
class SCheckBox;
class STextBlock;

/** Delegate that is executed to retrieve the current mask state */
DECLARE_DELEGATE_RetVal_OneParam(ECheckBoxState, FIsMasked, ERCMask /* MaskBit */);

/** Delegate that is executed when the mask state changes */
DECLARE_DELEGATE_TwoParams(FOnMaskStateChanged, ECheckBoxState /* NewState */, ERCMask /* MaskBit */);

/**
 * A custom widget represents a mask widget.
 */
class SRCProtocolMaskButton : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolMaskButton)
		: _MaskBit()
		, _MaskColor()
		, _DefaultLabel()
		, _OptionalTooltip()
	{}

		SLATE_ARGUMENT(ERCMask, MaskBit)
		SLATE_ARGUMENT(FLinearColor, MaskColor)

		SLATE_ATTRIBUTE(FText, DefaultLabel)
		SLATE_ATTRIBUTE(FText, OptionalTooltip)

		/** Whether the mask button is currently in a active state */
		SLATE_EVENT(FIsMasked, IsMasked)

		/** Called when the masked state has changed */
		SLATE_EVENT(FOnMaskStateChanged, OnMasked)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/**
	 * Retrieves mask bit.
	 */
	ERCMask GetMaskBit() const
	{
		return MaskBit;
	}

	/**
	 * Returns true if the mask is currently enabled
	 *
	 * @return	True if masked, otherwise false
	 */
	bool IsMasked() const;

	/**
	 * Toggles the masked state for this mask button, fire events as needed
	 */
	void ToggleMaskedState(bool bSoftToggle = false);

protected:

	/**
	 * Retrieves the tooltip of the mask button based on its state.
	 */
	FText HandleMaskTooltip() const;

	/**
	 * Retrieves the state of the mask.
	 */
	ECheckBoxState IsMaskEnabled() const;

	/**
	 * Handles masked state.
	 */
	void SetMaskEnabled(ECheckBoxState NewState);

private:

	/** Holds the label of this mask button. */
	TAttribute<FText> DefaultLabel;
	
	/** Holds the tooltip text of this mask button. */
	TAttribute<FText> OptionalToolTipText;

	/** Holds the mask that should be affected by this button as an uint8. */
	ERCMask MaskBit;

	/** The color of the mask to set */
	FLinearColor MaskColor = FLinearColor::White;

	/** Holds a shared reference to the underlying mask button. */
	TSharedPtr<SCheckBox> Mask;
	
	/** Holds a shared reference to the underlying mask label. */
	TSharedPtr<STextBlock> MaskLabel;

	/** Holds a shared reference to the underlying mask border. */
	TSharedPtr<SBorder> MaskBorder;

	/** Holds the "Mask" string as text. */
	static FText MaskText;

	/** Holds the "Unmask" string as text. */
	static FText UnmaskText;

	/** Are we Masked */
	TAttribute<ECheckBoxState> MaskedState;

	/** Are we masked */
	FIsMasked IsInMaskedState;

	/** Delegate called when the masking state changes. */
	FOnMaskStateChanged OnMaskStateChanged;

	/** Protocol Widget Style reference. */
	const FProtocolWidgetStyle* WidgetStyle;
};
