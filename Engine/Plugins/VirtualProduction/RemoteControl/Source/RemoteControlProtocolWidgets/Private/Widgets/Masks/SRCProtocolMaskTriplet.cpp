// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Masks/SRCProtocolMaskTriplet.h"
#include "SRCProtocolMaskButton.h"

#include "Styling/ProtocolPanelStyle.h"
#include "Styling/ProtocolStyles.h"

#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "SRCProtocolMaskTriplet"

const TMap<UScriptStruct*, EMaskingType>& FRemoteControlProtocolMasking::GetStructsToMaskingTypes()
{
	static TMap<UScriptStruct*, EMaskingType> StructsToMaskingTypes = { { TBaseStructure<FColor>::Get(), EMaskingType::Color }
		, { TBaseStructure<FLinearColor>::Get(), EMaskingType::Color }
		, { TBaseStructure<FRotator>::Get(), EMaskingType::Rotator }
		, { TBaseStructure<FVector>::Get(), EMaskingType::Vector }
		, { TBaseStructure<FIntVector>::Get(), EMaskingType::Vector }
		, { TBaseStructure<FVector4>::Get(), EMaskingType::Quat }
		, { TBaseStructure<FIntVector4>::Get(), EMaskingType::Quat }
		, { TBaseStructure<FQuat>::Get(), EMaskingType::Quat }
	};

	return StructsToMaskingTypes;
}

const TSet<UScriptStruct*>& FRemoteControlProtocolMasking::GetOptionalMaskStructs()
{
	static TSet<UScriptStruct*> OptionalMaskStructs = {
		TBaseStructure<FLinearColor>::Get(),
		TBaseStructure<FColor>::Get(),
		TBaseStructure<FVector4>::Get(),
		TBaseStructure<FQuat>::Get(),
		TBaseStructure<FIntVector4>::Get()
	};

	return OptionalMaskStructs;
}

void SRCProtocolMaskTriplet::Construct(const FArguments& InArgs)
{
	WidgetStyle = &FProtocolPanelStyle::Get()->GetWidgetStyle<FProtocolWidgetStyle>("ProtocolsPanel.Widgets.Mask");

	MaskingType = InArgs._MaskingType;

	const bool bCanBeMasked = InArgs._CanBeMasked.Get(false);

	TSharedPtr<SWidget> ContentWidget = SNullWidget::NullWidget;

	if (bCanBeMasked)
	{
		TSharedPtr<SHorizontalBox> ContentHBox;

		const FText MaskALabel = GetMaskLabel(EMasking::Bits::ETM_A);
		const FText MaskBLabel = GetMaskLabel(EMasking::Bits::ETM_B);
		const FText MaskCLabel = GetMaskLabel(EMasking::Bits::ETM_C);

		ContentWidget = SNew(SBorder) // Foreground Border
			.BorderImage(&WidgetStyle->ContentAreaBrushDark)
			[
				SAssignNew(ContentHBox, SHorizontalBox)

				// Mask A
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SAssignNew(MaskA, SRCProtocolMaskButton)
					.MaskBit(InArgs._MaskA)
					.MaskColor(FLinearColor::Red)
					.DefaultLabel(MaskALabel)
					.IsMasked(this, &SRCProtocolMaskTriplet::IsMaskEnabled)
					.OnMasked(this, &SRCProtocolMaskTriplet::SetMaskEnabled)
				]
		
				// Spacer
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]

				// Mask B
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SAssignNew(MaskB, SRCProtocolMaskButton)
					.MaskBit(InArgs._MaskB)
					.MaskColor(FLinearColor::Green)
					.DefaultLabel(MaskBLabel)
					.IsMasked(this, &SRCProtocolMaskTriplet::IsMaskEnabled)
					.OnMasked(this, &SRCProtocolMaskTriplet::SetMaskEnabled)
				]
				
				// Spacer
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				]

				// Mask C
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SAssignNew(MaskC, SRCProtocolMaskButton)
					.MaskBit(InArgs._MaskC)
					.MaskColor(FLinearColor::Blue)
					.DefaultLabel(MaskCLabel)
					.IsMasked(this, &SRCProtocolMaskTriplet::IsMaskEnabled)
					.OnMasked(this, &SRCProtocolMaskTriplet::SetMaskEnabled)
				]
			];

		if (InArgs._EnableOptionalMask && ContentHBox.IsValid())
		{
			const FText MaskDLabel = GetMaskLabel(EMasking::Bits::ETM_D);

			// Spacer
			ContentHBox->AddSlot()
				.FillWidth(1.f)
				[
					SNew(SSpacer)
				];

			// Optional Mask
			ContentHBox->AddSlot()
				.AutoWidth()
				.Padding(2.f, 0.f)
				[
					SAssignNew(MaskD, SRCProtocolMaskButton)
					.MaskBit(InArgs._OptionalMask)
					.MaskColor(FLinearColor::White)
					.DefaultLabel(MaskDLabel)
					.IsMasked(this, &SRCProtocolMaskTriplet::IsMaskEnabled)
					.OnMasked(this, &SRCProtocolMaskTriplet::SetMaskEnabled)
				];
		}
	}

	// Background Border
	SBorder::Construct(SBorder::FArguments()
		.BorderImage(bCanBeMasked ? &WidgetStyle->ContentAreaBrush : FAppStyle::Get().GetBrush("NoBorder"))
		.HAlign(HAlign_Center)
		.Padding(2.f)
		.Content()
		[
			ContentWidget.ToSharedRef()
		]
	);

	LoadMasks();
}

FText SRCProtocolMaskTriplet::GetMaskLabel(EMasking::Bits InMaskBitName) const
{
	FText MaskLabel = FText::GetEmpty();

	switch (MaskingType)
	{
		case EMaskingType::Rotator:
		case EMaskingType::Vector:
		case EMaskingType::Quat:
		{
			switch (InMaskBitName)
			{
				case EMasking::Bits::ETM_A:
					MaskLabel = LOCTEXT("MaskXLabel", "X");
					break;
				case EMasking::Bits::ETM_B:
					MaskLabel = LOCTEXT("MaskYLabel", "Y");
					break;
				case EMasking::Bits::ETM_C:
					MaskLabel = LOCTEXT("MaskZLabel", "Z");
					break;
				case EMasking::Bits::ETM_D:
					MaskLabel = LOCTEXT("MaskWLabel", "W");
					break;
			}
		}
		break;
		case EMaskingType::Polar:
		{
			switch (InMaskBitName)
			{
				case EMasking::Bits::ETM_A:
					MaskLabel = LOCTEXT("MaskILabel", "U");
					break;
				case EMasking::Bits::ETM_B:
					MaskLabel = LOCTEXT("MaskJLabel", "V");
					break;
				case EMasking::Bits::ETM_C:
					MaskLabel = LOCTEXT("MaskKLabel", "W");
					break;
			}
		}
		break;
		case EMaskingType::Color:
		{
			switch (InMaskBitName)
			{
				case EMasking::Bits::ETM_A:
					MaskLabel = LOCTEXT("MaskRLabel", "R");
					break;
				case EMasking::Bits::ETM_B:
					MaskLabel = LOCTEXT("MaskGLabel", "G");
					break;
				case EMasking::Bits::ETM_C:
					MaskLabel = LOCTEXT("MaskBLabel", "B");
					break;
				case EMasking::Bits::ETM_D:
					MaskLabel = LOCTEXT("MaskALabel", "A");
					break;
			}
		}
		break;
		case EMaskingType::Unsupported:
			check(false);
			break;
	}

	return MaskLabel;
}

void SRCProtocolMaskTriplet::LoadMasks()
{
	// Soft toggle enabled masks.
	constexpr bool bSoftToggle = true;

	if (MaskA.IsValid() && MaskA->IsMasked())
	{
		MaskA->ToggleMaskedState(bSoftToggle);
	}

	if (MaskB.IsValid() && MaskB->IsMasked())
	{
		MaskB->ToggleMaskedState(bSoftToggle);
	}

	if (MaskC.IsValid() && MaskC->IsMasked())
	{
		MaskC->ToggleMaskedState(bSoftToggle);
	}

	if (MaskD.IsValid() && MaskD->IsMasked())
	{
		MaskD->ToggleMaskedState(bSoftToggle);
	}
}

#undef LOCTEXT_NAMESPACE
