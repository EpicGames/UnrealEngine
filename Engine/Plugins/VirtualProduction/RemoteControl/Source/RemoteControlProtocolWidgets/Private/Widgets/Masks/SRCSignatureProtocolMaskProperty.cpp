// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureProtocolMaskProperty.h"

#include "IRemoteControlModule.h"
#include "RemoteControlCommon.h"
#include "Signature/RCSignatureProtocolAction.h"

namespace UE::RemoteControlProtocolWidgets::Private
{
	EMaskingType GetMaskingType(const UScriptStruct* InPropertyStruct)
	{
		if (const EMaskingType* FoundMaskingType = FRemoteControlProtocolMasking::GetStructsToMaskingTypes().Find(InPropertyStruct))
		{
			return *FoundMaskingType;
		}
		return EMaskingType::Unsupported;
	}
}

void SRCSignatureProtocolMaskProperty::Construct(const FArguments& InArgs
	, const TSharedRef<IPropertyHandle>& InMaskPropertyHandle
	, TConstArrayView<const FRCSignatureProtocolAction*> InProtocolActions)
{
	using namespace UE::RemoteControlProtocolWidgets;

	MaskPropertyHandle = InMaskPropertyHandle;

	// Find the Property Struct for all the selected protocol actions
	// if the protocol action have different structs, this will remain null
	const UScriptStruct* PropertyStruct = nullptr;
	for (const FRCSignatureProtocolAction* Action : InProtocolActions)
	{
		if (!Action)
		{
			continue;
		}

		const UScriptStruct* CurrentStruct = Action->GetPropertyStruct();
		if (!PropertyStruct)
		{
			PropertyStruct = CurrentStruct;
		}

		// If the property structs don't match, do not show mask widget (multiple different protocol actions are being visualized)
		if (PropertyStruct != CurrentStruct)
		{
			PropertyStruct = nullptr;
			break;
		}
	}

	if (PropertyStruct)
	{
		SRCProtocolMaskTriplet::Construct(SRCProtocolMaskTriplet::FArguments()
			.MaskA(ERCMask::MaskA)
			.MaskB(ERCMask::MaskB)
			.MaskC(ERCMask::MaskC)
			.OptionalMask(ERCMask::MaskD)
			.MaskingType(Private::GetMaskingType(PropertyStruct))
			.CanBeMasked(IRemoteControlModule::Get().SupportsMasking(PropertyStruct))
			.EnableOptionalMask(FRemoteControlProtocolMasking::GetOptionalMaskStructs().Contains(PropertyStruct))
		);
	}
}

ECheckBoxState SRCSignatureProtocolMaskProperty::IsMaskEnabled(ERCMask InMaskBit) const
{
	if (!MaskPropertyHandle.IsValid())
	{
		return ECheckBoxState::Unchecked;
	}

	uint8 MaskValue = 0;
	if (MaskPropertyHandle->GetValue(MaskValue) != FPropertyAccess::Success)
	{
		return ECheckBoxState::Unchecked;
	}

	return MaskValue & static_cast<uint8>(InMaskBit) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SRCSignatureProtocolMaskProperty::SetMaskEnabled(ECheckBoxState InNewState, ERCMask InNewMaskBit)
{
	if (InNewState == ECheckBoxState::Undetermined || !MaskPropertyHandle.IsValid())
	{
		return;
	}

	uint8 MaskValue = 0;
	if (MaskPropertyHandle->GetValue(MaskValue) != FPropertyAccess::Success)
	{
		return;
	}

	if (InNewState == ECheckBoxState::Checked)
	{
		MaskValue |= static_cast<uint8>(InNewMaskBit);
	}
	else
	{
		MaskValue &= ~static_cast<uint8>(InNewMaskBit);
	}

	MaskPropertyHandle->SetValue(MaskValue);
}
