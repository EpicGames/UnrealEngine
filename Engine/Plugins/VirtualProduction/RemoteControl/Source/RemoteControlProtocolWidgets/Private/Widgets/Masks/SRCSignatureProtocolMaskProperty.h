// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Masks/SRCProtocolMaskTriplet.h"

class IPropertyHandle;
struct FRCSignatureProtocolAction;

class SRCSignatureProtocolMaskProperty : public SRCProtocolMaskTriplet
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureProtocolMaskProperty) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<IPropertyHandle>& InMaskPropertyHandle
		, TConstArrayView<const FRCSignatureProtocolAction*> InProtocolActions);

private:
	//~ Begin SRCProtocolMaskTriplet
	virtual ECheckBoxState IsMaskEnabled(ERCMask InMaskBit) const override;
	virtual void SetMaskEnabled(ECheckBoxState InNewState, ERCMask InNewMaskBit) override;
	//~ End SRCProtocolMaskTriplet

	TSharedPtr<IPropertyHandle> MaskPropertyHandle;
};
