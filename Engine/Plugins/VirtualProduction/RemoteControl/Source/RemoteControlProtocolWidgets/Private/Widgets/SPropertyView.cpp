// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPropertyView.h"

#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "IRemoteControlModule.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlProtocolWidgetsModule.h"
#include "SRCProtocolShared.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

namespace Internal
{
	/** All matching name, returns true if one or more found. Could cache result as child indices. */
	bool FindWidgetsByName(const TSharedRef<SWidget>& InParent, const FString& InWidgetTypeName, TArray<TSharedRef<SWidget>>& OutFoundWidgets)
	{
		FChildren* PanelChildren = InParent->GetChildren();
		for (int ChildIdx = 0; ChildIdx < PanelChildren->Num(); ++ChildIdx)
		{
			TSharedRef<SWidget> ChildWidget = PanelChildren->GetChildAt(ChildIdx);
			FString ChildType = ChildWidget->GetTypeAsString();
			if (ChildType.Contains(InWidgetTypeName))
			{
				OutFoundWidgets.Add(ChildWidget);
			}
			else if (InParent->GetChildren()->Num() > 0)
			{
				if (FindWidgetsByName(ChildWidget, InWidgetTypeName, OutFoundWidgets))
				{
					return true;
				}
			}
		}

		return OutFoundWidgets.Num() > 0;
	}

	/** Returns first occurence. */
	template <typename WidgetType>
	TSharedPtr<WidgetType> FindWidgetByType(const TSharedRef<SWidget>& InParent)
	{
		FChildren* PanelChildren = InParent->GetChildren();
		for (int ChildIdx = 0; ChildIdx < PanelChildren->Num(); ++ChildIdx)
		{
			TSharedRef<SWidget> ChildWidget = PanelChildren->GetChildAt(ChildIdx);
			if (ChildWidget != SNullWidget::NullWidget)
			{
				if (WidgetType* ChildWidgetAsType = Cast<WidgetType>(ChildWidget.Get()))
				{
					return ChildWidgetAsType;
				}
				else if (InParent->GetChildren()->Num() > 0)
				{
					if (const TSharedPtr<WidgetType>& FoundWidget = FindWidgetByType<WidgetType>(ChildWidget))
					{
						return FoundWidget;
					}
				}
			}
		}

		return nullptr;
	}
}

int32 SPropertyView::DesiredWidth = 400.f;

void SPropertyView::Construct(const FArguments& InArgs)
{
	Object = TStrongObjectPtr<UObject>(InArgs._Object);
	RootPropertyName = InArgs._RootPropertyName;
	NameVisibility = InArgs._NameVisibility;
	DisplayNameOverride = InArgs._DisplayName;
	Struct = InArgs._Struct;
	Spacing = InArgs._Spacing;
	bResizableColumn = InArgs._ResizableColumn;

	if (InArgs._ColumnSizeData.IsValid())
	{
		ColumnSizeData = InArgs._ColumnSizeData;
	}
	else
	{
		ColumnWidth = 0.7f;
		ColumnSizeData = MakeShared<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>();
		ColumnSizeData->LeftColumnWidth = TAttribute<float>(this, &SPropertyView::OnGetLeftColumnWidth);
		ColumnSizeData->RightColumnWidth = TAttribute<float>(this, &SPropertyView::OnGetRightColumnWidth);
		ColumnSizeData->OnWidthChanged = SSplitter::FOnSlotResized::CreateSP(this, &SPropertyView::OnSetColumnWidth);
	}

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const FPropertyRowGeneratorArgs Args;
	Generator = PropertyEditorModule.CreatePropertyRowGenerator(Args);

	if (Object.IsValid())
	{
		Generator->SetObjects({Object.Get()});
	}
	else if (Struct.IsValid())
	{
		Generator->SetStructure(Struct);
	}

	OnPropertyChangedHandle = Generator->OnFinishedChangingProperties().AddSP(this, &SPropertyView::OnPropertyChanged);

	if (GEditor)
	{
		OnObjectReplacedHandle = FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SPropertyView::OnObjectReplaced);
		OnObjectPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SPropertyView::OnObjectPropertyChanged);
		OnObjectTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddSP(this, &SPropertyView::OnObjectTransacted);
	}

	ConstructInternal();
}

void SPropertyView::ConstructInternal()
{
	if (Object.IsValid() || Struct.IsValid())
	{
		GridPanel = SNew(SGridPanel).FillColumn(0.0f, 1.0f);

		TArray<TSharedRef<IDetailTreeNode>> RootNodes = Generator->GetRootTreeNodes();
		if (FindRootPropertyHandle(RootNodes))
		{
			int32 Index = 0;
			AddWidgets(RootNodes, Index, 0.0f);
		}

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(1.f, 5.0f)
			[
				GridPanel.ToSharedRef()
			]
		];
	}
	else
	{
		const FText ErrorText = LOCTEXT("InvalidObject", "Error: Not a valid Object");
		IRemoteControlModule::BroadcastError(ErrorText.ToString());
		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(5.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.FillWidth(1.0f)
				[
					SNew(STextBlock)
					.Font(IDetailLayoutBuilder::GetDetailFontBold())
					.Text(ErrorText)
					.Margin(FMargin(5.0f, 5.0f, 0.0f, 0.0f))
					.ColorAndOpacity(FLinearColor(1, 0, 0, 1))
				]
			]
		];
	}
}

SPropertyView::~SPropertyView()
{
	Generator->OnFinishedChangingProperties().Remove(OnPropertyChangedHandle);

	if (GEditor)
	{
		FCoreUObjectDelegates::OnObjectsReplaced.Remove(OnObjectReplacedHandle);
		FCoreUObjectDelegates::OnObjectTransacted.Remove(OnObjectTransactedHandle);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnObjectPropertyChangedHandle);
	}
}

void SPropertyView::SetProperty(UObject* InNewObject, const FName InPropertyName)
{
	if (!Object.IsValid() || Object.Get() != InNewObject)
	{
		Object.Reset(InNewObject);
		RootPropertyName = InPropertyName;
		Struct.Reset();
		Refresh();
	}
}

void SPropertyView::SetStruct(UObject* InNewObject, TSharedPtr<FStructOnScope>& InStruct)
{
	if (!Object.IsValid() || Object.Get() != InNewObject)
	{
		Object.Reset(InNewObject);
		Struct = InStruct;
		Property = nullptr;
		Refresh();
	}
}

TSharedPtr<IPropertyHandle> SPropertyView::GetPropertyHandle() const
{
	return Property;
}

void SPropertyView::Refresh()
{
	MarkPrepassAsDirty();
}

void SPropertyView::AddWidgets(const TArray<TSharedRef<IDetailTreeNode>>& InDetailTree, int32& InIndex, float InLeftPadding)
{
	// Check type and metadata for visibility/editability
	auto IsDisplayable = [](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		if (InPropertyHandle.IsValid() && InPropertyHandle->IsValidHandle() && InPropertyHandle->IsEditable())
		{
			FProperty* PropertyToVerify = InPropertyHandle->GetProperty();

			if (PropertyToVerify)
			{
				if (const FFieldVariant Outer = PropertyToVerify->GetOwnerVariant())
				{
					// if the outer is a container property (array,set or map) it's editable even without the proper flags.
					if (Outer.IsA<FArrayProperty>() || Outer.IsA<FSetProperty>() || Outer.IsA<FMapProperty>())
					{
						return true;
					}
				}

				return PropertyToVerify && !PropertyToVerify->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && PropertyToVerify->HasAnyPropertyFlags(CPF_Edit);
			}
		}

		// Ok to display DetailNode without property because at this stage the parent property was displayable
		return true;
	};

	// Check if array, map, set
	auto IsContainer = [](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		if (InPropertyHandle.IsValid() && InPropertyHandle->IsValidHandle() && InPropertyHandle->IsEditable())
		{
			if (FProperty* PropertyToVerify = InPropertyHandle->GetProperty())
			{
				FFieldClass* PropertyClass = PropertyToVerify->GetClass();
				if (PropertyClass == FArrayProperty::StaticClass()
					|| PropertyClass == FSetProperty::StaticClass()
					|| PropertyClass == FMapProperty::StaticClass())
				{
					return !PropertyToVerify->HasAnyPropertyFlags(CPF_DisableEditOnInstance) && PropertyToVerify->HasAnyPropertyFlags(CPF_Edit);
				}
			}
		}

		return false;
	};

	for (const TSharedRef<IDetailTreeNode>& ChildNode : InDetailTree)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = ChildNode->CreatePropertyHandle();
		if (ChildNode->GetNodeType() == EDetailNodeType::Category)
		{
			AddCategoryWidget(ChildNode, InIndex, InLeftPadding);
		}
		else if (IsContainer(PropertyHandle))
		{
			AddContainerWidget(ChildNode, InIndex, InLeftPadding);
		}
		else if (IsDisplayable(PropertyHandle))
		{
			TSharedPtr<SWidget> NameWidget;
			TSharedPtr<SWidget> ValueWidget;
			EHorizontalAlignment HAlign;
			EVerticalAlignment VAlign;
			TOptional<float> MinWidth;
			TOptional<float> MaxWidth;

			TSharedPtr<IDetailPropertyRow> DetailPropertyRow = ChildNode->GetRow();
			// Overrides the top-level property display name, if specified
			if (InIndex == 0 && DisplayNameOverride.IsSet())
			{
				DetailPropertyRow->DisplayName(DisplayNameOverride.GetValue());
			}

			if (DetailPropertyRow.IsValid())
			{
				FDetailWidgetRow Row;
				DetailPropertyRow->GetDefaultWidgets(NameWidget, ValueWidget, Row, true);
				HAlign = Row.ValueWidget.HorizontalAlignment;
				VAlign = Row.ValueWidget.VerticalAlignment;
				MinWidth = Row.ValueWidget.MinWidth;
				MaxWidth = Row.ValueWidget.MaxWidth;
			}
			else
			{
				FNodeWidgets NodeWidgets = ChildNode->CreateNodeWidgets();

				NameWidget = NodeWidgets.NameWidget;
				ValueWidget = NodeWidgets.ValueWidget;
				HAlign = NodeWidgets.ValueWidgetLayoutData.HorizontalAlignment;
				VAlign = NodeWidgets.ValueWidgetLayoutData.VerticalAlignment;
				MinWidth = NodeWidgets.ValueWidgetLayoutData.MinWidth;
				MaxWidth = NodeWidgets.ValueWidgetLayoutData.MaxWidth;
			}

			if (NameWidget.IsValid() && ValueWidget.IsValid())
			{
				InIndex++;

				// If root widget, single line (no children or bDisplayChildren == false), disable name widget + column, and not forcibly shown
				if ((InIndex <= 1 && NameVisibility != EPropertyNameVisibility::Show)
					// Or if forcibly hidden 
					|| NameVisibility == EPropertyNameVisibility::Hide)
				{
					NameWidget.Reset();
				}

				TArray<TSharedRef<IDetailTreeNode>> Children;
				ChildNode->GetChildren(Children);

				// Check if this row has input controls, like a color or vector
				bool bHasHeaderContent = RowHasInputContent(ValueWidget);

				// Handle special case for Vector4 as color grading
				bool bIsColorGrading = GetPropertyHandle()->HasMetaData(TEXT("ColorGradingMode"));
				
				if ((!bHasHeaderContent && Children.Num() > 0) || (bIsColorGrading && Children.Num() > 0))
				{
					// #ueent_todo: Find a way to add collapse/expand capability for property with children
					AddWidgets(Children, InIndex, InLeftPadding + 10.f);
				}
				else
				{
					// If root widget, single line (no children or bDisplayChildren == false), disable name widget + column, and not forcibly shown
					if ((InIndex <= 1 && NameVisibility != EPropertyNameVisibility::Show)
						// Or if forcibly hidden 
						|| NameVisibility == EPropertyNameVisibility::Hide)
					{
						NameWidget.Reset();
					}

					// Only creates row if no children or bDisplayChildren == false (single row property), so need to reverse index
					CreateDefaultWidget(FPropertyWidgetCreationArgs(InIndex - 1, NameWidget, ValueWidget, InLeftPadding, MinWidth, MaxWidth));
				}
			}
		}
	}
}

void SPropertyView::AddCategoryWidget(const TSharedRef<IDetailTreeNode>& InDetailTree, int32& InOutIndex, float InLeftPadding)
{
	if (InOutIndex > 0)
	{
		GridPanel->AddSlot(0, InOutIndex)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[
				SNew(SSpacer)
				.Size(FVector2D(0.f, 10.f))
			]
		];
		InOutIndex++;
	}

	TArray<TSharedRef<IDetailTreeNode>> Children;
	InDetailTree->GetChildren(Children);
	AddWidgets(Children, InOutIndex, InLeftPadding);
}

void SPropertyView::AddContainerWidget(const TSharedRef<IDetailTreeNode>& InDetailTree, int32& InOutIndex, float InLeftPadding)
{
	TSharedPtr<IDetailPropertyRow> DetailPropertyRow = InDetailTree->GetRow();
	if (DetailPropertyRow.IsValid())
	{
		FDetailWidgetRow Row;
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		DetailPropertyRow->GetDefaultWidgets(NameWidget, ValueWidget, Row, true);

		CreateDefaultWidget(FPropertyWidgetCreationArgs(InOutIndex, NameWidget, ValueWidget, InLeftPadding));
		InOutIndex++;

		TArray<TSharedRef<IDetailTreeNode>> Children;
		InDetailTree->GetChildren(Children);
		if (Children.Num() > 0)
		{
			// #ueent_todo: Find a way to add collapse/expand capability for property with children
			AddWidgets(Children, InOutIndex, InLeftPadding + 10.f);
		}
	}
}

TSharedRef<SWidget> SPropertyView::CreatePropertyWidget(const FPropertyWidgetCreationArgs& InCreationArgs)
{
	if (InCreationArgs.HasNameWidget() && InCreationArgs.bResizableColumn)
	{
		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 0.0f, InCreationArgs.Spacing)
				[
					SNew(RemoteControlProtocolWidgetUtils::SCustomSplitter)
					.LeftWidget(InCreationArgs.NameWidget)
					.RightWidget(InCreationArgs.ValueWidget)
					.ColumnSizeData(InCreationArgs.ColumnSizeData)
				];
	}
	else
	{
		TSharedRef<SHorizontalBox> NameValuePairWidget = SNew(SHorizontalBox);

		// Prepend name widget if present
		if (InCreationArgs.NameWidget.IsValid())
		{
			NameValuePairWidget
				->AddSlot()
				.Padding(0.0f, 0.0f, 12.0f, 0.0f)
				.AutoWidth()
				[
					InCreationArgs.NameWidget.ToSharedRef()
				];
		}

		NameValuePairWidget
			->AddSlot()
			.AutoWidth()
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				[
					SNew(RemoteControlProtocolWidgetUtils::SConstrainedBox)
					.MinWidth(InCreationArgs.ValueMinWidth)
					.MaxWidth(InCreationArgs.ValueMaxWidth)
					[
						InCreationArgs.ValueWidget.ToSharedRef()
					]
				]
			];

		return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Top)
				.HAlign(HAlign_Fill)
				.Padding(0.0f, 0.0f, 0.0f, InCreationArgs.Spacing)
				[
					NameValuePairWidget
				];
	}
}

void SPropertyView::CreateDefaultWidget(const FPropertyWidgetCreationArgs& InCreationArgs)
{
	TSharedPtr<SHorizontalBox> NameColumn = nullptr;
	if (InCreationArgs.NameWidget.IsValid())
	{
		NameColumn = SNew(SHorizontalBox)
					.Clipping(EWidgetClipping::OnDemand);

		InCreationArgs.NameWidget->SetClipping(EWidgetClipping::OnDemand);

		// Add the name widget
		NameColumn->AddSlot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Left)
			.Padding(FMargin(InCreationArgs.LeftPadding, 0.f, 0.f, 0.f))
			[
				InCreationArgs.NameWidget.ToSharedRef()
			];
	}

	GridPanel->AddSlot(0, InCreationArgs.Index)
	[
		CreatePropertyWidget(FPropertyWidgetCreationArgs(InCreationArgs, NameColumn, ColumnSizeData, Spacing, bResizableColumn))
	];
}

// @note: this broadcasts, rather than overloads handling, for use cases where the object property is changed outside of this widget
void SPropertyView::OnObjectPropertyChanged(UObject* InObject, FPropertyChangedEvent& InEvent)
{
	if(IsValid(InObject) && InObject == Object.Get())
	{
		OnFinishedChangingProperties().Broadcast(InEvent);
	}
}

void SPropertyView::OnPropertyChanged(const FPropertyChangedEvent& InEvent)
{
	if (Property.IsValid() && Property->GetProperty() == InEvent.Property)
	{
		Refresh();
	}
}

void SPropertyView::OnObjectReplaced(const TMap<UObject*, UObject*>& InReplacementObjectMap)
{
	if(Object.IsValid())
	{
		if (UObject* const* ObjectPtr = InReplacementObjectMap.Find(Object.Get()))
		{
			Object.Reset(*ObjectPtr);
			Refresh();
		}
	}
}

void SPropertyView::OnObjectTransacted(UObject* InObject, const FTransactionObjectEvent& InTransactionObjectEvent)
{
	if ((Object.IsValid() && Object.Get() == InObject) || (Object.IsValid() && Object->GetOuter() == InObject))
	{
		Refresh();
	}
}

bool SPropertyView::RowHasInputContent(const TSharedPtr<SWidget>& InValueWidget) const
{
	if (const TSharedPtr<SPanel> PanelWidget = StaticCastSharedPtr<SPanel>(InValueWidget))
	{
		TArray<TSharedRef<SWidget>> FoundWidgets;
		if (Internal::FindWidgetsByName(PanelWidget.ToSharedRef(), TEXT("SNumericEntryBox"), FoundWidgets))
		{
			return true;
		}

		if (Internal::FindWidgetsByName(PanelWidget.ToSharedRef(), TEXT("SColorBlock"), FoundWidgets))
		{
			return true;
		}
	}

	return false;
}

bool SPropertyView::FindRootPropertyHandle(TArray<TSharedRef<IDetailTreeNode>>& InOutNodes)
{
	if (InOutNodes.Num() > 0)
	{
		TFunction<TSharedPtr<IDetailTreeNode>(const TSharedRef<IDetailTreeNode>)> FindNodeRecursive;
		FindNodeRecursive = [&](const TSharedRef<IDetailTreeNode>& InNode) -> TSharedPtr<IDetailTreeNode>
		{
			// check input node
			if (const TSharedPtr<IPropertyHandle> PropertyHandle = InNode->CreatePropertyHandle())
			{
				if (PropertyHandle->GetProperty()->GetFName() == RootPropertyName)
				{
					return InNode;
				}
			}

			TSharedPtr<IDetailTreeNode> FoundNode = nullptr;

			TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
			InNode->GetChildren(ChildNodes);

			// search children
			for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
			{
				FoundNode = FindNodeRecursive(ChildNode);
				if (FoundNode)
				{
					return FoundNode;
				}
			}

			// return found or nullptr
			return FoundNode;
		};

		TSharedPtr<IDetailTreeNode> FoundNode = nullptr;
		for (const TSharedRef<IDetailTreeNode>& RootNode : InOutNodes)
		{
			FoundNode = FindNodeRecursive(RootNode);
			if (FoundNode)
			{
				break;
			}
		}

		if (FoundNode)
		{
			TSharedPtr<IPropertyHandle> PropertyHandle = FoundNode->CreatePropertyHandle();
			Swap(Property, PropertyHandle);

			InOutNodes.Empty();
			InOutNodes.Add(FoundNode.ToSharedRef());
			return true;
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
