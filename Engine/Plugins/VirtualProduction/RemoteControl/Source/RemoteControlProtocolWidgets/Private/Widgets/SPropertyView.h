// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPropertyRowGenerator.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace RemoteControlProtocolWidgetUtils {
	struct FPropertyViewColumnSizeData;
}

/** Controls visibility of the property name column */
enum class EPropertyNameVisibility : uint8
{
	Show = 0, // Always show the name column
    Hide = 1, // Never show the name column
    HideTopLevel = 2 // Hide if either: Property has no children, or is draw in one row; Property is the top-level/root, and has children
};

class SGridPanel;

/** Represents a single property, including a struct. */
class REMOTECONTROLPROTOCOLWIDGETS_API SPropertyView final : public SCompoundWidget
{
	struct FPropertyWidgetCreationArgs
	{
		FPropertyWidgetCreationArgs(
			const int32 InIndex,
			const TSharedPtr<SWidget>& InNameWidget,
			const TSharedPtr<SWidget>& InValueWidget,
			const float InLeftPadding,
			const TOptional<float>& InValueMinWidth = {},
			const TOptional<float>& InValueMaxWidth = {})
			: Index(InIndex)
			, NameWidget(InNameWidget)
			, ValueWidget(InValueWidget)
			, LeftPadding(InLeftPadding)
			, ValueMinWidth(InValueMinWidth)
			, ValueMaxWidth(InValueMaxWidth)
			, ColumnSizeData(nullptr)
			, Spacing(0.0f)
			, bResizableColumn(false)
		{ }

		FPropertyWidgetCreationArgs(
			const FPropertyWidgetCreationArgs& InOther,
			const TSharedPtr<SWidget>& InNameWidget,
			const TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>& InColumnSizeData,
			const float InSpacing,
			const bool bInResizeableColumn)
			: Index(InOther.Index)
			, NameWidget(InNameWidget ? InNameWidget : InOther.NameWidget)
			, ValueWidget(InOther.ValueWidget)
			, LeftPadding(InOther.LeftPadding)
			, ValueMinWidth(InOther.ValueMinWidth)
			, ValueMaxWidth(InOther.ValueMaxWidth)
			, ColumnSizeData(InColumnSizeData)
			, Spacing(InSpacing)
			, bResizableColumn(bInResizeableColumn)
		{ }

		int32 Index;
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		float LeftPadding;
		TOptional<float> ValueMinWidth;
		TOptional<float> ValueMaxWidth;
		TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> ColumnSizeData;
		float Spacing;
		bool bResizableColumn = true;
		
		bool HasNameWidget() const { return NameWidget.IsValid(); }
	};
	
public:
	SLATE_BEGIN_ARGS(SPropertyView)
		: _Object(nullptr)
		, _Struct(nullptr)
		, _RootPropertyName(NAME_None)
		, _NameVisibility(EPropertyNameVisibility::HideTopLevel)
		, _DisplayName({})
		, _Spacing(0)
		, _ColumnPadding(false)
		, _ResizableColumn(true)
		{}
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, ColumnSizeData)
	    SLATE_ARGUMENT(UObject*, Object)
		SLATE_ARGUMENT(TSharedPtr<FStructOnScope>, Struct)
		SLATE_ARGUMENT(FName, RootPropertyName)
		SLATE_ARGUMENT(EPropertyNameVisibility, NameVisibility)
		SLATE_ARGUMENT(TOptional<FText>, DisplayName)
	    SLATE_ARGUMENT(float, Spacing)
	    SLATE_ARGUMENT(bool, ColumnPadding)
	    SLATE_ARGUMENT(bool, ResizableColumn)
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	virtual ~SPropertyView() override;

	/** Set's the object/property pair */
	void SetProperty(UObject* InNewObject, const FName InPropertyName);

	/** Set's the object/struct pair */
	void SetStruct(UObject* InNewObject, TSharedPtr<FStructOnScope>& InStruct);

	/** Force a refresh/rebuild */
	void Refresh();

	static int32 GetDesiredWidth() { return DesiredWidth; }
	static void SetDesiredWidth(int32 InDesiredWidth) { DesiredWidth = InDesiredWidth; }

	/** Get the property handle for the property on this view */
	TSharedPtr<IPropertyHandle> GetPropertyHandle() const;

	/** Get the FOnFinishedChangingProperties delegate from the underlying PropertyRowGenerator */
	FOnFinishedChangingProperties& OnFinishedChangingProperties() const { return Generator->OnFinishedChangingProperties(); }

protected:
	// Begin SWidget overrides.
	// #ueent_todo: This is temporary until we find a better solution to the splitter issue
	//				See SConstrainedBox's trick in cpp file
	virtual FVector2D ComputeDesiredSize(float) const override
	{
		const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();
		return FVector2D( DesiredWidth, ChildSize.Y );
	}
	// End SWidget overrides.

private:	
	/** Fills up the details view with the detail nodes created by the property row manager */
	void ConstructInternal();

	/** Add widgets held by array of DetailTreeNode objects */
	void AddWidgets(const TArray<TSharedRef<class IDetailTreeNode>>& InDetailTree, int32& InIndex, float InLeftPadding);

	/** Add widget for a category node */
	void AddCategoryWidget(const TSharedRef<IDetailTreeNode>& InDetailTree, int32& InOutIndex, float InLeftPadding);

	/** Add widget for a container node (array, etc.) */
	void AddContainerWidget(const TSharedRef<IDetailTreeNode>& InDetailTree, int32& InOutIndex, float InLeftPadding);

	/** Creates a layout for the supplied Name/Value widgets according to the supplied creation options. */
	TSharedRef<SWidget> CreatePropertyWidget(const FPropertyWidgetCreationArgs& InCreationArgs);
	
	/** Inserts a generic widget for a property row into the grid panel	*/
	void CreateDefaultWidget(const FPropertyWidgetCreationArgs& InCreationArgs);

	/** Callback used by all splitters in the details view, so that they move in sync */
	void OnLeftColumnResized(float InNewWidth)
	{
		// This has to be bound or the splitter will take it upon itself to determine the size
		// We do nothing here because it is handled by the column size data
	}

	float OnGetLeftColumnWidth() const { return 1.0f - ColumnWidth; }
	float OnGetRightColumnWidth() const { return ColumnWidth; }
	void OnSetColumnWidth(float InWidth) { ColumnWidth = InWidth < 0.5f ? 0.5f : InWidth; }

	/** Callback to track property changes */
	void OnObjectPropertyChanged(UObject* InObject, struct FPropertyChangedEvent& InEvent);

	/** Callback to track property changes */
	void OnPropertyChanged(const struct FPropertyChangedEvent& InEvent);

	/** Callback used to detect the existence of a new object to display after a re-instancing process */
	void OnObjectReplaced(const TMap<UObject*, UObject*>& InReplacementObjectMap);

	/** Callback to track object transactions (undo/redo) */
	void OnObjectTransacted(UObject* InObject, const class FTransactionObjectEvent& InTransactionObjectEvent);

	/** Semi-reliable check to see if a row is an empty header row or not (customized). */
	bool RowHasInputContent(const TSharedPtr<SWidget>& InValueWidget) const;

	/** Attempts to find the property specified by RootPropertyName, returns true if found.  */
	bool FindRootPropertyHandle(TArray<TSharedRef<class IDetailTreeNode>>& InOutNodes);

private:
	/** Row generator applied on detailed object. */
	TSharedPtr<class IPropertyRowGenerator> Generator;

	/** Object to be detailed. */
	TStrongObjectPtr<UObject> Object = nullptr;

	/** Name of the root property represented. */
	FName RootPropertyName;

	/** Display name visibility option. */
	EPropertyNameVisibility NameVisibility = EPropertyNameVisibility::Show;

	/** Optional display name override. */
	TOptional<FText> DisplayNameOverride;

	/** Property represented by this widget. */
	TSharedPtr<IPropertyHandle> Property;

	/** Struct to be detailed. */
	TSharedPtr<FStructOnScope> Struct;

	/** Delegate handle to track property changes (from this widget). */
	FDelegateHandle OnPropertyChangedHandle;

	/** Delegate handle to track new object after a re-instancing process. */
	FDelegateHandle OnObjectReplacedHandle;

	/** Delegate handle to track when a object was transacted. */
	FDelegateHandle OnObjectTransactedHandle;

	/** Delegate handle to track property changes (from FCoreUObjectDelegates). */
	FDelegateHandle OnObjectPropertyChangedHandle;

	/** Relative width to control splitters. */
	float ColumnWidth = 0;

	/** Points to the currently used column size data. Can be provided via argument as well. */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> ColumnSizeData;

	/** Spacing between rows. 0 by default. */
	float Spacing = 0;

	/** Indicates if the name and value widgets have a splitter and can be resized. */
	bool bResizableColumn = false;

	/** Grid panel storing the row widgets. */
	TSharedPtr<SGridPanel> GridPanel;

	/** Ideal total width of the widget. */
	static int32 DesiredWidth;
};
