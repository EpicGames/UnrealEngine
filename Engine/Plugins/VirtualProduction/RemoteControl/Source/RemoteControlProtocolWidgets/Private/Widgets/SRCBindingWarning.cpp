// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBindingWarning.h"

#include "EditorFontGlyphs.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

void SRCBindingWarning::Construct(const FArguments& InArgs)
{
	Status = InArgs._Status;
	StatusMessage = InArgs._StatusMessage;
	
	ChildSlot
	[
		// Switches between warning and info
		SNew(SWidgetSwitcher)
		.Visibility_Lambda([this]() { return Status.Get() != ERCBindingWarningStatus::Ok ? EVisibility::Visible : EVisibility::Hidden; })
		.ToolTipText(StatusMessage)
		.WidgetIndex_Lambda([&]
		{
			// For now everything is a warning
			return 0;
		})

		// Warning ((will at one point) disable binding)
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ColorAndOpacity(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Warning").Normal.TintColor.GetSpecifiedColor())
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock) 
				.TextStyle(FAppStyle::Get(), "NormalText.Important")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
		]

		// Info (doesn't disable binding)
		+ SWidgetSwitcher::Slot()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.ColorAndOpacity(FAppStyle::Get().GetWidgetStyle<FButtonStyle>("FlatButton.Info").Normal.TintColor.GetSpecifiedColor())
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(STextBlock) 
				.TextStyle(FAppStyle::Get(), "NormalText.Important")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
		]
	];
}
