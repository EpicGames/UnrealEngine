// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

/** Status severity. */
enum class ERCBindingWarningStatus : uint8
{
	Ok = 0,
	Info = 1,
	Warning = 2,
	Error = 3
};

/** An icon/tooltip pair for validation. */ 
class SRCBindingWarning : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBindingWarning)
		: _Status(ERCBindingWarningStatus::Ok)
		, _StatusMessage(FText::GetEmpty())
	{}
		/** The current status and severity. */
		SLATE_ATTRIBUTE(ERCBindingWarningStatus, Status)

		/** The text displayed in the tooltip. */
		SLATE_ATTRIBUTE(FText, StatusMessage)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:
	/** The current status and severity. */
	TAttribute<ERCBindingWarningStatus> Status;

	/** The text displayed in the tooltip. */
	TAttribute<FText> StatusMessage;
};
