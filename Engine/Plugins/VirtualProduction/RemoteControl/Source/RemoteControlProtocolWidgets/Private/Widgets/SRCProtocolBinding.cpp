// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolBinding.h"

#include "SDropTarget.h"
#include "SRCBindingWarning.h"
#include "SRCProtocolRangeList.h"
#include "ViewModels/ProtocolBindingViewModel.h"
#include "Widgets/Masks/SRCProtocolBindingMask.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SRCProtocolEntity.h"
#include "Widgets/SRCProtocolShared.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

void SRCProtocolBinding::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FProtocolBindingViewModel>& InViewModel)
{
	constexpr float Padding = 2.0f;
	ViewModel = InViewModel;
	PrimaryColumnSizeData = InArgs._PrimaryColumnSizeData;
	SecondaryColumnSizeData = InArgs._SecondaryColumnSizeData;
	OnStartRecording = InArgs._OnStartRecording;
	OnStopRecording = InArgs._OnStopRecording;

	TSharedRef<SHorizontalBox> RowBox = SNew(SHorizontalBox);

	RowBox->AddSlot()
		.Padding(Padding)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(ViewModel->GetProtocolName())
			.TextStyle(FAppStyle::Get(), "LargeText")
		];

	// Validation warning
	RowBox->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SRCBindingWarning)
			.Status_Lambda([this]()
			{
				FText StatusMessage;
				return ViewModel->IsValid(StatusMessage) ? ERCBindingWarningStatus::Ok : ERCBindingWarningStatus::Warning;
			})
			.StatusMessage_Lambda([this]()
			{
				FText StatusMessage;
				ViewModel->IsValid(StatusMessage);
				return StatusMessage;
			})
		];

	// Delete binding
	RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ForegroundColor(FSlateColor::UseForeground())
			.IsFocusable(false)
			.OnClicked(this, &SRCProtocolBinding::OnDelete)
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.X"))
			]
		];

	// Record binding input
	RowBox->AddSlot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(2.f)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ToolTipText(LOCTEXT("RecordingButtonToolTip", "Status of the protocol entity binding"))
			.ForegroundColor(FSlateColor::UseForeground())
			.OnClicked(this, &SRCProtocolBinding::ToggleRecording)
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity_Raw(this, &SRCProtocolBinding::GetRecordingButtonColor)
				.Image(FAppStyle::Get().GetBrush(TEXT("Icons.FilledCircle")))
			]
		];

	STableRow::Construct(
		STableRow::FArguments()
		.Padding(Padding)
		.ShowSelection(false)
		.Content()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(Padding)
			.VAlign(VAlign_Fill)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(Padding)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				[
					RowBox
				]

				+ SVerticalBox::Slot()
				.Padding(Padding)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				[
					SNew(SRCProtocolEntity, ViewModel.ToSharedRef())
				]
				
				+ SVerticalBox::Slot()
				.Padding(Padding)
				.VAlign(VAlign_Center)
				.AutoHeight()
				[
					ConstructMaskingWidget()
				]

				+ SVerticalBox::Slot()
				.Padding(Padding)
				.VAlign(VAlign_Fill)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					[
						SAssignNew(RangeList, SRCProtocolRangeList, ViewModel.ToSharedRef())
						.PrimaryColumnSizeData(PrimaryColumnSizeData)
						.SecondaryColumnSizeData(SecondaryColumnSizeData)
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBox)
						.WidthOverride(16.0f)
						[
							SNullWidget::NullWidget
						]
					]
				]
			]
		],
		InOwnerTableView);
}

TSharedRef<SWidget> SRCProtocolBinding::ConstructMaskingWidget()
{
	const TSharedPtr<SWidget> LeftWidget = SNew(SHorizontalBox)

		// Masking Label
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(4.f, 2.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("OverrideMaskingLabel", "Override Masks"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.f, 1.f))
		]

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		];

		const TSharedPtr<SWidget> RightWidget = SNew(SHorizontalBox)

			// Masking Widget
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(4.f, 2.f)
			.AutoWidth()
			[
				SNew(SRCProtocolBindingMask, ViewModel.ToSharedRef())
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SSpacer)
			];

	TSharedRef<SWidget> Widget = SNew(RemoteControlProtocolWidgetUtils::SCustomSplitter)
		.LeftWidget(LeftWidget.ToSharedRef())
		.RightWidget(RightWidget.ToSharedRef())
		.ColumnSizeData(PrimaryColumnSizeData);

	return Widget;
}

FReply SRCProtocolBinding::OnDelete()
{
	ViewModel->Remove();
	SetVisibility(EVisibility::Collapsed);
	return FReply::Handled();
}

FReply SRCProtocolBinding::ToggleRecording() const
{
	// Binding can't be nullptr
	FRemoteControlProtocolBinding* Binding = ViewModel->GetBinding();
	const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity = Binding ? Binding->GetRemoteControlProtocolEntityPtr() : nullptr;
	if (ensureMsgf(ProtocolEntity && ProtocolEntity->IsValid(), TEXT("Unexpected invalid binding or entity, cannot toggle recording.")))
	{
		const ERCBindingStatus BindingStatus = (*ProtocolEntity)->ToggleBindingStatus();
		if (BindingStatus == ERCBindingStatus::Awaiting)
		{
			OnStartRecording.ExecuteIfBound(ProtocolEntity);
		}
		else if (BindingStatus == ERCBindingStatus::Bound)
		{
			OnStopRecording.ExecuteIfBound(ProtocolEntity);
		}
		else
		{
			checkNoEntry();
		}
	}

	return FReply::Handled();
}

FSlateColor SRCProtocolBinding::GetRecordingButtonColor() const
{
	FRemoteControlProtocolBinding* Binding = ViewModel->GetBinding();
	const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity = Binding ? Binding->GetRemoteControlProtocolEntityPtr() : nullptr;
	if (ProtocolEntity && ProtocolEntity->IsValid())
	{
		const ERCBindingStatus BindingStatus = (*ProtocolEntity)->GetBindingStatus();

		switch (BindingStatus)
		{
		case ERCBindingStatus::Awaiting:
			return FLinearColor::Red;
		case ERCBindingStatus::Bound:
			return FLinearColor::Green;
		case ERCBindingStatus::Unassigned:
			return FLinearColor::Gray;
		default:
			checkNoEntry();
		}
	}

	return FLinearColor::Black;
}

#undef LOCTEXT_NAMESPACE
