// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SRCProtocolShared.h"
#include "UObject/StructOnScope.h"
#include "ViewModels/ProtocolBindingViewModel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FRemoteControlProtocolEntity;
struct FSlateColor;

class FProtocolBindingViewModel;
class FProtocolRangeViewModel;
class ITableRow;
class SRCProtocolRangeList;
class STableViewBase;

class SRCProtocolBinding : public STableRow<TSharedPtr<FProtocolBindingViewModel>>
{
public:
	/** Delegate when start recording input from protocol */
	DECLARE_DELEGATE_OneParam(FOnStartRecording, TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity)

	/** Delegate when stop recording input from protocol */
	DECLARE_DELEGATE_OneParam(FOnStopRecording, TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> ProtocolEntity)

	SLATE_BEGIN_ARGS(SRCProtocolBinding)
		{}
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, PrimaryColumnSizeData)
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, SecondaryColumnSizeData)
		SLATE_EVENT(FOnStartRecording, OnStartRecording)
		SLATE_EVENT(FOnStopRecording, OnStopRecording)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FProtocolBindingViewModel>& InViewModel);

private:
	/** Constructs the masking widget. */
	REMOTECONTROLPROTOCOLWIDGETS_API TSharedRef<SWidget> ConstructMaskingWidget();

	/** Called when Delete button pressed. */
	REMOTECONTROLPROTOCOLWIDGETS_API FReply OnDelete();

	/** Toggle recording button handler */
	REMOTECONTROLPROTOCOLWIDGETS_API FReply ToggleRecording() const;

	/** Get Recording button color based on binding status of protocol entity */
	REMOTECONTROLPROTOCOLWIDGETS_API FSlateColor GetRecordingButtonColor() const;

private:
	/** ViewModel for the Protocol Binding. */
	TSharedPtr<FProtocolBindingViewModel> ViewModel;

	/** RangeList Widget. */
	TSharedPtr<SRCProtocolRangeList> RangeList;

	/** Container used by all primary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> PrimaryColumnSizeData;

	/** Container used by all secondary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> SecondaryColumnSizeData;

	/** Start recording delegate instance */
	FOnStartRecording OnStartRecording;

	/** Stop recording delegate instance */
	FOnStopRecording OnStopRecording;
};
