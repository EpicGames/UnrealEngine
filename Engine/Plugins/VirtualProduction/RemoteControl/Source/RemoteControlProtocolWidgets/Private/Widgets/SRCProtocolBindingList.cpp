// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolBindingList.h"

#include "Delegates/Delegate.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "Misc/MessageDialog.h"
#include "Styling/AppStyle.h"
#include "IRemoteControlProtocolModule.h"
#include "RemoteControlProtocolWidgetsModule.h"
#include "SRCProtocolBinding.h"
#include "Types/ISlateMetaData.h"
#include "ViewModels/ProtocolEntityViewModel.h"
#include "ViewModels/RCViewModelCommon.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

void SRCProtocolBindingList::Construct(const FArguments& InArgs, TSharedRef<FProtocolEntityViewModel> InViewModel)
{
	constexpr float Padding = 2.0f;
	ViewModel = InViewModel;
	Refresh();

	PrimaryColumnWidth = 0.7f;
	PrimaryColumnSizeData = MakeShared<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>();
	PrimaryColumnSizeData->LeftColumnWidth = TAttribute<float>(this, &SRCProtocolBindingList::OnGetPrimaryLeftColumnWidth);
	PrimaryColumnSizeData->RightColumnWidth = TAttribute<float>(this, &SRCProtocolBindingList::OnGetPrimaryRightColumnWidth);
	PrimaryColumnSizeData->OnWidthChanged = SSplitter::FOnSlotResized::CreateSP(this, &SRCProtocolBindingList::OnSetPrimaryColumnWidth);

	SecondaryColumnWidth = 0.7f;
	SecondaryColumnSizeData = MakeShared<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>();
	SecondaryColumnSizeData->LeftColumnWidth = TAttribute<float>(this, &SRCProtocolBindingList::OnGetSecondaryLeftColumnWidth);
	SecondaryColumnSizeData->RightColumnWidth = TAttribute<float>(this, &SRCProtocolBindingList::OnGetSecondaryRightColumnWidth);
	SecondaryColumnSizeData->OnWidthChanged = SSplitter::FOnSlotResized::CreateSP(this, &SRCProtocolBindingList::OnSetSecondaryColumnWidth);
	
	ViewModel->OnBindingAdded().AddLambda([&](const TSharedPtr<FProtocolBindingViewModel> InBindingViewModel)
	{
		if(Refresh())
		{
			Refresh(true);
			BindingList->RequestNavigateToItem(InBindingViewModel);
        }
	});

	ViewModel->OnBindingRemoved().AddLambda([&](FGuid)
	{
		Refresh();
	});

	ViewModel->OnChanged().AddLambda([&]()
	{
		Refresh();
	});

	SAssignNew(BindingList, SListView<TSharedPtr<IRCTreeNodeViewModel>>)
	.ListItemsSource(&FilteredBindings)
	.OnGenerateRow(this, &SRCProtocolBindingList::ConstructBindingWidget)
	.SelectionMode(ESelectionMode::None);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1, 1, 1, Padding)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(Padding)
			.AutoWidth()
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.Padding(Padding)
				.AutoWidth()
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "PropertyEditor.AssetComboStyle")
					.ForegroundColor(FSlateColor::UseForeground())
					.ToolTipText(this, &SRCProtocolBindingList::HandleAddBindingToolTipText)
					.IsEnabled_Lambda([this]() { return CanAddProtocol(); })
					.OnClicked_Lambda([this]()
						{
							IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();

							const FName& ProtocolName = RCProtocolWidgetsModule.GetSelectedProtocolName();

							ViewModel->AddBinding(ProtocolName);

							Refresh(false);

							return FReply::Handled();
						}
					)
					.ContentPadding(FMargin(4.f, 2.f))
					.Content()
					[
						SNew(SHorizontalBox)
						
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::GetBrush("Icons.Plus"))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(STextBlock)
							.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
							.Text(LOCTEXT("AddBindingText", "Add Binding"))
						]
					]
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(FMargin(3,0))
				[
					SNew(STextBlock)
					.ColorAndOpacity(FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor"))
					.IsEnabled_Lambda([&](){ return !StatusMessage.IsEmpty(); })
					.Text_Lambda([&]() { return StatusMessage; })
				]
			]
			
			+ SHorizontalBox::Slot()
			.Padding(Padding)
			[
				SNullWidget::NullWidget
			]
		]			 

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0)
		[
			SNew(SScrollBox)

			+ SScrollBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
				.Padding(5.0f)
				[
					BindingList.ToSharedRef()				
				]				
			]
		]
	];
}

SRCProtocolBindingList::~SRCProtocolBindingList()
{
	for (const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>& RemoteControlProtocolEntity : AwaitingProtocolEntities)
	{
		if (FRemoteControlProtocolEntity* RemoteControlProtocolEntityPtr = RemoteControlProtocolEntity->Get())
		{
			if (RemoteControlProtocolEntityPtr->GetBindingStatus() == ERCBindingStatus::Awaiting)
			{
				RemoteControlProtocolEntityPtr->ResetDefaultBindingState();
			}
		}
	}

	AwaitingProtocolEntities.Empty();
}

URemoteControlPreset* SRCProtocolBindingList::GetPreset() const
{
	return ViewModel.IsValid() ? ViewModel->GetPreset() : nullptr;
}

void SRCProtocolBindingList::AddProtocolBinding(const FName InProtocolName)
{
	if (!ViewModel.IsValid())
	{
		return;
	}

	FText CannotAddBinding;

	if (!ViewModel->CanAddBinding(InProtocolName, CannotAddBinding))
	{
		FMessageDialog::Open(EAppMsgType::Ok, CannotAddBinding);

		return;
	}

	ViewModel->AddBinding(InProtocolName);
}

TSharedRef<ITableRow> SRCProtocolBindingList::ConstructBindingWidget(TSharedPtr<IRCTreeNodeViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable)
{
	check(InViewModel.IsValid());
	
	return SNew(SRCProtocolBinding, InOwnerTable, StaticCastSharedPtr<FProtocolBindingViewModel>(InViewModel).ToSharedRef())
		.PrimaryColumnSizeData(PrimaryColumnSizeData)
		.SecondaryColumnSizeData(SecondaryColumnSizeData)
		.OnStartRecording(this, &SRCProtocolBindingList::OnStartRecording)
		.OnStopRecording(this, &SRCProtocolBindingList::OnStopRecording);
}

bool SRCProtocolBindingList::CanAddProtocol()
{
	IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
	const FName& SelectedProtocolName = RCProtocolWidgetsModule.GetSelectedProtocolName();
	if(!ViewModel->IsBound())
	{
		StatusMessage = FText::GetEmpty();
		return false;
	}
	
	if(ViewModel->CanAddBinding(SelectedProtocolName, StatusMessage))
	{
		StatusMessage = FText::GetEmpty();
		return true;
	}
	
	return false;	
}

FText SRCProtocolBindingList::HandleAddBindingToolTipText() const
{
	IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();

	const FName& SelectedProtocolName = RCProtocolWidgetsModule.GetSelectedProtocolName();

	if (!SelectedProtocolName.IsNone())
	{
		return FText::Format(LOCTEXT("AddBindingToolTip", "Add {0} protocol binding."), { FText::FromName(SelectedProtocolName) });
	}

	return FText::GetEmpty();
}

void SRCProtocolBindingList::OnStartRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity)
{
	AwaitingProtocolEntities.Add(InEntity);
}

void SRCProtocolBindingList::OnStopRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity)
{
	AwaitingProtocolEntities.Remove(InEntity);
}

bool SRCProtocolBindingList::Refresh(bool bNavigateToEnd)
{
	FilteredBindings.Empty(FilteredBindings.Num());

	IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();
	const FName& SelectedProtocolName = RCProtocolWidgetsModule.GetSelectedProtocolName();

	for (const TSharedPtr<FProtocolBindingViewModel>& ProtocolBinding : ViewModel->GetBindings())
	{
		if (ProtocolBinding->GetBinding()->GetProtocolName() == SelectedProtocolName)
		{
			FilteredBindings.Add(ProtocolBinding);
		}
	}
	
	if(BindingList.IsValid())
	{
		// Don't build list if not supported
		if(!CanAddProtocol())
		{
			return false;
		}

		GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([bNavigateToEnd, WeakListPtr = TWeakPtr<SRCProtocolBindingList>(StaticCastSharedRef<SRCProtocolBindingList>(AsShared()))]()
		{
			if (TSharedPtr<SRCProtocolBindingList> ListPtr = WeakListPtr.Pin())
			{
				ListPtr->BindingList->RequestListRefresh();
				if(bNavigateToEnd)
				{
					ListPtr->BindingList->ScrollToBottom();	
				}
			}
		}));
		
		return true;
	}
	
	return false;
}

#undef LOCTEXT_NAMESPACE
