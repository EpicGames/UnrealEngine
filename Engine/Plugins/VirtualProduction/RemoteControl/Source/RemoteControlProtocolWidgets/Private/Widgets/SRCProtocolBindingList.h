// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IRCProtocolBindingList.h"
#include "SRCProtocolShared.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class FProtocolBindingViewModel;
class FProtocolEntityViewModel;
class IRCTreeNodeViewModel;
class ITableRow;
class SRCProtocolBinding;
class STableViewBase;

/** The root view for a given entity. A (vertical) list of bindings, where each binding has a protocol. */
class REMOTECONTROLPROTOCOLWIDGETS_API SRCProtocolBindingList final : public SCompoundWidget, public IRCProtocolBindingList
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolBindingList)
	{ }
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FProtocolEntityViewModel> InViewModel);
	virtual ~SRCProtocolBindingList() override;

	//~ Begin IRCProtocolBindingList Interface
	virtual void AddProtocolBinding(const FName InProtocolName) override;
	virtual URemoteControlPreset* GetPreset() const;

	/** Get the set of entities which is awaiting state and waiting for binding. */
	virtual FRemoteControlProtocolEntitySet GetAwaitingProtocolEntities() const override
	{
		return AwaitingProtocolEntities;
	}

private:

	/** Start recording incoming protocol message handler */
	virtual void OnStartRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity) override;

	/** Stop recording incoming protocol message handler */
	virtual void OnStopRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity) override;

	/** Refresh protocol binding list. Returns false if the backing widget wasn't valid. */
	virtual bool Refresh(bool bNavigateToEnd = false) override;

	//~ End IRCProtocolBindingList Interface

private:
	/** Constructs a single item in the list. */
	TSharedRef<ITableRow> ConstructBindingWidget(TSharedPtr<IRCTreeNodeViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Check if the bound entity type is supported by Protocol. */
	bool CanAddProtocol();

	/** Retrieves the appropriate tooltip based on the active protocol. */
	FText HandleAddBindingToolTipText() const;

	/** Primary Column is between Input and Output values. */
	float OnGetPrimaryLeftColumnWidth() const
	{
		return 1.0f - PrimaryColumnWidth;
	}

	float OnGetPrimaryRightColumnWidth() const
	{
		return PrimaryColumnWidth;
	}

	void OnSetPrimaryColumnWidth(float InWidth)
	{
		PrimaryColumnWidth = InWidth;
	}

	/** Secondary Column is for output struct labels - doesn't apply when the type is drawn in a single row. */
	float OnGetSecondaryLeftColumnWidth() const
	{
		return 1.0f - SecondaryColumnWidth;
	}

	float OnGetSecondaryRightColumnWidth() const
	{
		return SecondaryColumnWidth;
	}

	void OnSetSecondaryColumnWidth(float InWidth)
	{
		SecondaryColumnWidth = InWidth;
	}

private:
	/** ViewModel for the Protocol Entity. */
	TSharedPtr<FProtocolEntityViewModel> ViewModel;

	/** Current status message, if any */
	FText StatusMessage;

	/** ListView widget for each protocol binding */
	TSharedPtr<SListView<TSharedPtr<IRCTreeNodeViewModel>>> BindingList;

	/** Binding view models, filtered by the current HiddenProtocolTypeNames list */
	TArray<TSharedPtr<IRCTreeNodeViewModel>> FilteredBindings;

	/** Container used by all primary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> PrimaryColumnSizeData;

	/** Relative width to control primary splitters */
	float PrimaryColumnWidth = 0.5f;

	/** Container used by all primary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> SecondaryColumnSizeData;

	/** Relative width to control secondary splitters */
	float SecondaryColumnWidth = 0.5f;

	/** Set of protocol entities with awaiting state and waiting for binding */
	FRemoteControlProtocolEntitySet AwaitingProtocolEntities;

	/** Map of protocol names and their support of awaiting protocol entities */
	TMap<FName, bool> ProtocolNameToSupportsAwaitingProtocolEntitiesMap;
};
