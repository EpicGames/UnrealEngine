// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolEntity.h"

#include "IDetailsView.h"
#include "IRemoteControlProtocolModule.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlProtocolBinding.h"
#include "Modules/ModuleManager.h"
#include "ViewModels/ProtocolBindingViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

void SRCProtocolEntity::Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel)
{
	ViewModel = InViewModel;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(0.f, 5.f)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				CreateStructureDetailView()
			]
		]
	];
}

TSharedRef<SWidget> SRCProtocolEntity::CreateStructureDetailView()
{
	const TSharedPtr<FStructOnScope> StructOnScope = ViewModel->GetBinding()->GetStructOnScope();
	if (!StructOnScope || !StructOnScope->IsValid())
	{
		return SNew(STextBlock).Text(LOCTEXT("MappingItemError", "Mapping Item Error"));
	}

	FStructureDetailsViewArgs StructViewArgs;
	FDetailsViewArgs ViewArgs;

	// create struct to display
	StructViewArgs.bShowObjects = true;
	StructViewArgs.bShowAssets = true;
	StructViewArgs.bShowClasses = true;
	StructViewArgs.bShowInterfaces = true;

	ViewArgs.bAllowSearch = false;
	ViewArgs.bHideSelectionTip = false;
	ViewArgs.bShowObjectLabel = false;

	static FName PropertyEditorModuleName = "PropertyEditor";
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>(PropertyEditorModuleName);

	TSharedPtr<IStructureDetailsView> StructureDetailsView = PropertyEditorModule.CreateStructureDetailView(ViewArgs, StructViewArgs, StructOnScope, LOCTEXT("Struct", "Struct View"));
	StructureDetailsView->GetOnFinishedChangingPropertiesDelegate().AddLambda([this](const FPropertyChangedEvent& PropertyChangedEvent)
	{
		// Ignore temporary interaction (dropdown opened, etc.)
		if(PropertyChangedEvent.ChangeType == EPropertyChangeType::ValueSet)
		{
			ViewModel->NotifyChanged();	
		}
		
		FRemoteControlProtocolBinding* Binding = ViewModel->GetBinding();
		
		TSharedPtr<IRemoteControlProtocol> Protocol = IRemoteControlProtocolModule::Get().GetProtocolByName(Binding->GetProtocolName());
		if (Protocol.IsValid())
		{
			// Re-bound to update channel, etc.

			// Unbind in case already bound
			const TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> EntityPtr = Binding->GetRemoteControlProtocolEntityPtr();
			Protocol->Unbind(EntityPtr);
		
			// Bind/Rebind
			Protocol->Bind(EntityPtr);
		}

		const TWeakObjectPtr<URemoteControlPreset> Preset = ViewModel->GetPreset();
		if (Preset.IsValid())
		{
			// Have to mark dirty when changed by this widget, it's not automatically propagated for StructOnScope
			Preset->MarkPackageDirty();
		}
	});

	return StructureDetailsView->GetWidget().ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
