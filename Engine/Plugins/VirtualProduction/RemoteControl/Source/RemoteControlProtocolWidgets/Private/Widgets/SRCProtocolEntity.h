// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FProtocolBindingViewModel;

/** Widget for a given FRemoteControlProtocolEntity implementation. */
class REMOTECONTROLPROTOCOLWIDGETS_API SRCProtocolEntity : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolEntity)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel);

private:
	/** ViewModel for a Protocol Binding. */
	TSharedPtr<FProtocolBindingViewModel> ViewModel;

	/** Creates the widget for the current Protocol Binding implementation struct. */
	TSharedRef<SWidget> CreateStructureDetailView();
};
