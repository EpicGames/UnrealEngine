// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolRange.h"

#include "Styling/AppStyle.h"
#include "IDetailPropertyRow.h"
#include "IDetailTreeNode.h"
#include "IStructureDetailsView.h"
#include "SRCBindingWarning.h"
#include "Components/Widget.h"
#include "ViewModels/ProtocolRangeViewModel.h"
#include "Widgets/SPropertyView.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

void SRCProtocolRange::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FProtocolRangeViewModel>& InViewModel)
{
	constexpr float Padding = 2.0f;
	static FLinearColor BackgroundColor = FLinearColor::FromSRGBColor({83, 83, 83});

	ViewModel = InViewModel;
	PrimaryColumnSizeData = InArgs._PrimaryColumnSizeData;
	SecondaryColumnSizeData = InArgs._SecondaryColumnSizeData;

	ViewModel->OnChanged().AddSP(this, &SRCProtocolRange::OnViewModelChanged);
	
	const TSharedPtr<SWidget> LeftWidget =
	SNew(SHorizontalBox)
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(4.f, 4.f, 0.f, 0.f)
	.AutoWidth()
	[
		MakeInput()
	];

	const TSharedPtr<SWidget> RightWidget =
	SNew(SHorizontalBox)
	.Clipping(EWidgetClipping::OnDemand)
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.Padding(4.f, 4.f, 0.f, 0.f)
	.FillWidth(1.0f)
	[
		SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			MakeOutput()
		]
	]

	// Copies current property value
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Left)
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.OnClicked(this, &SRCProtocolRange::CopyFromCurrentPropertyValue)
		.ToolTipText(LOCTEXT("UsePropertyValue", "Use current property value."))
		.ContentPadding(4.0f)
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Use"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	]

	// Validation warning
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.AutoWidth()
	[
		SNew(SRCBindingWarning)
		.Status_Lambda([this]()
		{
			FText StatusMessage;
			return ViewModel->IsValid(StatusMessage) ? ERCBindingWarningStatus::Ok : ERCBindingWarningStatus::Warning;
		})
		.StatusMessage_Lambda([this]()
		{
			FText StatusMessage;
			ViewModel->IsValid(StatusMessage);
			return StatusMessage;
		})
	]

	// Delete button container
	+ SHorizontalBox::Slot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(2.f, 0.f)
	.AutoWidth()
	[
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
		.ForegroundColor(FSlateColor::UseForeground())
		.IsFocusable(false)
		.OnClicked(this, &SRCProtocolRange::OnDelete)
		.ContentPadding(2.f)
		.Content()
		[
			SNew(SImage)
			.ColorAndOpacity(FSlateColor::UseForeground())
			.Image(FAppStyle::GetBrush("Icons.X"))
		]
	];

	STableRow::Construct(
		STableRow::FArguments()
		.Style(FAppStyle::Get(), "DetailsView.TreeView.TableRow")
		.ShowSelection(false)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(0.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			.FillWidth(1.f)
			[
				SNew(RemoteControlProtocolWidgetUtils::SCustomSplitter)
				.LeftWidget(LeftWidget.ToSharedRef())
				.RightWidget(RightWidget.ToSharedRef())
				.ColumnSizeData(PrimaryColumnSizeData)
			]
		],
		InOwnerTableView);
}

TSharedRef<SWidget> SRCProtocolRange::MakeInput()
{
	TSharedPtr<SWidget> Widget;

	if (UObject* InputContainer = ViewModel->GetInputContainer())
	{
		InputPropertyView = SNew(SPropertyView)
		.Object(InputContainer)
		.RootPropertyName("Value")
		.NameVisibility(EPropertyNameVisibility::Show)
		.DisplayName(LOCTEXT("ProtocolRangeInput", "Input"))
		.ResizableColumn(false)
		.Spacing(10.0f)
		.ColumnPadding(true);

		ViewModel->CopyInputValue(GetInputPropertyHandle());
		OnInputProxyPropertyChangedHandle = InputPropertyView->OnFinishedChangingProperties().AddSP(this, &SRCProtocolRange::OnInputProxyChanged);

		Widget = InputPropertyView;
	}

	if (!Widget.IsValid())
	{
		const FProperty* Property = ViewModel->GetInputProperty();
		Widget = SNew(SBox)
		[
			SNew(STextBlock)
			.Text(FText::FromString(FString::Printf(TEXT("Unsupported Type: %s"), (Property ? *Property->GetClass()->GetName() : TEXT("Unknown")))))
		];
	}

	return Widget.ToSharedRef();
}

TSharedRef<SWidget> SRCProtocolRange::MakeOutput()
{
	TSharedPtr<SWidget> Widget;

	if (UObject* OutputContainer = ViewModel->GetOutputContainer())
	{
		OutputPropertyView = SNew(SPropertyView)
		.Object(OutputContainer)
		.RootPropertyName("Value")
		.ColumnSizeData(SecondaryColumnSizeData)
		.Spacing(10.0f)
		.ColumnPadding(true);

		ViewModel->CopyOutputValue(GetOutputPropertyHandle());
		OutputPropertyView->Refresh(); // in case child items added after CopyOutputValue
		OnOutputProxyPropertyChangedHandle = OutputPropertyView->OnFinishedChangingProperties().AddSP(this, &SRCProtocolRange::OnOutputProxyChanged);

		Widget = OutputPropertyView;
	}

	if (!Widget.IsValid())
	{
		const FProperty* Property = ViewModel->GetProperty().Get();
		Widget = SNew(SBox)
		[
			SNew(STextBlock).Text(FText::FromString(FString::Printf(TEXT("Unsupported Type: %s"), (Property ? *Property->GetClass()->GetName() : TEXT("Unknown")))))
		];
	}

	return Widget.ToSharedRef();
}

FReply SRCProtocolRange::CopyFromCurrentPropertyValue() const
{
	ViewModel->CopyFromCurrentPropertyValue();
	return FReply::Handled();
}

FReply SRCProtocolRange::OnDelete() const
{
	ViewModel->Remove();
	return FReply::Handled();
}

static void OnProxyPropertyChanged(const FPropertyChangedEvent& InEvent, const TSharedPtr<IPropertyHandle>& InPropertyHandle, const TFunction<void(const TSharedPtr<IPropertyHandle>&)>& InSetter)
{
	// It's possible this call is delayed and the property has been changed or invalidated
	if (InPropertyHandle.IsValid() && InPropertyHandle->IsValidHandle())
	{
		InSetter(InPropertyHandle);
	}
}

TSharedPtr<IPropertyHandle> SRCProtocolRange::GetInputPropertyHandle() const
{
	if (InputPropertyView.IsValid())
	{
		return InputPropertyView->GetPropertyHandle();
	}

	return nullptr;
}

TSharedPtr<IPropertyHandle> SRCProtocolRange::GetOutputPropertyHandle() const
{
	if (OutputPropertyView.IsValid())
	{
		return OutputPropertyView->GetPropertyHandle();
	}

	return nullptr;
}

void SRCProtocolRange::OnInputProxyChanged(const FPropertyChangedEvent& InEvent)
{
	OnProxyPropertyChanged(InEvent, GetInputPropertyHandle(), [&](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		ViewModel->SetInputData(InPropertyHandle);
	});
	InputPropertyView->Refresh();
}

void SRCProtocolRange::OnOutputProxyChanged(const FPropertyChangedEvent& InEvent)
{
	OnProxyPropertyChanged(InEvent, GetOutputPropertyHandle(), [&](const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	{
		ViewModel->SetOutputData(InPropertyHandle);
	});
	OutputPropertyView->Refresh();
}

void SRCProtocolRange::OnViewModelChanged() const
{
	// May not have been created yet
	if(InputPropertyView.IsValid())
	{
		InputPropertyView->Refresh();	
	}
}

#undef LOCTEXT_NAMESPACE
