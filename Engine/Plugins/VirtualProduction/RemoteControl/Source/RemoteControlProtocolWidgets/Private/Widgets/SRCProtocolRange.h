// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPropertyRowGenerator.h"
#include "Serialization/BufferArchive.h"
#include "ViewModels/ProtocolRangeViewModel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SRCProtocolRangeList.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

class FProtocolBindingViewModel;
class IPropertyRowGenerator;
class SPropertyView;
class SRemoteControlProtocolWidgetExtension;
struct FRemoteControlProtocolMapping;

class SRCProtocolRange : public STableRow<TSharedPtr<FProtocolRangeViewModel>>
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolRange)
	{}
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, PrimaryColumnSizeData)
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, SecondaryColumnSizeData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, const TSharedRef<FProtocolRangeViewModel>& InViewModel);

protected:
	/** Resolve the input widget. */
	TSharedRef<SWidget> MakeInput();

	/** Resolve the output widget. */
	TSharedRef<SWidget> MakeOutput();

	/** Copies current property value to this range's output value. */
	FReply CopyFromCurrentPropertyValue() const;

	/** Called when this ProtocolRange is deleted. */
	FReply OnDelete() const;

private:
	FDelegateHandle OnInputProxyPropertyChangedHandle;
	FDelegateHandle OnOutputProxyPropertyChangedHandle;

	TSharedPtr<IPropertyHandle> GetInputPropertyHandle() const;
	TSharedPtr<IPropertyHandle> GetOutputPropertyHandle() const;

	/** Applies proxy data to actual. */
	void OnInputProxyChanged(const FPropertyChangedEvent& InEvent);

	/** Applies proxy data to actual. */
	void OnOutputProxyChanged(const FPropertyChangedEvent& InEvent);
	
	/** Called whenever the viewmodel changes. */
	void OnViewModelChanged() const;
	
protected:
	/** Represents an individual range binding for a protocol binding. */
	TSharedPtr<FProtocolRangeViewModel> ViewModel;

	TSharedPtr<SSplitter> Splitter;

	TSharedPtr<IPropertyRowGenerator> InputPropertyRowGenerator;
	TSharedPtr<SPropertyView> InputPropertyView;

	TSharedPtr<IPropertyRowGenerator> OutputPropertyRowGenerator;
	TSharedPtr<SPropertyView> OutputPropertyView;

	/** Container used by all primary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> PrimaryColumnSizeData;

	/** Container used by all secondary splitters in the details view, so that they move in sync */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> SecondaryColumnSizeData;
};
