// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolRangeList.h"
#include "SRCProtocolRange.h"
#include "ViewModels/ProtocolBindingViewModel.h"
#include "ViewModels/ProtocolRangeViewModel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "RemoteControlProtocolWidgets"

void SRCProtocolRangeList::Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel)
{
	constexpr float Padding = 2.0f;
	ViewModel = InViewModel;
	PrimaryColumnSizeData = InArgs._PrimaryColumnSizeData;
	SecondaryColumnSizeData = InArgs._SecondaryColumnSizeData;

	ListView = SNew(SListView<TSharedPtr<FProtocolRangeViewModel>>)
			   .SelectionMode(ESelectionMode::None)
			   .ListItemsSource(&ViewModel->GetRanges())
			   .OnGenerateRow(this, &SRCProtocolRangeList::OnGenerateRow)
			   .AllowOverscroll(EAllowOverscroll::No);

	ViewModel->OnRangeMappingAdded().AddSP(this, &SRCProtocolRangeList::OnRangeMappingAdded);
	ViewModel->OnRangeMappingRemoved().AddSP(this, &SRCProtocolRangeList::OnRangeMappingRemoved);
	ViewModel->OnChanged().AddSP(this, &SRCProtocolRangeList::OnViewModelChanged);

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(1, 1, 1, Padding)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("DetailsView.CategoryTop"))
			.Padding(FMargin(0.0f, 4.0f, 4.0f, 4.0f))
			.BorderBackgroundColor(FLinearColor(0.6f, 0.6f, 0.6f, 1.0f))
			.HAlign(HAlign_Fill)
			[
				ConstructHeader()
			]
		]

		// Channel muting
		/*
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Top)
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.VAlign(VAlign_Center)
			.FillHeight(1.0f)
			[
				ConstructChannelMuter()
			]

			+ SVerticalBox::Slot()
			.VAlign(VAlign_Top)
			.AutoHeight()
			[
				SNew(SSeparator)
				.Orientation(EOrientation::Orient_Horizontal)
			]
		]
		*/

		+ SVerticalBox::Slot()
		.Padding(Padding)
		[
			ListView.ToSharedRef()
		]
	];
}

void SRCProtocolRangeList::GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth)
{
	OutMinDesiredWidth = 170.0f;
	OutMaxDesiredWidth = 170.0f;
}

TSharedRef<SWidget> SRCProtocolRangeList::ConstructHeader()
{
	const TSharedPtr<SWidget> LeftWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNullWidget::NullWidget
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Ranges", "Ranges"))
			.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
			.ShadowOffset(FVector2D(1.0f, 1.0f))
		];

	const TSharedPtr<SWidget> RightWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SNew(SSpacer)
		]

		// Add new range button
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText(LOCTEXT("AddRangeTooltip", "Add a new range"))
			.OnClicked_Lambda([this]()
			{
				ViewModel->AddRangeMapping();
				return FReply::Handled();
			})
			.ContentPadding(FMargin(4.0f, 2.0f))
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.Plus"))
			]
		]

		// Delete *all* ranges button
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Right)
		.AutoWidth()
		[		
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ToolTipText(LOCTEXT("DeleteRangesTooltip", "Delete all ranges"))
			.OnClicked_Lambda([this]()
			{
				ViewModel->RemoveAllRangeMappings();
				return FReply::Handled();
			})
			.ContentPadding(FMargin(4.0f, 2.0f))
			.Content()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.X"))
			]
		];

	TSharedRef<SHorizontalBox> Widget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(RemoteControlProtocolWidgetUtils::SCustomSplitter)
			.LeftWidget(LeftWidget.ToSharedRef())
			.RightWidget(RightWidget.ToSharedRef())
			.ColumnSizeData(PrimaryColumnSizeData)
		];

	return Widget;
}

TSharedRef<SWidget> SRCProtocolRangeList::ConstructChannelMuter()
{
	const TSharedPtr<SWidget> LeftWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		.Padding(2.0f)
		.AutoWidth()
		[
			SNullWidget::NullWidget
		]

		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
			.Text(LOCTEXT("Channels", "Channels"))
		];

	const TSharedPtr<SWidget> RightWidget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
		];

	TSharedRef<SHorizontalBox> Widget =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		[
			SNew(RemoteControlProtocolWidgetUtils::SCustomSplitter)
			.LeftWidget(LeftWidget.ToSharedRef())
			.RightWidget(RightWidget.ToSharedRef())
			.ColumnSizeData(PrimaryColumnSizeData)
		];

	return Widget;
}

TSharedRef<ITableRow> SRCProtocolRangeList::OnGenerateRow(TSharedPtr<FProtocolRangeViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable) const
{
	check(InViewModel.IsValid());
	return SNew(SRCProtocolRange, InOwnerTable, InViewModel.ToSharedRef())
		   .PrimaryColumnSizeData(PrimaryColumnSizeData)
		   .SecondaryColumnSizeData(SecondaryColumnSizeData);
}

void SRCProtocolRangeList::OnRangeMappingAdded(TSharedRef<FProtocolRangeViewModel> InRangeViewModel) const
{
	Refresh();
}

void SRCProtocolRangeList::OnRangeMappingRemoved(FGuid InRangeId) const
{
	Refresh();
}

void SRCProtocolRangeList::OnViewModelChanged() const
{
	Refresh();
}

void SRCProtocolRangeList::Refresh() const
{
	if (ListView.IsValid())
	{
		ListView->RequestListRefresh();
	}
}

#undef LOCTEXT_NAMESPACE
