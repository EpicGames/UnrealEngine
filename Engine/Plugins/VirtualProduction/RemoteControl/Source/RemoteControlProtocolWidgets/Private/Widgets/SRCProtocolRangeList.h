// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SRCProtocolShared.h"
#include "Widgets/Views/SListView.h"

class FProtocolRangeViewModel;
class FProtocolBindingViewModel;
class SSplitter;
template <typename ItemType> class SListView;
class ITableRow;
class STableViewBase;

class REMOTECONTROLPROTOCOLWIDGETS_API SRCProtocolRangeList : public SCompoundWidget
{
public:
SLATE_BEGIN_ARGS(SRCProtocolRangeList)
		{}
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, PrimaryColumnSizeData)
		SLATE_ARGUMENT(TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData>, SecondaryColumnSizeData)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FProtocolBindingViewModel>& InViewModel);
	void GetDesiredWidth(float& OutMinDesiredWidth, float& OutMaxDesiredWidth);

private:
	/** Constructs the header widget with array controls. */
	TSharedRef<SWidget> ConstructHeader();

	/** Constructs  the channel muting widget (if supported for type). */
	TSharedRef<SWidget> ConstructChannelMuter();

	/** Constructs an individual widget per range binding. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FProtocolRangeViewModel> InViewModel, const TSharedRef<STableViewBase>& InOwnerTable) const;

	/** Called when a range mapping is added. */
	void OnRangeMappingAdded(TSharedRef<FProtocolRangeViewModel> InRangeViewModel) const;

	/** Called when a range mapping is removed. */
	void OnRangeMappingRemoved(FGuid InRangeId) const;

	/** Called whenever the viewmodel changes. */
	void OnViewModelChanged() const;

	/** Refresh ListView. */
	void Refresh() const;

private:
	/** Represents an individual protocol binding for an entity. */
	TSharedPtr<FProtocolBindingViewModel> ViewModel;

	/** Container widget for all range bindings. */
	TSharedPtr<SListView<TSharedPtr<FProtocolRangeViewModel>>> ListView;

	/** Container used by all primary splitters in the details view, so that they move in sync. */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> PrimaryColumnSizeData;

	/** Container used by all secondary splitters in the details view, so that they move in sync. Secondary splitter is used for struct name/value separation. */
	TSharedPtr<RemoteControlProtocolWidgetUtils::FPropertyViewColumnSizeData> SecondaryColumnSizeData;
};
