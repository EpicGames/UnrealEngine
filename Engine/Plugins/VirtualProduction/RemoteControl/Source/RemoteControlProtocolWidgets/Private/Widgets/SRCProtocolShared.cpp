// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCProtocolShared.h"

#include "Styling/AppStyle.h"
#include "Widgets/SBoxPanel.h"

namespace RemoteControlProtocolWidgetUtils
{
	void SConstrainedBox::Construct(const FArguments& InArgs)
	{
		MinWidth = InArgs._MinWidth;
		MaxWidth = InArgs._MaxWidth;

		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	FVector2D SConstrainedBox::ComputeDesiredSize(float InLayoutScaleMultiplier) const
	{
		const float MinWidthVal = MinWidth.Get().Get(0.0f);
		const float MaxWidthVal = MaxWidth.Get().Get(0.0f);

		if (MinWidthVal == 0.0f && MaxWidthVal == 0.0f)
		{
			return SCompoundWidget::ComputeDesiredSize(InLayoutScaleMultiplier);
		}
		else
		{
			const FVector2D ChildSize = ChildSlot.GetWidget()->GetDesiredSize();

			float XVal = FMath::Max(MinWidthVal, ChildSize.X);
			if (MaxWidthVal >= MinWidthVal)
			{
				XVal = FMath::Min(MaxWidthVal, XVal);
			}

			return FVector2D(XVal, ChildSize.Y);
		}
	}

	void SCustomSplitter::Construct(const FArguments& InArgs)
	{
		SSplitter::FArguments Args;
		SSplitter::Construct(Args
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.PhysicalSplitterHandleSize(1.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			.ResizeMode(ESplitterResizeMode::Fill));

		ColumnSizeData = InArgs._ColumnSizeData;

		AddSlot()
			.Value(ColumnSizeData->LeftColumnWidth)
			.OnSlotResized(FOnSlotResized::CreateLambda([](float InNewWidth) -> void
			{
				// This has to be bound or the splitter will take it upon itself to determine the size
				// We do nothing here because it is handled by the column size data
			}))
			[
				SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand)
				+ SHorizontalBox::Slot()
				.Padding(0.0f, 0.0f, 5.0f, 0.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillWidth(1.f)
				[
					InArgs._LeftWidget.ToSharedRef()
				]
			];

		AddSlot()
			.Value(InArgs._ColumnSizeData->RightColumnWidth)
			.OnSlotResized(ColumnSizeData->OnWidthChanged)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.0f, 0.0f, 0.0f, 0.0f)
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Fill)
				.FillWidth(1.f)
				[
					InArgs._RightWidget.ToSharedRef()
				]
			];
	}

	void SCustomSplitter::OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const
	{
		if (Children.Num() > 1)
		{
			const FSlot& LeftChild = Children[0];

			const FVector2D AllottedSize = AllottedGeometry.GetLocalSize();

			FVector2D LocalPosition = FVector2D::ZeroVector;
			FVector2D LocalSize(AllottedSize.X * ColumnSizeData->LeftColumnWidth.Get(0.f), AllottedSize.Y);

			ArrangedChildren.AddWidget(EVisibility::Visible, AllottedGeometry.MakeChild(LeftChild.GetWidget(), LocalPosition, LocalSize));

			const FSlot& RightChild = Children[1];

			LocalPosition = FVector2D(LocalSize.X, 0.f);
			LocalSize.X = AllottedSize.X * ColumnSizeData->RightColumnWidth.Get(0.f);

			ArrangedChildren.AddWidget(EVisibility::Visible, AllottedGeometry.MakeChild(RightChild.GetWidget(), LocalPosition, LocalSize));
		}
	}
}
