// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SSplitter.h"

namespace RemoteControlProtocolWidgetUtils
{
	/** Helper class to force a widget to fill in a space. Copied from SDetailSingleItemRow.cpp */
	class REMOTECONTROLPROTOCOLWIDGETS_API SConstrainedBox : public SCompoundWidget
	{
	public:
			SLATE_BEGIN_ARGS(SConstrainedBox)
				: _MinWidth()
				, _MaxWidth()
			{}
			SLATE_DEFAULT_SLOT(FArguments, Content)
			SLATE_ATTRIBUTE(TOptional<float>, MinWidth)
			SLATE_ATTRIBUTE(TOptional<float>, MaxWidth)
		SLATE_END_ARGS()

		virtual ~SConstrainedBox() = default;

		void Construct(const FArguments& InArgs);
		virtual FVector2D ComputeDesiredSize(float InLayoutScaleMultiplier) const override;

	private:
		TAttribute<TOptional<float>> MinWidth;
		TAttribute<TOptional<float>> MaxWidth;
	};

	/** Copied from SDetailsViewBase.h */
	struct FPropertyViewColumnSizeData
	{
		TAttribute<float> LeftColumnWidth;
		TAttribute<float> RightColumnWidth;
		SSplitter::FOnSlotResized OnWidthChanged;

		void SetColumnWidth(float InWidth)
		{
			OnWidthChanged.ExecuteIfBound(InWidth);
		}
	};

	class SCustomSplitter : public SSplitter
	{
	public:
	SLATE_BEGIN_ARGS(SCustomSplitter)
			{}
			SLATE_ARGUMENT(TSharedPtr<SWidget>, LeftWidget)
			SLATE_ARGUMENT(TSharedPtr<SWidget>, RightWidget)
			SLATE_ARGUMENT(TSharedPtr<FPropertyViewColumnSizeData>, ColumnSizeData)
		SLATE_END_ARGS();

		virtual ~SCustomSplitter() = default;

		void Construct(const FArguments& InArgs);

		virtual void OnArrangeChildren(const FGeometry& AllottedGeometry, FArrangedChildren& ArrangedChildren) const override;

	private:
		TSharedPtr<FPropertyViewColumnSizeData> ColumnSizeData;
	};

	/** Info about a given property customization */
	struct FRCPropertyViewCustomizationInfo
	{
		/** A name to uniquely identify this info, ie. the path or type name. */
		FName Id = NAME_None;

		/** Used to determine if the first row needs to be drawn or not. */
		bool bHasHeaderContent = false;

		/** Size of each column in a detail row. */
		TArray<float> ColumnSizes;

		/** Label for each column in a detail row, if applicable (ie. X Y Z). */
		TArray<FText> ColumnLabels;
	};
}
