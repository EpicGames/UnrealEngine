// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/StructOnScope.h"

class URemoteControlPreset;
struct FRemoteControlProtocolEntity;

/**
 * Interface class for RCProtocolBindingList widget
 */
class IRCProtocolBindingList
{
public:
	using FRemoteControlProtocolEntitySet = TSet<TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>>>;

	/** Virtual destructor */
	virtual ~IRCProtocolBindingList() = default;

	/** Returns the remote control preset for which this list is currently drawn */
	virtual URemoteControlPreset* GetPreset() const = 0;

	/** Adds a new protocol binding of specified type to the current view model. */
	virtual void AddProtocolBinding(const FName InProtocolName) = 0;

	/** Get the set of entities which is awaiting state and waiting for binding */
	virtual FRemoteControlProtocolEntitySet GetAwaitingProtocolEntities() const = 0;

	/** Start recording incoming protocol message handler */
	virtual void OnStartRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity) = 0;

	/** Stop recording incoming protocol message handler */
	virtual void OnStopRecording(TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> InEntity) = 0;

	/** Refresh protocol binding list. Returns false if the backing widget wasn't valid. */
	virtual bool Refresh(bool bNavigateToEnd = false) = 0;
};
