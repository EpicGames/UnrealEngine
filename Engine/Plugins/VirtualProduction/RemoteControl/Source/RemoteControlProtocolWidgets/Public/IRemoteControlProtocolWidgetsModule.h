// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "RemoteControlField.h"

class IRCProtocolBindingList;
class SWidget;
class URemoteControlPreset;
class SWidget;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnProtocolBindingAddedOrRemoved, ERCProtocolBinding::Op /* BindingOperation */);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnActiveProtocolChanged, const FName /* NewActiveProtocolName */);

/** A Remote Control module that provides editor widgets for protocol bindings. */
class IRemoteControlProtocolWidgetsModule : public IModuleInterface
{
public:
	/**
	* Singleton-like access to this module's interface.  This is just for convenience!
	* Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	*
	* @return Returns singleton instance, loading the module on demand if needed
	*/
	static IRemoteControlProtocolWidgetsModule& Get()
	{
		static const FName ModuleName = "RemoteControlProtocolWidgets";
		return FModuleManager::LoadModuleChecked<IRemoteControlProtocolWidgetsModule>(ModuleName);
	}

	/** Adds a new protocol binding of specified type to the current view model. */
	virtual void AddProtocolBinding(const FName InProtocolName) = 0;

	/** Creates a widget for the given Preset Field and FieldType */
	virtual TSharedRef<SWidget> GenerateDetailsForEntity(URemoteControlPreset* InPreset, const FGuid& InFieldId, const EExposedFieldType& InFieldType = EExposedFieldType::Invalid) = 0;

	/** Reset protocol binding widget */
	virtual void ResetProtocolBindingList() = 0;

	/** Get the binding list public reference */
	virtual TSharedPtr<IRCProtocolBindingList> GetProtocolBindingList() const = 0;
	
	/** Get the selected protocol name. */
	virtual const FName GetSelectedProtocolName() const = 0;

	/** Called when binding is added or removed. */
	virtual FOnProtocolBindingAddedOrRemoved& OnProtocolBindingAddedOrRemoved() = 0;

	/** Called when active protocol selection changed. */
	virtual FOnActiveProtocolChanged& OnActiveProtocolChanged() = 0;
};
