// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SRCProtocolMaskTriplet.h"

enum class ERCMask : uint8;
struct FRemoteControlField;

/**
 * A widget that represents a mask.
 */
class UE_DEPRECATED(5.6, "SRCProtocolMask deprecated. Prefer using SRCProtocolMaskTriplet for mask widgets") SRCProtocolMask;
class REMOTECONTROLPROTOCOLWIDGETS_API SRCProtocolMask : public SRCProtocolMaskTriplet, public IHasMaskExtensibility
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolMask)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> InField);

protected:

	//~ Begin : IHasMaskExtensibility Interface
	virtual bool CanBeMasked() const override;
	virtual EMaskingType GetMaskingType() override;
	virtual bool HasOptionalMask() const override;
	//~ End : IHasMaskExtensibility Interface

	//~ Begin : SRCProtocolMaskTriplet Interface
	virtual ECheckBoxState IsMaskEnabled(ERCMask InMaskBit) const override;
	virtual void SetMaskEnabled(ECheckBoxState NewState, ERCMask NewMaskBit) override;
	//~ End : SRCProtocolMaskTriplet Interface

private:

	/** Holds a weak reference to the housing RC field. */
	TWeakPtr<FRemoteControlField> WeakField;
};
