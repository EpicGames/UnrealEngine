// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SBorder.h"

enum class ERCMask : uint8;
struct FProtocolWidgetStyle;
class SRCProtocolMaskButton;

namespace EMasking
{
	enum class Bits : uint8
	{
		ETM_A,

		ETM_B,

		ETM_C,

		ETM_D
	};
}

enum class EMaskingType : uint8
{
	Color,

	Polar,

	Rotator,

	Vector,

	Quat,

	Unsupported
};

class FRemoteControlProtocolMasking
{
public:
	static const TMap<UScriptStruct*, EMaskingType>& GetStructsToMaskingTypes();
	static const TSet<UScriptStruct*>& GetOptionalMaskStructs();
};

/**
 * A widget that represents a triplet mask.
 */
class SRCProtocolMaskTriplet : public SBorder
{
public:
	SLATE_BEGIN_ARGS(SRCProtocolMaskTriplet)
		: _MaskA()
		, _MaskB()
		, _MaskC()
		, _OptionalMask()
		, _EnableOptionalMask(false)
	{}
		SLATE_ARGUMENT(ERCMask, MaskA)
		SLATE_ARGUMENT(ERCMask, MaskB)
		SLATE_ARGUMENT(ERCMask, MaskC)
		SLATE_ARGUMENT(ERCMask, OptionalMask)

		SLATE_ARGUMENT(EMaskingType, MaskingType)

		SLATE_ATTRIBUTE(bool, CanBeMasked)
		SLATE_ARGUMENT(bool, EnableOptionalMask)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:

	/**
	 * Retrieves the label of the mask button based on type.
	 */
	FText GetMaskLabel(EMasking::Bits InMaskBitName) const;

protected:

	/**
	 * Retrieves the state of the mask.
	 */
	virtual ECheckBoxState IsMaskEnabled(ERCMask InMaskBit) const { return ECheckBoxState::Unchecked; }

	/** Soft toggles masks based on active masks. */
	void LoadMasks();

	/**
	 * Handles masked state.
	 */
	virtual void SetMaskEnabled(ECheckBoxState NewState, ERCMask NewMaskBit) {}

protected:

	/** Protocol Widget Style reference. */
	const FProtocolWidgetStyle* WidgetStyle;

private:

	/** Holds the type of this triplet mask. */
	EMaskingType MaskingType;

	/** Holds a shared reference to the underlying mask button for A. */
	TSharedPtr<SRCProtocolMaskButton> MaskA;

	/** Holds a shared reference to the underlying mask button for B. */
	TSharedPtr<SRCProtocolMaskButton> MaskB;

	/** Holds a shared reference to the underlying mask button for C. */
	TSharedPtr<SRCProtocolMaskButton> MaskC;

	/** Holds a shared reference to the underlying mask button for D. */
	TSharedPtr<SRCProtocolMaskButton> MaskD;
};

/**
 * Ensures the maskability of the derived classes.
 */
class IHasMaskExtensibility
{
	/**
	 * True when the derived classes supports masking, false otherwise.
	 */
	virtual bool CanBeMasked() const = 0;

	/**
	 * Retrieves the masking type based on the property type.
	 */
	virtual EMaskingType GetMaskingType() = 0;

	/**
	 * True when derived classes has optional mask.
	 */
	virtual bool HasOptionalMask() const = 0;
};
