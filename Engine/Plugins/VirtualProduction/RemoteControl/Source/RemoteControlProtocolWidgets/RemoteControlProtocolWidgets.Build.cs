// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RemoteControlProtocolWidgets : ModuleRules
{
	public RemoteControlProtocolWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(new string[] {
			"Core",
			"CoreUObject",
			"EditorStyle",
			"EditorSubsystem",
			"EditorWidgets",
			"Engine",
			"GraphEditor",
			"InputCore",
			"Projects",
			"PropertyEditor",
			"RemoteControl",
			"RemoteControlCommon",
			"RemoteControlProtocol",
			"Slate",
			"SlateCore",
			"StructUtilsEditor",
			"UnrealEd",
		});
	}
}
