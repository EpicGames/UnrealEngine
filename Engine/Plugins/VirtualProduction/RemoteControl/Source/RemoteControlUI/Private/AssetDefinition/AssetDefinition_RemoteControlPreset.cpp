// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition_RemoteControlPreset.h"

#include "RemoteControlPreset.h"
#include "AssetEditor/RemoteControlPresetEditorToolkit.h"

#define LOCTEXT_NAMESPACE "AssetDefinition_RemoteControlPreset"

FText UAssetDefinition_RemoteControlPreset::GetAssetDisplayName() const
{
	return LOCTEXT("RemoteControlPreset_AssetName", "Remote Control Preset");
}

TSoftClassPtr<UObject> UAssetDefinition_RemoteControlPreset::GetAssetClass() const
{
	return URemoteControlPreset::StaticClass();
}

FLinearColor UAssetDefinition_RemoteControlPreset::GetAssetColor() const
{
	return FColor(200, 80, 80);
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition_RemoteControlPreset::GetAssetCategories() const
{
	static const FAssetCategoryPath Categories[] =
	{
		FAssetCategoryPath(FAssetCategoryPath(LOCTEXT("RemoteControlPreset_AssetCategory", "Virtual Production")), LOCTEXT("RemoteControlPreset_CategorySection", "Remote Control"), ECategoryMenuType::Section)
	};

	return Categories;
}

EAssetCommandResult UAssetDefinition_RemoteControlPreset::OpenAssets(const FAssetOpenArgs& OpenArgs) const
{
	TArray<UObject*> Objects = OpenArgs.LoadObjects<UObject>();
	for (UObject* Obj : Objects)
	{
		if (URemoteControlPreset* Asset = Cast<URemoteControlPreset>(Obj))
		{
			FRemoteControlPresetEditorToolkit::CreateEditor(OpenArgs.GetToolkitMode(), OpenArgs.ToolkitHost, Asset);
		}
	}

	return EAssetCommandResult::Handled;
}

FAssetSupportResponse UAssetDefinition_RemoteControlPreset::CanLocalize(const FAssetData& InAsset) const
{
	return FAssetSupportResponse::NotSupported();
}

FAssetOpenSupport UAssetDefinition_RemoteControlPreset::GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
{
	FAssetOpenSupport Support(OpenSupportArgs.OpenMethod, OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit);
	Support.RequiredToolkitMode = EToolkitMode::WorldCentric;
	return Support;
}

#undef LOCTEXT_NAMESPACE
