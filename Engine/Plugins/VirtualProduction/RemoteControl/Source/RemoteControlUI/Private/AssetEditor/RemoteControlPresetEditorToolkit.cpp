// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPresetEditorToolkit.h"

#include "Framework/Docking/TabManager.h"
#include "RemoteControlPreset.h"
#include "RemoteControlUIModule.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Docking/SDockTab.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelTreeNode.h"
#include "UI/SRemoteControlPanel.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "RemoteControlPresetEditorToolkit"

const FName FRemoteControlPresetEditorToolkit::RemoteControlPanelAppIdentifier(TEXT("RemoteControlPanel"));


TSharedRef<FRemoteControlPresetEditorToolkit> FRemoteControlPresetEditorToolkit::CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& InitToolkitHost, URemoteControlPreset* InPreset)
{
	TSharedRef<FRemoteControlPresetEditorToolkit> NewEditor = MakeShared<FRemoteControlPresetEditorToolkit>();

	NewEditor->InitRemoteControlPresetEditor(Mode, InitToolkitHost, InPreset);

	return NewEditor;
}

void FRemoteControlPresetEditorToolkit::InitRemoteControlPresetEditor(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost> & InitToolkitHost, URemoteControlPreset* InPreset)
{
	Preset = InPreset;

	PanelTab = FRemoteControlUIModule::Get().CreateRemoteControlPanel(InPreset, InitToolkitHost);

	// Required, will cause the previous toolkit to close bringing down the RemoteControlPreset and unsubscribing the
	// tab spawner. Without this, the InitAssetEditor call below will trigger an ensure as the RemoteControlPreset
	// tab ID will already be registered within EditorTabManager
	TSharedPtr<FTabManager> HostTabManager;

	if (InitToolkitHost.IsValid())
	{
		HostTabManager = InitToolkitHost->GetTabManager();
	}
	else
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		HostTabManager = LevelEditorModule.GetLevelEditorTabManager();
	}

	if (HostTabManager.IsValid() && HostTabManager->FindExistingLiveTab(FRemoteControlUIModule::RemoteControlPanelTabName).IsValid())
	{
		HostTabManager->TryInvokeTab(FRemoteControlUIModule::RemoteControlPanelTabName)->RequestCloseTab();
	}

	RegisterTabSpawners(InitToolkitHost->GetTabManager().ToSharedRef());

	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_RemoteControlPresetEditor")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->Split
			(
				FTabManager::NewStack()
				->AddTab(FRemoteControlUIModule::RemoteControlPanelTabName, ETabState::OpenedTab)
			)
		);

	constexpr bool bCreateDefaultStandaloneMenu = true;
	constexpr bool bCreateDefaultToolbar = true;
	FAssetEditorToolkit::InitAssetEditor(Mode, InitToolkitHost, RemoteControlPanelAppIdentifier, StandaloneDefaultLayout, bCreateDefaultStandaloneMenu, bCreateDefaultStandaloneMenu, InPreset);

	InvokePanelTab();
}

FRemoteControlPresetEditorToolkit::~FRemoteControlPresetEditorToolkit()
{
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		if (TSharedPtr<FTabManager> EditorTabManager = LevelEditorModule.GetLevelEditorTabManager())
		{
			UnregisterTabSpawners(EditorTabManager.ToSharedRef());
			if (TSharedPtr<SDockTab> Tab = EditorTabManager->FindExistingLiveTab(FRemoteControlUIModule::RemoteControlPanelTabName))
			{
				Tab->RequestCloseTab();
			}
		}
	}
}

void FRemoteControlPresetEditorToolkit::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	// In case of double registration
	if (!InTabManager->HasTabSpawner(FRemoteControlUIModule::RemoteControlPanelTabName))
	{
		InTabManager->RegisterTabSpawner(FRemoteControlUIModule::RemoteControlPanelTabName, FOnSpawnTab::CreateSP(this, &FRemoteControlPresetEditorToolkit::HandleTabManagerSpawnPanelTab))
			.SetDisplayName(LOCTEXT("RemoteControlPanelMainTab", "Remote Control Panel"))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorCategory())
			.SetIcon(FSlateIcon(FRemoteControlPanelStyle::GetStyleSetName(), "Icons.RemoteControlPanel"));
	}
}

void FRemoteControlPresetEditorToolkit::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(FRemoteControlUIModule::RemoteControlPanelTabName);
}

bool FRemoteControlPresetEditorToolkit::OnRequestClose(EAssetEditorCloseReason InCloseReason)
{
	return true;
}

void FRemoteControlPresetEditorToolkit::FocusWindow(UObject* ObjectToFocusOn)
{
	InvokePanelTab();
	BringToolkitToFront();
}

FText FRemoteControlPresetEditorToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("PanelToolkitName", "Remote Control Panel");
}

FName FRemoteControlPresetEditorToolkit::GetToolkitFName() const
{
	static const FName RemoteControlPanelName("RemoteControlPanel");
	return RemoteControlPanelName;
}

FLinearColor FRemoteControlPresetEditorToolkit::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.7f, 0.0f, 0.0f, 0.5f);
}

FString FRemoteControlPresetEditorToolkit::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("RemoteControlTabPrefix", "RemoteControl ").ToString();
}

TSharedRef<SDockTab> FRemoteControlPresetEditorToolkit::HandleTabManagerSpawnPanelTab(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == FRemoteControlUIModule::RemoteControlPanelTabName);

	/*
	FText TabName;
	
	if (Preset)
	{
		TabName = FText::FromString(Preset->GetName());
	}
	else
	{
		TabName = LOCTEXT("ControlPanelLabel", "Control Panel");
	}
	*/

	return SNew(SDockTab)
		.Label(LOCTEXT("RCPanelTabName", "Remote Control"))
		.TabColorScale(GetTabColorScale())	
		[
			PanelTab.ToSharedRef()
		];
}

void FRemoteControlPresetEditorToolkit::InvokePanelTab()
{
	struct Local
	{
		static void OnRemoteControlPresetClosed(TSharedRef<SDockTab> DockTab, TWeakPtr<IAssetEditorInstance> InRemoteControlPreset)
		{
			if (const TSharedPtr<IAssetEditorInstance> AssetEditorInstance = InRemoteControlPreset.Pin())
			{
				InRemoteControlPreset.Pin()->CloseWindow(EAssetEditorCloseReason::AssetEditorHostClosed);
			}
		}
	};

	TSharedPtr<FTabManager> HostTabManager;	

	if (ToolkitHost.IsValid())
	{
		HostTabManager = ToolkitHost.Pin()->GetTabManager();
	}
	else
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		HostTabManager = LevelEditorModule.GetLevelEditorTabManager();
	}

	// Create a new DockTab and add the RemoteControlPreset widget to it.
	if (HostTabManager.IsValid())
	{
		if (TSharedPtr<SDockTab> Tab = HostTabManager->TryInvokeTab(FRemoteControlUIModule::RemoteControlPanelTabName))
		{
			Tab->SetContent(PanelTab.ToSharedRef());
			Tab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateStatic(&Local::OnRemoteControlPresetClosed, TWeakPtr<IAssetEditorInstance>(SharedThis(this))));

			// Force the parent window to have minimum width.
			TSharedPtr<SWindow> Window = Tab->GetParentWindow();
			if (Window.IsValid())
			{
				FWindowSizeLimits SizeLimits = Window->GetSizeLimits();
				SizeLimits.SetMinWidth(SRemoteControlPanel::MinimumPanelWidth);
				Window->SetSizeLimits(SizeLimits);
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE /*RemoteControlPresetEditorToolkit*/