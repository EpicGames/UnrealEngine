// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Framework/Docking/TabManager.h"

class SDockTab;
class SRemoteControlPanel;
struct SRCPanelTreeNode;

/** Viewer/editor for a Remote Control Preset */
class FRemoteControlPresetEditorToolkit : public FAssetEditorToolkit
{
public:
	/**
	 *  Create an editor for a remote control preset asset.
	 */
	static TSharedRef<FRemoteControlPresetEditorToolkit> CreateEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class URemoteControlPreset* InPreset);

	/**
	 * Initialize a remote control preset editor module. 
	 */
	void InitRemoteControlPresetEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class URemoteControlPreset* InPreset);
	~FRemoteControlPresetEditorToolkit();
	
	//~ Begin IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	virtual bool OnRequestClose(EAssetEditorCloseReason InCloseReason) override;
	//~ End IToolkit interface


	/** Begin IAssetEditorInstance interface */
	virtual void FocusWindow(UObject* ObjectToFocusOn = nullptr) override;
	/** End IAssetEditorInstance interface */

private:
	//~ Handlers that spawn remote control tabs.
	TSharedRef<SDockTab> HandleTabManagerSpawnPanelTab(const FSpawnTabArgs& Args);

	/** Handle invoking the remote control tab. */
	void InvokePanelTab();
private:
	/** Holds the remote control panel app identifier. */
	static const FName RemoteControlPanelAppIdentifier;
	/** Holds the preset being edited. */
	URemoteControlPreset* Preset = nullptr;
	/** Holds the panel widget */
	TSharedPtr<SRemoteControlPanel> PanelTab;
};