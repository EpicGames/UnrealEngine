// Copyright Epic Games, Inc. All Rights Reserved.

#include "Commands/RemoteControlCommands.h"

#include "UI/RemoteControlPanelStyle.h"

#define LOCTEXT_NAMESPACE "RemoteControlCommands"

FRemoteControlCommands::FRemoteControlCommands()
	: TCommands<FRemoteControlCommands>
	(
		TEXT("RemoteControl"),
		LOCTEXT("RemoteControl", "Remote Control API"),
		NAME_None,
		FRemoteControlPanelStyle::GetStyleSetName()
	)
{
}

void FRemoteControlCommands::RegisterCommands()
{
	// Save Preset
	UI_COMMAND(SavePreset, 
		"Save", "Saves this preset",
		EUserInterfaceActionType::Button,
		FInputChord(EModifierKey::Control, EKeys::S));

	// Protocol Mode
	UI_COMMAND(ActivateProtocolsMode,
		"Protocols", "View list of protocols mapped to active selection.",
		EUserInterfaceActionType::RadioButton,
		FInputChord());

	// Entity Details Mode
	UI_COMMAND(ActivateDetailsMode,
		"Details", "Activate the Entity Details Panel Mode",
		EUserInterfaceActionType::RadioButton,
		FInputChord());

	// Logic Mode
	UI_COMMAND(ActivateLogicMode,
		"Logic", "View the logic applied to active selection.",
		EUserInterfaceActionType::RadioButton,
		FInputChord());

	// Signature Mode
	UI_COMMAND(ActivateSignatureMode,
		"Signature",
		"View the current signatures available",
		EUserInterfaceActionType::RadioButton,
		FInputChord());

	// Output Log Mode
	UI_COMMAND(ActivateOutputLogMode,
		"Output Log",
		"View the output log",
		EUserInterfaceActionType::RadioButton,
		FInputChord());

	// Delete Entity
	UI_COMMAND(DeleteEntity,
		"Delete",
		"Delete the selected group/exposed entity from the list.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::Delete));

	// Rename Entity
	UI_COMMAND(RenameEntity,
		"Rename", "Rename the selected group/exposed entity.",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::F2));

	// Modify Entity Prop Id
	UI_COMMAND(ChangePropId,
		"Change Property ID", "Change the selected property Ids",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::F3, EModifierKey::Control));

	// Copy Item
	UI_COMMAND(CopyItem,
		"Copy",
		"Copy the selected UI item",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::C, EModifierKey::Control));

	// Paste Item
	UI_COMMAND(PasteItem,
		"Paste", "Paste the selected UI item",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::V, EModifierKey::Control));

	// Duplicate Item
	UI_COMMAND(DuplicateItem,
		"Duplicate",
		"Duplicate the selected UI item",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::D, EModifierKey::Control));

	// Update Value
	UI_COMMAND(UpdateValue,
		"Update Value",
		"Update the selected UI item value with the one in the fields list",
		EUserInterfaceActionType::Button,
		FInputChord(EKeys::U, EModifierKey::Control));

	// Select Source
	UI_COMMAND(SelectSource,
		"Select Source",
		"Selects the source of the bound value",
		EUserInterfaceActionType::Button,
		FInputChord());

	// Protocols generate property changed events
	UI_COMMAND(ProtocolsGeneratePropertyChangeEvents,
		"Protocols Generate Property Change Events",
		"When checked, protocols generate Property change events whenever they change a property.\nEnabling this option may impact performance significantly, but may be required for certain editor controls.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());

	// Protocols generate transactions
	UI_COMMAND(ProtocolsGenerateTransactions,
		"Protocols Generate Transactions",
		"When checked, protocols generate Undo Redo events whenever they change a property.\nEnabling this option is not recommended as protocols will raise many transactions and impact performance significantly.",
		EUserInterfaceActionType::ToggleButton,
		FInputChord());
}

#undef LOCTEXT_NAMESPACE
