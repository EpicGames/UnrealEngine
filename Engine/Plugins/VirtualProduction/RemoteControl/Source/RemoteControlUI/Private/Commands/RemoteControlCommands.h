// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Commands/Commands.h"

/**
 * Defines commands for the Remote Control API which enables most functionality of the Remote Control Panel.
 */
class FRemoteControlCommands : public TCommands<FRemoteControlCommands>
{
public:

	FRemoteControlCommands();

	//~ BEGIN : TCommands<> Implementation(s)

	virtual void RegisterCommands() override;

	//~ END : TCommands<> Implementation(s)

	/**
	 * Holds the information about UI Command that saves the actively edited preset.
	 */
	TSharedPtr<FUICommandInfo> SavePreset;

	/**
	 * Holds the information about UI Command that brings up a panel which holds the active protocol mappings.
	 */
	TSharedPtr<FUICommandInfo> ActivateProtocolsMode;

	/**
	 * Holds the information about UI Command that brings up the Entity Details UI panel.
	 */
	TSharedPtr<FUICommandInfo> ActivateDetailsMode;

	/**
	 * Holds the information about UI Command that brings up a panel which enables the RC Logical Behaviour.
	 */
	TSharedPtr<FUICommandInfo> ActivateLogicMode;

	/**
	 * Holds the information about UI Command that brings up the RC Signature UI panel.
	 */
	TSharedPtr<FUICommandInfo> ActivateSignatureMode;

	/**
	 * Holds the information about UI Command that brings up the Output panel.
	 */
	TSharedPtr<FUICommandInfo> ActivateOutputLogMode;

	/**
	 * Holds the information about UI Command that deletes currently selected group/exposed entity.
	 */
	TSharedPtr<FUICommandInfo> DeleteEntity;

	/**
	 * Holds the information about UI Command that renames selected group/exposed entity.
	 */
	TSharedPtr<FUICommandInfo> RenameEntity;

	/**
	 * Holds the information about UI Command that change the selected property Ids.
	 */
	TSharedPtr<FUICommandInfo> ChangePropId;

	/**
	 * UI Command for copying a UI item in the Remote Control preset. Currently used for Logic panel
	 */
	TSharedPtr<FUICommandInfo> CopyItem;

	/**
	 * UI Command for pasting a UI item in the Remote Control preset. Currently used for Logic panel
	 */
	TSharedPtr<FUICommandInfo> PasteItem;

	/**
	 * UI Command for duplicating a UI item in the Remote Control preset. Currently used for Logic panel
	 */
	TSharedPtr<FUICommandInfo> DuplicateItem;

	/**
	 * UI Command for updating the action in the action list with the value in the field list. Currently used for Logic panel
	 */
	TSharedPtr<FUICommandInfo> UpdateValue;

	/**
	 * UI Command for selecting the source of a property in an action. Currently used for Logic panel
	 */
	TSharedPtr<FUICommandInfo> SelectSource;

	/**
	 * UI Command to Toggle if protocols generate property changed events
	 */
	TSharedPtr<FUICommandInfo> ProtocolsGeneratePropertyChangeEvents;

	/**
	 * UI Command to Toggle if protocols generate transactions
	 */
	TSharedPtr<FUICommandInfo> ProtocolsGenerateTransactions;
};
