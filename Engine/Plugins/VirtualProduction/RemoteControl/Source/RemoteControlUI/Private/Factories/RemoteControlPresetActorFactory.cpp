// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlPresetActorFactory.h"

#include "AssetRegistry/AssetData.h"
#include "Engine/Level.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPreset.h"
#include "RemoteControlPresetActor.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/Notifications/SNotificationList.h"

URemoteControlPresetActorFactory::URemoteControlPresetActorFactory()
{
	DisplayName = NSLOCTEXT("RemoteControlPresetActorFactory", "RemoteControlPresetDisplayName", "RemoteControlPreset");
	NewActorClass = ARemoteControlPresetActor::StaticClass();
}

bool URemoteControlPresetActorFactory::CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg )
{
	if (AssetData.IsValid() && !AssetData.GetClass()->IsChildOf(URemoteControlPreset::StaticClass()))
	{
		OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoRemoteControlPresetAsset", "A valid remote control preset asset must be specified.");
		IRemoteControlModule::BroadcastError(OutErrorMsg.ToString());
		return false;
	}

	return true;
}

AActor* URemoteControlPresetActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	ARemoteControlPresetActor* NewActor = Cast<ARemoteControlPresetActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));

	if (NewActor)
	{
		if (URemoteControlPreset* Preset = Cast<URemoteControlPreset>(InAsset))
		{
			NewActor->Preset = Preset;
		}
	}

	return NewActor;
}

UObject* URemoteControlPresetActorFactory::GetAssetFromActorInstance(AActor* ActorInstance)
{
	if (ARemoteControlPresetActor* RemoteControlPresetActor = Cast<ARemoteControlPresetActor>(ActorInstance))
	{
		return RemoteControlPresetActor->Preset;
	}
	return nullptr;
}
