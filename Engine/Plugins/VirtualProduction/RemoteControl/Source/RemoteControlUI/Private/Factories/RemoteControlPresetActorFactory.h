// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "ActorFactories/ActorFactory.h"
#include "RemoteControlPresetActorFactory.generated.h"

class AActor;
struct FActorSpawnParameters;
struct FAssetData;
class ULevel;
class SNotificationItem;

UCLASS()
class URemoteControlPresetActorFactory : public UActorFactory
{
	GENERATED_BODY()

public:
	URemoteControlPresetActorFactory();

	//~ Begin UActorFactory Interface
	virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	virtual bool CanCreateActorFrom( const FAssetData& AssetData, FText& OutErrorMsg ) override;
	virtual UObject* GetAssetFromActorInstance(AActor* ActorInstance) override;
	//~ End UActorFactory Interface

private:
	TWeakPtr<SNotificationItem> ActiveNotification;
};