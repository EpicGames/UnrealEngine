// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/RemoteControlPresetFactory.h"

#include "AssetToolsModule.h"
#include "EngineAnalytics.h"
#include "IAssetTools.h"
#include "RemoteControlPreset.h"

#define LOCTEXT_NAMESPACE "RemoteControlPresetFactory"

URemoteControlPresetFactory::URemoteControlPresetFactory()
{
	bCreateNew = true;
	bEditAfterNew = true;
	SupportedClass = URemoteControlPreset::StaticClass();
}

UObject* URemoteControlPresetFactory::FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn)
{
	if (FEngineAnalytics::IsAvailable())
	{
		FEngineAnalytics::GetProvider().RecordEvent(TEXT("RemoteControl.CreateRemoteControlPreset"));	
	}
	return NewObject<URemoteControlPreset>(InParent, Name, Flags);
}

bool URemoteControlPresetFactory::ShouldShowInNewMenu() const
{
	return true;
}

uint32 URemoteControlPresetFactory::GetMenuCategories() const
{
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools").Get();
	return AssetTools.FindAdvancedAssetCategory("VirtualProduction");
}

#undef LOCTEXT_NAMESPACE
