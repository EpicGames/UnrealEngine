// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "RemoteControlPresetFactory.generated.h"

/**
 * Implements a factory for URemoteControlPreset objects.
 */
UCLASS(BlueprintType, hidecategories=Object)
class URemoteControlPresetFactory
	: public UFactory
{
	GENERATED_BODY()
public:
	URemoteControlPresetFactory();

	//~ Begin UFactory Interface
	virtual UObject* FactoryCreateNew(UClass* Class, UObject* InParent, FName Name, EObjectFlags Flags, UObject* Context, FFeedbackContext* Warn) override;
	virtual bool ShouldShowInNewMenu() const override;
	virtual uint32 GetMenuCategories() const override;
	//~ End UFactory Interface
};
