// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlUIModule.h"

#include "AssetEditor/RemoteControlPresetEditorToolkit.h"
#include "AssetToolsModule.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Commands/RemoteControlCommands.h"
#include "Components/SkeletalMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "DetailRowMenuContext.h"
#include "DetailTreeNode.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/StaticMesh.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IRemoteControlModule.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "LevelEditorSubsystem.h"
#include "MaterialDomain.h"
#include "MaterialEditor/DEditorParameterValue.h"
#include "Materials/Material.h"
#include "PropertyHandle.h"
#include "RemoteControlActor.h"
#include "RemoteControlField.h"
#include "RemoteControlInstanceMaterial.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Behaviour/Builtin/Conditional/SRCBehaviourConditional.h"
#include "UI/Behaviour/SRCBehaviourPanel.h"
#include "UI/Controller/CustomControllers/SCustomStatelessEventControllerWidget.h"
#include "UI/Controller/CustomControllers/SCustomStringListControllerWidget.h"
#include "UI/Controller/CustomControllers/SCustomTextureControllerWidget.h"
#include "UI/Customizations/FPassphraseCustomization.h"
#include "UI/Customizations/NetworkAddressCustomization.h"
#include "UI/Customizations/RCAssetPathElementCustomization.h"
#include "UI/Customizations/RemoteControlEntityCustomization.h"
#include "UI/IRCPanelExposedEntitiesGroupWidgetFactory.h"
#include "UI/IRCPanelExposedEntitiesListSettingsForProtocol.h"
#include "UI/IRCPanelExposedEntityWidgetFactory.h"
#include "UI/RemoteControlExposeMenuStyle.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelExposedActor.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRCPanelExposedField.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"

#define LOCTEXT_NAMESPACE "RemoteControlUI"

const FName FRemoteControlUIModule::RemoteControlPanelTabName = "RemoteControl_RemoteControlPanel";
const FString IRemoteControlUIModule::SettingsIniSection = TEXT("RemoteControl");

static const FName DetailsTabIdentifiers_LevelEditor[] = {
	"LevelEditorSelectionDetails",
	"LevelEditorSelectionDetails2",
	"LevelEditorSelectionDetails3",
	"LevelEditorSelectionDetails4"
};

static const int32 DetailsTabIdentifiers_LevelEditor_Count = 4;

namespace UE::RemoteControlUI::Private
{
	static const FLazyName DisplayClusterConfigDataClassName = "DisplayClusterConfigurationData";
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs()
	: Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs(const FOnGenerateGlobalRowExtensionArgs& InExtensionArgs)
	: PropertyHandle(InExtensionArgs.PropertyHandle)
	, OwnerObject(InExtensionArgs.OwnerObject)
	, PropertyPath(InExtensionArgs.PropertyPath)
	, Property(InExtensionArgs.Property)
	, Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs(FOnGenerateGlobalRowExtensionArgs&& InExtensionArgs)
	: PropertyHandle(InExtensionArgs.PropertyHandle)
	, OwnerObject(InExtensionArgs.OwnerObject)
	, PropertyPath(InExtensionArgs.PropertyPath)
	, Property(InExtensionArgs.Property)
	, Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs(TSharedPtr<IPropertyHandle>& InPropertyHandle)
	: PropertyHandle(InPropertyHandle)
	, Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs(const TSharedPtr<IPropertyHandle>& InPropertyHandle)
	: PropertyHandle(InPropertyHandle)
	, Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::FRCExposesPropertyArgs(UObject* InOwnerObject, const FString& InPropertyPath, FProperty* InProperty)
	: OwnerObject(InOwnerObject)
	, PropertyPath(InPropertyPath)
	, Property(InProperty)
	, Id(FGuid::NewGuid())
{
}

FRCExposesPropertyArgs::EType FRCExposesPropertyArgs::GetType() const
{
	if (PropertyHandle.IsValid())
	{
		return EType::E_Handle;
	}
	if (OwnerObject.IsValid() && !PropertyPath.IsEmpty() && Property.IsValid())
	{
		return EType::E_OwnerObject;
	}

	return EType::E_None;
}

bool FRCExposesPropertyArgs::IsValid() const
{
	const EType Type = GetType();
	return Type == EType::E_Handle || Type == EType::E_OwnerObject;
}


FProperty* FRCExposesPropertyArgs::GetProperty() const
{
	const EType Type = GetType();
	if (Type == EType::E_Handle)
	{
		return PropertyHandle->GetProperty();
	}
	return Property.Get();
}

FProperty* FRCExposesPropertyArgs::GetPropertyChecked() const
{
	FProperty* PropertyChecked = GetProperty();
	check(PropertyChecked);
	return PropertyChecked;
}

namespace RemoteControlUIModule
{
	const TArray<FName>& GetCustomizedStructNames()
	{
		static TArray<FName> CustomizedStructNames =
		{
			FRemoteControlProperty::StaticStruct()->GetFName(),
			FRemoteControlFunction::StaticStruct()->GetFName(),
			FRemoteControlActor::StaticStruct()->GetFName(),
		};

		return CustomizedStructNames;
	};

	static void OnOpenRemoteControlPanel(const EToolkitMode::Type Mode, const TSharedPtr<class IToolkitHost>& EditWithinLevelEditor, class URemoteControlPreset* Preset)
	{
		TSharedRef<FRemoteControlPresetEditorToolkit> Toolkit = FRemoteControlPresetEditorToolkit::CreateEditor(Mode, EditWithinLevelEditor,  Preset);
		Toolkit->InitRemoteControlPresetEditor(Mode, EditWithinLevelEditor, Preset);
	}

	int32 GetNumStaticMaterials(UObject* InOuterObject)
	{
		if (UStaticMesh* StaticMesh = Cast<UStaticMesh>(InOuterObject))
		{
			return StaticMesh->GetStaticMaterials().Num();
		}
		if (USkeletalMesh* SkeletalMesh = Cast<USkeletalMesh>(InOuterObject))
		{
			return SkeletalMesh->GetMaterials().Num();
		}

		return 0;
	};

	FObjectProperty* GetObjectProperty(const FRCExposesPropertyArgs& InPropertyArgs)
	{
		if (!ensureMsgf(InPropertyArgs.IsValid(), TEXT("Extension Property Args not valid")))
		{
			return nullptr;
		}

		FRCFieldPathInfo RCFieldPathInfo(InPropertyArgs.PropertyPath);
		RCFieldPathInfo.Resolve(InPropertyArgs.OwnerObject.Get());
		FRCFieldResolvedData ResolvedData = RCFieldPathInfo.GetResolvedData();
		return CastField<FObjectProperty>(InPropertyArgs.GetProperty());
	}

	bool IsStaticOrSkeletalMaterial(const FRCExposesPropertyArgs& InPropertyArgs)
	{
		if (!InPropertyArgs.IsValid())
		{
			return false;
		}

		const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

		// Only in case of Owner Object is applicable for MaterialInterface
		if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
		{
			FProperty* Property = InPropertyArgs.GetProperty();

			if (!ensure(Property))
			{
				return false;
			}

			FObjectProperty* ObjectProperty = GetObjectProperty(InPropertyArgs);

			if (!ObjectProperty)
			{
				return false;
			}

			return Property->GetFName() == UMaterialInterface::StaticClass()->GetFName() && (InPropertyArgs.OwnerObject->GetClass()->IsChildOf(UStaticMesh::StaticClass()) || InPropertyArgs.OwnerObject->GetClass()->IsChildOf(USkeletalMesh::StaticClass()));
		}

		return false;
	}


	bool IsTransientObjectAllowListed(UObject* Object)
	{
		if (!Object)
		{
			return false;
		}

		if (Object->IsA<UDEditorParameterValue>())
		{
			return true;
		}

		 // Embedded preset worlds may be part of a transient package.
		if (UWorld* ObjectWorld = Object->GetWorld())
		{
			IRemoteControlModule& RCModule = IRemoteControlModule::Get();
			TArray<TWeakObjectPtr<URemoteControlPreset>> EmbeddedPresets;
			RCModule.GetEmbeddedPresets(EmbeddedPresets);

			for (TWeakObjectPtr<URemoteControlPreset> RCPreset : EmbeddedPresets)
			{
				if (RCPreset.IsValid() && RCPreset->GetEmbeddedWorld() == ObjectWorld)
				{
					return true;
				}
			}
		}

		return false;
	}

	void UpdateDetailViewExtensionWidth(const TSharedPtr<IDetailsView>& InDetailsView, bool bInOnOpen)
	{		
		const float DeltaWidth = bInOnOpen ? 22.0f : -22.0f;
		InDetailsView->SetRightColumnMinWidth(InDetailsView->GetRightColumnMinWidth() + DeltaWidth);
		InDetailsView->ForceRefresh();
	}
}

void FRemoteControlUIModule::StartupModule()
{
	FRemoteControlPanelStyle::Initialize();
	FRemoteControlExposeMenuStyle::Initialize();
	BindRemoteControlCommands();
	RegisterDetailRowExtension();
	RegisterContextMenuExtender();
	RegisterEvents();
	RegisterStructCustomizations();
	RegisterSettings();
	RegisterWidgetFactories();
	RegisterDefaultPropertyResolvers();
	
	if (FAssetToolsModule* AssetToolsModule = FModuleManager::GetModulePtr<FAssetToolsModule>("AssetTools"))
	{
		RemoteControlAssetCategoryBit = AssetToolsModule->Get().RegisterAdvancedAssetCategory("VirtualProduction", LOCTEXT("VirtualProductionCategory", "Virtual Production"));
	}
}

void FRemoteControlUIModule::ShutdownModule()
{
	UnregisterSettings();
	UnregisterStructCustomizations();
	UnregisterEvents();
	UnregisterContextMenuExtender();
	UnregisterDetailRowExtension();
	UnbindRemoteControlCommands();
	FRemoteControlPanelStyle::Shutdown();
	FRemoteControlExposeMenuStyle::Shutdown();
}

FDelegateHandle FRemoteControlUIModule::AddPropertyFilter(FOnDisplayExposeIcon OnDisplayExposeIcon)
{
	FDelegateHandle Handle = OnDisplayExposeIcon.GetHandle();
	ExternalFilterDelegates.Add(Handle, MoveTemp(OnDisplayExposeIcon));
	return Handle;
}

void FRemoteControlUIModule::RemovePropertyFilter(const FDelegateHandle& Handle)
{
	ExternalFilterDelegates.Remove(Handle);
}

void FRemoteControlUIModule::RegisterMetadataCustomization(FName MetadataKey, FOnCustomizeMetadataEntry OnCustomizeCallback)
{
	ExternalEntityMetadataCustomizations.FindOrAdd(MetadataKey) = MoveTemp(OnCustomizeCallback);
}

void FRemoteControlUIModule::UnregisterMetadataCustomization(FName MetadataKey)
{
	ExternalEntityMetadataCustomizations.Remove(MetadataKey);
}

TSharedRef<SRemoteControlPanel> FRemoteControlUIModule::CreateRemoteControlPanel(URemoteControlPreset* Preset, const TSharedPtr<IToolkitHost>& ToolkitHost)
{
	constexpr bool bIsInLiveMode = true;

	if (TSharedPtr<SRemoteControlPanel> Panel = WeakActivePanel.Pin())
	{
		Panel->SetActiveMode(ERCPanelMode::Live);
	}

	TSharedRef<SRemoteControlPanel> PanelRef = SNew(SRemoteControlPanel, Preset, ToolkitHost)
		.OnLiveModeChange_Lambda(
			[this](TSharedPtr<SRemoteControlPanel> InPanel, bool bInLiveMode)
			{
				// Activating the live mode on a panel sets it as the active panel
				if (bInLiveMode)
				{
					if (const TSharedPtr<SRemoteControlPanel> ActivePanel = WeakActivePanel.Pin())
					{
						if (ActivePanel != InPanel)
						{
							ActivePanel->SetActiveMode(ERCPanelMode::Live);
						}
					}
					WeakActivePanel = MoveTemp(InPanel);
				}
			});

	WeakActivePanel = PanelRef;

	RegisteredRemoteControlPanels.Add(TWeakPtr<SRemoteControlPanel>(PanelRef));

	// NOTE : Reregister the module with the detail panel when this panel is created.

	constexpr bool bResolveIfInvalid = true;
	if (TSharedPtr<IDetailsView> DetailsPanel = GetDetailsView(bResolveIfInvalid, Preset->GetDetailsTabIdentifierOverrides()))
	{
		RemoteControlUIModule::UpdateDetailViewExtensionWidth(DetailsPanel, /*bInOnOpen*/ true);
	}

	OnRemoteControlPresetOpened().Broadcast(Preset);

	return PanelRef;
}

void FRemoteControlUIModule::RegisterContextMenuExtender()
{
	if (IsRunningGame())
	{
		return;
	}

	// Extend the level viewport context menu to add an option to copy the object path.
	LevelViewportContextMenuRemoteControlExtender = FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FRemoteControlUIModule::ExtendLevelViewportContextMenuForRemoteControl);
	FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();
	MenuExtenders.Add(LevelViewportContextMenuRemoteControlExtender);
	MenuExtenderDelegateHandle = MenuExtenders.Last().GetHandle();
}

void FRemoteControlUIModule::UnregisterContextMenuExtender()
{
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetAllLevelViewportContextMenuExtenders().RemoveAll(
			[&](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& Delegate)
			{
				return Delegate.GetHandle() == MenuExtenderDelegateHandle;
			});
	}
}

void FRemoteControlUIModule::RegisterDetailRowExtension()
{
	FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FOnGenerateGlobalRowExtension& RowExtensionDelegate = Module.GetGlobalRowExtensionDelegate();
	RowExtensionDelegate.AddRaw(this, &FRemoteControlUIModule::HandleCreatePropertyRowExtension);
}

void FRemoteControlUIModule::UnregisterDetailRowExtension()
{
	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& Module = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		Module.GetGlobalRowExtensionDelegate().RemoveAll(this);
	}
}

void FRemoteControlUIModule::RegisterEvents()
{
	FEditorDelegates::PostUndoRedo.AddRaw(this, &FRemoteControlUIModule::RefreshPanels);
}

void FRemoteControlUIModule::UnregisterEvents()
{
	FEditorDelegates::PostUndoRedo.RemoveAll(this);
}

TSharedPtr<IDetailsView> FRemoteControlUIModule::GetDetailsView(bool bInResolveIfInvalid, TConstArrayView<FName> InDetailsTabIdentifierOverrides) const
{
	if (TSharedPtr<IDetailsView> ExistingDetailsView = DetailsPanelWeak.Pin())
	{
		return ExistingDetailsView;
	}

	if (!bInResolveIfInvalid)
	{
		// Skip resolve if saved details panel is destroyed/null
		return nullptr;
	}

	FPropertyEditorModule& PropertyEditor = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	if (InDetailsTabIdentifierOverrides.IsEmpty())
	{
		InDetailsTabIdentifierOverrides = MakeArrayView(DetailsTabIdentifiers_LevelEditor, DetailsTabIdentifiers_LevelEditor_Count);
	}

	for (FName DetailsTabIdentifier : InDetailsTabIdentifierOverrides)
	{
		if (TSharedPtr<IDetailsView> DetailsPanel = PropertyEditor.FindDetailView(DetailsTabIdentifier))
		{
			DetailsPanelWeak = DetailsPanel;
			return DetailsPanel;
		}
	}

	return nullptr;
}

void FRemoteControlUIModule::ExtendPropertyRowContextMenu() const
{
	UToolMenus* Menus = UToolMenus::Get();

	check(Menus);

	if (UToolMenu* ContextMenu = Menus->FindMenu(UE::PropertyEditor::RowContextMenuName))
	{
		ContextMenu->AddDynamicSection(
			TEXT("FillRemoteControlRowContextSection")
			, FNewToolMenuDelegate::CreateRaw(this, &FRemoteControlUIModule::FillRemoteControlRowContextSection));
	}
}

void FRemoteControlUIModule::FillRemoteControlRowContextSection(UToolMenu* InToolMenu) const
{
	if (!InToolMenu)
	{
		return;
	}

	// For context menu in details view
	const UDetailRowMenuContext* Context = InToolMenu->FindContext<UDetailRowMenuContext>();

	if (!Context || Context->PropertyHandles.IsEmpty())
	{
		return;
	}

	if (!Context->PropertyHandles[0].IsValid())
	{
		return;
	}

	TSharedPtr<IPropertyHandle> PropertyHandleToUse = Context->PropertyHandles[0];

	// If there are multiple handles and the parent property is a struct use that
	if (Context->PropertyHandles.Num() > 1)
	{
		if (const TSharedPtr<IPropertyHandle>& ParentHandle = PropertyHandleToUse->GetParentHandle())
		{
			if (const FProperty* ParentProperty = ParentHandle->GetProperty())
			{
				if (ParentProperty->IsA<FStructProperty>())
				{
					PropertyHandleToUse = ParentHandle;
				}
			}
		}
	}

	FRCExposesPropertyArgs ExposesPropertyArgs(PropertyHandleToUse);

	if (!ExposesPropertyArgs.IsValid())
	{
		return;
	}

	static const FName RemoteControlSectionName("ContextRemoteControlActions");

	if (!GetActivePreset())
	{
		return;
	}

	FToolMenuSection* RemoteControlSection = nullptr;
	RemoteControlSection = InToolMenu->FindSection(RemoteControlSectionName);
	if (!RemoteControlSection)
	{
		RemoteControlSection = &InToolMenu->AddSection(RemoteControlSectionName
			, LOCTEXT("ContextRemoteControlActions", "Remote Control Actions")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}

	// Unexpose/Expose entry for the Menu
	RemoteControlSection->AddMenuEntry(
		TEXT("RemoteControlExposeUnexposeEntry")
		, TAttribute<FText>::CreateRaw(this, &FRemoteControlUIModule::GetPropertyActionText, ExposesPropertyArgs)
		, TAttribute<FText>::CreateRaw(this, &FRemoteControlUIModule::GetPropertyActionTooltip, ExposesPropertyArgs)
		, TAttribute<FSlateIcon>::CreateRaw(this, &FRemoteControlUIModule::OnGetPropertyActionIcon, ExposesPropertyArgs)
		, FUIAction(
			FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::ExecutePropertyAction, ExposesPropertyArgs),
			FCanExecuteAction::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, ExposesPropertyArgs),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, ExposesPropertyArgs)
			)
		);

	// Unexpose/Expose SubProperty SubMenu for the Menu
	if (ExposesPropertyArgs.PropertyHandle.IsValid() && HasChildProperties(ExposesPropertyArgs.PropertyHandle->GetProperty()))
	{
		RemoteControlSection->AddSubMenu(
			TEXT("ExposeUnexposeSubPropertyMenu")
			, LOCTEXT("ExposeUnexposeSubPropertyMenu_Label", "Toggle Sub Property of this field")
			, LOCTEXT("ExposeUnexposeSubPropertyMenu_Tooltip", "Let you toggle Sub Property of the given field if any")
			, FNewToolMenuDelegate::CreateRaw(this, &FRemoteControlUIModule::GetSubPropertySubMenu, ExposesPropertyArgs));
	}
}

void FRemoteControlUIModule::GetSubPropertySubMenu(UToolMenu* InToolMenu, FRCExposesPropertyArgs InExposesPropertyArgs) const
{
	if (!InToolMenu || !InExposesPropertyArgs.IsValid())
	{
		return;
	}
	TArray<FRCExposesAllPropertiesArgs> ExposeAllArgs;
	GetAllExposableSubPropertyFromStruct(InExposesPropertyArgs, ExposeAllArgs);

	FToolMenuSection* RemoteControlExposeUnexposeAllSubMenuSection = nullptr;
	const FName RemoteControlExposeUnexposeAllSubMenuSectionName(InExposesPropertyArgs.GetProperty()->GetDisplayNameText().ToString());
	RemoteControlExposeUnexposeAllSubMenuSection = InToolMenu->FindSection(RemoteControlExposeUnexposeAllSubMenuSectionName);
	if (!RemoteControlExposeUnexposeAllSubMenuSection)
	{
		RemoteControlExposeUnexposeAllSubMenuSection = &InToolMenu->AddSection(RemoteControlExposeUnexposeAllSubMenuSectionName
			, InExposesPropertyArgs.GetProperty()->GetDisplayNameText()
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}
	
	// Expose and Unexpose ALL section
	RemoteControlExposeUnexposeAllSubMenuSection->AddMenuEntry(
		TEXT("SubMenuExposeAllEntry")
		, LOCTEXT("RCExposeAll", "Expose All")
		, LOCTEXT("RCExposeAllTooltip", "Expose all sub-properties")
		, FSlateIcon()
		, FUIAction(FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnExposeAll, ExposeAllArgs))
		, EUserInterfaceActionType::CollapsedButton);

	RemoteControlExposeUnexposeAllSubMenuSection->AddMenuEntry(
	TEXT("SubMenuUnexposeAllEntry")
		, LOCTEXT("RCUnexposeAll", "Unexpose All")
		, LOCTEXT("RCUnexposeAllToolTip", "Unexpose all sub-properties")
		, FSlateIcon()
		, FUIAction(FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnUnexposeAll, ExposeAllArgs))
		, EUserInterfaceActionType::CollapsedButton);

	FToolMenuSection* RemoteControlSubMenuSection = nullptr;
	const FName RemoteControlSubMenuSectionName("EXPOSE");
	RemoteControlSubMenuSection = InToolMenu->FindSection(RemoteControlSubMenuSectionName);
	if (!RemoteControlSubMenuSection)
	{
		RemoteControlSubMenuSection = &InToolMenu->AddSection(RemoteControlSubMenuSectionName
			, LOCTEXT("ContextRemoteControlActionsSubProperties", "EXPOSE")
			, FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	}

	// Expose and Unexpose single property section
	for (FRCExposesAllPropertiesArgs PropArgs : ExposeAllArgs)
	{
		const FName SubPropertyName = FName("ExposeUnexposeSingleEntry_" + PropArgs.PropName.ToString());
		RemoteControlSubMenuSection->AddMenuEntry(
			SubPropertyName,
			PropArgs.ExposedPropertyLabel,
			PropArgs.ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnToggleExposeSubProperty, PropArgs.PropertyArgs, PropArgs.DesiredName),
				FCanExecuteAction::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, PropArgs.PropertyArgs),
				FGetActionCheckState::CreateRaw(this, &FRemoteControlUIModule::GetPropertyExposedCheckState, PropArgs.PropertyArgs),
				FIsActionButtonVisible::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, PropArgs.PropertyArgs)),
			EUserInterfaceActionType::ToggleButton);
	}
}

URemoteControlPreset* FRemoteControlUIModule::GetActivePreset() const
{
	if (const TSharedPtr<SRemoteControlPanel> Panel = GetPanelForObject(nullptr))
	{
		return Panel->GetPreset();
	}
	return nullptr;
}

uint32 FRemoteControlUIModule::GetRemoteControlAssetCategory() const
{
	return RemoteControlAssetCategoryBit;
}

void FRemoteControlUIModule::RegisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization)
{
	if (InCustomization.IsValid())
	{
		SignatureCustomizations.AddUnique(InCustomization.ToSharedRef());
	}
}

void FRemoteControlUIModule::UnregisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization)
{
	if (InCustomization.IsValid())
	{
		SignatureCustomizations.Remove(InCustomization.ToSharedRef());
	}
}

void FRemoteControlUIModule::RegisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings)
{
	ExposedEntitiesListSettingsForProtocols.AddUnique(InSettings);
}

void FRemoteControlUIModule::UnregisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings)
{
	ExposedEntitiesListSettingsForProtocols.RemoveSingle(InSettings);
}

void FRemoteControlUIModule::RegisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender)
{
	ExposedEntitiesPanelExtenders.AddUnique(InExtender);
}

void FRemoteControlUIModule::UnregisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender)
{
	ExposedEntitiesPanelExtenders.RemoveSingle(InExtender);
}

void FRemoteControlUIModule::RegisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory)
{
	ExposedEntitiesGroupWidgetFactories.AddUnique(InFactory);
}

void FRemoteControlUIModule::UnregisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory)
{
	ExposedEntitiesGroupWidgetFactories.RemoveSingle(InFactory);
}

void FRemoteControlUIModule::RegisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory)
{
	ExposedEntityWidgetFactories.AddUnique(InFactory);
}

void FRemoteControlUIModule::UnregisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory)
{
	ExposedEntityWidgetFactories.RemoveSingle(InFactory);
}

void FRemoteControlUIModule::BindRemoteControlCommands()
{
	FRemoteControlCommands::Register();
}

void FRemoteControlUIModule::UnbindRemoteControlCommands()
{
	FRemoteControlCommands::Unregister();
}

void FRemoteControlUIModule::HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions)
{
	const FRCExposesPropertyArgs ExposesPropertyArgs(InArgs);

	WeakDetailsTreeNode.Reset();
	if (ExposesPropertyArgs.IsValid())
	{
		// Extend context row menu
		ExtendPropertyRowContextMenu();

		// Expose/Unexpose button.
		FPropertyRowExtensionButton& ExposeButton = OutExtensions.AddDefaulted_GetRef();
		ExposeButton.Icon = TAttribute<FSlateIcon>::CreateRaw(this, &FRemoteControlUIModule::OnGetPropertyActionIcon, ExposesPropertyArgs);
		ExposeButton.Label = TAttribute<FText>::CreateRaw(this, &FRemoteControlUIModule::GetPropertyActionText, ExposesPropertyArgs);
		ExposeButton.ToolTip = TAttribute<FText>::CreateRaw(this, &FRemoteControlUIModule::GetPropertyActionTooltip, ExposesPropertyArgs);
		ExposeButton.UIAction = FUIAction(
			FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::ExecutePropertyAction, ExposesPropertyArgs),
			FCanExecuteAction::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, ExposesPropertyArgs),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, ExposesPropertyArgs)
		);

		// Override material(s) warning.
		FPropertyRowExtensionButton& OverrideMaterialButton = OutExtensions.AddDefaulted_GetRef();
		OverrideMaterialButton.Icon = TAttribute<FSlateIcon>::CreateRaw(this, &FRemoteControlUIModule::OnGetOverrideMaterialsIcon, ExposesPropertyArgs);
		OverrideMaterialButton.Label = LOCTEXT("OverrideMaterial", "Override Material");
		OverrideMaterialButton.ToolTip = LOCTEXT("OverrideMaterialToolTip", "Click to override this material in order to expose this property to Remote Control.");
		OverrideMaterialButton.UIAction = FUIAction(
			FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::TryOverridingMaterials, ExposesPropertyArgs),
			FCanExecuteAction::CreateRaw(this, &FRemoteControlUIModule::IsStaticOrSkeletalMaterialProperty, ExposesPropertyArgs),
			FGetActionCheckState(),
			FIsActionButtonVisible::CreateRaw(this, &FRemoteControlUIModule::IsStaticOrSkeletalMaterialProperty, ExposesPropertyArgs)
		);

		WeakDetailsTreeNode = InArgs.OwnerTreeNode;
	}
}

TSharedPtr<SRemoteControlPanel> FRemoteControlUIModule::GetPanelForObject(const UObject* Object) const
{
	if (Object)
	{
		if (UWorld* OwnerWorld = Object->GetWorld())
		{
			for (const TWeakPtr<SRemoteControlPanel>& RegisteredPanelWeak : RegisteredRemoteControlPanels)
			{
				if (RegisteredPanelWeak.IsValid())
				{
					TSharedPtr<SRemoteControlPanel> RegisteredPanel = RegisteredPanelWeak.Pin();
					const URemoteControlPreset* Preset = RegisteredPanel->GetPreset();

					if (Preset && Preset->IsEmbeddedPreset() && Preset->GetEmbeddedWorld() == OwnerWorld)
					{
						return RegisteredPanel;
					}
				}
			}
		}
	}

	return WeakActivePanel.Pin();
}

TSharedPtr<SRemoteControlPanel> FRemoteControlUIModule::GetPanelForProperty(const FRCExposesPropertyArgs& InPropertyArgs) const
{
	if (!InPropertyArgs.IsValid())
	{
		return nullptr;
	}

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		TArray<UObject*> OuterObjects;
		InPropertyArgs.PropertyHandle->GetOuterObjects(OuterObjects);

		if (OuterObjects.Num() > 0)
		{
			return GetPanelForObject(OuterObjects[0]);
		}
	}
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		return GetPanelForObject(InPropertyArgs.OwnerObject.Get());
	}

	return GetPanelForObject(nullptr);
}

TSharedPtr<SRemoteControlPanel> FRemoteControlUIModule::GetPanelForPropertyChangeEvent(const FPropertyChangedEvent& InPropertyChangeEvent) const
{
	if (InPropertyChangeEvent.GetNumObjectsBeingEdited() > 0)
	{
		return GetPanelForObject(InPropertyChangeEvent.GetObjectBeingEdited(0));
	}

	return GetPanelForObject(nullptr);
}

FSlateIcon FRemoteControlUIModule::OnGetPropertyActionIcon(const FRCExposesPropertyArgs InPropertyArgs) const
{
	FName BrushName("RemoteControlExposeMenu.NoBrush");

	TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs);

	if (Panel.IsValid() && Panel->GetPreset())
	{
		if (Panel->IsModeActive(ERCPanelMode::Signature))
		{
			return FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("BlueprintEditor.AddNewFunction"));
		}

		if (Panel->IsExposed(InPropertyArgs))
		{
			BrushName = HasChildPropertiesExposed(InPropertyArgs) ? "RemoteControlExposeMenu.VisibleAndVisibleChildren" : "RemoteControlExposeMenu.Visible";
		}
		else
		{
			BrushName = HasChildPropertiesExposed(InPropertyArgs) ? "RemoteControlExposeMenu.HiddenAndVisibleChildren" : "RemoteControlExposeMenu.Hidden";
		}
	}

	return FSlateIcon(FRemoteControlExposeMenuStyle::GetStyleSetName(), BrushName);
}

bool FRemoteControlUIModule::CanExecutePropertyAction(const FRCExposesPropertyArgs InPropertyArgs) const
{
	if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs))
	{
		return InPropertyArgs.IsValid() && ShouldDisplayExposeIcon(InPropertyArgs);
	}

	return false;
}


ECheckBoxState FRemoteControlUIModule::GetPropertyExposedCheckState(const FRCExposesPropertyArgs InPropertyArgs) const
{
	TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs);
	if (Panel.IsValid() && Panel->GetPreset() && Panel->IsExposed(InPropertyArgs))
	{
		return ECheckBoxState::Checked;
	}
	return ECheckBoxState::Unchecked;
}

void FRemoteControlUIModule::ExecutePropertyAction(const FRCExposesPropertyArgs InPropertyArgs) const
{
	if (!ensureMsgf(InPropertyArgs.IsValid(), TEXT("Property could not be exposed because the extension args was invalid.")))
	{
		return;
	}

	TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs);
	if (!Panel.IsValid())
	{
		return;
	}

	if (ShouldCreateSubMenuForChildProperties(Panel, InPropertyArgs))
	{
		CreateSubMenuForChildProperties(InPropertyArgs);
	}
	else
	{
		Panel->ExecutePropertyAction(InPropertyArgs);
	}
}

void FRemoteControlUIModule::OnToggleExposeSubProperty(const FRCExposesPropertyArgs InPropertyArgs, const FString InDesiredName) const
{
	if (!ensureMsgf(InPropertyArgs.IsValid(), TEXT("Property could not be exposed because the extension args was invalid.")))
	{
		return;
	}
	if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs))
	{
		Panel->ExecutePropertyAction(InPropertyArgs, InDesiredName);
	}
}

void FRemoteControlUIModule::OnToggleExposePropertyWithChild(const FRCExposesPropertyArgs InPropertyArgs)
{
	if (!ensureMsgf(InPropertyArgs.IsValid(), TEXT("Property could not be exposed because the extension args was invalid.")))
	{
		return;
	}
	CreateSubMenuForChildProperties(InPropertyArgs);
}

void FRemoteControlUIModule::OnExposeAll(const TArray<FRCExposesAllPropertiesArgs> InExposeAllArgs) const
{
	for (const FRCExposesAllPropertiesArgs& PropInfo : InExposeAllArgs)
	{
		TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(PropInfo.PropertyArgs);
		if (Panel.IsValid() && !Panel->IsExposed(PropInfo.PropertyArgs))
		{
			OnToggleExposeSubProperty(PropInfo.PropertyArgs, PropInfo.DesiredName);
		}
	}
}

void FRemoteControlUIModule::OnUnexposeAll(const TArray<FRCExposesAllPropertiesArgs> InExposeAllArgs) const
{
	for (const FRCExposesAllPropertiesArgs& PropInfo : InExposeAllArgs)
	{
		TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(PropInfo.PropertyArgs);
		if (Panel.IsValid() && Panel->IsExposed(PropInfo.PropertyArgs))
		{
			OnToggleExposeSubProperty(PropInfo.PropertyArgs, PropInfo.DesiredName);
		}
	}
}

FRemoteControlUIModule::EPropertyExposeStatus FRemoteControlUIModule::GetPropertyExposeStatus(const FRCExposesPropertyArgs& InPropertyArgs) const
{
	if (InPropertyArgs.IsValid())
	{
		if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs))
		{
			return Panel->IsExposed(InPropertyArgs) ? EPropertyExposeStatus::Exposed : EPropertyExposeStatus::Unexposed;
		}
	}

	return EPropertyExposeStatus::Unexposable;
}


FSlateIcon FRemoteControlUIModule::OnGetOverrideMaterialsIcon(const FRCExposesPropertyArgs InPropertyArgs) const
{
	FName BrushName("NoBrush");

	if (IsStaticOrSkeletalMaterialProperty(InPropertyArgs))
	{
		BrushName = "Icons.Warning";
	}

	return FSlateIcon(FAppStyle::Get().GetStyleSetName(), BrushName);
}


bool FRemoteControlUIModule::IsStaticOrSkeletalMaterialProperty(const FRCExposesPropertyArgs InPropertyArgs) const
{
	if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs)) // Check whether the panel is active.
	{
		if (Panel->GetPreset() && InPropertyArgs.IsValid()) // Ensure that we have a valid preset and handle.
		{
			return RemoteControlUIModule::IsStaticOrSkeletalMaterial(InPropertyArgs);
		}
	}

	return false;
}


void FRemoteControlUIModule::AddGetPathOption(FMenuBuilder& MenuBuilder, AActor* SelectedActor)
{
	auto CopyLambda = [SelectedActor]()
	{
		if (SelectedActor)
		{
			FPlatformApplicationMisc::ClipboardCopy(*(SelectedActor->GetPathName()));
		}
	};

	FUIAction CopyObjectPathAction(FExecuteAction::CreateLambda(MoveTemp(CopyLambda)));
	MenuBuilder.BeginSection("RemoteControl", LOCTEXT("RemoteControlHeading", "Remote Control"));
	MenuBuilder.AddMenuEntry(LOCTEXT("CopyObjectPath", "Copy path"), LOCTEXT("CopyObjectPath_Tooltip", "Copy the actor's path."), FSlateIcon(FCoreStyle::Get().GetStyleSetName(), "GenericCommands.Copy"), CopyObjectPathAction);

	MenuBuilder.EndSection();
}

TSharedRef<FExtender> FRemoteControlUIModule::ExtendLevelViewportContextMenuForRemoteControl(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors)
{
	TSharedPtr<FExtender> Extender = MakeShared<FExtender>();

	if (SelectedActors.Num() == 1)
	{
		Extender->AddMenuExtension("ActorTypeTools", EExtensionHook::After, CommandList,
			FMenuExtensionDelegate::CreateRaw(this, &FRemoteControlUIModule::AddGetPathOption, SelectedActors[0]));
	}

	return Extender.ToSharedRef();
}

bool FRemoteControlUIModule::ShouldDisplayExposeIcon(const FRCExposesPropertyArgs& InPropertyArgs) const
{
	if (!InPropertyArgs.IsValid())
	{
		return false;
	}


	if (RemoteControlUIModule::IsStaticOrSkeletalMaterial(InPropertyArgs))
	{
		return false;
	}

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyArgs.PropertyHandle;

		if (PropertyHandle->GetNumOuterObjects() == 1)
		{
			TArray<UObject*> OuterObjects;
			PropertyHandle->GetOuterObjects(OuterObjects);

			if (!IsAllowedOwnerObjects(OuterObjects))
			{
				return false;
			}
		}
	}
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		if (!IsAllowedOwnerObjects({ InPropertyArgs.OwnerObject.Get()}))
		{
			return false;
		}
	}


	for (const TPair<FDelegateHandle, FOnDisplayExposeIcon>& DelegatePair : ExternalFilterDelegates)
	{
		if (DelegatePair.Value.IsBound())
		{
			if (!DelegatePair.Value.Execute(InPropertyArgs))
			{
				return false;
			}
		}
	}


	return true;
}

bool FRemoteControlUIModule::ShouldSkipOwnPanelProperty(const FRCExposesPropertyArgs& InPropertyArgs) const
{
	if (!InPropertyArgs.IsValid())
	{
		return false;
	}

	auto CheckOwnProperty = [](FProperty* InProp)
	{
		if (!InProp)
		{
			return false;
		}

		// Don't display an expose icon for RCEntities since they're only displayed in the Remote Control Panel.
		if (InProp->GetOwnerStruct() && InProp->GetOwnerStruct()->IsChildOf(FRemoteControlEntity::StaticStruct()))
		{
			return true;
		}

		return false;
	};

	const FRCExposesPropertyArgs::EType ArgsType = InPropertyArgs.GetType();

	if (ArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = InPropertyArgs.PropertyHandle)
		{
			CheckOwnProperty(PropertyHandle->GetProperty());
		}
	}
	else if (ArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		return CheckOwnProperty(InPropertyArgs.Property.Get());
	}

	return false;
}

bool FRemoteControlUIModule::IsAllowedOwnerObjects(TArray<UObject*> InOuterObjects) const
{
	if (InOuterObjects[0])
	{
		// Don't display an expose icon for default objects.
		if (InOuterObjects[0]->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
		{
			return false;
		}

		// Don't display an expose icon for transient objects such as material editor parameters.
		if (InOuterObjects[0]->GetOutermost()->HasAnyFlags(RF_Transient) && !RemoteControlUIModule::IsTransientObjectAllowListed(InOuterObjects[0]))
		{
			return false;
		}

		/** Disallow exposing properties that come from RC behavior nodes */
		if (InOuterObjects[0]->IsA<URCBehaviourNode>() || InOuterObjects[0]->GetTypedOuter<URCBehaviourNode>())
		{
			return false;
		}
	}

	return true;
}


void FRemoteControlUIModule::RegisterStructCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (FName Name : RemoteControlUIModule::GetCustomizedStructNames())
	{
		PropertyEditorModule.RegisterCustomClassLayout(Name, FOnGetDetailCustomizationInstance::CreateStatic(&FRemoteControlEntityCustomization::MakeInstance));
	}

	PropertyEditorModule.RegisterCustomPropertyTypeLayout(FRCAssetPathElement::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FRCAssetPathElementCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(FRCNetworkAddress::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FNetworkAddressCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout(FRCPassphrase::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPassphraseCustomization::MakeInstance));
}

void FRemoteControlUIModule::UnregisterStructCustomizations()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	// Unregister delegates in reverse order.
	for (int8 NameIndex = RemoteControlUIModule::GetCustomizedStructNames().Num() - 1; NameIndex >= 0; NameIndex--)
	{
		PropertyEditorModule.UnregisterCustomClassLayout(RemoteControlUIModule::GetCustomizedStructNames()[NameIndex]);
	}

	if (UObjectInitialized())
	{
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FRCAssetPathElement::StaticStruct()->GetFName());
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FRCAssetPathElementNew::StaticStruct()->GetFName());
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FRCNetworkAddress::StaticStruct()->GetFName());
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout(FRCPassphrase::StaticStruct()->GetFName());
	}
}

void FRemoteControlUIModule::RegisterSettings()
{
	GetMutableDefault<URemoteControlSettings>()->OnSettingChanged().AddRaw(this, &FRemoteControlUIModule::OnSettingsModified);
}

void FRemoteControlUIModule::UnregisterSettings()
{
	if (UObjectInitialized())
	{
		GetMutableDefault<URemoteControlSettings>()->OnSettingChanged().RemoveAll(this);
	}
}

void FRemoteControlUIModule::OnSettingsModified(UObject*, FPropertyChangedEvent& InPropertyChangeEvent)
{
	if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForPropertyChangeEvent(InPropertyChangeEvent))
	{
		if (TSharedPtr<SRCPanelExposedEntitiesList> EntityList = Panel->GetEntityList())
		{
			EntityList->Refresh();
		}
	}
}

void FRemoteControlUIModule::RegisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType, const FOnGenerateRCWidget& OnGenerateRCWidgetDelegate)
{
	if (!GenerateWidgetDelegates.Contains(RemoteControlEntityType))
	{
		GenerateWidgetDelegates.Add(RemoteControlEntityType, OnGenerateRCWidgetDelegate);
	}
}

void FRemoteControlUIModule::UnregisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType)
{
	GenerateWidgetDelegates.Remove(RemoteControlEntityType);
}

void FRemoteControlUIModule::HighlightPropertyInDetailsPanel(const FPropertyPath& Path) const
{
	if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView())
	{
		DetailsView->HighlightProperty(Path);
	}
}

void FRemoteControlUIModule::SelectObjects(const TArray<UObject*>& Objects) const
{
	if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
	{
		FTypedElementListRef ElementHandles = UTypedElementRegistry::GetInstance()->CreateElementList();
		for (UObject* Object : Objects)
		{
			if (UActorComponent* Component = Cast<UActorComponent>(Object))
			{
				if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component))
				{
					ElementHandles->Add(Handle);
				}
			}
			else if (AActor* Actor = Cast<AActor>(Object))
			{
				if (FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor))
				{
					ElementHandles->Add(Handle);
				}
			}
		}

		UTypedElementSelectionSet* SelectionSet = LevelEditorSubsystem->GetSelectionSet();
		SelectionSet->SetSelection(ElementHandles->GetElementHandles(), FTypedElementSelectionOptions());
	}
}

TSharedPtr<SWidget> FRemoteControlUIModule::CreateCustomControllerWidget(URCVirtualPropertyBase* InController, TSharedPtr<IPropertyHandle> InOriginalPropertyHandle) const
{
	TSharedPtr<SWidget> CustomControllerWidget;
	const FString& CustomName = UE::RCCustomControllers::GetCustomControllerTypeName(InController);
	
	if (CustomName == UE::RCCustomControllers::CustomTextureControllerName)
	{
		CustomControllerWidget = SNew(SCustomTextureControllerWidget, Cast<URCController>(InController), InOriginalPropertyHandle);
	}
	else if (CustomName == UE::RCCustomControllers::CustomStringListControllerName)
	{
		CustomControllerWidget = SNew(SCustomStringListControllerWidget, Cast<URCController>(InController), InOriginalPropertyHandle);
	}
	else if (CustomName == UE::RCCustomControllers::CustomStatelessEventControllerName)
	{
		CustomControllerWidget = SNew(SCustomStatelessEventControllerWidget, Cast<URCController>(InController), InOriginalPropertyHandle);
	}

	return CustomControllerWidget;
}

void FRemoteControlUIModule::RegisterPropertyResolver(FName InResolverId, const FRCPropertyResolver& InResolver)
{
	PropertyResolvers.FindOrAdd(InResolverId, InResolver);
}

void FRemoteControlUIModule::UnregisterPropertyResolver(FName InResolverId)
{
	PropertyResolvers.Remove(InResolverId);
}

void FRemoteControlUIModule::RegisterDefaultPropertyResolvers()
{
	using namespace UE::RemoteControlUI::Private;

	PropertyResolvers.Add(
		DisplayClusterConfigDataClassName,
		FRCPropertyResolver::CreateStatic(&FRemoteControlUIModule::ResolveDisplayClusterConfigurationDataProperty)
	);
}

bool FRemoteControlUIModule::ResolveDisplayClusterConfigurationDataProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects, TSharedPtr<FPropertyPath>& OutPropertyPath)
{
	for (auto It = InOutBoundObjects.CreateIterator(); It; ++It)
	{
		const UObject* BindingObject = *It;

		AActor* OwnerActor = BindingObject->GetTypedOuter<AActor>();

		if (!OwnerActor)
		{
			continue;
		}

		// --- Special NDisplay handling because of their customization ---
		// Since display cluster config data is not created as a default subobject, therefore we have no way of retrieving the CurrentConfigData property from the config object itself.
		using namespace UE::RemoteControlUI::Private;

		if (BindingObject->GetClass()->GetFName() != DisplayClusterConfigDataClassName)
		{
			continue;
		}

		FProperty* Property = OwnerActor->GetClass()->FindPropertyByName("CurrentConfigData");

		if (!Property)
		{
			continue;
		}

		// Append "CurrentConfigData" to the beginning of the path.
		TSharedRef<FPropertyPath> CurrentPropertyPath = InProperty->FieldPathInfo.ToPropertyPath();
		OutPropertyPath = FPropertyPath::Create(Property);

		for (int32 Index = 0; Index < CurrentPropertyPath->GetNumProperties(); Index++)
		{
			OutPropertyPath->AddProperty(CurrentPropertyPath->GetPropertyInfo(Index));
		}

		return true;
	}

	return false;
}

TSharedPtr<FPropertyPath> FRemoteControlUIModule::ResolveProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects)
{
	// Resolve it to ensure the property path.
	InProperty->FieldPathInfo.Resolve(InProperty->GetBoundObject());

	if (!InProperty->FieldPathInfo.IsResolved())
	{
		return nullptr;
	}

	TSharedPtr<FPropertyPath> ResolvedPath;

	for (const TPair<FName, FRCPropertyResolver>& ResolverPair : PropertyResolvers)
	{
		const bool bIsResolved = ResolverPair.Value.Execute(InProperty, InOutBoundObjects, ResolvedPath);

		if (bIsResolved)
		{
			break;
		}
	}

	TArray<UObject*> AdditionalObjects;

	// By default, convert non-actor/actor components to the owning actors.
	for (auto It = InOutBoundObjects.CreateIterator(); It; ++It)
	{
		const UObject* BindingObject = *It;

		AActor* OwnerActor = BindingObject->GetTypedOuter<AActor>();

		if (!OwnerActor)
		{
			continue;
		}

		// When we encounter a non-component object, 
		if (!BindingObject->IsA<UActorComponent>())
		{
			AdditionalObjects.Add(OwnerActor);
			It.RemoveCurrent();
		}
	}

	InOutBoundObjects.Append(AdditionalObjects);

	// And then return the default path resolution
	return InProperty->FieldPathInfo.ToPropertyPath();
}

void FRemoteControlUIModule::SelectSource(TSharedPtr<FRemoteControlEntity> InEntity)
{
	const TSharedPtr<FRemoteControlField> RCField = StaticCastSharedPtr<FRemoteControlField>(InEntity);

	if (!RCField.IsValid() || RCField->GetStruct() != FRemoteControlProperty::StaticStruct())
	{
		return;
	}

	const TSharedPtr<FRemoteControlProperty> RCProp = StaticCastSharedPtr<FRemoteControlProperty>(RCField);

	TArray<UObject*> BoundObjects = RCProp->GetBoundObjects();
	TSharedPtr<FPropertyPath> PropertyPath = ResolveProperty(RCProp.ToSharedRef(), BoundObjects);

	SelectObjects(BoundObjects);

	if (!PropertyPath.IsValid())
	{
		return;
	}

	FTimerHandle Handle;
	const FTimerDelegate Delegate = FTimerDelegate::CreateLambda(([PropertyPath]()
		{
			FRemoteControlUIModule::Get().HighlightPropertyInDetailsPanel(*PropertyPath);
		}));

	// Needed because modifying the selection set is asynchronous.
	GEditor->GetTimerManager()->SetTimer(Handle, Delegate, 0.1f, false, -1);
}

void FRemoteControlUIModule::RegisterWidgetFactories()
{
	RegisterWidgetFactoryForType(FRemoteControlActor::StaticStruct(), FOnGenerateRCWidget::CreateStatic(&SRCPanelExposedActor::MakeInstance));
	RegisterWidgetFactoryForType(FRemoteControlProperty::StaticStruct(), FOnGenerateRCWidget::CreateStatic(&SRCPanelExposedField::MakeInstance));
	RegisterWidgetFactoryForType(FRemoteControlFunction::StaticStruct(), FOnGenerateRCWidget::CreateStatic(&SRCPanelExposedField::MakeInstance));
	RegisterWidgetFactoryForType(FRemoteControlInstanceMaterial::StaticStruct(), FOnGenerateRCWidget::CreateStatic(&SRCPanelExposedField::MakeInstance));
}

FText FRemoteControlUIModule::GetPropertyActionTooltip(const FRCExposesPropertyArgs InPropertyArgs) const
{
	if (const TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs))
	{
		if (const URemoteControlPreset* Preset = Panel->GetPreset())
		{
			const FText PresetName = FText::FromString(Preset->GetName());

			if (Panel->IsModeActive(ERCPanelMode::Signature))
			{
				return FText::Format(LOCTEXT("SignaturePropertyToolTip", "Add this property to the selected or a new signature in RemoteControl Preset '{0}'."), PresetName);
			}

			if (Panel->IsExposed(InPropertyArgs))
			{
				return FText::Format(LOCTEXT("ExposePropertyToolTip", "Unexpose this property from RemoteControl Preset '{0}'."), PresetName);
			}

			return FText::Format(LOCTEXT("UnexposePropertyToolTip", "Expose this property in RemoteControl Preset '{0}'."), PresetName);
		}
	}

	return LOCTEXT("InvalidExposePropertyTooltip", "Invalid Preset");
}

FText FRemoteControlUIModule::GetPropertyActionText(const FRCExposesPropertyArgs InPropertyArgs) const
{
	TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs);
	if (Panel.IsValid() && Panel->GetPreset())
	{
		if (Panel->IsModeActive(ERCPanelMode::Signature))
		{
			return LOCTEXT("AddSignaturePropertyText", "Add to Signature");
		}

		if (Panel->IsExposed(InPropertyArgs))
		{
			return LOCTEXT("ExposePropertyText", "Unexpose property");
		}
		else
		{
			return LOCTEXT("UnexposePropertyText", "Expose property");
		}
	}
	return FText::GetEmpty();
}

void FRemoteControlUIModule::TryOverridingMaterials(const FRCExposesPropertyArgs InPropertyArgs)
{
	if (!ensureMsgf(InPropertyArgs.IsValid(), TEXT("Property could not be exposed because the handle was invalid.")))
	{
		return;
	}

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		ensureMsgf(false, TEXT("Override materials can't be done with property handle arguments type"));
	}
	// Override material only if PropertyArgs is OwnerObject
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		FRCFieldPathInfo RCFieldPathInfo(InPropertyArgs.PropertyPath);
		RCFieldPathInfo.Resolve(InPropertyArgs.OwnerObject.Get());
		const FRCFieldResolvedData ResolvedData = RCFieldPathInfo.GetResolvedData();
		FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InPropertyArgs.Property.Get());

		if (!ObjectProperty)
		{
			return;
		}

		// Material can't be null
		UMaterialInterface* OriginalMaterial = Cast<UMaterialInterface>(ObjectProperty->GetObjectPropertyValue(ResolvedData.ContainerAddress));
		OriginalMaterial = OriginalMaterial ? OriginalMaterial : UMaterial::GetDefaultMaterial(MD_Surface);
		if (OriginalMaterial)
		{
			if (UMeshComponent* MeshComponentToBeModified = GetSelectedMeshComponentToBeModified(InPropertyArgs.OwnerObject.Get(), OriginalMaterial))
			{
				const int32 NumStaticMaterials = RemoteControlUIModule::GetNumStaticMaterials(InPropertyArgs.OwnerObject.Get());
				if (NumStaticMaterials > 0)
				{
					FScopedTransaction Transaction(LOCTEXT("OverrideMaterial", "Override Material"));

					if (FComponentEditorUtils::AttemptApplyMaterialToComponent(MeshComponentToBeModified, OriginalMaterial, NumStaticMaterials - 1))
					{
						RefreshPanels();
					}
				}
			}
		}
	}
}

UMeshComponent* FRemoteControlUIModule::GetSelectedMeshComponentToBeModified(UObject* InOwnerObject, UMaterialInterface* InOriginalMaterial)
{
	// NOTE : Obtain the actor and/or object information from the details panel.

	TArray<TWeakObjectPtr<AActor>> SelectedActors;

	TArray<TWeakObjectPtr<UObject>> SelectedObjects;

	if (TSharedPtr<IDetailTreeNode> OwnerTreeNode = WeakDetailsTreeNode.Pin())
	{
		if (TSharedPtr<IDetailsView> DetailsView = OwnerTreeNode->GetNodeDetailsViewSharedPtr())
		{
			SelectedActors = DetailsView->GetSelectedActors();

			SelectedObjects = DetailsView->GetSelectedObjects();
		}
	}
	else if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView()) // Fallback to global detail panel reference if Detail Tree Node is invalid.
	{
		SelectedActors = DetailsView->GetSelectedActors();
		SelectedObjects = DetailsView->GetSelectedObjects();
	}

	UMeshComponent* MeshComponentToBeModified = nullptr;

	auto GetComponentToBeModified = [InOwnerObject](UMeshComponent* InMeshComponent)
	{
		UMeshComponent* ModifinedComponent = nullptr;

		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(InMeshComponent))
		{
			if (StaticMeshComponent->GetStaticMesh() == InOwnerObject)
			{
				ModifinedComponent = InMeshComponent;
			}
		}
		else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InMeshComponent))
		{
			if (SkeletalMeshComponent->GetSkeletalMeshAsset() == InOwnerObject)
			{
				ModifinedComponent = InMeshComponent;
			}
		}

		return ModifinedComponent;
	};

	if (SelectedActors.Num()) // If user selected actor then get the component from it.
	{
		// NOTE : Allow single selection only.

		if (!ensureMsgf(SelectedActors.Num() == 1, TEXT("Property could not be exposed as multiple actor(s) are selected.")))
		{
			return nullptr;
		}

		for (TWeakObjectPtr<AActor> SelectedActor : SelectedActors)
		{
			TInlineComponentArray<UMeshComponent*> MeshComponents(SelectedActor.Get());

			// NOTE : First mesh component that has the material slot gets served (FCFS approach).

			for (UMeshComponent* MeshComponent : MeshComponents)
			{
				MeshComponentToBeModified = GetComponentToBeModified(MeshComponent);
			}
		}
	}
	else if (SelectedObjects.Num()) // If user selected a component then proceed with it.
	{
		// NOTE : Allow single selection only.
		if (!ensureMsgf(SelectedObjects.Num() == 1, TEXT("Property could not be exposed as multiple component(s) are selected.")))
		{
			return nullptr;
		}

		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(SelectedObject))
			{
				MeshComponentToBeModified = GetComponentToBeModified(MeshComponent);
			}
		}
	}

	return MeshComponentToBeModified;
}


void FRemoteControlUIModule::RefreshPanels()
{
	if (TSharedPtr<IDetailTreeNode> OwnerTreeNode = WeakDetailsTreeNode.Pin())
	{
		if (TSharedPtr<IDetailsView> DetailsView = OwnerTreeNode->GetNodeDetailsViewSharedPtr())
		{
			DetailsView->ForceRefresh();
		}
	}
	else if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView()) // Fallback to global detail panel reference if Detail Tree Node is invalid.
	{
		DetailsView->ForceRefresh();
	}

	if (TSharedPtr<SRemoteControlPanel> Panel = GetPanelForObject(nullptr))
	{
		Panel->Refresh();
		// Propagate that the PropertyId updated during the Undo/Redo to update the action as well.
		if (Panel->GetPreset() && Panel->GetPreset()->GetPropertyIdRegistry())
		{
			Panel->GetPreset()->GetPropertyIdRegistry()->OnPropertyIdUpdated().Broadcast();
		}
	}
}

bool FRemoteControlUIModule::ShouldCreateSubMenuForChildProperties(const TSharedPtr<SRemoteControlPanel>& InPanel, const FRCExposesPropertyArgs InPropertyArgs) const
{
	if (!InPanel->IsModeActive(ERCPanelMode::Signature) && InPropertyArgs.PropertyHandle.IsValid() && FSlateApplication::Get().GetModifierKeys().IsControlDown())
	{
		return HasChildProperties(InPropertyArgs.GetProperty());
	}
	return false;
}

bool FRemoteControlUIModule::HasChildProperties(const FProperty* InProperty) const
{
	if (const FStructProperty* StructProperty = CastField<FStructProperty>(InProperty))
	{
		if (const TObjectPtr<UScriptStruct>& ScriptStruct = StructProperty->Struct)
		{
			// If property contains at least one child, than it is considered valid to expose its child properties
			
			if (const TObjectPtr<UField>& StructChildren = ScriptStruct->Children)
			{
				if (StructChildren->IsValidLowLevel())
				{
					return true;
				}
			}

			if (const FField* ChildProperties = StructProperty->Struct->ChildProperties)
			{
				if (ChildProperties->IsValidLowLevel())
				{
					return true;
				}
			}
		}
	}
	return false;
}

void FRemoteControlUIModule::CreateSubMenuForChildProperties(const FRCExposesPropertyArgs InPropertyArgs) const
{
	FMenuBuilder MenuBuilder(false, nullptr, nullptr, true );
	MenuBuilder.SetStyle(FRemoteControlExposeMenuStyle::Get().Get(), FRemoteControlExposeMenuStyle::GetStyleSetName());

	TArray<FRCExposesAllPropertiesArgs> ExposeAllArgs;
	GetAllExposableSubPropertyFromStruct(InPropertyArgs, ExposeAllArgs);
	MenuBuilder.BeginSection(NAME_None, FText::Format(LOCTEXT("ExposeUnexposeExpansion", "{0}"), InPropertyArgs.GetProperty()->GetDisplayNameText()));

	// Expose and Unexpose ALL section
	MenuBuilder.AddMenuEntry(
	LOCTEXT("RCExposeAll", "Expose All"),
	LOCTEXT("RCExposeAllTooltip", "Expose all sub-properties"),
	FSlateIcon(),
	FUIAction(FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnExposeAll, ExposeAllArgs)),
	NAME_None,
	EUserInterfaceActionType::CollapsedButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCUnexposeAll", "Unexpose All"),
		LOCTEXT("RCUnexposeAllToolTip", "Unexpose all sub-properties"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnUnexposeAll, ExposeAllArgs)),
		NAME_None,
		EUserInterfaceActionType::CollapsedButton);

	MenuBuilder.EndSection();

	// Sub-Properties section
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ExposeUnexposeExpansionSection", "EXPOSE"));

	for (FRCExposesAllPropertiesArgs PropArgs : ExposeAllArgs)
	{
		MenuBuilder.AddMenuEntry(
			PropArgs.ExposedPropertyLabel,
			PropArgs.ToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FRemoteControlUIModule::OnToggleExposeSubProperty, PropArgs.PropertyArgs, PropArgs.DesiredName),
				FCanExecuteAction::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, PropArgs.PropertyArgs),
				FGetActionCheckState::CreateRaw(this, &FRemoteControlUIModule::GetPropertyExposedCheckState, PropArgs.PropertyArgs),
				FIsActionButtonVisible::CreateRaw(this, &FRemoteControlUIModule::CanExecutePropertyAction, PropArgs.PropertyArgs)),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}

	MenuBuilder.EndSection();

	UE::Slate::FDeprecateVector2DParameter MousePos = FSlateApplication::Get().GetCursorPos();
	FWidgetPath WindowPath = FSlateApplication::Get().LocateWindowUnderMouse(MousePos, FSlateApplication::Get().GetInteractiveTopLevelWindows(), false, 0);

	FSlateApplication::Get().PushMenu(WindowPath.GetLastWidget(), WindowPath, MenuBuilder.MakeWidget(), MousePos, FPopupTransitionEffect::ContextMenu);
}

bool FRemoteControlUIModule::HasChildPropertiesExposed(const FRCExposesPropertyArgs& InPropertyArgs) const
{
	TSharedPtr<SRemoteControlPanel> Panel = GetPanelForProperty(InPropertyArgs);

	if (Panel.IsValid() && InPropertyArgs.PropertyHandle.IsValid())
	{
		if (FProperty* Property = InPropertyArgs.PropertyHandle->GetProperty())
		{
			if (Property->IsA<FStructProperty>())
			{
				// Check if it has sub-properties exposed
				const FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(Property);
				int32 ChildHandleIndex = 0;
				for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
				{
					FRCExposesPropertyArgs ChildArgs;
					ChildArgs.OwnerObject = InPropertyArgs.OwnerObject.Get();
					ChildArgs.Property = *It;
					ChildArgs.PropertyPath = (*It)->GetPathName(nullptr);
					ChildArgs.PropertyHandle = InPropertyArgs.PropertyHandle->GetChildHandle(ChildHandleIndex++);

					if (ChildArgs.IsValid() && Panel->IsExposed(ChildArgs))
					{
						return true;
					}
				}
			}
		}
	}
	return false;
}

void FRemoteControlUIModule::GetAllExposableSubPropertyFromStruct(const FRCExposesPropertyArgs InPropertyArgs, TArray<FRCExposesAllPropertiesArgs>& OutAllProperty) const
{
	if (FProperty* PropertyHandleProperty = InPropertyArgs.PropertyHandle->GetProperty())
	{
		FStructProperty* StructProperty = CastFieldChecked<FStructProperty>(PropertyHandleProperty);
		int32 ChildHandleIndex = 0;
		for (TFieldIterator<FProperty> It(StructProperty->Struct); It; ++It)
		{
			FRCExposesPropertyArgs ChildArgs;
			ChildArgs.OwnerObject = InPropertyArgs.OwnerObject.Get();
			ChildArgs.Property = *It;
			ChildArgs.PropertyPath = (*It)->GetPathName(nullptr);
			ChildArgs.PropertyHandle = InPropertyArgs.PropertyHandle->GetChildHandle(ChildHandleIndex++);

			// Skip if not valid
			if (!ChildArgs.IsValid())
			{
				continue;
			}

			// Skip if FColor and FLinearColor since they can decide to show the Alpha channel or not
			if (const FProperty* Property = InPropertyArgs.GetProperty())
			{
				if (Property->HasMetaData("HideAlphaChannel"))
				{
					TWeakFieldPtr<FProperty> ChildProperty = ChildArgs.Property;

					if (ChildProperty.IsValid())
					{
						if (ChildProperty->GetFName() == FName("A"))
						{
							continue;
						}
					}
				}
			}

			FText PropertyName = It->GetDisplayNameText();
			if (StructProperty->GetFName() == USceneComponent::GetRelativeRotationPropertyName() || StructProperty->GetFName() == USceneComponent::GetAbsoluteRotationPropertyName())
			{
				PropertyName = FText::FromString(It->GetName());
			}
			FText ExposedPropertyLabel = FText::Format(LOCTEXT("RC_ChildProperty", "{0}"), PropertyName);
			TAttribute<FText> ToolTip = TAttribute<FText>::CreateLambda([this, ChildArgs, PropertyName]()
			{
				const EPropertyExposeStatus ChildStatus = GetPropertyExposeStatus(ChildArgs);
				switch (ChildStatus)
				{
					case EPropertyExposeStatus::Exposed:
						return FText::Format(LOCTEXT("RCUnexpose", "Unexpose {0}"), PropertyName);

					case EPropertyExposeStatus::Unexposed:
						return FText::Format(LOCTEXT("RCExpose", "Expose {0}"), PropertyName);

					case EPropertyExposeStatus::Unexposable:
				default:
					return FText::Format(LOCTEXT("RCUnexposable", "Property {0} cannot be exposed"), PropertyName);
				}
			});
			OutAllProperty.Add(FRCExposesAllPropertiesArgs({ChildArgs, PropertyName, PropertyName.ToString(), ExposedPropertyLabel, ToolTip}));
		}
	}
}

TSharedPtr<SRCPanelTreeNode> FRemoteControlUIModule::GenerateEntityWidget(const FGenerateWidgetArgs& Args)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRemoteControlUIModule::GenerateEntityWidget);

	if (Args.Preset && Args.Entity)
	{
		const UScriptStruct* EntityType = Args.Preset->GetExposedEntityType(Args.Entity->GetId());

		if (FOnGenerateRCWidget* OnGenerateWidget = GenerateWidgetDelegates.Find(const_cast<UScriptStruct*>(EntityType)))
		{
			return OnGenerateWidget->Execute(Args);
		}
	}

	return nullptr;
}

void FRemoteControlUIModule::UnregisterRemoteControlPanel(SRemoteControlPanel* Panel)
{
	// When unregistering remote control panel, skip this step if the original details view is already destroyed (do not resolve to a new one)
	constexpr bool bResolveIfInvalid = false;
	if (TSharedPtr<IDetailsView> DetailsView = GetDetailsView(bResolveIfInvalid))
	{
		// reset column min width to standard size
		RemoteControlUIModule::UpdateDetailViewExtensionWidth(DetailsView, /*bInOnOpen*/ false);
	}

	if (!Panel)
	{
		return;
	}

	for (int32 PanelIndex = 0; PanelIndex < RegisteredRemoteControlPanels.Num(); ++PanelIndex)
	{
		if (RegisteredRemoteControlPanels[PanelIndex].HasSameObject(Panel))
		{
			RegisteredRemoteControlPanels.RemoveAt(PanelIndex);
			return;
		}
	}
}

const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>* FRemoteControlUIModule::GetExposedEntitiesListSettingsForProtocol(const FName& ProtocolName) const
{
	return Algo::FindByPredicate(ExposedEntitiesListSettingsForProtocols, [&ProtocolName, this](const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& Settings)
		{
			return Settings->GetProtocolName() == ProtocolName;
		});
}

const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>* FRemoteControlUIModule::GetExposedEntitiesGroupWidgetFactory(const FName& ForColumnName, const FName& InActiveProtocol) const
{
	return Algo::FindByPredicate(ExposedEntitiesGroupWidgetFactories, [&ForColumnName, &InActiveProtocol](const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& WidgetFactory)
		{
			if (WidgetFactory->GetColumnName() == ForColumnName)
			{
				const FName ProtocolName = WidgetFactory->GetProtocolName();

				// Customize the column for any protcol if the protocol name is none
				return 
					ProtocolName == InActiveProtocol || 
					ProtocolName == NAME_None;
			}
			
			return false;
		});
}

const TSharedRef<IRCPanelExposedEntityWidgetFactory>* FRemoteControlUIModule::GetExposedEntityWidgetFactory(const FName& ForColumnName, const FName& InActiveProtocol) const
{
	return Algo::FindByPredicate(ExposedEntityWidgetFactories, [&ForColumnName, &InActiveProtocol](const TSharedRef<IRCPanelExposedEntityWidgetFactory>& ExposedEntityCustomization)
		{
			if (ExposedEntityCustomization->GetColumnName() == ForColumnName)
			{
				const FName ProtocolName = ExposedEntityCustomization->GetProtocolName();

				// Customize the column for any protcol if the protocol name is none
				return
					ProtocolName == InActiveProtocol ||
					ProtocolName == NAME_None;
			}

			return false;
		});
}

IMPLEMENT_MODULE(FRemoteControlUIModule, RemoteControlUI);

#undef LOCTEXT_NAMESPACE /*RemoteControlUI*/
