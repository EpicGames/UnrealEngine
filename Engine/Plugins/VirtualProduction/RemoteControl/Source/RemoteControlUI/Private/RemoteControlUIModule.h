// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRemoteControlUIModule.h"

#include "AssetTypeCategories.h"
#include "CoreMinimal.h"
#include "LevelEditor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"

class IRCPanelExposedEntitiesListSettingsForProtocol;
class IToolkitHost;
class SRemoteControlPanel;
class UMeshComponent;
class URemoteControlPreset;
enum class ERCPanelMode : uint8;

/**
 * A Remote Control module that allows exposing objects and properties from the editor.
 */
class FRemoteControlUIModule : public IRemoteControlUIModule
{
public:

	FRemoteControlUIModule()
		: RemoteControlAssetCategoryBit(EAssetTypeCategories::Misc)
	{
	}

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static FRemoteControlUIModule& Get()
	{
		static const FName ModuleName = "RemoteControlUI";
		return FModuleManager::LoadModuleChecked<FRemoteControlUIModule>(ModuleName);
	}

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface

	//~ Begin IRemoteControlUIModule interface
	virtual FOnGenerateExtensions& GetExtensionGenerators() override { return ExtensionsGenerator; }
	virtual FOnRemoteControlPresetOpened& OnRemoteControlPresetOpened() override { return RemoteControlPresetOpenedDelegate; }
	virtual FOnGenerateControllerExtensionsWidgets& OnGenerateControllerExtensionsWidgets() override { return GenerateControllerExtensionsWidgetsDelegate; }
	virtual FOnAddControllerExtensionColumn& OnAddControllerExtensionColumn() override { return OnAddControllerExtensionColumnDelegate; }
	virtual FDelegateHandle AddPropertyFilter(FOnDisplayExposeIcon OnDisplayExposeIcon) override;
	virtual void RemovePropertyFilter(const FDelegateHandle& FilterDelegateHandle) override;
	virtual void RegisterMetadataCustomization(FName MetadataKey, FOnCustomizeMetadataEntry OnCustomizeCallback) override;
	virtual void UnregisterMetadataCustomization(FName MetadataKey) override;
	virtual URemoteControlPreset* GetActivePreset() const override;
	virtual uint32 GetRemoteControlAssetCategory() const override;
	virtual void RegisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization) override;
	virtual void UnregisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization) override;
	virtual void RegisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings) override;
	virtual void UnregisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings) override;
	virtual void RegisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender) override;
	virtual void UnregisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender) override;
	virtual void RegisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory) override;
	virtual void UnregisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory) override;
	virtual void RegisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory) override;
	virtual void UnregisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory) override;
	virtual void RegisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType, const FOnGenerateRCWidget& OnGenerateRCWidgetDelegate) override;
	virtual void UnregisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType) override;
	virtual void HighlightPropertyInDetailsPanel(const FPropertyPath& Path) const override;
	virtual void SelectObjects(const TArray<UObject*>& Objects) const override;
	virtual TSharedPtr<SWidget> CreateCustomControllerWidget(URCVirtualPropertyBase* InController, TSharedPtr<IPropertyHandle> InOriginalPropertyHandle) const override;
	virtual void RegisterPropertyResolver(FName InResolverId, const FRCPropertyResolver& InResolver) override;
	virtual void UnregisterPropertyResolver(FName InResolverId) override;
	//~ End IRemoteControlUIModule interface

	void SelectSource(TSharedPtr<FRemoteControlEntity> InEntity);

	/**
	 * Create a remote control panel to display a given preset.
	 * @param The preset to display.
	 * @return The remote control widget.
	 */
	TSharedRef<SRemoteControlPanel> CreateRemoteControlPanel(URemoteControlPreset* Preset, const TSharedPtr<IToolkitHost>& ToolkitHost = TSharedPtr<IToolkitHost>());

	void UnregisterRemoteControlPanel(SRemoteControlPanel* Panel);

	/**
	 * Get the map of entity metadata entry customizations.
	 */
	const TMap<FName, FOnCustomizeMetadataEntry>& GetEntityMetadataCustomizations() const
	{
		return ExternalEntityMetadataCustomizations;
	}

	TConstArrayView<TSharedRef<IRCSignatureCustomization>> GetSignatureCustomizations() const
	{
		return SignatureCustomizations;
	}

	TConstArrayView<TSharedRef<IRCExposedEntitiesPanelExtender>> GetExposedEntitiesPanelExtenders() const
	{
		return ExposedEntitiesPanelExtenders;
	}

	/** Returns the exposed entity list settings for a protocol, or nullptr if no entities list settings could be found */
	const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>* GetExposedEntitiesListSettingsForProtocol(const FName& ProtocolName) const;

	/** Returns the exposed entities group widget factory, or nullptr if no widget factory is registered for the specified protocol and column name. */
	const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>* GetExposedEntitiesGroupWidgetFactory(const FName& ForColumnName, const FName& InActiveProtocol) const;

	/** Returns the exposed entity widget factory, or nullptr if no widget factory is registered for the specified protocol and column name. */
	const TSharedRef<IRCPanelExposedEntityWidgetFactory>* GetExposedEntityWidgetFactory(const FName& ForColumnName, const FName& InActiveProtocol) const;

public:
	static const FName RemoteControlPanelTabName;

	TSharedPtr<SRCPanelTreeNode> GenerateEntityWidget(const FGenerateWidgetArgs& Args);

private:
	/**
	 * The status of a property.
	 */
	enum class EPropertyExposeStatus : uint8
	{
		Exposed,
		Unexposed,
		Unexposable
	};

private:
	//~ Remote Control Commands
	void BindRemoteControlCommands();
	void UnbindRemoteControlCommands();

	//~ Context menu extenders
	void RegisterContextMenuExtender();
	void UnregisterContextMenuExtender();

	//~ Detail row extensions
	void RegisterDetailRowExtension();
	void UnregisterDetailRowExtension();

	//~ Common Events
	void RegisterEvents();
	void UnregisterEvents();

	/** Retrieves the saved details view. If invalid, optionally resolve by finding a new details panel to store in DetailsPanelWeak and return that */
	TSharedPtr<IDetailsView> GetDetailsView(bool bInResolveIfInvalid=true, TConstArrayView<FName> InDetailsTabIdentifierOverrides = {}) const;

	/** Extend context row menu */
	void ExtendPropertyRowContextMenu() const;

	/** Fills the RemoteControl details view row context menu  */
	void FillRemoteControlRowContextSection(UToolMenu* InToolMenu) const;

	/** Create the SubProperty expose/unexpose sub menu for the details view row context menu */
	void GetSubPropertySubMenu(UToolMenu* InToolMenu, FRCExposesPropertyArgs InExposesPropertyArgs) const;

	/** Handle creating the row extensions.  */
	void HandleCreatePropertyRowExtension(const FOnGenerateGlobalRowExtensionArgs& InArgs, TArray<FPropertyRowExtensionButton>& OutExtensions);

	/*
	 * Returns the Panel widget associated with an property by, in the case of embedded presets, examining the
	 * world that the property's owning object belongs to. In the case of assets, just returns the Panel ref in this module.
	 */
	TSharedPtr<SRemoteControlPanel> GetPanelForObject(const UObject* Object) const;
	TSharedPtr<SRemoteControlPanel> GetPanelForProperty(const FRCExposesPropertyArgs& InPropertyArgs) const;
	TSharedPtr<SRemoteControlPanel> GetPanelForPropertyChangeEvent(const FPropertyChangedEvent& InPropertyChangeEvent) const;

	/** Handle getting the icon displayed in the row extension. */
	FSlateIcon OnGetPropertyActionIcon(const FRCExposesPropertyArgs InArgs) const;

	/** Handle getting the expose button visibility. */
	bool CanExecutePropertyAction(const FRCExposesPropertyArgs InArgs) const;

	/** Is the property currently exposed? */
	ECheckBoxState GetPropertyExposedCheckState(const FRCExposesPropertyArgs InArgs) const;

	/** Handle clicking the expose button. */
	void ExecutePropertyAction(const FRCExposesPropertyArgs InArgs) const;

	/** Handle clicking the expose SubProperty button. */
	void OnToggleExposeSubProperty(const FRCExposesPropertyArgs InArgs, const FString InDesiredName = TEXT("")) const;

	void OnToggleExposePropertyWithChild(const FRCExposesPropertyArgs InArgs);

	/** Store the information to expose/unexpose all properties from the parent one */
	struct FRCExposesAllPropertiesArgs
	{
		FRCExposesPropertyArgs PropertyArgs;
		FText PropName;
		FString DesiredName;
		FText ExposedPropertyLabel;
		TAttribute<FText> ToolTip;
	};
	/** Handle clicking the expose all in the SubProperty menu */
	void OnExposeAll(const TArray<FRCExposesAllPropertiesArgs> InExposeAllArgs) const;

	/** Handle clicking the expose all in the SubProperty menu */
	void OnUnexposeAll(const TArray<FRCExposesAllPropertiesArgs> InExposeAllArgs) const;

	/** Returns whether a property is exposed, unexposed or unexposable. */
	EPropertyExposeStatus GetPropertyExposeStatus(const FRCExposesPropertyArgs& InArgs) const;

	/** Handle getting the icon displayed in the row extension. */
	FSlateIcon OnGetOverrideMaterialsIcon(const FRCExposesPropertyArgs InArgs) const;

	/** Handle getting the override materials button visibility. */
	bool IsStaticOrSkeletalMaterialProperty(const FRCExposesPropertyArgs InArgs) const;

	/** Handle adding an option to get the object path in the actors' context menu. */
	void AddGetPathOption(class FMenuBuilder& MenuBuilder, AActor* SelectedActor);

	/** Handle adding the menu extender for the actors. */
	TSharedRef<FExtender> ExtendLevelViewportContextMenuForRemoteControl(const TSharedRef<FUICommandList> CommandList, TArray<AActor*> SelectedActors);

	/** Returns whether a given extension args should have an exposed icon. */
	bool ShouldDisplayExposeIcon(const FRCExposesPropertyArgs& InArgs) const;

	/** Return true if the extension args from remote control panel */
	bool ShouldSkipOwnPanelProperty(const FRCExposesPropertyArgs& InArgs) const;

	bool IsAllowedOwnerObjects(TArray<UObject*> InOuterObjects) const;

	//~ Handle struct details customizations for common RC types.
	void RegisterStructCustomizations();
	void UnregisterStructCustomizations();

	//~ Handle registering settings and reacting to setting changes.
	void RegisterSettings();
	void UnregisterSettings();
	void OnSettingsModified(UObject*, struct FPropertyChangedEvent&);

	void RegisterWidgetFactories();

	/** Returns expose button tooltip based on exposed state */
	FText GetPropertyActionTooltip(const FRCExposesPropertyArgs InArgs) const;

	/** Returns expose button text based on exposed state */
	FText GetPropertyActionText(const FRCExposesPropertyArgs InArgs) const;

	/** Attempts to replace the static or skeletal materials with their corressponding overrides. */
	void TryOverridingMaterials(const FRCExposesPropertyArgs InArgs);

	/** Return Selected Mesh Component by given OwnerObject and MaterialInterface */
	UMeshComponent* GetSelectedMeshComponentToBeModified(UObject* InOwnerObject,  UMaterialInterface* InOriginalMaterial);

	/** Attempts to refresh the details panel. */
	void RefreshPanels();

	/** Check if a sub menu should be created when ctrl+click on the eyeball icon in the details view */
	bool ShouldCreateSubMenuForChildProperties(const TSharedPtr<SRemoteControlPanel>& InPanel, const FRCExposesPropertyArgs InPropertyArgs) const;

	/** Check if the property has child properties that can be exposed */
	bool HasChildProperties(const FProperty* InProperty) const;

	/** Create a sub menu for each sub property */
	void CreateSubMenuForChildProperties(const FRCExposesPropertyArgs InPropertyArgs) const;

	/** Check if any sub-property is exposed */
	bool HasChildPropertiesExposed(const FRCExposesPropertyArgs& InPropertyArgs) const;

	void GetAllExposableSubPropertyFromStruct(const FRCExposesPropertyArgs InPropertyArgs, TArray<FRCExposesAllPropertiesArgs>& OutAllProperty) const;

	void RegisterDefaultPropertyResolvers();

	static bool ResolveDisplayClusterConfigurationDataProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects, TSharedPtr<FPropertyPath>& OutPropertyPath);

	TSharedPtr<FPropertyPath> ResolveProperty(const TSharedRef<FRemoteControlProperty>& InProperty, TArray<UObject*>& InOutBoundObjects);

private:
	/** Holds the context menu extender. */
	FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors LevelViewportContextMenuRemoteControlExtender;

	/** Holds the menu extender delegate handle. */
	FDelegateHandle MenuExtenderDelegateHandle;

	/** Holds a weak ptr to the active control panel. */
	TWeakPtr<SRemoteControlPanel> WeakActivePanel;

	/** Holds a weak ptr to the owner tree node of the active details panel. */
	TWeakPtr<IDetailTreeNode> WeakDetailsTreeNode;

	/** Holds a weak ptr to the global details panel. */
	mutable TWeakPtr<IDetailsView> DetailsPanelWeak;

	/** Delegate called to gather extensions added externally to the panel. */
	FOnGenerateExtensions ExtensionsGenerator;

	/** Delegate called when a Remote Control Preset panel is created for the specified Remote Control Preset */
	FOnRemoteControlPresetOpened RemoteControlPresetOpenedDelegate;

	/** Called when setting up/resetting the Controller panel list. Register to add custom column. */
	FOnAddControllerExtensionColumn OnAddControllerExtensionColumnDelegate;
	
	/** Delegate called when a Remote Control Preset panel is created/opened for the specified Remote Control Preset */
	FOnGenerateControllerExtensionsWidgets GenerateControllerExtensionsWidgetsDelegate;

	/** Filters added by other plugins queried to determine if a property should display an expose icon. */
	TMap<FDelegateHandle, FOnDisplayExposeIcon> ExternalFilterDelegates;

	/** Map of metadata key to customization handler. */
	TMap<FName, FOnCustomizeMetadataEntry> ExternalEntityMetadataCustomizations;

	/** Registered Signature Customizations */
	TArray<TSharedRef<IRCSignatureCustomization>> SignatureCustomizations; 

	/** Registered Exposed Entities List Settings for Protocols */
	TArray<TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>> ExposedEntitiesListSettingsForProtocols;

	/** Registered Exposed Entities Panel Extenders */
	TArray<TSharedRef<IRCExposedEntitiesPanelExtender>> ExposedEntitiesPanelExtenders;

	/** Registered Exposed Entities Group Widget Factories */
	TArray<TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>> ExposedEntitiesGroupWidgetFactories;

	/** Registered Exposed Entity Widget Factories */
	TArray<TSharedRef<IRCPanelExposedEntityWidgetFactory>> ExposedEntityWidgetFactories;

	TMap<TWeakObjectPtr<UScriptStruct>, FOnGenerateRCWidget> GenerateWidgetDelegates;

	/** Holds the advanced asset type category bit registered by Remote Control. */
	EAssetTypeCategories::Type RemoteControlAssetCategoryBit;

	TArray<TWeakPtr<SRemoteControlPanel>> RegisteredRemoteControlPanels;

	/** List of property sources resolvers. */
	TMap<FName, FRCPropertyResolver> PropertyResolvers;
};
