// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlStyles.h"

FRCPanelStyle::FRCPanelStyle()
	: HeaderRowPadding(4.f, 2.f)
	, HeaderTextStyle(FTextBlockStyle::GetDefault())
	, IconSize(FVector2D(20.f))
	, PanelPadding(0.f)
	, PanelTextStyle(FTextBlockStyle::GetDefault())
	, SectionHeaderTextStyle(FTextBlockStyle::GetDefault())
	, SplitterHandleSize(4.f)
{
}

const FName FRCPanelStyle::TypeName(TEXT("FRemoteControlStyle"));

const FRCPanelStyle& FRCPanelStyle::GetDefault()
{
	static FRCPanelStyle Default;
	return Default;
}

void FRCPanelStyle::GetResources(TArray<const FSlateBrush*>& OutBrushes) const
{
	HeaderRowStyle.GetResources(OutBrushes);
	FlatButtonStyle.GetResources(OutBrushes);
	TableRowStyle.GetResources(OutBrushes);
	TableViewStyle.GetResources(OutBrushes);
	SwitchButtonStyle.GetResources(OutBrushes);
	ToggleButtonStyle.GetResources(OutBrushes);
	OutBrushes.Add(&ContentAreaBrush);
	OutBrushes.Add(&ContentAreaBrushDark);
	OutBrushes.Add(&ContentAreaBrushLight);
	OutBrushes.Add(&SectionHeaderBrush);
}

