// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Layout/Margin.h"

#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Styling/SlateWidgetStyle.h"

#include "RemoteControlStyles.generated.h"

/**
 * Represents the appearance of an RC Panel.
 */
USTRUCT()
struct FRCPanelStyle : public FSlateWidgetStyle
{
	GENERATED_BODY()

	FRCPanelStyle();

	virtual ~FRCPanelStyle() {}

	//~ Begin : FSlateWidgetStyle
	virtual void GetResources(TArray<const FSlateBrush*>& OutBrushes) const override;

	static const FName TypeName;
	virtual const FName GetTypeName() const override { return TypeName; };
	
	static const FRCPanelStyle& GetDefault();
	//~ End : FSlateWidgetStyle

	/** Brush used by the RC Panels to draw the content associated with this panel. */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrush;
	FRCPanelStyle& SetContentAreaBrush(const FSlateBrush& InContentAreaBrush) { ContentAreaBrush = InContentAreaBrush; return *this; }
	
	/** Brush used by the RC Panels to draw the content associated with this panel (Dark). */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrushDark;
	FRCPanelStyle& SetContentAreaBrushDark(const FSlateBrush& InContentAreaBrushDark) { ContentAreaBrushDark = InContentAreaBrushDark; return *this; }
	
	/** Brush used by the RC Panels to draw the content associated with this panel (Light). */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush ContentAreaBrushLight;
	FRCPanelStyle& SetContentAreaBrushLight(const FSlateBrush& InContentAreaBrushLight) { ContentAreaBrushLight = InContentAreaBrushLight; return *this; }

	/** Style used for the flat button */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FButtonStyle FlatButtonStyle;
	FRCPanelStyle& SetFlatButtonStyle(const FButtonStyle& InFlatButtonStyle) { FlatButtonStyle = InFlatButtonStyle; return *this; }

	/** Panel Header Row Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FMargin HeaderRowPadding;
	FRCPanelStyle& SetHeaderRowPadding(const FMargin& InHeaderRowPadding) { HeaderRowPadding = InHeaderRowPadding; return *this; }
	
	/** Panel Header Row Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FHeaderRowStyle HeaderRowStyle;
	FRCPanelStyle& SetHeaderRowStyle(const FHeaderRowStyle& InHeaderRowStyle) { HeaderRowStyle = InHeaderRowStyle; return *this; }
	
	/** Panel Header Text Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTextBlockStyle HeaderTextStyle;
	FRCPanelStyle& SetHeaderTextStyle(const FTextBlockStyle& InHeaderTextStyle) { HeaderTextStyle = InHeaderTextStyle; return *this; }

	/** Padding used around this panel */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FVector2D IconSize;
	FRCPanelStyle& SetIconSize(const FVector2D& InIconSize) { IconSize = InIconSize; return *this; }

	/** Padding used around this panel */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FMargin PanelPadding;
	FRCPanelStyle& SetPanelPadding(const FMargin& InPanelPadding) { PanelPadding = InPanelPadding; return *this; }

	/** Panel Text Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTextBlockStyle PanelTextStyle;
	FRCPanelStyle& SetPanelTextStyle(const FTextBlockStyle& InPanelTextStyle) { PanelTextStyle = InPanelTextStyle; return *this; }

	/** Panel Section Header Text Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTextBlockStyle SectionHeaderTextStyle;
	FRCPanelStyle& SetSectionHeaderTextStyle(const FTextBlockStyle& InSectionHeaderTextStyle) { SectionHeaderTextStyle = InSectionHeaderTextStyle; return *this; }

	/** Brush used by the RC Panels to draw the section header associated with this panel. */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FSlateBrush SectionHeaderBrush;
	FRCPanelStyle& SetSectionHeaderBrush(const FSlateBrush& InSectionHeaderBrush) { SectionHeaderBrush = InSectionHeaderBrush; return *this; }

	/** Panel Table Row Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTableRowStyle TableRowStyle;
	FRCPanelStyle& SetTableRowStyle(const FTableRowStyle& InTableRowStyle) { TableRowStyle = InTableRowStyle; return *this; }
	
	/** Panel Table View Style */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FTableViewStyle TableViewStyle;
	FRCPanelStyle& SetTableViewStyle(const FTableViewStyle& InTableViewStyle) { TableViewStyle = InTableViewStyle; return *this; }

	/** Style used for the switch button */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FCheckBoxStyle SwitchButtonStyle;
	FRCPanelStyle& SetSwitchButtonStyle(const FCheckBoxStyle& InSwitchButtonStyle) { SwitchButtonStyle = InSwitchButtonStyle; return *this; }

	/** Style used for the toggle button */
	UPROPERTY(EditAnywhere, Category = Appearance)
		FCheckBoxStyle ToggleButtonStyle;
	FRCPanelStyle& SetToggleButtonStyle(const FCheckBoxStyle& InToggleButtonStyle) { ToggleButtonStyle = InToggleButtonStyle; return *this; }

	/** Splitter handle size*/
	UPROPERTY(EditAnywhere, Category = Appearance)
	float SplitterHandleSize;
	FRCPanelStyle& SetSplitterHandleSize(const float InSplitterHandleSize) { SplitterHandleSize = InSplitterHandleSize; return *this; }
};
