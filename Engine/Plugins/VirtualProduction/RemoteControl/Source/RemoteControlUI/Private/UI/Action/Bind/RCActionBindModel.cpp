// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCActionBindModel.h"

#include "Action/RCPropertyAction.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Controller/RCController.h"
#include "RemoteControlField.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Behaviour/Builtin/Bind/RCBehaviourBindModel.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelDragHandle.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCActionModel"

namespace UE::RCActionPanelBindList
{
	namespace Columns
	{
		const FName VariableColor = TEXT("VariableColor");
		const FName DragDropHandle = TEXT("DragDropHandle");
		const FName Description = TEXT("Description");
		const FName Controller = TEXT("Controller");
	}

	class SBindActionRow : public SMultiColumnTableRow<TSharedRef<FRCActionModel>>
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TSharedRef<FRCActionBindModel> InActionItem)
		{
			ActionItem = InActionItem;
			FSuperRowType::Construct(InArgs, OwnerTableView);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (!ensure(ActionItem.IsValid()))
			{
				return SNullWidget::NullWidget;
			}
		
			if (ColumnName == UE::RCActionPanelBindList::Columns::VariableColor)
			{
				return ActionItem->GetTypeColorTagWidget();
			}
			else if (ColumnName == UE::RCActionPanelBindList::Columns::Description)
			{
				return ActionItem->GetNameWidget();
			}
			else if (ColumnName == UE::RCActionPanelBindList::Columns::Controller)
			{
				if (URCPropertyBindAction* BindAction = Cast< URCPropertyBindAction>(ActionItem->GetAction()))
				{
					if (ensure(BindAction->Controller))
					{
						const FText ControllerName = FText::FromName(BindAction->Controller->DisplayName);

						return SNew(SBox)
									.Padding(FMargin(6.f))
									[
										SNew(STextBlock).Text(ControllerName)
									];
							
					}
				}
			}

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FRCActionBindModel> ActionItem;
	};
}

TSharedPtr<SHeaderRow> FRCActionBindModel::GetHeaderRow()
{
	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	return SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)

		+ SHeaderRow::Column(UE::RCActionPanelBindList::Columns::VariableColor)
		.DefaultLabel(FText())
		.FixedWidth(5.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelBindList::Columns::Description)
		.DefaultLabel(LOCTEXT("DescriptionColumnHeader", "Description"))
		.FillWidth(0.4f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelBindList::Columns::Controller)
		.DefaultLabel(LOCTEXT("ControllerColumnHeader", "Controller"))
		.FillWidth(0.4f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);
}

TSharedPtr<FRCActionBindModel> FRCActionBindModel::GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
{
	if (URCPropertyBindAction* PropertyBindAction = Cast<URCPropertyBindAction>(InAction))
	{
		return MakeShared<FRCPropertyActionBindModel>(PropertyBindAction, InBehaviourItem, InRemoteControlPanel);
	}

	return nullptr;
}

TSharedRef<ITableRow> FRCActionBindModel::OnGenerateWidgetForList(TSharedPtr<FRCActionBindModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	typedef UE::RCActionPanelBindList::SBindActionRow ActionRowType;

	return SNew(ActionRowType, OwnerTable, InItem.ToSharedRef())
		.Style(&RCPanelStyle->TableRowStyle)
		.Padding(FMargin(3.f));
}

FLinearColor FRCPropertyActionBindModel::GetActionTypeColor() const
{
	if (const URCPropertyBindAction* PropertyAction = Cast< URCPropertyBindAction>(ActionWeakPtr.Get()))
	{
		if (TSharedPtr<FRemoteControlProperty> RemoteControlProperty = PropertyAction->GetRemoteControlProperty())
		{
			if (RemoteControlProperty->IsBound())
			{
				return UE::RCUIHelpers::GetFieldClassTypeColor(RemoteControlProperty->GetProperty());
			}
		}
	}

	return FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE