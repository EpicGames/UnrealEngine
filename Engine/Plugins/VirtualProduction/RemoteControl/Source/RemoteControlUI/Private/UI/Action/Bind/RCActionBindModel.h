// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/Bind/RCPropertyBindAction.h"
#include "Action/RCPropertyIdAction.h"
#include "UI/Action/RCActionModel.h"

class SActionItemListRow;
class SBox;

/* 
* ~ FRCActionBindModel ~
*
* UI model for representing an Action of the Bind Behaviour.
*/
class FRCActionBindModel : public FRCActionModel
{
public:
	FRCActionBindModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionModel(InAction, InBehaviourItem, InRemoteControlPanel)
	{
	}

	/** OnGenerateRow delegate for the Actions List View*/
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCActionBindModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the Header Row appropriate for this particular Action model */
	static TSharedPtr<SHeaderRow> GetHeaderRow();

	/** Chooses the appropriate Action model for the current class and field type*/
	static TSharedPtr<FRCActionBindModel> GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);
};

/*
* ~ FRCPropertyActionBindModel ~
*
* UI model for Property based Bind Actions
*/
class FRCPropertyActionBindModel : public FRCActionBindModel
{
public:
	FRCPropertyActionBindModel(URCPropertyBindAction* InPropertyAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCActionBindModel(InPropertyAction, InBehaviourItem, InRemoteControlPanel)
	{
	}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override;
};

/**
 * FRCPropertyIdActionBindModel
 *
 * UI model for PropertyId based Bind Actions
 */
class FRCPropertyIdActionBindModel : public FRCActionBindModel, public FRCPropertyIdActionType
{
public:
	FRCPropertyIdActionBindModel(URCPropertyIdAction* InPropertyIdAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionBindModel(InPropertyIdAction, InBehaviourItem, InRemoteControlPanel)
		, FRCPropertyIdActionType(InPropertyIdAction)
	{
	}
	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyIdTypeColor();
	}
	/** Widget representing Action Name field */
	virtual TSharedRef<SWidget> GetNameWidget() const override
	{
		return GetPropertyIdNameWidget();
	}
	/** Widget representing the Value field */
	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyIdValueWidget();
	}
};

// Note: Bind only supports Property based Actions, so Function action model is not necessary here