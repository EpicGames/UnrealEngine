// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCActionConditionalModel.h"

#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Controller/RCController.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCVirtualPropertyWidget.h"
#include "UI/Behaviour/Builtin/Conditional/RCBehaviourConditionalModel.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelDragHandle.h"
#include "UI/SRCPanelExposedEntity.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCActionConditionalModel"

namespace UE::RCActionPanelConditionalList
{
	namespace Columns
	{
		const FName TypeColorTag = TEXT("TypeColorTag");
		const FName DragDropHandle = TEXT("DragDropHandle");
		const FName Condition = TEXT("Condition");
		const FName Description = TEXT("PropertyID");
		const FName Value = TEXT("Value");
	}

	class SActionItemListRow : public SMultiColumnTableRow<TSharedRef<FRCActionModel>>
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TSharedRef<FRCActionConditionalModel> InActionItem)
		{
			ActionItem = InActionItem;
			FSuperRowType::Construct(InArgs, OwnerTableView);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (!ensure(ActionItem.IsValid()))
				return SNullWidget::NullWidget;

			if (ColumnName == UE::RCActionPanelConditionalList::Columns::TypeColorTag)
			{
				return ActionItem->GetTypeColorTagWidget();
			}
			else if (ColumnName == UE::RCActionPanelConditionalList::Columns::Condition)
			{
				return ActionItem->GetConditionWidget();
			}
			else if (ColumnName == UE::RCActionPanelConditionalList::Columns::Description)
			{
				return ActionItem->GetNameWidget();
			}
			else if (ColumnName == UE::RCActionPanelConditionalList::Columns::Value)
			{
				return ActionItem->GetWidget();
			}

			// @todo: Implement Drag-Drop-handle column with Actions reordering support

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FRCActionConditionalModel> ActionItem;
	};
}

TSharedPtr<SHeaderRow> FRCActionConditionalModel::GetHeaderRow()
{
	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	return SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)

		+ SHeaderRow::Column(UE::RCActionPanelConditionalList::Columns::TypeColorTag)
		.DefaultLabel(FText())
		.FixedWidth(5.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelConditionalList::Columns::DragDropHandle)
		.DefaultLabel(FText())
		.FixedWidth(25.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelConditionalList::Columns::Condition)
		.DefaultLabel(LOCTEXT("RCActionConditionColumnHeader", "Condition"))
		.FillWidth(0.2f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelConditionalList::Columns::Description)
		.DefaultLabel(LOCTEXT("RCActionDescColumnHeader", "PropertyID"))
		.FillWidth(0.2f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelConditionalList::Columns::Value)
		.DefaultLabel(LOCTEXT("RCActionValueColumnHeader", "Value"))
		.FillWidth(0.6f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);
}

FRCActionConditionalModel::FRCActionConditionalModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCActionModel(InAction, InBehaviourItem, InRemoteControlPanel)
{
	SAssignNew(ConditionWidget, STextBlock);
}

TSharedPtr<FRCActionConditionalModel> FRCActionConditionalModel::GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
{
	if (URCPropertyAction* PropertyAction = Cast<URCPropertyAction>(InAction))
	{
		return MakeShared<FRCPropertyActionConditionalModel>(PropertyAction, InBehaviourItem, InRemoteControlPanel);
	}
	else if (URCFunctionAction* FunctionAction = Cast<URCFunctionAction>(InAction))
	{
		return MakeShared<FRCFunctionActionConditionalModel>(FunctionAction, InBehaviourItem, InRemoteControlPanel);
	}
	else if (URCPropertyIdAction* PropertyIdAction = Cast<URCPropertyIdAction>(InAction))
	{
		return MakeShared<FRCPropertyIdActionConditionalModel>(PropertyIdAction, InBehaviourItem, InRemoteControlPanel);
	}

	return nullptr;
}

FText FRCActionConditionalModel::CreateConditionComparandText(const URCBehaviourConditional* InBehaviour, const FRCBehaviourCondition* InCondition)
{
	if (!InBehaviour || !InCondition)
	{
		return FText::GetEmpty();
	}

	const FText ConditionText = InBehaviour->GetConditionTypeAsText(InCondition->ConditionType);

	if (InCondition->ConditionType == ERCBehaviourConditionType::Else)
	{
		return ConditionText; // Else doesn't need to display the comparand
	}

	if (!InCondition->Comparand)
	{
		return FText::GetEmpty();
	}

	const FText ComparandValueText = FText::FromString(InCondition->Comparand->GetDisplayValueAsString());

	return FText::Format(INVTEXT("{0} {1}"), ConditionText, ComparandValueText);
}

void FRCActionConditionalModel::UpdateConditionWidget(const FText& InNewText) const
{
	if (ConditionWidget.IsValid())
	{
		ConditionWidget->SetText(InNewText);
	}
}

TSharedRef<SWidget> FRCActionConditionalModel::GetConditionWidget()
{
	if (TSharedPtr<FRCBehaviourConditionalModel> BehaviourItem = StaticCastSharedPtr<FRCBehaviourConditionalModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCBehaviourConditional* Behaviour = Cast<URCBehaviourConditional>(BehaviourItem->GetBehaviour()))
		{
			if (FRCBehaviourCondition* Condition = Behaviour->Conditions.Find(GetAction()))
			{
				if(ensure(Condition->Comparand))
				{
					return SAssignNew(EditableVirtualPropertyWidget, SRCVirtualPropertyWidget, Condition->Comparand)
						.OnGenerateWidget(this, &FRCActionConditionalModel::OnGenerateConditionWidget)
						.OnExitingEditMode(this, &FRCActionConditionalModel::OnExitingEditingMode);
				}
			}
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FRCActionConditionalModel::OnGenerateConditionWidget(URCVirtualPropertySelfContainer* InComparand)
{
	if (!ensure(InComparand))
	{
		return SNullWidget::NullWidget;
	}

	if (const TSharedPtr<FRCBehaviourConditionalModel> BehaviourItem = StaticCastSharedPtr<FRCBehaviourConditionalModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCBehaviourConditional* Behaviour = Cast<URCBehaviourConditional>(BehaviourItem->GetBehaviour()))
		{
			if (const FRCBehaviourCondition* Condition = Behaviour->Conditions.Find(GetAction()))
			{
				const FText ConditionText = Behaviour->GetConditionTypeAsText(Condition->ConditionType);

				FText ConditionComparandText = CreateConditionComparandText(Behaviour, Condition);

				UpdateConditionWidget(ConditionComparandText);

				if (ConditionWidget.IsValid())
				{
					return ConditionWidget.ToSharedRef();
				}
			}
		}
	}

	return SNullWidget::NullWidget;
}

void FRCActionConditionalModel::OnExitingEditingMode()
{
	if (const TSharedPtr<FRCBehaviourConditionalModel> BehaviourItem = StaticCastSharedPtr<FRCBehaviourConditionalModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCBehaviourConditional* Behaviour = Cast<URCBehaviourConditional>(BehaviourItem->GetBehaviour()))
		{
			if (const FRCBehaviourCondition* Condition = Behaviour->Conditions.Find(GetAction()))
			{
				const FText ConditionText = Behaviour->GetConditionTypeAsText(Condition->ConditionType);

				const FText ConditionComparandText = CreateConditionComparandText(Behaviour, Condition);

				UpdateSelectedConditionalActionModel(Condition, ConditionComparandText);
			}
		}
	}
}

void FRCActionConditionalModel::UpdateSelectedConditionalActionModel(const FRCBehaviourCondition* InConditionToCopy, const FText& InNewConditionText) const
{
	if (!InConditionToCopy || !InConditionToCopy->Comparand)
	{
		return;
	}

	// Update other selected action based on the edited one
	if (const TSharedPtr<SRemoteControlPanel> RCPanel = GetRemoteControlPanel())
    {
    	if (const TSharedPtr<SRCActionPanel> LogicPanel = RCPanel->GetLogicActionPanel())
    	{
    		const TArray<TSharedPtr<FRCLogicModeBase>> SelectedLogicItems = LogicPanel->GetSelectedLogicItems();

    		// If the selected items don't contain this one then skip it
    		if (SelectedLogicItems.Contains(SharedThis(this)))
    		{
    			for (const TSharedPtr<FRCLogicModeBase>& SelectedLogicItem : SelectedLogicItems)
    			{
    				if (const TSharedPtr<FRCActionConditionalModel> SelectedConditionalActionModel = StaticCastSharedPtr<FRCActionConditionalModel>(SelectedLogicItem))
    				{
    					if (const TSharedPtr<FRCBehaviourModel> SelectedBehaviourModel = SelectedConditionalActionModel->GetParentBehaviour())
    					{
    						if (URCBehaviourConditional* SelectedBehaviour = Cast<URCBehaviourConditional>(SelectedBehaviourModel->GetBehaviour()))
    						{
    							if (const FRCBehaviourCondition* SelectedCondition = SelectedBehaviour->Conditions.Find(SelectedConditionalActionModel->GetAction()))
    							{
    								if (SelectedCondition->Comparand)
    								{
    									SelectedCondition->Comparand->UpdateValueWithProperty(InConditionToCopy->Comparand);
    									SelectedConditionalActionModel->UpdateConditionWidget(InNewConditionText);
    								}
    							}
    						}
    					}
    				}
    			}
    		}
    	}
    }
}

TSharedRef<ITableRow> FRCActionConditionalModel::OnGenerateWidgetForList(TSharedPtr<FRCActionConditionalModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	typedef UE::RCActionPanelConditionalList::SActionItemListRow ActionRowType;

	return SNew(ActionRowType, OwnerTable, InItem.ToSharedRef())
		.Style(&RCPanelStyle->TableRowStyle)
		.Padding(FMargin(3.f));
}

#undef LOCTEXT_NAMESPACE
