// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Action/RCPropertyIdAction.h"
#include "UI/Action/RCActionModel.h"

struct FRCBehaviourCondition;
class SActionItemListRow;
class SBox;
class SRCVirtualPropertyWidget;
class STextBlock;
class URCBehaviourConditional;

/* 
* ~ FRCActionConditionalModel ~
*
* UI model for representing an Action of the Conditional Behaviour.
* Contains a Condition widget for representing the comparand/comparator and target property/value widgets
* 
*  <Actions Table>       <Condition>                <Description>         <Value>
* 
*  Color coding | Comparator+Comparand | Target Property | Value To Set
*/
class FRCActionConditionalModel : public FRCActionModel
{
public:
	FRCActionConditionalModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);	

	/** Widget representing the Condition (Comparator|Comparand) used
	* to trigger the corresponding action
	*/
	TSharedRef<SWidget> GetConditionWidget();

	/** OnGenerateRow delegate for the Actions List View*/
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCActionConditionalModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the Header Row appropriate for this particular Action model */
	static TSharedPtr<SHeaderRow> GetHeaderRow();

	/** Chooses the appropriate Action model for the current class and field type*/
	static TSharedPtr<FRCActionConditionalModel> GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

private:
	/** Create the text to be assigned to the ConditionWidget */
	FText CreateConditionComparandText(const URCBehaviourConditional* InBehaviour, const FRCBehaviourCondition* InCondition);

	/** Update the condition widget text with the new text given */
	void UpdateConditionWidget(const FText& InNewText) const;

	/** Called when constructing the widget or after editing the condition value */
	TSharedRef<SWidget> OnGenerateConditionWidget(class URCVirtualPropertySelfContainer* InComparand);

	/** Called when exiting edit mode to update other selected actions */
	void OnExitingEditingMode();

	/** Called when updating the condition to update every other actions currently selected with the new condition */
	void UpdateSelectedConditionalActionModel(const FRCBehaviourCondition* InConditionToCopy, const FText& InNewConditionText) const;

	/** Holding the widget that display the condition as text */
	TSharedPtr<STextBlock> ConditionWidget;
};

/*
* ~ FRCPropertyActionConditionalModel ~
*
* UI model for Property based Conditional Actions
*/
class FRCPropertyActionConditionalModel : public FRCActionConditionalModel, public FRCPropertyActionType
{
public:
	FRCPropertyActionConditionalModel(URCPropertyAction* InPropertyAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCActionConditionalModel(InPropertyAction, InBehaviourItem, InRemoteControlPanel),
		FRCPropertyActionType(InPropertyAction) { }

	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyWidget();
	}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyTypeColor();
	}
};

/*
* ~ FRCFunctionActionConditionalModel ~
*
* UI model for Function based Conditional Actions
*/
class FRCFunctionActionConditionalModel : public FRCActionConditionalModel, public FRCFunctionActionType
{
public:
	FRCFunctionActionConditionalModel(URCFunctionAction* InFunctionAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionConditionalModel(InFunctionAction, InBehaviourItem, InRemoteControlPanel)
		, FRCFunctionActionType(InFunctionAction) { }

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetFunctionTypeColor();
	}
};

/**
 * FRCPropertyIdActionConditionalModel
 *
 * UI model for PropertyId based Conditional Actions
 */
class FRCPropertyIdActionConditionalModel : public FRCActionConditionalModel, public FRCPropertyIdActionType
{
public:
	FRCPropertyIdActionConditionalModel(URCPropertyIdAction* InPropertyIdAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionConditionalModel(InPropertyIdAction, InBehaviourItem, InRemoteControlPanel)
		, FRCPropertyIdActionType(InPropertyIdAction)
	{}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyIdTypeColor();
	}

	/** Widget representing Action Name field */
	virtual TSharedRef<SWidget> GetNameWidget() const override
	{
		return GetPropertyIdNameWidget();
	}

	/** Widget representing the Value field */
	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyIdValueWidget();
	}
};
