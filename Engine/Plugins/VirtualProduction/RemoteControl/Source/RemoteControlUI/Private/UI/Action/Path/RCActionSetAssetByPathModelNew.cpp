// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Action/Path/RCActionSetAssetByPathModelNew.h"

#include "Action/Path/RCSetAssetByPathActionNew.h"
#include "RemoteControlField.h"
#include "ScopedTransaction.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCActionSetAssetByPathModelNew"

namespace UE::RemoteControl::UI::Private::ActionSetAssetByPathModelNew
{
	namespace Columns
	{
		const FName VariableColor = TEXT("VariableColor");
		const FName Description = TEXT("Description");
	}

	class SActionSetAssetByPathModelNewRow : public SMultiColumnTableRow<TSharedRef<FRCActionModel>>
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FRCActionSetAssetByPathModelNew> InActionItem)
		{
			ActionItem = InActionItem;
			FSuperRowType::Construct(InArgs, InOwnerTableView);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (!ensure(ActionItem.IsValid()))
			{
				return SNullWidget::NullWidget;
			}

			using namespace UE::RemoteControl::UI::Private::ActionSetAssetByPathModelNew;
		
			if (ColumnName == Columns::VariableColor)
			{
				return ActionItem->GetTypeColorTagWidget();
			}
			else if (ColumnName == Columns::Description)
			{
				return ActionItem->GetNameWidget();
			}

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FRCActionSetAssetByPathModelNew> ActionItem;
	};
}

TSharedRef<ITableRow> FRCActionSetAssetByPathModelNew::OnGenerateWidgetForList(TSharedPtr<FRCActionSetAssetByPathModelNew> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	using SActionRow = UE::RemoteControl::UI::Private::ActionSetAssetByPathModelNew::SActionSetAssetByPathModelNewRow;

	return SNew(SActionRow, OwnerTable, InItem.ToSharedRef())
		.Style(&RCPanelStyle->TableRowStyle)
		.Padding(FMargin(3.f));
}

TSharedPtr<SHeaderRow> FRCActionSetAssetByPathModelNew::GetHeaderRow()
{
	using namespace UE::RemoteControl::UI::Private;

	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	return SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)

		+ SHeaderRow::Column(ActionSetAssetByPathModelNew::Columns::VariableColor)
		.DefaultLabel(FText())
		.FixedWidth(5.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(ActionSetAssetByPathModelNew::Columns::Description)
		.DefaultLabel(LOCTEXT("RCActionDescColumnHeader", "Description"))
		.FillWidth(0.5f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);
}

TSharedPtr<FRCActionSetAssetByPathModelNew> FRCActionSetAssetByPathModelNew::GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
{
	if (URCSetAssetByPathActionNew* PropertyBindAction = Cast<URCSetAssetByPathActionNew>(InAction))
	{
		return MakeShared<FRCPropertyActionSetAssetByPathModelNew>(PropertyBindAction, InBehaviourItem, InRemoteControlPanel);
	}

	return nullptr;
}

FLinearColor FRCPropertyActionSetAssetByPathModelNew::GetActionTypeColor() const
{
	if (const URCSetAssetByPathActionNew* PropertyAction = Cast<URCSetAssetByPathActionNew>(ActionWeakPtr.Get()))
	{
		if (TSharedPtr<FRemoteControlProperty> RemoteControlProperty = PropertyAction->GetRemoteControlProperty())
		{
			if (RemoteControlProperty->IsBound())
			{
				return UE::RCUIHelpers::GetFieldClassTypeColor(RemoteControlProperty->GetProperty());
			}
		}
	}

	return FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE
