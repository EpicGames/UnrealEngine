// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/Path/RCSetAssetByPathActionNew.h"
#include "Action/RCPropertyIdAction.h"
#include "UI/Action/RCActionModel.h"

class SActionItemListRow;
class SBox;
class URCSetAssetByPathActionNew;

/* 
* ~ FRCActionSetAssetByPathModelNew ~
*
* UI model for representing an Action of the Set Asset (new)
*/
class FRCActionSetAssetByPathModelNew : public FRCActionModel
{
public:
	FRCActionSetAssetByPathModelNew(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionModel(InAction, InBehaviourItem, InRemoteControlPanel)
	{
	}

	/** OnGenerateRow delegate for the Actions List View*/
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCActionSetAssetByPathModelNew> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Get the Header Row appropriate for this particular Action model */
	static TSharedPtr<SHeaderRow> GetHeaderRow();

	/** Chooses the appropriate Action model for the current class and field type*/
	static TSharedPtr<FRCActionSetAssetByPathModelNew> GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);
};

/*
* ~ FRCPropertyActionSetAssetByPathModelNew ~
*
* UI model for representing a Property Action of the Set Asset (new)
*/
class FRCPropertyActionSetAssetByPathModelNew : public FRCActionSetAssetByPathModelNew
{
public:
	FRCPropertyActionSetAssetByPathModelNew(URCSetAssetByPathActionNew* InPropertyAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionSetAssetByPathModelNew(InPropertyAction, InBehaviourItem, InRemoteControlPanel)
	{
	}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override;
};
