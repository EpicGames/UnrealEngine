// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCActionModel.h"

#include "Algo/Count.h"
#include "Commands/RemoteControlCommands.h"
#include "Controller/RCController.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCVirtualPropertyWidget.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelDragHandle.h"
#include "UI/SRCPanelExposedEntity.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCActionModel"

namespace UE::RCActionPanelList
{
	namespace Columns
	{
		const FName VariableColor = TEXT("VariableColor");
		const FName DragDropHandle = TEXT("DragDropHandle");
		const FName Description = TEXT("Description");
		const FName Value = TEXT("Value");
	}

	class SActionItemListRow : public SMultiColumnTableRow<TSharedRef<FRCActionModel>>
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TSharedRef<FRCActionModel> InActionItem)
		{
			ActionItem = InActionItem;
			FSuperRowType::Construct(InArgs, OwnerTableView);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (!ensure(ActionItem.IsValid()))
				return SNullWidget::NullWidget;

			if (ColumnName == UE::RCActionPanelList::Columns::VariableColor)
			{
				return ActionItem->GetTypeColorTagWidget();
			}
			else if (ColumnName == UE::RCActionPanelList::Columns::Description)
			{
				return ActionItem->GetNameWidget();
			}
			else if (ColumnName == UE::RCActionPanelList::Columns::Value)
			{
				return ActionItem->GetWidget();
			}

			// @todo: Implement Drag-Drop-handle column with Actions reordering support

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FRCActionModel> ActionItem;
	};
}

const FLazyName FRCActionModel::ActionModelName = "ActionModel";

FRCActionModel::FRCActionModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCLogicModeBase(InRemoteControlPanel),
	ActionWeakPtr(InAction)
{
	BehaviourItemWeakPtr = InBehaviourItem;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.LogicControllersPanel");
}

TSharedRef<SWidget> FRCActionModel::GetNameWidget() const
{
	if(URCAction* Action = GetAction())
	{
		if (URemoteControlPreset* Preset = GetPreset())
		{
			if (const TSharedPtr<FRemoteControlField> RemoteControlField = Preset->GetExposedEntity<FRemoteControlField>(Action->ExposedFieldId).Pin())
			{
				return SNew(SBox)
					.Padding(FMargin(8.f))
					.VAlign(EVerticalAlignment::VAlign_Center)
					[
						SNew(STextBlock).Text(FText::FromName(RemoteControlField->GetLabel()))
					];
			}
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FRCActionModel::GetWidget() const
{
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FRCActionModel::GetTypeColorTagWidget() const
{
	const FLinearColor TypeColor = GetActionTypeColor();

	// Type Color Bar
	return SNew(SBox)
		.HeightOverride(5.f)
		[
			SNew(SBorder)
			.Visibility(EVisibility::HitTestInvisible)
			.BorderImage(FAppStyle::Get().GetBrush("NumericEntrySpinBox.Decorator"))
			.BorderBackgroundColor(TypeColor)
			.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.f))
		];
}

URCAction* FRCActionModel::GetAction() const
{
	return ActionWeakPtr.Get();
}

TSharedRef<ITableRow> FRCActionModel::OnGenerateWidgetForList(TSharedPtr<FRCActionModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	typedef UE::RCActionPanelList::SActionItemListRow ActionRowType;

	return SNew(ActionRowType, OwnerTable, InItem.ToSharedRef())
		.Style(&RCPanelStyle->TableRowStyle)
		.Padding(FMargin(3.f));
}

TSharedPtr<SHeaderRow> FRCActionModel::GetHeaderRow()
{
	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	return SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)

		+ SHeaderRow::Column(UE::RCActionPanelList::Columns::VariableColor)
		.DefaultLabel(FText())
		.FixedWidth(5.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelList::Columns::DragDropHandle)
		.DefaultLabel(FText())
		.FixedWidth(25.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelList::Columns::Description)
		.DefaultLabel(LOCTEXT("RCActionDescColumnHeader", "Description"))
		.FillWidth(0.5f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelList::Columns::Value)
		.DefaultLabel(LOCTEXT("RCActionValueColumnHeader", "Value"))
		.FillWidth(0.5f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);
}

TSharedPtr<FRCActionModel> FRCActionModel::GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
{
	if (URCPropertyAction* PropertyAction = Cast<URCPropertyAction>(InAction))
	{
		return MakeShared<FRCPropertyActionModel>(PropertyAction, InBehaviourItem, InRemoteControlPanel);
	}
	else if (URCFunctionAction* FunctionAction = Cast<URCFunctionAction>(InAction))
	{
		return MakeShared<FRCFunctionActionModel>(FunctionAction, InBehaviourItem, InRemoteControlPanel);
	}
	else if (URCPropertyIdAction* PropertyIdAction = Cast<URCPropertyIdAction>(InAction))
	{
		return MakeShared<FRCPropertyIdActionModel>(PropertyIdAction, InBehaviourItem, InRemoteControlPanel);
	}
	else
		return nullptr;
}

FRCPropertyActionType::FRCPropertyActionType(URCPropertyAction* InPropertyAction)
{
	PropertyActionWeakPtr = InPropertyAction;

	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties = true;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);

	if (InPropertyAction)
	{
		// In case the Action is a Vector/Rotator we don't want the Lock widget
		if (FProperty* ActionProperty = InPropertyAction->GetProperty())
		{
			ActionProperty->RemoveMetaData(TEXT("AllowPreserveRatio"));
		}

		// Generate UI widget for Action input
		if (const TSharedPtr<FStructOnScope> StructOnScope = InPropertyAction->PropertySelfContainer->CreateStructOnScope())
		{
			PropertyRowGenerator->SetStructure(StructOnScope);
			PropertyRowGenerator->OnFinishedChangingProperties().AddRaw(this, &FRCPropertyActionType::OnFinishedChangingProperties);

			for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGenerator->GetRootTreeNodes())
			{
				TArray<TSharedRef<IDetailTreeNode>> Children;
				CategoryNode->GetChildren(Children);

				for (const TSharedRef<IDetailTreeNode>& Child : Children)
				{
					// Special handling for any Remote Control Property that is actually a single element of an Array
					// For example: "Override Materials" of Static Mesh Actor when exposed as a property in the Remote Control preset
					//
					if (TSharedPtr<FRemoteControlProperty> RemoteControlProperty = InPropertyAction->GetRemoteControlProperty())
					{
						if (ensure(RemoteControlProperty->FieldPathInfo.GetSegmentCount() > 0))
						{
							const FRCFieldPathSegment& LastSegment = RemoteControlProperty->FieldPathInfo.Segments.Last();
							const int32 ArrayIndex = LastSegment.ArrayIndex;

							// Is this a Container element?
							if (ArrayIndex != INDEX_NONE)
							{
								TArray<TSharedRef<IDetailTreeNode>> InnerChildren;
								Child->GetChildren(InnerChildren);

								// If yes, extract the Detail node corresponding to that element (rather than the parent container)
								if (ensure(InnerChildren.IsValidIndex(ArrayIndex)))
								{
									DetailTreeNodeWeakPtr = InnerChildren[ArrayIndex];
								}
							}
						}
					}

					// For regular properties (non-container)
					if (!DetailTreeNodeWeakPtr.IsValid())
					{
						DetailTreeNodeWeakPtr = Child;
						break;
					}
				}
			}
		}
	}
}

FRCPropertyActionType::~FRCPropertyActionType()
{
	if (PropertyRowGenerator.IsValid())
	{
		PropertyRowGenerator->OnFinishedChangingProperties().RemoveAll(this);
	}
}

const FName& FRCPropertyActionType::GetPropertyName() const
{
	return PropertyActionWeakPtr.Get()->PropertySelfContainer->PropertyName;
}

TSharedRef<SWidget> FRCPropertyActionType::GetPropertyWidget() const
{
	return UE::RCUIHelpers::GetGenericFieldWidget(DetailTreeNodeWeakPtr.Pin());
}

FLinearColor FRCPropertyActionType::GetPropertyTypeColor() const
{
	FLinearColor TypeColor = FLinearColor::White;

	if (!ensure(PropertyActionWeakPtr.IsValid()))
	{
		return TypeColor;
	}

	if (const URCPropertyAction* PropertyAction = PropertyActionWeakPtr.Get())
	{
		if (const FProperty* Property = PropertyAction->PropertySelfContainer->GetProperty())
		{
			TypeColor = UE::RCUIHelpers::GetFieldClassTypeColor(Property);
		}
	}

	return TypeColor;
}

void FRCPropertyActionType::OnActionValueChange() const
{
	if (PropertyActionWeakPtr.IsValid())
	{
		PropertyActionWeakPtr->NotifyActionValueChanged();
	}
}

void FRCPropertyActionType::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangeEvent) const
{
	if (InPropertyChangeEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		OnActionValueChange();
	}
}

void FRCActionModel::AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder)
{
	if (TSharedPtr<SRemoteControlPanel> RemoteControlPanel = PanelWeakPtr.Pin())
	{
		// 1. Edit (if available)
		if (EditableVirtualPropertyWidget)
		{
			EditableVirtualPropertyWidget->AddEditContextMenuOption(MenuBuilder);
		}

		const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

		// 2. Select Source
		MenuBuilder.AddMenuEntry(Commands.SelectSource);
	}
}

void FRCActionModel::OnSelectionExit()
{
	if (EditableVirtualPropertyWidget)
	{
		EditableVirtualPropertyWidget->ExitEditMode();
	}
}

int32 FRCActionModel::RemoveModel()
{
	URCAction* Action = GetAction();

	if (!Action)
	{
		return 0;
	}

	TSharedPtr<FRCBehaviourModel> BehaviorModel = GetParentBehaviour();

	if (!BehaviorModel.IsValid())
	{
		return 0;
	}

	URCBehaviour* Behavior = BehaviorModel->GetBehaviour();

	if (!Behavior || !Behavior->ActionContainer)
	{
		return 0;
	}

	return Behavior->ActionContainer->RemoveAction(Action);
}

FRCPropertyIdActionType::FRCPropertyIdActionType(URCPropertyIdAction* InPropertyIdAction)
	: PropertyIdActionWeakPtr(InPropertyIdAction)
{
	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties = true;
	
	PropertyIdNameRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
	if (InPropertyIdAction)
	{
		RefreshNameWidget();
		RefreshValueWidget();
#if WITH_EDITOR
		if (const URemoteControlPreset* Preset = InPropertyIdAction->PresetWeakPtr.Get())
		{
			Preset->GetPropertyIdRegistry()->OnPropertyIdActionNeedsRefresh().AddRaw(this, &FRCPropertyIdActionType::RefreshValueWidget);
		}
#endif
	}
}

FRCPropertyIdActionType::~FRCPropertyIdActionType()
{
#if WITH_EDITOR
	if (const URCPropertyIdAction* PropertyIdAction = PropertyIdActionWeakPtr.Get())
	{
		if (const URemoteControlPreset* Preset = PropertyIdAction->PresetWeakPtr.Get())
		{
			Preset->GetPropertyIdRegistry()->OnPropertyIdActionNeedsRefresh().RemoveAll(this);
		}
	}
#endif

	for (const TPair<FPropertyIdContainerKey, TSharedPtr<IPropertyRowGenerator>>& CachedGenerator : CachedPropertyIdValueRowGenerator)
	{
		if (CachedGenerator.Value.IsValid())
		{
			CachedGenerator.Value->OnFinishedChangingProperties().RemoveAll(this);
		}
	}
	PropertyIdValueRowGenerator.Reset();
}

FLinearColor FRCPropertyIdActionType::GetPropertyIdTypeColor() const
{
	// @todo: Confirm color to be used for this with VP team.
	return FLinearColor(FColor(32, 191, 107));
}

TSharedRef<SWidget> FRCPropertyIdActionType::GetPropertyIdNameWidget() const
{
	if (!FieldIdTreeNodeWeakPtr.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const FNodeWidgets FieldIdNodeWidgets = FieldIdTreeNodeWeakPtr.Pin()->CreateNodeWidgets();
	const TSharedRef<SHorizontalBox> NameWidget = SNew(SHorizontalBox);

	if (FieldIdNodeWidgets.ValueWidget)
	{
		NameWidget->AddSlot()
			.Padding(10.f, 2.f)
			.VAlign(VAlign_Center)
			[
				FieldIdNodeWidgets.ValueWidget.ToSharedRef()
			];
	}

	return NameWidget;
}

TSharedRef<SWidget> FRCPropertyIdActionType::GetPropertyIdValueWidget() const
{
	TSharedRef<SVerticalBox> VerticalBox = SNew(SVerticalBox);

	TArray<FPropertyIdContainerKey> SortedTreeNodesKeys;
	ValueTreeNodeWeakPtr.GetKeys(SortedTreeNodesKeys);
	SortedTreeNodesKeys.Sort();

	FName LastPropId = NAME_None;
	constexpr float Offset = 5.f;

	FMargin TitlePadding = FMargin(0.f);
	FMargin ValuePadding = FMargin(0.f);
	ValuePadding.Right = Offset;

	int32 OriginalPropIdLength = 0;
	int32 OriginalDotCount = 0;

	if (const URCPropertyIdAction* PropIdAction = PropertyIdActionWeakPtr.Get())
	{
		const FString PropIdAsString = PropIdAction->PropertyId.ToString();
		OriginalDotCount = Algo::Count(PropIdAsString, '.');
		OriginalPropIdLength = PropIdAction->PropertyId.GetStringLength();
	}

	for (const FPropertyIdContainerKey& TreeNodeKey : SortedTreeNodesKeys)
	{
		FString PropIdAsString = TEXT("");

		if (LastPropId != TreeNodeKey.PropertyId)
		{
			if (LastPropId != NAME_None)
			{
				TitlePadding.Top = Offset;
			}

			LastPropId = TreeNodeKey.PropertyId;
			PropIdAsString = LastPropId.ToString();
			if (PropIdAsString.Len() != OriginalPropIdLength)
			{
				FString LabelToUse = PropIdAsString.RightChop(OriginalPropIdLength);
				const int32 CurrentDotCount = Algo::Count(LabelToUse, '.');
				TitlePadding.Left = Offset * (CurrentDotCount - OriginalDotCount);
				ValuePadding.Left = TitlePadding.Left + 2.f;

				VerticalBox->AddSlot()
					.AutoHeight()
					.Padding(TitlePadding)
					[
						SNew(STextBlock)
						.Text(FText::FromString(LabelToUse))
					];
			}
		}

		if (const TWeakPtr<IDetailTreeNode>* Node = ValueTreeNodeWeakPtr.Find(TreeNodeKey))
		{
			if (Node->IsValid())
			{
				VerticalBox->AddSlot()
					.Padding(ValuePadding)
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.FillWidth(0.7f)
						[
							UE::RCUIHelpers::GetGenericFieldWidget(Node->Pin())
						]
					];
			}
		}
	}

	return VerticalBox;
}

void FRCPropertyIdActionType::OnActionValueChange() const
{
	if (PropertyIdActionWeakPtr.IsValid())
	{
		PropertyIdActionWeakPtr->NotifyActionValueChanged();
	}
}

void FRCPropertyIdActionType::OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangeEvent) const
{
	if (InPropertyChangeEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		OnActionValueChange();
	}
}

void FRCPropertyIdActionType::RefreshNameWidget()
{
	if (URCPropertyIdAction* PropertyIdAction = PropertyIdActionWeakPtr.Get())
	{
		// Since we must keep many PRG objects alive in order to access the handle data, validating the nodes each tick is very taxing.
		// We can override the validation with a lambda since the validation function in PRG is not necessary for our implementation
		auto ValidationLambda = ([](const FRootPropertyNodeList& PropertyNodeList) { return true; });
		PropertyIdNameRowGenerator->SetObjects({ PropertyIdAction });
		PropertyIdNameRowGenerator->SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes::CreateLambda(MoveTemp(ValidationLambda)));
		for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyIdNameRowGenerator->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			CategoryNode->GetChildren(Children);
			bool bFoundFieldId = false;
			for (const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				const TSharedPtr<IPropertyHandle> PropertyHandle = Child->CreatePropertyHandle();
				if (PropertyHandle && PropertyHandle->IsValidHandle())
				{
					if (const FProperty* Property = PropertyHandle->GetProperty())
					{
						if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(URCPropertyIdAction, PropertyId))
						{
							FieldIdTreeNodeWeakPtr = Child;
							bFoundFieldId = true;
						}
						if (bFoundFieldId)
						{
							break;
						}
					}
				}
			}
		}
	}
}

void FRCPropertyIdActionType::RefreshValueWidget()
{
	if (URCPropertyIdAction* PropertyIdAction = PropertyIdActionWeakPtr.Get())
	{
		// Generate UI widget for Action input
		PropertyIdValueRowGenerator.Reset();
		ValueTreeNodeWeakPtr.Reset();
		PropertyIdAction->PropertySelfContainer.KeySort([] (const FPropertyIdContainerKey& InFirst, const FPropertyIdContainerKey& InSecond)
		{
			return InSecond < InFirst;
		});


		for (const TPair<FPropertyIdContainerKey, TObjectPtr<URCVirtualPropertySelfContainer>>& PropertyContainer : PropertyIdAction->PropertySelfContainer)
		{
			if (IsValid(PropertyContainer.Value))
			{
				if (const TSharedPtr<FStructOnScope> StructOnScope = PropertyContainer.Value->CreateStructOnScope())
				{
					if (const TSharedPtr<IPropertyRowGenerator>* Generator = CachedPropertyIdValueRowGenerator.Find(PropertyContainer.Key))
					{
						PropertyIdValueRowGenerator.Add(PropertyContainer.Key, (*Generator));
					}
					else
					{
						FPropertyRowGeneratorArgs Args;
						Args.bShouldShowHiddenProperties = true;

						CachedPropertyIdValueRowGenerator.Add(PropertyContainer.Key, FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args));
						CachedPropertyIdValueRowGenerator[PropertyContainer.Key]->SetStructure(StructOnScope);
						CachedPropertyIdValueRowGenerator[PropertyContainer.Key]->OnFinishedChangingProperties().AddRaw(this, &FRCPropertyIdActionType::OnFinishedChangingProperties);

						PropertyIdValueRowGenerator.Add(PropertyContainer.Key, CachedPropertyIdValueRowGenerator[PropertyContainer.Key]);
					}

					for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyIdValueRowGenerator[PropertyContainer.Key]->GetRootTreeNodes())
					{
						TArray<TSharedRef<IDetailTreeNode>> Children;
						CategoryNode->GetChildren(Children);
						if (Children.Num() > 0)
						{
							const TSharedRef<IDetailTreeNode>& Child = Children[0];
							// For regular properties (non-container)
							if (const TWeakPtr<IDetailTreeNode>* ValueTreeNode = CachedValueTreeNodeWeakPtr.Find(PropertyContainer.Key))
							{
								ValueTreeNodeWeakPtr.Add(PropertyContainer.Key, (*ValueTreeNode));
							}
							else
							{
								CachedValueTreeNodeWeakPtr.Add(PropertyContainer.Key, Child);
								ValueTreeNodeWeakPtr.Add(PropertyContainer.Key, CachedValueTreeNodeWeakPtr[PropertyContainer.Key]);
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE