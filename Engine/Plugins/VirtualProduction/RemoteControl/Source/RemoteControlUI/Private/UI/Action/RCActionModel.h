// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyIdAction.h"
#include "Action/RCPropertyAction.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"

#include "UObject/WeakFieldPtr.h"

class FMenuBuilder;
class FRCBehaviourModel;
struct FRCPanelStyle;
class IDetailTreeNode;
class ITableRow;
class IPropertyRowGenerator;
class SHeaderRow;
class SHorizontalBox;
class SRCVirtualPropertyWidget;
class STableViewBase;
class SWidget;
class URCAction;
class URCPropertyAction;
class URCVirtualProperty;
class URCFunctionAction;

/* 
* ~ FRCActionModel ~
*
* UI model for representing an Action.
* Contains a row widget with Action related metadata and a generic value widget
*/
class FRCActionModel : public FRCLogicModeBase
{
public:
	static const FLazyName ActionModelName;

	FRCActionModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	virtual FName GetModelType() const override
	{
		return ActionModelName;
	}
	
	/** Fetch the Action (data model) associated with us*/
	URCAction* GetAction() const;

	/** Widget representing Action Name field */
	virtual TSharedRef<SWidget> GetNameWidget() const;

	/** Widget representing the Value field */
	virtual TSharedRef<SWidget> GetWidget() const override;

	virtual int32 RemoveModel() override;

	/** Widget representing Color Coding for the Action Type*/
	virtual TSharedRef<SWidget> GetTypeColorTagWidget() const;

	/** Color code for this Action. Customized per action type*/
	virtual FLinearColor GetActionTypeColor() const = 0;

	/** Allows Logic panels to add special functionality to the Context Menu based on context */
	virtual void AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder);

	/** OnGenerateRow delegate for the Actions List View*/
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCActionModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the Header Row appropriate for this particular Action model */
	static TSharedPtr<SHeaderRow> GetHeaderRow();

	/** Chooses the appropriate Action model for the current class and field type*/
	static TSharedPtr<FRCActionModel> GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	/** Invoked when this Action item no longer has focus*/
	virtual void OnSelectionExit();

	/** Returns the Behaviour Item associated with us*/
	TSharedPtr<FRCBehaviourModel> GetParentBehaviour()
	{
		return BehaviourItemWeakPtr.Pin();
	}

protected:
	/** The Action (data model) associated with us*/
	TWeakObjectPtr<URCAction> ActionWeakPtr;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** The Behaviour (UI model) associated with us*/
	TWeakPtr<FRCBehaviourModel> BehaviourItemWeakPtr;

	/** Optional: An editable virtual property widget associated with this row
	* Child action models can assign this variable and receive the Edit context menu option automatically*/
	TSharedPtr<SRCVirtualPropertyWidget> EditableVirtualPropertyWidget;
};

/*
* ~ FRCPropertyActionType ~
*
* Reusable class containing Property action related data and functions
*/
class FRCPropertyActionType
{
public:
	FRCPropertyActionType(URCPropertyAction* InPropertyAction);

	~FRCPropertyActionType();

	/** Property Name associated with this Action */
	const FName& GetPropertyName() const;

	/** The widget to be rendered for this Property
	* Represents the input field which the user will use to set a value for this Action
	*/
	TSharedRef<SWidget> GetPropertyWidget() const;

	FLinearColor GetPropertyTypeColor() const;

protected:
	/** Callback for when the ChangeProperty type is ValueSet */
	void OnActionValueChange() const;

	/** Callback when the action value change */
	void OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangeEvent) const;

	/** The Property Action (data model) associated with us*/
	TWeakObjectPtr<URCPropertyAction> PropertyActionWeakPtr;

	/** The row generator used to represent this widget as a row, when used with SListView */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Used to create a generic Value Widget for the property row widget*/
	TWeakPtr<IDetailTreeNode> DetailTreeNodeWeakPtr;
};

/*
* ~ FRCFunctionActionType ~
*
* Reusable class containing Function action related data and functions
*/
class FRCFunctionActionType
{
public:
	FRCFunctionActionType(URCFunctionAction* InFunctionAction) : FunctionActionWeakPtr(InFunctionAction) {}

	/** Color code for function actions in the Actions table */
	FLinearColor GetFunctionTypeColor() const
	{
		// @todo: Confirm color to be used for this with VP team.
		return FLinearColor::Blue;
	}

protected:
	/** The Function Action (data model) associated with us*/
	TWeakObjectPtr<URCFunctionAction> FunctionActionWeakPtr;
};

/**
 * FRCPropertyIdActionType
 *
 * Reusable class containing PropertyID action related data and functions
 */
class FRCPropertyIdActionType
{
public:
	FRCPropertyIdActionType(URCPropertyIdAction* InPropertyIdAction);

	~FRCPropertyIdActionType();

	/** Color code for function actions in the Actions table */
	FLinearColor GetPropertyIdTypeColor() const;

	/** Widget representing Action Name field */
	TSharedRef<SWidget> GetPropertyIdNameWidget() const;

	/** The widget to be rendered for this PropertyId
	 * Represents the input field which the user will use to set a value for this Action
	 */
	TSharedRef<SWidget> GetPropertyIdValueWidget() const;

private:
	/** Callback for when the ChangeProperty type is ValueSet */
	void OnActionValueChange() const;

	/** Callback when the action value change */
	void OnFinishedChangingProperties(const FPropertyChangedEvent& InPropertyChangeEvent) const;

	/** Recreates the name section of this row widget. */
	void RefreshNameWidget();

	/** Recreates the value section of this row widget. */
	void RefreshValueWidget();

protected:
	/** The PropertyId Action (data model) associated with this. */
	TWeakObjectPtr<URCPropertyIdAction> PropertyIdActionWeakPtr;

	/** The row generator used to represent name of this widget as a row, when used with SListView */
	TSharedPtr<IPropertyRowGenerator> PropertyIdNameRowGenerator;
	
	/** The cache of the row generator used to not recreate it */
	TMap<FPropertyIdContainerKey, TSharedPtr<IPropertyRowGenerator>> PropertyIdValueRowGenerator;

	/** The row generator used to represent value of this widget as a row, when used with SListView */
	TMap<FPropertyIdContainerKey, TSharedPtr<IPropertyRowGenerator>> CachedPropertyIdValueRowGenerator;

	/** Used to create a generic Field Id Widget for the property row widget */
	TWeakPtr<IDetailTreeNode> FieldIdTreeNodeWeakPtr;
	
	/** Used to create a generic Value Widget for the property row widget */
	TMap<FPropertyIdContainerKey, TWeakPtr<IDetailTreeNode>> ValueTreeNodeWeakPtr;

	/** Used to cache a generic Value Widget for the property row widget to not recreate it */
	TMap<FPropertyIdContainerKey, TWeakPtr<IDetailTreeNode>> CachedValueTreeNodeWeakPtr;
};

/*
* ~ FRCPropertyActionModel ~
*
* UI model for Property based Actions
*/
class FRCPropertyActionModel : public FRCActionModel, public FRCPropertyActionType
{
public:
	FRCPropertyActionModel(URCPropertyAction* InPropertyAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionModel(InPropertyAction, InBehaviourItem, InRemoteControlPanel),
		FRCPropertyActionType(InPropertyAction)	{ }

	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyWidget();
	}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyTypeColor();
	}
};

/*
* ~ FRCFunctionActionModel ~
*
* UI model for Function based Actions
*/
class FRCFunctionActionModel : public FRCActionModel, public FRCFunctionActionType
{
public:
	FRCFunctionActionModel(URCFunctionAction* InFunctionAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionModel(InFunctionAction, InBehaviourItem, InRemoteControlPanel)
		, FRCFunctionActionType(InFunctionAction) { }

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetFunctionTypeColor();
	}
};

/**
 * FRCPropertyIdActionModel
 *
 * UI model for PropertyId based Actions
 */
class FRCPropertyIdActionModel : public FRCActionModel, public FRCPropertyIdActionType
{
public:
	FRCPropertyIdActionModel(URCPropertyIdAction* InPropertyIdAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionModel(InPropertyIdAction, InBehaviourItem, InRemoteControlPanel)
		, FRCPropertyIdActionType(InPropertyIdAction)
	{}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyIdTypeColor();
	}

	/** Widget representing Action Name field */
	virtual TSharedRef<SWidget> GetNameWidget() const override
	{
		return GetPropertyIdNameWidget();
	}

	/** Widget representing the Value field */
	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyIdValueWidget();
	}
};
