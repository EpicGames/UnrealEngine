// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCActionRangeMapModel.h"

#include "Behaviour/Builtin/RangeMap/RCRangeMapBehaviour.h"
#include "Controller/RCController.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCVirtualPropertyWidget.h"
#include "UI/Behaviour/Builtin/RangeMap/RCBehaviourRangeMapModel.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelDragHandle.h"
#include "UI/SRCPanelExposedEntity.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCActionRangeMapModel"

namespace UE::RCActionPanelRangeMapList
{
	namespace Columns
	{
		const FName TypeColorTag = TEXT("TypeColorTag");
		const FName DragDropHandle = TEXT("DragDropHandle");
		const FName Input = TEXT("Input");
		const FName Name = TEXT("Name");
		const FName Value = TEXT("Value");
	}

	class SActionItemListRow : public SMultiColumnTableRow<TSharedRef<FRCActionModel>>
	{
	public:
		void Construct(const FTableRowArgs& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, TSharedRef<FRCActionRangeMapModel> InActionItem)
		{
			ActionItem = InActionItem;
			FSuperRowType::Construct(InArgs, OwnerTableView);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (!ensure(ActionItem.IsValid()))
				return SNullWidget::NullWidget;

			if (ColumnName == UE::RCActionPanelRangeMapList::Columns::TypeColorTag)
			{
				return ActionItem->GetTypeColorTagWidget();
			}
			else if (ColumnName == UE::RCActionPanelRangeMapList::Columns::Input)
			{
				return ActionItem->GetInputWidget();
			}
			else if (ColumnName == UE::RCActionPanelRangeMapList::Columns::Name)
			{
				return ActionItem->GetNameWidget();
			}
			else if (ColumnName == UE::RCActionPanelRangeMapList::Columns::Value)
			{
				return ActionItem->GetWidget();
			}

			// TODO: Implement Drag-Drop-handle column with Actions reordering support

			return SNullWidget::NullWidget;
		}

	private:
		TSharedPtr<FRCActionRangeMapModel> ActionItem;
	};
}

TSharedPtr<SHeaderRow> FRCActionRangeMapModel::GetHeaderRow()
{
	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	return SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)

		+ SHeaderRow::Column(UE::RCActionPanelRangeMapList::Columns::TypeColorTag)
		.DefaultLabel(FText())
		.FixedWidth(5.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelRangeMapList::Columns::DragDropHandle)
		.DefaultLabel(FText())
		.FixedWidth(25.f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelRangeMapList::Columns::Input)
		.DefaultLabel(LOCTEXT("RCActionInputColumnHeader", "Input"))
		.FillWidth(0.2f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelRangeMapList::Columns::Name)
		.DefaultLabel(LOCTEXT("RCActionDescColumnHeader", "Description"))
		.FillWidth(0.4f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

		+ SHeaderRow::Column(UE::RCActionPanelRangeMapList::Columns::Value)
		.DefaultLabel(LOCTEXT("RCActionValueColumnHeader", "Value"))
		.FillWidth(0.4f)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);
}

FRCActionRangeMapModel::FRCActionRangeMapModel(URCAction* InAction, const TSharedPtr<FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCActionModel(InAction, InBehaviourItem, InRemoteControlPanel)
{
	SAssignNew(ConditionWidget, STextBlock);
}

TSharedPtr<FRCActionRangeMapModel> FRCActionRangeMapModel::GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
{
	if (URCPropertyAction* PropertyAction = Cast<URCPropertyAction>(InAction))
	{
		return MakeShared<FRCPropertyActionRangeMapModel>(PropertyAction, InBehaviourItem, InRemoteControlPanel);
	}
	else if (URCFunctionAction* FunctionAction = Cast<URCFunctionAction>(InAction))
	{
		return MakeShared<FRCFunctionActionRangeMapModel>(FunctionAction, InBehaviourItem, InRemoteControlPanel);
	}

	return nullptr;
}

TSharedRef<SWidget> FRCActionRangeMapModel::GetInputWidget()
{
	if (const TSharedPtr<FRCRangeMapBehaviourModel> BehaviourItem = StaticCastSharedPtr<FRCRangeMapBehaviourModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCRangeMapBehaviour* Behaviour = Cast<URCRangeMapBehaviour>(BehaviourItem->GetBehaviour()))
		{
			if (const FRCRangeMapInput* RangeMapInput = Behaviour->RangeMapActionContainer.Find(GetAction()))
			{
				if (ensure(RangeMapInput->InputProperty))
				{
					return SAssignNew(EditableVirtualPropertyWidget, SRCVirtualPropertyWidget, RangeMapInput->InputProperty)
						.OnGenerateWidget(this, &FRCActionRangeMapModel::OnGenerateInputWidget)
						.OnExitingEditMode(this, &FRCActionRangeMapModel::OnExitingEditingMode);
				}
			}
		}
	}
	
	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FRCActionRangeMapModel::OnGenerateInputWidget(class URCVirtualPropertySelfContainer* StepValueProperty) const
{
	if (!ensure(StepValueProperty))
	{
		return SNullWidget::NullWidget;
	}

	if (const TSharedPtr<FRCRangeMapBehaviourModel> BehaviourItem = StaticCastSharedPtr<FRCRangeMapBehaviourModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCRangeMapBehaviour* Behaviour = Cast<URCRangeMapBehaviour>(BehaviourItem->GetBehaviour()))
		{
			if (const FRCRangeMapInput* RangeInput = Behaviour->RangeMapActionContainer.Find(GetAction()))
			{
				double Value;
				if(RangeInput->GetInputValue(Value))
				{
					const FText StepAsText = FText::FromString(FString::SanitizeFloat(Value));

					UpdateConditionWidget(StepAsText);

					if (ConditionWidget.IsValid())
					{
						return ConditionWidget.ToSharedRef();
					}
				}
			}
		}
	}

	return SNullWidget::NullWidget;
}

void FRCActionRangeMapModel::OnExitingEditingMode() const
{
	if (const TSharedPtr<FRCRangeMapBehaviourModel> BehaviourItem = StaticCastSharedPtr<FRCRangeMapBehaviourModel>(BehaviourItemWeakPtr.Pin()))
	{
		if (URCRangeMapBehaviour* Behaviour = Cast<URCRangeMapBehaviour>(BehaviourItem->GetBehaviour()))
		{
			if (const FRCRangeMapInput* RangeInput = Behaviour->RangeMapActionContainer.Find(GetAction()))
			{
				double Value;
				if(RangeInput->GetInputValue(Value))
				{
					const FText StepAsText = FText::FromString(FString::SanitizeFloat(Value));

					UpdateSelectedRangeMapActionModel(Value, StepAsText);
				}
			}
		}
	}
}

TSharedRef<ITableRow> FRCActionRangeMapModel::OnGenerateWidgetForList(TSharedPtr<FRCActionRangeMapModel> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	typedef UE::RCActionPanelRangeMapList::SActionItemListRow ActionRowType;

	return SNew(ActionRowType, OwnerTable, InItem.ToSharedRef())
		.Style(&RCPanelStyle->TableRowStyle)
		.Padding(3.f);
}

void FRCActionRangeMapModel::UpdateConditionWidget(const FText& InNewText) const
{
	if (ConditionWidget.IsValid())
	{
		ConditionWidget->SetText(InNewText);
	}
}

void FRCActionRangeMapModel::UpdateSelectedRangeMapActionModel(double InNewValue, const FText& InNewConditionText) const
{
	// Update other selected action based on the edited one
	if (const TSharedPtr<SRemoteControlPanel> RCPanel = GetRemoteControlPanel())
	{
		if (const TSharedPtr<SRCActionPanel> LogicPanel = RCPanel->GetLogicActionPanel())
		{
			const TArray<TSharedPtr<FRCLogicModeBase>> SelectedLogicItems = LogicPanel->GetSelectedLogicItems();

			// If the selected items don't contain this one then skip it
			if (SelectedLogicItems.Contains(SharedThis(this)))
			{
				for (const TSharedPtr<FRCLogicModeBase>& SelectedLogicItem : SelectedLogicItems)
				{
					if (const TSharedPtr<FRCActionRangeMapModel> SelectedRangeMapActionModel = StaticCastSharedPtr<FRCActionRangeMapModel>(SelectedLogicItem))
					{
						if (const TSharedPtr<FRCBehaviourModel> SelectedBehaviourModel = SelectedRangeMapActionModel->GetParentBehaviour())
						{
							if (URCRangeMapBehaviour* SelectedBehaviour = Cast<URCRangeMapBehaviour>(SelectedBehaviourModel->GetBehaviour()))
							{
								if (const FRCRangeMapInput* SelectedRangeInput = SelectedBehaviour->RangeMapActionContainer.Find(SelectedRangeMapActionModel->GetAction()))
								{
									if (SelectedRangeInput->SetInputValue(InNewValue))
									{
										SelectedRangeMapActionModel->UpdateConditionWidget(InNewConditionText);
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
