// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyIdAction.h"
#include "Action/RCPropertyAction.h"
#include "UI/Action/RCActionModel.h"

class STextBlock;

/*
 * ~ FRCActionRangeMapModel ~
 *
 * UI Model for representing Actions as Input for the RangeMap Behaviour.
 * Contains and represents, other than name and value, a step value being a value between 0 to 1.
 *
 * <Input Value> <Name> <Value>
 */
class FRCActionRangeMapModel : public FRCActionModel
{
public:
	FRCActionRangeMapModel(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	/**
	 * Simple Widget representing the Normalized Value or Input Value for the Range and this particular Value.
	 */
	virtual TSharedRef<SWidget> GetInputWidget();

	/** OnGenerateRow delegate for the Actions List View*/
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCActionRangeMapModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Get the Header Row appropriate for this particular Action model */
	static TSharedPtr<SHeaderRow> GetHeaderRow();

	/** Chooses the appropriate Action model for the current class and field type*/
	static TSharedPtr<FRCActionRangeMapModel> GetModelByActionType(URCAction* InAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

private:
	/** Update the condition widget text with the new text given */
	void UpdateConditionWidget(const FText& InNewText) const;

	/** Called when updating the condition to update every other actions currently selected with the new condition */
	void UpdateSelectedRangeMapActionModel(double InNewValue, const FText& InNewConditionText) const;

	/** Called when constructing the widget or after editing the condition value */
	TSharedRef<SWidget> OnGenerateInputWidget(class URCVirtualPropertySelfContainer* InComparand) const;

	/** Called when exiting edit mode to update other selected actions */
	void OnExitingEditingMode() const;

	/** Holding the widget that display the condition as text */
	TSharedPtr<STextBlock> ConditionWidget;
};

/*
* ~ FRCPropertyActionRangeMapModel ~
*
* UI model for Property based RangeMap Actions
*/
class FRCPropertyActionRangeMapModel : public FRCActionRangeMapModel, public FRCPropertyActionType
{
public:
	FRCPropertyActionRangeMapModel(URCPropertyAction* InPropertyAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCActionRangeMapModel(InPropertyAction, InBehaviourItem, InRemoteControlPanel),
		FRCPropertyActionType(InPropertyAction) { }

	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyWidget();
	}

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyTypeColor();
	}
};

/*
* ~ FRCFunctionActionRangeMapModel ~
*
* UI model for Function based RangeMap Actions
*/
class FRCFunctionActionRangeMapModel : public FRCActionRangeMapModel, public FRCFunctionActionType
{
public:
	FRCFunctionActionRangeMapModel (URCFunctionAction* InFunctionAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionRangeMapModel(InFunctionAction, InBehaviourItem, InRemoteControlPanel)
		, FRCFunctionActionType(InFunctionAction) { }

	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetFunctionTypeColor();
	}
};

/**
 * FRCPropertyIdActionRangeMapModel
 *
 * UI model for PropertyId based RangeMap Actions
 */
class FRCPropertyIdActionRangeMapModel : public FRCActionRangeMapModel, public FRCPropertyIdActionType
{
public:
	FRCPropertyIdActionRangeMapModel(URCPropertyIdAction* InPropertyIdAction, const TSharedPtr<class FRCBehaviourModel> InBehaviourItem, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
		: FRCActionRangeMapModel(InPropertyIdAction, InBehaviourItem, InRemoteControlPanel)
		, FRCPropertyIdActionType(InPropertyIdAction)
	{}
	/** Color code for this Action*/
	virtual FLinearColor GetActionTypeColor() const override
	{
		return GetPropertyIdTypeColor();
	}
	/** Widget representing Action Name field */
	virtual TSharedRef<SWidget> GetNameWidget() const override
	{
		return GetPropertyIdNameWidget();
	}
	/** Widget representing the Value field */
	virtual TSharedRef<SWidget> GetWidget() const override
	{
		return GetPropertyIdValueWidget();
	}
};
