// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCActionPanel.h"

#include "RemoteControlField.h"
#include "RemoteControlPropertyIdRegistry.h"
#include "RemoteControlPreset.h"

#include "Action/RCActionContainer.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyIdAction.h"
#include "Action/RCPropertyAction.h"

#include "Controller/RCController.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Behaviour/Builtin/RCBehaviourOnValueChangedNode.h"
#include "Behaviour/RCBehaviour.h"

#include "EdGraphSchema_K2.h"
#include "EdGraph/EdGraphPin.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Misc/MessageDialog.h"

#include "SlateOptMacros.h"
#include "SRCActionPanelList.h"
#include "SRCBehaviourDetails.h"
#include "Styling/RemoteControlStyles.h"

#include "UI/Behaviour/RCBehaviourModel.h"
#include "UI/Panels/SRCDockPanel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRemoteControlPanel.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SRCActionPanel"

TSharedRef<SBox> SRCActionPanel::GetNoneSelectedWidget()
{
	return SNew(SBox)
	.Padding(0.f)
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("NoneSelected", "Select a behavior to view its actions."))
		.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
		.Justification(ETextJustify::Center)
	];
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCActionPanel::Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	SRCLogicPanelBase::Construct(SRCLogicPanelBase::FArguments(), InPanel);
	
	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	UpdateWrappedWidget();
	
	ChildSlot
		.Padding(RCPanelStyle->PanelPadding)
		[
			WrappedBoxWidget.ToSharedRef()
		];

	// Register delegates
	InPanel->OnBehaviourSelectionChanged.AddSP(this, &SRCActionPanel::OnBehaviourSelectionChanged);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCActionPanel::OnBehaviourSelectionChanged(TSharedPtr<FRCBehaviourModel> InBehaviourItem)
{
	SelectedBehaviourItemWeakPtr = InBehaviourItem;
	UpdateWrappedWidget(InBehaviourItem);
}

void SRCActionPanel::UpdateWrappedWidget(TSharedPtr<FRCBehaviourModel> InBehaviourItem)
{
	if (!AreActionPanelWidgetsValid())
	{
		CreateActionPanelWidgets();
	}

	// Update Behavior details widget
	BehaviourDetailsWidget->SetNewBehavior(InBehaviourItem);

	if (InBehaviourItem.IsValid())
	{
		// Create action list based on the behavior
		ActionPanelList = InBehaviourItem->GetActionsListWidget(SharedThis(this));
		ActionDockPanel->SetContent(ActionPanelList.ToSharedRef());

		const bool bIsBehaviourEnabled = InBehaviourItem->IsBehaviourEnabled();
		InBehaviourItem->RefreshIsBehaviourEnabled(bIsBehaviourEnabled);
		RefreshIsBehaviourEnabled(bIsBehaviourEnabled);
	}
}

FReply SRCActionPanel::OnClickOverrideBlueprintButton()
{
	if (!SelectedBehaviourItemWeakPtr.IsValid())
	{
		return FReply::Unhandled();
	}

	if (TSharedPtr<FRCBehaviourModel> Behaviour = SelectedBehaviourItemWeakPtr.Pin())
	{
		Behaviour->OnOverrideBlueprint();
	}

	return FReply::Handled();
}

void SRCActionPanel::SetIsBehaviourEnabled(const bool bIsEnabled)
{
	if (BehaviourDetailsWidget)
	{
		BehaviourDetailsWidget->SetIsBehaviourEnabled(bIsEnabled);
	}
}

void SRCActionPanel::RefreshIsBehaviourEnabled(const bool bIsEnabled)
{
	if (ActionPanelList)
	{
		ActionPanelList->SetEnabled(bIsEnabled);
	}
}

TSharedRef<SWidget> SRCActionPanel::GetActionMenuContentWidget()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	if (!SelectedBehaviourItemWeakPtr.IsValid())
	{
		return MenuBuilder.MakeWidget();
	}

	if (const URCBehaviour* Behaviour = SelectedBehaviourItemWeakPtr.Pin()->GetBehaviour())
	{
		if (Behaviour->IsA<URCBehaviourConditional>() || Behaviour->IsA<URCBehaviourOnValueChangedNode>())
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("PropertyIdTitle", "PropertyId"));
			// Create property identity entry
			FUIAction PropertyIdAction(FExecuteAction::CreateSP(this, &SRCActionPanel::OnAddActionClicked));
			MenuBuilder.AddMenuEntry(
				LOCTEXT("AddPropertyIdAction", "Add PropertyId (Property)"),
				LOCTEXT("AddPropertyIdAction_Tooltip", "Add a PropertyId action."),
				FSlateIcon(),
				MoveTemp(PropertyIdAction));

			if (URemoteControlPreset* Preset = GetPreset())
			{
				if (Preset->GetPropertyIdRegistry())
				{
					TSet<FName> IdList = Preset->GetPropertyIdRegistry().Get()->GetFullPropertyIdsNamePossibilitiesList();
					if (!IdList.IsEmpty())
					{
						MenuBuilder.AddSubMenu(
						LOCTEXT("AddActionSubMenu", "Add specific ID action"),
						LOCTEXT("AddActionSubMenu_ToolTip", "Choose the ID based on the current list of different you have"),
						FNewMenuDelegate::CreateLambda([this, IdList](FMenuBuilder& InMenuBuilder)
						{
							for (FName Id : IdList)
							{
								FText LabelName = FText::FromString(TEXT("ID: ") + Id.ToString());
								const FText ToolTipName = LOCTEXT("AddAction_SpecificToolTip", "Create an action widget with this Id");
								FUIAction PropertyIdAction(FExecuteAction::CreateSP(this, &SRCActionPanel::OnAddActionClicked, Id));
								InMenuBuilder.AddMenuEntry(LabelName, ToolTipName, FSlateIcon(), PropertyIdAction);
							}
						}));
					}
				}
			}
			MenuBuilder.EndSection();
		}
	}

	// List of exposed entities
	if (URemoteControlPreset* Preset = GetPreset())
	{
		const TArray<TWeakPtr<FRemoteControlField>>& RemoteControlFields = Preset->GetExposedEntities<FRemoteControlField>();

		if (!RemoteControlFields.IsEmpty())
		{
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("FieldsTitle", "Fields"));
		}

		for (const TWeakPtr<FRemoteControlField>& RemoteControlFieldWeakPtr : RemoteControlFields)
		{
			if (const TSharedPtr<FRemoteControlField> RemoteControlField = RemoteControlFieldWeakPtr.Pin())
			{
				if (const URCBehaviour* Behaviour = SelectedBehaviourItemWeakPtr.Pin()->GetBehaviour())
				{
					// Skip if we already have an Action created for this exposed entity
					if(!Behaviour->CanHaveActionForField(RemoteControlField))
					{
						continue;
					}

					// Create menu entry
					FUIAction Action(FExecuteAction::CreateSP(this, &SRCActionPanel::OnAddActionClicked, RemoteControlField));
					MenuBuilder.AddMenuEntry(
						FText::Format(LOCTEXT("AddAction", "{0}"), FText::FromName(RemoteControlField->GetLabel())),
						FText::Format(LOCTEXT("AddActionTooltip", "Add {0}"), FText::FromName(RemoteControlField->GetLabel())),
						FSlateIcon(),
						MoveTemp(Action));
				}
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void SRCActionPanel::CreateActionPanelWidgets()
{
	// Action Dock Panel
	if (!ActionDockPanel.IsValid())
	{
		ActionDockPanel = SNew(SRCMinorPanel)
			.HeaderLabel(LOCTEXT("ActionsLabel", "Actions"))
			.EnableHeader(true)
			.EnableFooter(false);
	}

	bool bIsAnyToolbarItemCreated = false;

	// Add new action combo button
	if (!AddNewActionButton.IsValid())
	{
		bIsAnyToolbarItemCreated = true;

		AddNewActionButton = SNew(SComboButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Action")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.ForegroundColor(FSlateColor::UseForeground())
			.CollapseMenuOnParentFocus(true)
			.HasDownArrow(false)
			.ContentPadding(FMargin(4.f, 2.f))
			.ButtonContent()
			[
				SNew(SBox)
				.WidthOverride(RCPanelStyle->IconSize.X)
				.HeightOverride(RCPanelStyle->IconSize.Y)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
				]
			]
			.MenuContent()
			[
				GetActionMenuContentWidget()
			];
		AddNewActionButton->SetOnGetMenuContent(FOnGetContent::CreateSP(this, &SRCActionPanel::GetActionMenuContentWidget));
	}

	// Add all button
	if (!AddAllActionsButton.IsValid())
	{
		bIsAnyToolbarItemCreated = true;

		AddAllActionsButton = SNew(SButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add All Actions")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.ToolTipText(LOCTEXT("AddAllToolTip", "Adds all the available actions."))
			.OnClicked(this, &SRCActionPanel::OnAddAllFields)
			.Visibility(this, &SRCActionPanel::HandleAddAllButtonVisibility)
			[
				SNew(SBox)
				.WidthOverride(RCPanelStyle->IconSize.X)
				.HeightOverride(RCPanelStyle->IconSize.Y)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.Duplicate"))
				]
			];
	}

	// Add all selected button
	if (!AddAllSelectedActionsButton.IsValid())
	{
		bIsAnyToolbarItemCreated = true;

		AddAllSelectedActionsButton = SNew(SButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add All Selected Fields")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.ToolTipText(LOCTEXT("RCAddAllSelectedToolTip", "Adds all the selected fields."))
			.OnClicked(this, &SRCActionPanel::OnAddAllSelectedFields)
			.Visibility(this, &SRCActionPanel::HandleAddAllButtonVisibility)
			[
				SNew(SBox)
				.WidthOverride(RCPanelStyle->IconSize.X)
				.HeightOverride(RCPanelStyle->IconSize.Y)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("DataLayerBrowser.AddSelection"))
				]
			];
	}

	// Add all selected button
	if (!SelectSourceButton.IsValid())
	{
		bIsAnyToolbarItemCreated = true;

		SelectSourceButton = SNew(SButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Select Action Property Source")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.ToolTipText(LOCTEXT("RCSelectSourceToolTip", "Select the source of the property this action is bound to."))
			.OnClicked(this, &SRCActionPanel::OnSelectSource)
			.Visibility(this, &SRCActionPanel::HandleSelectSourceVisibility)
			[
				SNew(SBox)
				.WidthOverride(RCPanelStyle->IconSize.X)
				.HeightOverride(RCPanelStyle->IconSize.Y)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.SelectInViewport"))
				]
			];
	}

	// Adding header toolbar items
	if (bIsAnyToolbarItemCreated)
	{
		ActionDockPanel->ClearHeaderToolbarItems();
		ActionDockPanel->AddHeaderToolbarItem(EToolbar::Left, AddNewActionButton.ToSharedRef());
		ActionDockPanel->AddHeaderToolbarItem(EToolbar::Right, SelectSourceButton.ToSharedRef());
		ActionDockPanel->AddHeaderToolbarItem(EToolbar::Right, AddAllSelectedActionsButton.ToSharedRef());
		ActionDockPanel->AddHeaderToolbarItem(EToolbar::Right, AddAllActionsButton.ToSharedRef());
	}

	// Behavior Details widget
	if (!BehaviourDetailsWidget.IsValid())
	{
		BehaviourDetailsWidget = SNew(SRCBehaviourDetails, SharedThis(this), nullptr);
	}

	// Behavior Details panel
	if (!BehaviorDetailsPanel.IsValid())
	{
		BehaviorDetailsPanel = SNew(SRCMinorPanel)
			.EnableHeader(false)
			.Visibility_Lambda([this] ()
			{
				if (const TSharedPtr<FRCBehaviourModel>& Behavior = SelectedBehaviourItemWeakPtr.Pin())
				{
					return Behavior->HasBehaviourDetailsWidget()? EVisibility::Visible : EVisibility::Collapsed;
				}
				return EVisibility::Collapsed;
			})
		[
			BehaviourDetailsWidget.ToSharedRef()
		];
	}

	// Action Major Panel
	if (!ActionsPanel.IsValid())
	{
		ActionsPanel = SNew(SRCMajorPanel)
			.EnableFooter(false)
			.EnableHeader(false)
			.ChildOrientation(Orient_Vertical);

		if (BehaviorDetailsPanel.IsValid())
		{
			// Panel size of zero forces use of "SizeToContent" ensuring that each Behaviour only takes up as much space as necessary
			ActionsPanel->AddPanel(BehaviorDetailsPanel.ToSharedRef(), 0.f);
		}
		ActionsPanel->AddPanel(ActionDockPanel.ToSharedRef(), 0.5f);
	}

	// Wrapped Box
	if (!WrappedBoxWidget.IsValid())
	{
		WrappedBoxWidget = SNew(SBox)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this] () { return SelectedBehaviourItemWeakPtr.IsValid()? 0 : 1; })

			// Index 0 is for valid behavior
			+ SWidgetSwitcher::Slot()
			[
				ActionsPanel.ToSharedRef()
			]

			// Index 1 is for invalid behavior
			+ SWidgetSwitcher::Slot()
			[
				GetNoneSelectedWidget()
			]
		];
	}
}

bool SRCActionPanel::AreActionPanelWidgetsValid() const
{
	return WrappedBoxWidget.IsValid() &&
			ActionDockPanel.IsValid() &&
			AddNewActionButton.IsValid() &&
			AddAllActionsButton.IsValid() &&
			AddAllSelectedActionsButton.IsValid() &&
			BehaviorDetailsPanel.IsValid() &&
			ActionsPanel.IsValid();
}

URCAction* SRCActionPanel::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
	{
		if (const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour())
		{
			Behaviour->ActionContainer->Modify();

			URCAction* NewAction = BehaviourItem->AddAction(InRemoteControlField);

			if (NewAction)
			{
				AddNewActionToList(NewAction);

				// Broadcast new Action to other panels
				if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
				{
					RemoteControlPanel->OnActionAdded.Broadcast(NewAction);
				}

				return NewAction;
			}
		}
	}

	return nullptr;
}

bool SRCActionPanel::CanHaveActionForField(const FGuid& InRemoteControlFieldId)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (TSharedPtr<FRemoteControlField> RemoteControlField = Preset->GetExposedEntity<FRemoteControlField>(InRemoteControlFieldId).Pin())
		{
			if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
			{
				if (const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour())
				{
					return Behaviour->CanHaveActionForField(RemoteControlField);
				}
			}
		}
	}

	return false;
}

void SRCActionPanel::OnAddActionClicked(TSharedPtr<FRemoteControlField> InRemoteControlField)
{
	if (!SelectedBehaviourItemWeakPtr.IsValid() || !InRemoteControlField)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddActionTransaction", "Add Action"));

	AddAction(InRemoteControlField.ToSharedRef());
}

void SRCActionPanel::OnAddActionClicked()
{
	if (!SelectedBehaviourItemWeakPtr.IsValid())
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("AddActionTransaction", "Add Action"));
	AddAction();
}

void SRCActionPanel::OnAddActionClicked(FName InFieldId)
{
	if (!SelectedBehaviourItemWeakPtr.IsValid())
	{
		return;
	}
	FScopedTransaction Transaction(LOCTEXT("AddActionTransaction", "Add Action"));
	AddAction(InFieldId);
}

FReply SRCActionPanel::OnClickEmptyButton()
{
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
	{
		if (const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour())
		{
			FScopedTransaction Transaction(LOCTEXT("EmptyActionsTransaction", "Empty Actions"));
			Behaviour->ActionContainer->Modify();

			Behaviour->ActionContainer->EmptyActions();
		}
	}

	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		RemoteControlPanel->OnEmptyActions.Broadcast();
	}

	return FReply::Handled();
}

FReply SRCActionPanel::OnAddAllFields()
{
	if (!SelectedBehaviourItemWeakPtr.IsValid())
	{
		return FReply::Handled();
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
		{
			const TArray<TWeakPtr<FRemoteControlField>>& RemoteControlFields = Preset->GetExposedEntities<FRemoteControlField>();

			FScopedTransaction Transaction(LOCTEXT("AddAllActionsTransaction", "Add All Actions"));

			// Enumerate the list of Exposed Entities and Functions available in this Preset for our use as Actions
			for (const TWeakPtr<FRemoteControlField>& RemoteControlFieldWeakPtr : RemoteControlFields)
			{
				if (const TSharedPtr<FRemoteControlField> RemoteControlField = RemoteControlFieldWeakPtr.Pin())
				{
					URCBehaviour* Behaviour = BehaviourItem->GetBehaviour();
					
					// Only add the Behaviour if it's listed as addable.
					if (Behaviour && Behaviour->CanHaveActionForField(RemoteControlField))
					{
						AddAction(RemoteControlField.ToSharedRef());
					}
				}
			}
		}
	}

	return FReply::Handled();
}

FReply SRCActionPanel::OnAddAllSelectedFields()
{
	if (!SelectedBehaviourItemWeakPtr.IsValid() || !PanelWeakPtr.IsValid())
	{
		return FReply::Handled();
	}

	const TSharedPtr<SRemoteControlPanel> RCPanel = PanelWeakPtr.Pin();
	if (const TSharedPtr<SRCPanelExposedEntitiesList> RCEntitiesList = RCPanel->GetEntityList())
	{
		if (const TSharedPtr<FRCBehaviourModel>& BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
		{
			FScopedTransaction Transaction(LOCTEXT("RCAddAllSelectedActionsTransaction", "Add All Selected Fields"));

			auto AddSelectedActionLambda = [this, BehaviourItem] (const TSharedPtr<SRCPanelTreeNode>& RCEntity)
			{
				if (URemoteControlPreset* Preset = GetPreset())
				{
					const TWeakPtr<FRemoteControlField> RCWeakField = Preset->GetExposedEntity<FRemoteControlField>(RCEntity->GetRCId());
					if (const TSharedPtr<FRemoteControlField> RCField = RCWeakField.Pin())
					{
						const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour();

						if (Behaviour && Behaviour->CanHaveActionForField(RCField))
						{
							AddAction(RCField.ToSharedRef());
						}
					}
				}
			};

			for (const TSharedPtr<SRCPanelTreeNode>& RCEntity : RCEntitiesList->GetSelectedEntities())
			{
				if (RCEntity->GetRCId().IsValid())
				{
					AddSelectedActionLambda(RCEntity);
				}
				else if (RCEntity->GetRCType() == SRCPanelTreeNode::FieldGroup)
				{
					if (const TSharedPtr<SRCPanelExposedEntitiesGroup> RCFieldGroup = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(RCEntity))
					{
						if (RCFieldGroup->GetGroupType() == ERCFieldGroupType::PropertyId)
						{
							const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour();
							if (Behaviour && Behaviour->SupportPropertyId())
							{
								AddAction(RCFieldGroup->GetFieldKey());
							}
						}
						else
						{
							TArray<TSharedPtr<SRCPanelTreeNode>> RCFieldGroupEntities;
							RCFieldGroup->GetNodeChildren(RCFieldGroupEntities);

							for (const TSharedPtr<SRCPanelTreeNode>& RCFieldGroupEntity : RCFieldGroupEntities)
							{
								if (RCFieldGroupEntity->GetRCId().IsValid())
								{
									AddSelectedActionLambda(RCFieldGroupEntity);
								}
							}
						}
					}
				}
			}
		}
	}

	return FReply::Handled();
}

FReply SRCActionPanel::OnSelectSource()
{
	if (ActionPanelList.IsValid())
	{
		ActionPanelList->SelectSource_Execute();
	}

	return FReply::Handled();
}

bool SRCActionPanel::IsListFocused() const
{
	return ActionPanelList.IsValid() && ActionPanelList->IsListFocused();
}

void SRCActionPanel::DeleteSelectedPanelItems()
{
	ActionPanelList->DeleteSelectedPanelItems();
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCActionPanel::GetSelectedLogicItems() const
{
	if (ActionPanelList)
	{
		return ActionPanelList->GetSelectedLogicItems();
	}

	return {};
}

void SRCActionPanel::DuplicateAction(URCAction* InAction)
{
	URCBehaviour* BehaviourTarget = nullptr;
	URCBehaviour* BehaviourSource = nullptr;

	// Behaviour Target - The Behaviour into which the input Action is to copied
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
	{
		BehaviourTarget = BehaviourItem->GetBehaviour();
		if (!ensure(BehaviourTarget))
		{
			return;
		}
	}
	
	// Behaviour Source - The Behaviour holding the input Action
	if (InAction)
	{
		BehaviourSource = InAction->GetParentBehaviour();
		if (!ensure(BehaviourSource))
		{
			return;
		}
	}

	URCAction* NewAction = BehaviourSource->DuplicateAction(InAction, BehaviourTarget);

	AddNewActionToList(NewAction);

	// Broadcast new Action to other panels
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		RemoteControlPanel->OnActionAdded.Broadcast(NewAction);
	}
}

void SRCActionPanel::AddNewActionToList(URCAction* NewAction)
{
	if(ActionPanelList.IsValid())
	{
		ActionPanelList->AddNewLogicItem(NewAction);
	}
}

void SRCActionPanel::DuplicateSelectedPanelItems()
{
	if (!ensure(ActionPanelList.IsValid()))
	{
		return;
	}

	for (const TSharedPtr<FRCLogicModeBase>& Action : GetSelectedLogicItems())
	{
		if (const TSharedPtr<FRCActionModel> ActionItem = StaticCastSharedPtr<FRCActionModel>(Action))
		{
			DuplicateAction(ActionItem->GetAction());
		}
	}
}

void SRCActionPanel::CopySelectedPanelItems()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		TArray<UObject*> ItemsToCopy;
		const TArray<TSharedPtr<FRCLogicModeBase>> LogicItems = GetSelectedLogicItems();
		ItemsToCopy.Reserve(LogicItems.Num());

		for (const TSharedPtr<FRCLogicModeBase>& LogicItem : LogicItems)
		{
			if (const TSharedPtr<FRCActionModel> ActionItem = StaticCastSharedPtr<FRCActionModel>(LogicItem))
			{
				ItemsToCopy.Add(ActionItem->GetAction());
			}
		}

		RemoteControlPanel->SetLogicClipboardItems(ItemsToCopy, SharedThis(this));
	}
}

void SRCActionPanel::PasteItemsFromClipboard()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		if (RemoteControlPanel->LogicClipboardItemSource == SharedThis(this))
		{
			for (UObject* LogicClipboardItem : RemoteControlPanel->GetLogicClipboardItems())
			{
				if(URCAction* Action = Cast<URCAction>(LogicClipboardItem))
				{
					DuplicateAction(Action);
				}
			}
		}
	}
}

bool SRCActionPanel::CanPasteClipboardItems(const TArrayView<const TObjectPtr<UObject>> InLogicClipboardItems) const
{
	for (const UObject* LogicClipboardItem : InLogicClipboardItems)
	{
		const URCAction* LogicClipboardAction = Cast<URCAction>(LogicClipboardItem);
		if (!LogicClipboardAction)
		{
			return false;
		}

		if (const URCBehaviour* BehaviourSource = LogicClipboardAction->GetParentBehaviour())
		{
			if (const TSharedPtr<FRCBehaviourModel>& BehaviourItemTarget = SelectedBehaviourItemWeakPtr.Pin())
			{
				if (const URCBehaviour* BehaviourTarget = BehaviourItemTarget->GetBehaviour())
				{
					// Copy-paste is allowed between compatible Behaviour types only
					//
					return BehaviourSource->GetClass() == BehaviourTarget->GetClass();
				}
			}
		}
	}

	return false;
}

void SRCActionPanel::UpdateValue()
{
	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (const TSharedPtr<FRCActionModel>& ActionLogicItem = StaticCastSharedPtr<FRCActionModel>(LogicItem))
		{
			if (const URCPropertyAction* RCPropertyAction = Cast<URCPropertyAction>(ActionLogicItem->GetAction()))
			{
				RCPropertyAction->UpdateValueBasedOnRCProperty();
			}
		}
	}
}

bool SRCActionPanel::CanUpdateValue() const
{
	const TArray<TSharedPtr<FRCLogicModeBase>>& LogicItems = GetSelectedLogicItems();

	if (LogicItems.IsEmpty())
	{
		return false;
	}

	if (const TSharedPtr<FRCActionModel> ActionLogicItem = StaticCastSharedPtr<FRCActionModel>(LogicItems[0]))
	{
		if (const TSharedPtr<FRCBehaviourModel> ParentBehaviour = ActionLogicItem->GetParentBehaviour())
		{
			if (const URCBehaviour* Behaviour = ParentBehaviour->GetBehaviour())
			{
				if (Behaviour->IsA<URCBehaviourBind>())
				{
					return false;
				}
			}
		}
	}

	for (const TSharedPtr<FRCLogicModeBase>& ActionItem : LogicItems)
	{
		if (const TSharedPtr<FRCActionModel> ActionLogicItem = StaticCastSharedPtr<FRCActionModel>(ActionItem))
		{
			if (const URCAction* RCAction = ActionLogicItem->GetAction())
			{
				if (const URemoteControlPreset* RCPreset = GetPreset())
				{
					if (RCAction->IsA<URCPropertyAction>()	&&
						RCPreset->GetExposedEntity(RCAction->ExposedFieldId).IsValid())
					{
						// if at least one of the selected actions can update, then enable it
						return true;
					}
				}
			}
		}
	}
	return false;
}

FText SRCActionPanel::GetPasteItemMenuEntrySuffix()
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		// This function should only have been called if we were the source of the item copied.
		if (ensure(RemoteControlPanel->LogicClipboardItemSource == SharedThis(this)))
		{
			TArray<UObject*> LogicClipboardItems = RemoteControlPanel->GetLogicClipboardItems();

			if (LogicClipboardItems.Num() > 0)
			{
				if (const URCAction* Action = Cast<URCAction>(LogicClipboardItems[0]))
				{
					if (URCBehaviour* Behaviour = Action->GetParentBehaviour())
					{
						if (LogicClipboardItems.Num() > 1)
						{
							return FText::Format(LOCTEXT("ActionPanelPasteMenuMultiEntrySuffix", "Action {0} and {1} other(s)"), Behaviour->GetDisplayName(), (LogicClipboardItems.Num() - 1));
						}
						return FText::Format(LOCTEXT("ActionPanelPasteMenuEntrySuffix", "Action {0}"), Behaviour->GetDisplayName());
					}
				}
			}
		}
	}

	return FText::GetEmpty();
}

URCAction* SRCActionPanel::AddAction()
{
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
	{
		if (const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour())
		{
			Behaviour->ActionContainer->Modify();
			URCAction* NewAction = BehaviourItem->AddAction();
			AddNewActionToList(NewAction);
			// Broadcast new Action to other panels
			if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
			{
				RemoteControlPanel->OnActionAdded.Broadcast(NewAction);
			}
			return NewAction;
		}
	}
	return nullptr;
}

URCAction* SRCActionPanel::AddAction(FName InFieldId)
{
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = SelectedBehaviourItemWeakPtr.Pin())
	{
		if (const URCBehaviour* Behaviour = BehaviourItem->GetBehaviour())
		{
			Behaviour->ActionContainer->Modify();
			URCAction* NewAction = BehaviourItem->AddAction(InFieldId);
			AddNewActionToList(NewAction);
			// Broadcast new Action to other panels
			if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
			{
				RemoteControlPanel->OnActionAdded.Broadcast(NewAction);
			}
			return NewAction;
		}
	}
	return nullptr;
}

FReply SRCActionPanel::RequestDeleteSelectedItem()
{
	if (!ActionPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = LOCTEXT("DeleteActionWarning", "Delete the selected Actions?");

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		DeleteSelectedPanelItems();
	}

	return FReply::Handled();
}

FReply SRCActionPanel::RequestDeleteAllItems()
{
	if (!ActionPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = FText::Format(LOCTEXT("DeleteAllWarning", "You are about to delete {0} actions. Are you sure you want to proceed?"), ActionPanelList->Num());

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		return OnClickEmptyButton();
	}

	return FReply::Handled();
}

EVisibility SRCActionPanel::HandleAddAllButtonVisibility() const
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		return Preset->HasEntities() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

EVisibility SRCActionPanel::HandleSelectSourceVisibility() const
{
	if (ActionPanelList.IsValid())
	{
		if (ActionPanelList->NumSelectedLogicItems() > 0)
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

#undef LOCTEXT_NAMESPACE
