// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/BaseLogicUI/SRCLogicPanelBase.h"

class FRCActionModel;
class FRCBehaviourModel;
class SBox;
class SCheckBox;
class SComboButton;
class SRCBehaviourDetails;
class SRCLogicPanelListBase;
class SRCMajorPanel;
class SRCMinorPanel;
class SRemoteControlPanel;
class URCAction;
class URCBehaviour;

struct FRCPanelStyle;
struct FRemoteControlField;

enum class ECheckBoxState : uint8;

/*
* ~ SRCActionPanel ~
*
* UI Widget for Action Panel.
* Contains a header, footer, list of actions and a behaviour specific panel
*/
class REMOTECONTROLUI_API SRCActionPanel : public SRCLogicPanelBase
{
public:
	SLATE_BEGIN_ARGS(SRCActionPanel)
		{
		}

	SLATE_END_ARGS()
	
	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Whether the Actions list widget currently has focus.*/
	bool IsListFocused() const;

	/** Delete Items UI command implementation for this panel */
	virtual void DeleteSelectedPanelItems() override;

	/** Returns the UI items currently selected by the user (if any). To be implemented per child panel */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() const override;

	/** "Duplicate Items" UI command implementation for Action panel*/
	virtual void DuplicateSelectedPanelItems() override;

	/** "Copy Items" UI command implementation for Action panel*/
	virtual void CopySelectedPanelItems() override;

	/** "Paste Items" UI command implementation for Action panel*/
	virtual void PasteItemsFromClipboard() override;

	/** Whether clipboard items can be successfully pasted into this panel */
	virtual bool CanPasteClipboardItems(const TArrayView<const TObjectPtr<UObject>> InLogicClipboardItems) const override;

	/** "Update Value" UI command implementation for panels */
	virtual void UpdateValue() override;

	/** Whether this panel can call the UpdateValue command */
	virtual bool CanUpdateValue() const override;

	/** Provides an item suffix for the Paste context menu to provide users with useful context on the nature of the item being pasted */
	virtual FText GetPasteItemMenuEntrySuffix() override;

	/** Adds an PropertyId Action for the currently active Behaviour and broadcasts to parent panels */
	URCAction* AddAction();

	/** Adds an PropertyId Action for the currently active Behaviour and broadcasts to parent panels */
	URCAction* AddAction(FName InFieldId);

	/** Adds an Action for the currently active Behaviour and broadcasts to parent panels */
	URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Whether this Actions panel can create an action for the given remote control field */
	bool CanHaveActionForField(const FGuid& InRemoteControlFieldId);

	/** Set the Enabled state of our parent Behaviour */
	void SetIsBehaviourEnabled(const bool bIsEnabled);

	/** Fetches the currently selected behaviour item*/
	TSharedPtr<FRCBehaviourModel> GetSelectedBehaviourItem()
	{
		return SelectedBehaviourItemWeakPtr.Pin();
	}

	/** Refreshes the UI widgets enabled state depending on whether the parent behaviour is currently enabled */
	void RefreshIsBehaviourEnabled(const bool bIsEnabled);

protected:
	/** Warns user before deleting a selected panel item. */
	virtual FReply RequestDeleteSelectedItem() override;

	/** Warns user before deleting all items in a panel. */
	virtual FReply RequestDeleteAllItems() override;

private:
	/** Determines the visibility Add All Button. */
	EVisibility HandleAddAllButtonVisibility() const;

	/** Determines the visibility Select Source Button. */
	EVisibility HandleSelectSourceVisibility() const;

	/** 
	* Behaviour selection change listener.
	* Updates the list of actions from newly selected Behaviour
	*/
	void OnBehaviourSelectionChanged(TSharedPtr<FRCBehaviourModel> InBehaviourItem);

	/* Rebuilds the Action Panel for a newly selected Behaviour*/
	void UpdateWrappedWidget(TSharedPtr<FRCBehaviourModel> InBehaviourItem = nullptr);

	/** Handles click event for Open Behaviour Blueprint button*/
	FReply OnClickOverrideBlueprintButton();

	/**
	 * Builds a menu containing the list of all possible Actions
	 * These are derived from the list of Exposed entities of the Remote Control Preset associated with us.
	 */
	TSharedRef<SWidget> GetActionMenuContentWidget();

	/** Check the validity of the panel widgets and if not valid will create them */
	void CreateActionPanelWidgets();

	/** Return whether or not all panel widgets are valid */
	bool AreActionPanelWidgetsValid() const;

	/** Handles click event for Add Action button*/
	void OnAddActionClicked(TSharedPtr<FRemoteControlField> InRemoteControlField);

	/** Handles click event for Add Action (PropertyId) button */
	void OnAddActionClicked();

	/** Handles click event for Add specific PropertyId Action buttons */
	void OnAddActionClicked(FName InFieldId);

	/** Handles click event for Empty button; clear all Actions from the panel*/
	FReply OnClickEmptyButton();

	/** Handles click event for Add All button; Adds all possible actions for the active Remote Control Preset*/
	FReply OnAddAllFields();

	/** Handles click event for Add All Selected button; Adds all selected actions for the active Remote Control Preset*/
	FReply OnAddAllSelectedFields();

	/** Handles click event for Select Source button; Selects the actor/component in the scene whose properties are bound in this action*/
	FReply OnSelectSource();

private:
	/** Helper widget for behavior details. */
	static TSharedRef<SBox> GetNoneSelectedWidget();

	void DuplicateAction(URCAction* InAction);

	void AddNewActionToList(URCAction* NewAction);

	/** The parent Behaviour that this Action panel is associated with */
	TWeakPtr<FRCBehaviourModel> SelectedBehaviourItemWeakPtr;
	
	/** Parent Box for the entire widget */
	TSharedPtr<SBox> WrappedBoxWidget;

	/** Widget representing List of Actions */
	TSharedPtr<SRCLogicPanelListBase> ActionPanelList;

	/** Behaviour specific details widget*/
	TSharedPtr<SRCBehaviourDetails> BehaviourDetailsWidget;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle = nullptr;

	/** Action Minor Panel */
	TSharedPtr<SRCMinorPanel> ActionDockPanel;

	/** ComboButton to add new action */
	TSharedPtr<SComboButton> AddNewActionButton;

	/** Add all actions button */
	TSharedPtr<SWidget> AddAllActionsButton;

	/** Add all selected actions button */
	TSharedPtr<SWidget> AddAllSelectedActionsButton;

	/** Select source for action button */
	TSharedPtr<SWidget> SelectSourceButton;

	/** Action Major Panel */
	TSharedPtr<SRCMajorPanel> ActionsPanel;

	/** Behavior Details Minor Panel */
	TSharedPtr<SRCMinorPanel> BehaviorDetailsPanel;
};
