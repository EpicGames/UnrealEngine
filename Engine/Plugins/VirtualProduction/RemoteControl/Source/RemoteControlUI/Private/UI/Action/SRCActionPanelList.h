// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Action/RCAction.h"
#include "Action/RCActionContainer.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Commands/RemoteControlCommands.h"
#include "Framework/Commands/UIAction.h"
#include "RCActionModel.h"
#include "RemoteControlPreset.h"
#include "RemoteControlUIModule.h"
#include "SDropTarget.h"
#include "Selection.h"
#include "SlateOptMacros.h"
#include "SRCActionPanel.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/Bind/RCActionBindModel.h"
#include "UI/Action/Conditional/RCActionConditionalModel.h"
#include "UI/BaseLogicUI/SRCLogicPanelListBase.h"
#include "UI/Behaviour/RCBehaviourModel.h"
#include "UI/RCFieldGroupType.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelExposedEntitiesGroup.h"
#include "UI/SRCPanelExposedField.h"
#include "UI/SRCPanelFieldGroup.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "SRCActionPanelList"

class FRCActionModel;

/*
* ~ SRCActionPanelList ~
*
* UI Widget for Actions List
* Used as part of the RC Logic Actions Panel.
*/
template <typename ActionType
	UE_REQUIRES(TIsDerivedFrom<ActionType, FRCActionModel>::Value)>
class REMOTECONTROLUI_API SRCActionPanelList : public SRCLogicPanelListBase
{
public:
	SLATE_BEGIN_ARGS(SRCActionPanelList<ActionType>)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRCActionPanel> InActionPanel, TSharedPtr<FRCBehaviourModel> InBehaviourItem)
	{
		const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = InActionPanel->GetRemoteControlPanel();
		check(RemoteControlPanel);

		SRCLogicPanelListBase::Construct(SRCLogicPanelListBase::FArguments(), InActionPanel, RemoteControlPanel.ToSharedRef());

		const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

		CommandList->MapAction(
			Commands.SelectSource,
			FUIAction(FExecuteAction::CreateSP(this, &SRCActionPanelList::SelectSource_Execute))
		);

		ActionPanelWeakPtr = InActionPanel;
		BehaviourItemWeakPtr = InBehaviourItem;

		RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

		ListView = SNew(SListView<TSharedPtr<ActionType>>)
			.ListItemsSource(&ActionItems)
			.SelectionMode(ESelectionMode::Multi)
			.HeaderRow(ActionType::GetHeaderRow())
			.OnGenerateRow(this, &SRCActionPanelList::OnGenerateWidgetForList)
			.OnSelectionChanged(this, &SRCActionPanelList::OnSelectionChanged)
			.OnContextMenuOpening(this, &SRCLogicPanelListBase::GetContextMenuWidget);

		ChildSlot
			[
				SNew(SDropTarget)
				.VerticalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.VerticalDash"))
				.HorizontalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.HorizontalDash"))
				.OnDropped_Lambda([this](const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) { return SRCActionPanelList::OnExposedFieldDrop(InDragDropEvent.GetOperation()); })
				.OnAllowDrop(this, &SRCActionPanelList::OnAllowDrop)
				.OnIsRecognized(this, &SRCActionPanelList::OnAllowDrop)
				[
					ListView.ToSharedRef()
				]
			];

		// Add delegates
		RemoteControlPanel->OnActionAdded.AddSP(this, &SRCActionPanelList::OnActionAdded);
		RemoteControlPanel->OnEmptyActions.AddSP(this, &SRCActionPanelList::OnEmptyActions);

		if (URCBehaviour* Behaviour = InBehaviourItem->GetBehaviour())
		{
			if (Behaviour->ActionContainer)
			{
				Behaviour->ActionContainer->OnActionsListModified.AddSP(this, &SRCActionPanelList::OnActionsListModified);
			}
		}

		Reset();

		if (URemoteControlPreset* Preset = RemoteControlPanel->GetPreset())
		{
			Preset->GetPropertyIdRegistry()->OnPropertyIdActionNeedsRefresh().AddSP(this, &SRCActionPanelList::Refresh);
		}
	}

	virtual ~SRCActionPanelList()
	{
		if (ActionPanelWeakPtr.IsValid())
		{
			if (URemoteControlPreset* Preset = ActionPanelWeakPtr.Pin()->GetPreset())
			{
				Preset->GetPropertyIdRegistry()->OnPropertyIdActionNeedsRefresh().RemoveAll(this);
			}
		}
	}

	void OnActionsListModified()
	{
		Reset();
	}

	/** Returns true if the underlying list is valid and empty. */
	virtual bool IsEmpty() const override
	{
		return ActionItems.IsEmpty();
	}

	/** Returns number of items in the list. */
	virtual int32 Num() const override
	{
		return ActionItems.Num();
	}

	/** The number of Controllers currently selected */
	virtual int32 NumSelectedLogicItems() const override
	{
		return ListView->GetNumItemsSelected();
	}

	/** Whether the Actions List View currently has focus. */
	virtual bool IsListFocused() const override
	{
		return ListView->HasAnyUserFocus().IsSet() || ContextMenuWidgetCached.IsValid();
	}

	/** Deletes currently selected items from the list view */
	virtual void DeleteSelectedPanelItems() override
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteSelectedItems", "Delete Selected Items"));
		if (!DeleteItemsFromLogicPanel<ActionType>(ActionItems, ListView->GetSelectedItems()))
		{
			Transaction.Cancel();
		}
	}

	/** Returns the UI items currently selected by the user (if any). */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() override
	{
		TArray<TSharedPtr<FRCLogicModeBase>> SelectedValidLogicItems;
		if (ListView.IsValid())
		{
			TArray<TSharedPtr<ActionType>> AllSelectedLogicItems = ListView->GetSelectedItems();
			SelectedValidLogicItems.Reserve(AllSelectedLogicItems.Num());

			for (const TSharedPtr<ActionType>& LogicItem : AllSelectedLogicItems)
			{
				if (LogicItem.IsValid())
				{
					SelectedValidLogicItems.Add(LogicItem);
				}
			}
		}
		return SelectedValidLogicItems;
	}

	/** The currently selected Action item */
	TSharedPtr<FRCActionModel> GetSelectedActionItem()
	{
		return SelectedActionItem;
	}

	/** Fetches the parent Action panel */
	TSharedPtr<SRCActionPanel> GetActionPanel()
	{
		return ActionPanelWeakPtr.Pin();
	}

	/** Fetches the Behaviour (UI model) associated with us */
	TSharedPtr<FRCBehaviourModel> GetBehaviourItem()
	{
		return BehaviourItemWeakPtr.Pin();
	}

	/** Adds an Action by Remote Control Field Guid */
	URCAction* AddAction(const FGuid& InRemoteControlFieldId)
	{
		if (const URemoteControlPreset* Preset = GetPreset())
		{
			if (TSharedPtr<const FRemoteControlField> RemoteControlField = Preset->GetExposedEntity<FRemoteControlField>(InRemoteControlFieldId).Pin())
			{
				if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
				{
					return ActionPanel->AddAction(RemoteControlField.ToSharedRef());
				}
			}
		}

		return nullptr;
	}

	/** Adds a PropertyId action with the given PropertyId */
	URCAction* AddAction(const FName InPropertyId)
	{
		if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
		{
			return ActionPanel->AddAction(InPropertyId);
		}
		return nullptr;
	}

	virtual void AddNewLogicItem(UObject* InLogicItem) override
	{
		AddRowToList(Cast<URCAction>(InLogicItem));

		RequestRefresh();
	}

	void AddRowToList(URCAction* InAction)
	{
		if (!ensure(InAction))
		{
			return;
		}

		if (TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
		{
			if (TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ActionPanel->GetRemoteControlPanel())
			{
				if (TSharedPtr<FRCBehaviourModel> BehaviourItem = BehaviourItemWeakPtr.Pin())
				{
					TSharedPtr<ActionType> ActionItem = ActionType::GetModelByActionType(InAction, BehaviourItem, RemoteControlPanel);

					if (ensure(ActionItem.IsValid()))
					{
						ActionItems.Add(ActionItem);
						UpdateEntityUsage();
					}
				}
			}
		}
	}

	void RequestRefresh()
	{
		ListView->RequestListRefresh();
	}

	/** Allows Logic panels to add special functionality to the Context Menu based on context */
	virtual void AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder) override
	{
		if (SelectedActionItem)
		{
			SelectedActionItem->AddSpecialContextMenuOptions(MenuBuilder);
		}
	}

	void SelectSource_Execute() override
	{
		TArray<TSharedPtr<ActionType>> SelectedItems = ListView->GetSelectedItems();
		TArray<UObject*> ObjectsToSelect;

		for (const TSharedPtr<ActionType>& SelectedAction : SelectedItems)
		{
			if (SelectedAction.IsValid())
			{
				if (URCAction* Action = SelectedAction->GetAction())
				{
					if (URemoteControlPreset* Preset = Action->PresetWeakPtr.Get())
					{
						if (TSharedPtr<FRemoteControlEntity> Entity = Preset->GetExposedEntity(Action->ExposedFieldId).Pin())
						{
							FRemoteControlUIModule::Get().SelectSource(Entity);
							return;
						}
					}
				}
			}
		}
	}

private:

	/** OnGenerateRow delegate for the Actions List View */
	TSharedRef<ITableRow> OnGenerateWidgetForList( TSharedPtr<ActionType> InItem, const TSharedRef<STableViewBase>& OwnerTable )
	{
		if (ensure(InItem))
		{
			return InItem->OnGenerateWidgetForList(InItem, OwnerTable);
		}

		return SNew(STableRow<TSharedPtr<FString>>, OwnerTable)
			[
				SNullWidget::NullWidget
			];
	}

	/** Responds to the selection of a newly created action. Resets UI state */
	void OnActionAdded(URCAction* InAction)
	{
		// Historical note: Previously we used to call Reset here after adding an Action
		// This is now covered by use of AddRowToList function (invoked from SRCActionPanel)
		// @todo: This improvement needs to be done for "Remove Action" as well.
	}

	/** Responds to the removal of all actions. Rests UI state */
	void OnEmptyActions()
	{
		Reset();
	}

	/** Refreshes the list from the latest state of the data model */
	virtual void Reset() override
	{
		ActionItems.Empty();

		if (TSharedPtr<FRCBehaviourModel> BehaviourItem = BehaviourItemWeakPtr.Pin())
		{
			if (URCBehaviour* Behaviour = Cast<URCBehaviour>(BehaviourItem->GetBehaviour()))
			{
				for (URCAction* Action : Behaviour->ActionContainer->GetActions())
				{
					AddRowToList(Action);
				}
			}
		}

		ListView->RebuildList();

		UpdateEntityUsage();
	}
	
	/** Refreshes the list */
	void Refresh()
	{
		ListView->RebuildList();
	}

	/** Handles broadcasting of a successful remove item operation. */
	virtual void BroadcastOnItemRemoved() override {}

	/** Fetches the Remote Control preset associated with the parent panel */
	virtual URemoteControlPreset* GetPreset() override
	{
		if (ActionPanelWeakPtr.IsValid())
		{
			return ActionPanelWeakPtr.Pin()->GetPreset();
		}

		return nullptr;
	}

	/*Drag and drop event for creating an Action from an exposed field */
	FReply OnExposedFieldDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
	{
		if (DragDropOperation)
		{
			if (DragDropOperation->IsOfType<FExposedEntityDragDrop>())
			{
				if (const TSharedPtr<FExposedEntityDragDrop> DragDropOp = StaticCastSharedPtr<FExposedEntityDragDrop>(DragDropOperation))
				{
					// Check this beforehand to avoid calling all this for each PropertyId group later on
					bool bBehaviorSupportPropertyId = false;
					if (const TSharedPtr<FRCBehaviourModel>& BehaviorModel = GetBehaviourItem())
					{
						if (const URCBehaviour* Behavior = BehaviorModel->GetBehaviour())
						{
							bBehaviorSupportPropertyId = Behavior->SupportPropertyId();
						}
					}

					// Fetch the Exposed Entities
					for (const TSharedPtr<SRCPanelTreeNode>& ExposedEntity : DragDropOp->GetSelectedEntities())
					{
						const SRCPanelTreeNode::ENodeType NodeType = ExposedEntity->GetRCType();
						if (NodeType == SRCPanelTreeNode::FieldGroup)
						{
							if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroup = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(ExposedEntity))
							{
								const ERCFieldGroupType GroupType = FieldGroup->GetGroupType();
								// Property Id group
								if (GroupType == ERCFieldGroupType::PropertyId)
								{
									if (bBehaviorSupportPropertyId)
									{
										AddAction(FieldGroup->GetFieldKey());
									}
								}
								// Owner group
								else if (GroupType == ERCFieldGroupType::Owner)
								{
									TArray<TSharedPtr<SRCPanelTreeNode>> OwnerGroupChildren;
									FieldGroup->GetNodeChildren(OwnerGroupChildren);
									for (const TSharedPtr<SRCPanelTreeNode>& Child : OwnerGroupChildren)
									{
										if (Child->GetRCType() == SRCPanelTreeNode::Field)
										{
											if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
											{
												if (ActionPanel->CanHaveActionForField(Child->GetRCId()))
												{
													AddAction(Child->GetRCId());
												}
											}
										}
									}
								}
							}
						}
						// Fields
						else if (NodeType == SRCPanelTreeNode::Field)
						{
							if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
							{
								if (ActionPanel->CanHaveActionForField(ExposedEntity->GetRCId()))
								{
									AddAction(ExposedEntity->GetRCId());
								}
							}
						}
					}
				}
			}
			else if (DragDropOperation->IsOfType<FFieldGroupDragDropOp>())
			{
				if (TSharedPtr<FFieldGroupDragDropOp> DragDropOp = StaticCastSharedPtr<FFieldGroupDragDropOp>(DragDropOperation))
				{
					if (URemoteControlPreset* Preset = GetPreset())
					{
						// Fetch the Group
						const FGuid GroupId = DragDropOp->GetGroupId();
						const FRemoteControlPresetGroup* Group = Preset->Layout.GetGroup(GroupId);

						if (ensure(Group))
						{
							const TArray<FGuid> GroupFields = Group->GetFields();

							// Add Action for all fields in the Group
							if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
							{
								for (const FGuid RemoteControlFieldId : GroupFields)
								{
									// Add Action
									if (ActionPanel->CanHaveActionForField(RemoteControlFieldId))
									{
										AddAction(RemoteControlFieldId);
									}
								}
							}
						}
					}
				}
			}
		}

		return FReply::Handled();
	}

	/** Whether drag and drop is possible from the current exposed property to the Actions table */
	bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
	{
		if (DragDropOperation)
		{
			if (DragDropOperation->IsOfType<FExposedEntityDragDrop>())
			{
				if (const TSharedPtr<FExposedEntityDragDrop> DragDropOp = StaticCastSharedPtr<FExposedEntityDragDrop>(DragDropOperation))
				{
					// Check this beforehand to avoid calling all this for each PropertyId group later on
					bool bBehaviorSupportPropertyId = false;
					if (const TSharedPtr<FRCBehaviourModel>& BehaviorModel = GetBehaviourItem())
					{
						if (const URCBehaviour* Behavior = BehaviorModel->GetBehaviour())
						{
							bBehaviorSupportPropertyId = Behavior->SupportPropertyId();
						}
					}

					// Fetch the Exposed Entity
					for (const TSharedPtr<SRCPanelTreeNode>& ExposedEntity : DragDropOp->GetSelectedEntities())
					{
						const SRCPanelTreeNode::ENodeType NodeType = ExposedEntity->GetRCType();
						if (NodeType == SRCPanelTreeNode::FieldGroup)
						{
							if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroup = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(ExposedEntity))
							{
								const ERCFieldGroupType GroupType = FieldGroup->GetGroupType();
								// Property Id Group
								if (GroupType == ERCFieldGroupType::PropertyId)
								{
									if (bBehaviorSupportPropertyId)
									{
										return true;
									}
								}
								// Owner Group
								else if (GroupType == ERCFieldGroupType::Owner)
								{
									TArray<TSharedPtr<SRCPanelTreeNode>> OwnerGroupChildren;
									FieldGroup->GetNodeChildren(OwnerGroupChildren);
									for (const TSharedPtr<SRCPanelTreeNode>& Child : OwnerGroupChildren)
									{
										if (Child->GetRCType() == SRCPanelTreeNode::Field)
										{
											if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
											{
												// if at least 1 can be created then enable it
												if (ActionPanel->CanHaveActionForField(Child->GetRCId()))
												{
													return true;
												}
											}
										}
									}
								}
							}
						}
						// Fields
						else if (NodeType == SRCPanelTreeNode::Field)
						{
							if (const TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
							{
								// if at least 1 can be created then enable it
								if (ActionPanel->CanHaveActionForField(ExposedEntity->GetRCId()))
								{
									return true;
								}
							}
						}
					}
				}
			}
			else if (DragDropOperation->IsOfType<FFieldGroupDragDropOp>())
			{
				if (TSharedPtr<FFieldGroupDragDropOp> DragDropOp = StaticCastSharedPtr<FFieldGroupDragDropOp>(DragDropOperation))
				{
					if (URemoteControlPreset* Preset = GetPreset())
					{
						// Fetch the Group
						const FGuid GroupId = DragDropOp->GetGroupId();
						const FRemoteControlPresetGroup* Group = Preset->Layout.GetGroup(GroupId);

						if (ensure(Group))
						{
							const TArray<FGuid> GroupFields = Group->GetFields();

							// For Groups we accept the drag-drop operation if even just one field in the group can have an action added successfully
							for (const FGuid RemoteControlFieldId : GroupFields)
							{
								if (TSharedPtr<SRCActionPanel> ActionPanel = GetActionPanel())
								{
									if (ActionPanel->CanHaveActionForField(RemoteControlFieldId))
									{
										return true; // we have at least one compatible field
									}
								}
							}
						}
					}
				}
			}
		}

		return false;
	}

	/** OnSelectionChanged delegate for Actions List View */
	void OnSelectionChanged(TSharedPtr<ActionType> InItem, ESelectInfo::Type)
	{
		if (SelectedActionItem)
		{
			SelectedActionItem->OnSelectionExit();
		}

		SelectedActionItem = InItem;
	}

private:

	/** The currently selected Action item */
	TSharedPtr<FRCActionModel> SelectedActionItem;

	/** The parent Action Panel widget */
	TWeakPtr<SRCActionPanel> ActionPanelWeakPtr;

	/** The Behaviour (UI model) associated with us */
	TWeakPtr<FRCBehaviourModel> BehaviourItemWeakPtr;

	/** List of Actions (UI model) active in this widget */
	TArray<TSharedPtr<ActionType>> ActionItems;

	/** List View widget for representing our Actions List */
	TSharedPtr<SListView<TSharedPtr<ActionType>>> ListView;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
};

#undef LOCTEXT_NAMESPACE
