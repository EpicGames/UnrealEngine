// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourDetails.h"

#include "Action/RCActionContainer.h"
#include "Action/RCFunctionAction.h"
#include "Action/RCPropertyAction.h"
#include "Behaviour/RCBehaviour.h"
#include "Controller/RCController.h"
#include "RemoteControlPreset.h"
#include "SlateOptMacros.h"
#include "SRCActionPanel.h"
#include "Styling/CoreStyle.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Behaviour/RCBehaviourModel.h"
#include "UI/BaseLogicUI/RCLogicHelpers.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"

#define LOCTEXT_NAMESPACE "SRCBehaviourDetails"

void SRCBehaviourDetails::Construct(const FArguments& InArgs, TSharedRef<SRCActionPanel> InActionPanel, TSharedPtr<FRCBehaviourModel> InBehaviourItem)
{
	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	ActionPanelWeakPtr = InActionPanel;
	BehaviourItemWeakPtr = InBehaviourItem;

	bool bIsEnabled = false;

	BehaviourDetailsBox = SNew(SBox);

	if (InBehaviourItem.IsValid())
	{
		BehaviourDetailsWidget = InBehaviourItem->GetBehaviourDetailsWidget();
		bIsEnabled = InBehaviourItem->IsBehaviourEnabled();
	}
	else
	{
		BehaviourDetailsWidget = SNullWidget::NullWidget;
	}
	
	BehaviourDetailsBox->SetContent(BehaviourDetailsWidget.ToSharedRef());

	ChildSlot
		.Padding(RCPanelStyle->PanelPadding)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this] ()
			{
				if (const TSharedPtr<FRCBehaviourModel>& Behavior = BehaviourItemWeakPtr.Pin())
				{
					return Behavior->HasBehaviourDetailsWidget()? 0 : 1;
				}
				return 1;
			})

			// Index 0 is for valid behavior
			+ SWidgetSwitcher::Slot()
			[
				SNew(SVerticalBox)

				// Behaviour Specific Details Panel
				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					BehaviourDetailsBox.ToSharedRef()
				]

				// Spacer to fill the gap.
				+ SVerticalBox::Slot()
				.Padding(0)
				.FillHeight(1.f)
				[
					SNew(SSpacer)
				]
			]

			// Index 1 is for invalid behavior
			+ SWidgetSwitcher::Slot()
			[
				SNullWidget::NullWidget
			]
		];

	RefreshIsBehaviourEnabled(bIsEnabled);
}

void SRCBehaviourDetails::SetIsBehaviourEnabled(const bool bIsEnabled) const
{
	if (const TSharedPtr<FRCBehaviourModel> BehaviourItem = BehaviourItemWeakPtr.Pin())
	{
		BehaviourItem->SetIsBehaviourEnabled(bIsEnabled);

		RefreshIsBehaviourEnabled(bIsEnabled);
	}
}

void SRCBehaviourDetails::SetNewBehavior(const TSharedPtr<FRCBehaviourModel>& InBehaviourItem)
{
	// TODO: do not update if the Behavior type is the same, this needs further change
	const TSharedPtr<FRCBehaviourModel>& CurrentBehavior = GetCurrentBehavior();
	if (CurrentBehavior != InBehaviourItem)
	{
		BehaviourItemWeakPtr = InBehaviourItem;
		Refresh();
	}
}

TSharedPtr<FRCBehaviourModel> SRCBehaviourDetails::GetCurrentBehavior() const
{
	if (BehaviourItemWeakPtr.IsValid())
	{
		return BehaviourItemWeakPtr.Pin();
	}
	return nullptr;
}

void SRCBehaviourDetails::RefreshIsBehaviourEnabled(const bool bIsEnabled) const
{
	if(BehaviourDetailsWidget && BehaviourTitleWidget)
	{
		BehaviourDetailsWidget->SetEnabled(bIsEnabled);
		BehaviourTitleWidget->SetEnabled(bIsEnabled);
	}

	if (const TSharedPtr<SRCActionPanel> ActionPanel = ActionPanelWeakPtr.Pin())
	{
		ActionPanel->RefreshIsBehaviourEnabled(bIsEnabled);
	}
}

void SRCBehaviourDetails::Refresh()
{
	const TSharedPtr<FRCBehaviourModel>& CurrentBehavior = GetCurrentBehavior();

	if (CurrentBehavior.IsValid() && CurrentBehavior->HasBehaviourDetailsWidget())
	{
		BehaviourDetailsWidget = CurrentBehavior->GetBehaviourDetailsWidget();
		const bool bIsEnabled = CurrentBehavior->IsBehaviourEnabled();

		if (BehaviourDetailsWidget.IsValid() && BehaviourDetailsBox.IsValid())
		{
			BehaviourDetailsBox->SetContent(BehaviourDetailsWidget.ToSharedRef());
		}

		RefreshIsBehaviourEnabled(bIsEnabled);
	}
	else
	{
		BehaviourDetailsWidget = SNullWidget::NullWidget;
		if (BehaviourDetailsBox.IsValid())
		{
			BehaviourDetailsBox->SetContent(BehaviourDetailsWidget.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
