// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "RemoteControlField.h"
#include "Widgets/SCompoundWidget.h"

struct FRCPanelStyle;
class SBox;
class SRCActionPanel;
class STextBlock;
class FRCBehaviourModel;

/**
 * A custom widget dedicated to show the details of the selected behaviour.
 */
class REMOTECONTROLUI_API SRCBehaviourDetails : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourDetails)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<SRCActionPanel> InActionPanel, TSharedPtr<FRCBehaviourModel> InBehaviourItem);

	/** Set the Enabled state of our parent Behaviour */
	void SetIsBehaviourEnabled(const bool bIsEnabled) const;

	/** Set a new behavior for this Details widget */
	void SetNewBehavior(const TSharedPtr<FRCBehaviourModel>& InBehaviourItem);

	/** Get the current Behavior used for this details */
	TSharedPtr<FRCBehaviourModel> GetCurrentBehavior() const;

private:
	/** Refreshes the UI widgets enabled state depending on whether the parent behaviour is currently enabled */
	void RefreshIsBehaviourEnabled(const bool bIsEnabled) const;

	/** Refresh the behaviour widgets */
	void Refresh();

	/** The parent Action panel UI widget holding this Header*/
	TWeakPtr<SRCActionPanel> ActionPanelWeakPtr;
	
	/** The active Behaviour (UI Model) associated with the Actions being shown*/
	TWeakPtr<FRCBehaviourModel> BehaviourItemWeakPtr;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Behaviour Box containing the details widget */
	TSharedPtr<SBox> BehaviourDetailsBox;

	/** Behaviour specific details panel*/
	TSharedPtr<SWidget> BehaviourDetailsWidget;

	/** Behaviour Title text widget*/
	TSharedPtr<STextBlock> BehaviourTitleWidget;
};
