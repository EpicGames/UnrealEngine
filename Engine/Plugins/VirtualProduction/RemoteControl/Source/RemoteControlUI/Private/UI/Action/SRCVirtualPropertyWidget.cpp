// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCVirtualPropertyWidget.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PropertyHandle.h"
#include "RCVirtualProperty.h"
#include "SlateOptMacros.h"
#include "UI/RCUIHelpers.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"

#define LOCTEXT_NAMESPACE "SRCVirtualPropertyWidget"

void SRCVirtualPropertyWidget::Construct(const FArguments& InArgs, URCVirtualPropertySelfContainer* InVirtualProperty)
{
	if (!ensure(InVirtualProperty))
	{
		return;
	}

	OnGenerateWidget = InArgs._OnGenerateWidget;
	OnExitingEditModeDelegate = InArgs._OnExitingEditMode;

	VirtualPropertyWeakPtr = InVirtualProperty;

	TSharedRef<SWidget> VirtualPropertyDisplayWidget = SNullWidget::NullWidget;

	if (ensureMsgf(OnGenerateWidget.IsBound(), TEXT("OnGenerateWidget must be implemented for read-only display of the virtual property")))
	{
		VirtualPropertyDisplayWidget = OnGenerateWidget.Execute(InVirtualProperty);
	}

	ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("NoBorder"))
			.OnMouseDoubleClick(this, &SRCVirtualPropertyWidget::OnMouseDoubleClick)
			[
				SAssignNew(VirtualPropertyWidgetBox, SBox)
				.VAlign(VAlign_Center)
				.Padding(FMargin(6.f))
				[
					VirtualPropertyDisplayWidget
				]
			]
		];
}

FReply SRCVirtualPropertyWidget::OnMouseDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	EnterEditMode();

	return FReply::Handled();
}

void SRCVirtualPropertyWidget::EnterEditMode()
{
	if (bIsEditMode)
	{
		return;
	}

	if (URCVirtualPropertySelfContainer* VirtualProperty = VirtualPropertyWeakPtr.Get())
	{
		DetailTreeNodeWeakPtr = UE::RCUIHelpers::GetDetailTreeNodeForVirtualProperty(VirtualProperty, PropertyRowGenerator);

		if (ensure(DetailTreeNodeWeakPtr.IsValid()))
		{
			TSharedPtr<IPropertyHandle> PropertyHandle;

			TSharedRef<SWidget> Widget = UE::RCUIHelpers::GetGenericFieldWidget(DetailTreeNodeWeakPtr.Pin(), &PropertyHandle, true);

			// Updates the parent box with the editable field widget for the virtual property which facilitates user input
			VirtualPropertyWidgetBox->SetContent(Widget);

			if (ensure(PropertyHandle.IsValid()))
			{
				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &SRCVirtualPropertyWidget::OnPropertyValueChanged));
			}

			bIsEditMode = true;
		}
	}
}

void SRCVirtualPropertyWidget::ExitEditMode()
{
	if (!bIsEditMode)
	{
		return;
	}

	if (ensure(OnGenerateWidget.IsBound()))
	{
		TSharedRef<SWidget> VirtualPropertyDisplayWidget = OnGenerateWidget.Execute(VirtualPropertyWeakPtr.Get());

		// Used to update other selected actions for Conditional and Range behaviour
		OnExitingEditModeDelegate.ExecuteIfBound();

		// Restore the readonly view of the virtual property widget
		VirtualPropertyWidgetBox->SetContent(VirtualPropertyDisplayWidget);
	}

	bIsEditMode = false;
}

void SRCVirtualPropertyWidget::OnPropertyValueChanged()
{
	ExitEditMode();
}

void SRCVirtualPropertyWidget::AddEditContextMenuOption(FMenuBuilder& MenuBuilder)
{
	FUIAction Action(FExecuteAction::CreateSP(this, &SRCVirtualPropertyWidget::EnterEditMode));

	MenuBuilder.AddMenuEntry(LOCTEXT("ContextMenuEdit", "Edit"),
		LOCTEXT("ContextMenuEditTooltip", "Edit the selected value"),
		FSlateIcon(), Action);
}

#undef LOCTEXT_NAMESPACE