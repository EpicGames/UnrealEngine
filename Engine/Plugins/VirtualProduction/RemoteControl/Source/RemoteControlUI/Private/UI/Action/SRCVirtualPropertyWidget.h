// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/SlateDelegates.h"
#include "Widgets/SCompoundWidget.h"

class FMenuBuilder;
class IDetailTreeNode;
class IPropertyRowGenerator;
class SBox;
class URCVirtualPropertySelfContainer;

DECLARE_DELEGATE(FOnExitingEditMode)

/*
* ~ SRCVirtualPropertyWidget ~
*
* Multi purpose widget for Virtual Properties covering both read-only and Edit usage scenarios
* 
* 1) In Read-Only mode, the OnGenerateWidget delegate is used to build a view widget as required by the caller.
* 2) In Edit mode, a generic field widget is generated from the Virtual property
* 
* Supported edit actions: 1) Double-click 2) Right-click -> Contet menu -> Edit
* Context menu support is automatically provided when this widget is set to the EditableVirtualPropertyWidget member of FRCActionModel.
*/
class REMOTECONTROLUI_API SRCVirtualPropertyWidget : public SCompoundWidget
{
public:
	using FOnGenerateWidget = typename TSlateDelegates<URCVirtualPropertySelfContainer*>::FOnGenerateWidget;

	SLATE_BEGIN_ARGS(SRCVirtualPropertyWidget)
		: _OnGenerateWidget()
		{
		}

	SLATE_EVENT(FOnGenerateWidget, OnGenerateWidget)

	SLATE_EVENT(FOnExitingEditMode, OnExitingEditMode)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, URCVirtualPropertySelfContainer* InVirtualProperty);

	/** Activates Edit Mode, generates a generic field widget for the virtual property to facilitate user entry */
	void EnterEditMode();

	/** Exits Edit Mode, restores the original read-only widget view as defined by the caller (via OnGenerateWidget) */
	void ExitEditMode();

	/** Automatically provides Right-click Edit support (invoked from FRCActionModel's GetContextMenuWidget function*/
	void AddEditContextMenuOption(FMenuBuilder& MenuBuilder);

protected:
	/** Delegate to be invoked when the list needs to generate a new widget from a data item. */
	FOnGenerateWidget OnGenerateWidget;

private:
	/** Double-click event for the border wrapping the Virtual Property widget. Provides Edit mode entry*/
	FReply OnMouseDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Property value change listener for the generic field widget. Used to exit edit mode. 
	* This is used in lieu of regular commit events as we don't have access to the generic field widget's internals */
	void OnPropertyValueChanged();

	/** The Virtual Property which this widget represents */
	TWeakObjectPtr<URCVirtualPropertySelfContainer> VirtualPropertyWeakPtr;

	/** Parent box encompassing read-only and edit modes.*/
	TSharedPtr<SBox> VirtualPropertyWidgetBox;

	/** Whether the virtual property is currently in edit mode */
	bool bIsEditMode = false;

	/** The property generator used to build a generic value widget*/
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Detail tree node used to create a generic value widget*/
	TWeakPtr<IDetailTreeNode> DetailTreeNodeWeakPtr;

	/** Delegate executed when exiting edit mode if bound */
	FOnExitingEditMode OnExitingEditModeDelegate;
};
