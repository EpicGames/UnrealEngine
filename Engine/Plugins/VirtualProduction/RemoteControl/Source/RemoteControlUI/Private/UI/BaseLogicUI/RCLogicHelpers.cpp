// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCLogicHelpers.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Behaviour/RCBehaviourNode.h"
#include "BlueprintEditorModule.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetEditorUtilities.h"

#define LOCTEXT_NAMESPACE "RCLogicHelpers"

UBlueprint* UE::RCLogicHelpers::CreateBlueprintWithDialog(UClass* InBlueprintParentClass, const UPackage* InSourcePackage, TSubclassOf<UBlueprint> InBlueprintClassType, TSubclassOf<UBlueprintGeneratedClass> InBlueprintGeneratedClassType)
{
	UBlueprint* NewBlueprint = nullptr;
	
	if (!InBlueprintParentClass)
	{
		return NewBlueprint;
	}

	
	const FString ClassName = FBlueprintEditorUtils::GetClassNameWithoutSuffix(InBlueprintParentClass);

	FString PathName = InSourcePackage->GetPathName();
	PathName = FPaths::GetPath(PathName);

	// Now that we've generated some reasonable default locations/names for the package, allow the user to have the final say
	// before we create the package and initialize the blueprint inside of it.
	FSaveAssetDialogConfig SaveAssetDialogConfig;
	SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("SaveAssetDialogTitle", "Save Asset As");
	SaveAssetDialogConfig.DefaultPath = PathName;
	SaveAssetDialogConfig.DefaultAssetName = ClassName + TEXT("_New");
	SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::Disallow;
			
	const FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	const FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
	if (!SaveObjectPath.IsEmpty())
	{
		const FString SavePackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
		const FString SavePackagePath = FPaths::GetPath(SavePackageName);
		const FString SaveAssetName = FPaths::GetBaseFilename(SavePackageName);

		UPackage* Package = CreatePackage(*SavePackageName);
		if (ensure(Package))
		{
			NewBlueprint = FKismetEditorUtilities::CreateBlueprint(InBlueprintParentClass, Package, FName(*SaveAssetName), BPTYPE_Normal, InBlueprintClassType, InBlueprintGeneratedClassType);

			if (NewBlueprint)
			{
				// Notify the asset registry
				FAssetRegistryModule::AssetCreated(NewBlueprint);

				// Mark the package dirty...
				Package->MarkPackageDirty();
			}
		}
	}

	return NewBlueprint;
}

bool UE::RCLogicHelpers::OpenBlueprintEditor(UBlueprint* InBlueprint)
{
	if (InBlueprint)
	{
		FBlueprintEditorModule& BlueprintEditorModule = FModuleManager::LoadModuleChecked<FBlueprintEditorModule>( "Kismet" );
		BlueprintEditorModule.CreateBlueprintEditor(  EToolkitMode::Standalone, TSharedPtr<IToolkitHost>(), InBlueprint, false );
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
