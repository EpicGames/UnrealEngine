// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SubclassOf.h"

class URCBehaviourBlueprint;
class UBlueprint;
class UBlueprintGeneratedClass;

/** Helpers for "Blueprint Override" functionality - for creating custom BP Behaviours*/
namespace UE::RCLogicHelpers
{
	/** Launches the Create Blueprint dialog for creating custom Behaviour blueprints */
	UBlueprint* CreateBlueprintWithDialog(UClass* InBlueprintParentClass, const UPackage* InSourcePackage, TSubclassOf<UBlueprint> InBlueprintClassType, TSubclassOf<UBlueprintGeneratedClass> InBlueprintGeneratedCClassType);

	/** Opens the Blueprint Editor for a given custom Behaviour blueprint*/
	bool OpenBlueprintEditor(UBlueprint* InBlueprint);
}