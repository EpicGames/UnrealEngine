// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCLogicModeBase.h"
#include "UI/SRemoteControlPanel.h"

#define LOCTEXT_NAMESPACE "FRCLogicModeBase"

FRCLogicModeBase::FRCLogicModeBase(const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel) : PanelWeakPtr(InRemoteControlPanel)
{
}

TSharedRef<SWidget> FRCLogicModeBase::GetWidget() const
{
	return SNew(STextBlock).Text(LOCTEXT("NotImplemented", "Not Implemented"));
}

URemoteControlPreset* FRCLogicModeBase::GetPreset() const
{
	if (!PanelWeakPtr.IsValid())
	{
		return nullptr;
	}

	return PanelWeakPtr.Pin()->GetPreset();
}

TSharedPtr< SRemoteControlPanel> FRCLogicModeBase::GetRemoteControlPanel() const
{
	return PanelWeakPtr.Pin();
}

#undef LOCTEXT_NAMESPACE