// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SRemoteControlPanel;
class SWidget;
class URemoteControlPreset;

/*
* ~ FRCLogicModeBase ~
*
* Base UI model for representing a Logic item (Controllers / Behaviours / Actions)
* Provides a row widget that can be used in list views.
*/
class FRCLogicModeBase : public TSharedFromThis<FRCLogicModeBase>
{
public:
	FRCLogicModeBase(){}
	FRCLogicModeBase(const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	virtual ~FRCLogicModeBase() = default;

	/** Returns a name to identify the type of logic model*/
	virtual FName GetModelType() const = 0;

	/** Represents the widget to be rendered for associated UI Model.
	* Implemented by child classes to supply the row widget for various panel lists
	*/
	virtual TSharedRef<SWidget> GetWidget() const;

	/** Returns the Remote Control Preset object associated with us*/
	URemoteControlPreset* GetPreset() const;

	/** Returns the Remote Control Preset object associated with us*/
	TSharedPtr<SRemoteControlPanel> GetRemoteControlPanel() const;

	/** Removes this model from the owning UI panel. Returns the number of items removed. */
	virtual int32 RemoveModel() = 0;

protected:
	/** The parent Remote Control Panel widget*/
	TWeakPtr<SRemoteControlPanel> PanelWeakPtr;
};
