// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCLogicPanelBase.h"

#include "SlateOptMacros.h"
#include "UI/SRemoteControlPanel.h"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCLogicPanelBase::Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	PanelWeakPtr = InPanel;
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

URemoteControlPreset* SRCLogicPanelBase::GetPreset() const
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		return RemoteControlPanel->GetPreset();
	}

	return nullptr;
}

TSharedPtr<SRemoteControlPanel> SRCLogicPanelBase::GetRemoteControlPanel() const
{
	return PanelWeakPtr.Pin();
}
