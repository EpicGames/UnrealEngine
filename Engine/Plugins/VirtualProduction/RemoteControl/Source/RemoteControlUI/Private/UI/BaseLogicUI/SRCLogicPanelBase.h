// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "SRCLogicPanelListBase.h"
#include "Widgets/SCompoundWidget.h"

class FRCLogicModeBase;
class SRemoteControlPanel;
class URemoteControlPreset;

/*
* 
* ~ SRCLogicPanelBase ~
*
* Base UI Widget for Logic Panels (Controllers / Behaviours / Actions).
* Provides access to parent Remote Control Preset object and common Delete Panel Item functionality
*/
class REMOTECONTROLUI_API SRCLogicPanelBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCLogicPanelBase)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Returns the Remote Control Preset object associated with us*/
	virtual URemoteControlPreset* GetPreset() const;

	/** Returns the parent Remote Control Panel widget,
	* This is the main container holding all the Remote Control panels including Logic and Exposed Properties*/
	virtual TSharedPtr<SRemoteControlPanel> GetRemoteControlPanel() const;

	/** "Delete Items" UI command implementation for panels*/
	virtual void DeleteSelectedPanelItems() = 0;

	/** Whether this panel can Duplicate Items */
	virtual bool CanDuplicateItems() const { return !GetSelectedLogicItems().IsEmpty(); }

	/** "Duplicate Items" UI command implementation for action panels */
	virtual void DuplicateSelectedPanelItems() {}

	/** Whether this panel can Copy Items */
	virtual bool CanCopyItems() const { return !GetSelectedLogicItems().IsEmpty(); }

	/** "Copy Items" UI command implementation for panels*/
	virtual void CopySelectedPanelItems() {}

	/** "Paste Items" UI command implementation for panels*/
	virtual void PasteItemsFromClipboard() {}

	/** Whether a given clipboard items can be successfully pasted into this panel */
	virtual bool CanPasteClipboardItems(const TArrayView<const TObjectPtr<UObject>> InLogicClipboardItems) const { return true; }

	/** "Update Value" UI command implementation for panels */
	virtual void UpdateValue() {}

	/** Whether this panel can call the UpdateValue command */
	virtual bool CanUpdateValue() const { return false; }

	/** Returns the UI items currently selected by the user (if any). To be implemented per child panel*/
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() const = 0;

	/** Provides an item suffix for the Paste context menu to provide users with useful context on the nature of the item being pasted */
	virtual FText GetPasteItemMenuEntrySuffix() { return FText::GetEmpty(); }

	/** Warns user before deleting a selected panel item. */
	virtual FReply RequestDeleteSelectedItem() = 0;

	/** Warns user before deleting all items in a panel. */
	virtual FReply RequestDeleteAllItems() = 0;

protected:
	/** Widget representing List of Controllers/Behaviours/Actions */
	TSharedPtr<class SRCLogicPanelListBase> PanelList;

	/** The parent Remote Control Panel widget*/
	TWeakPtr<SRemoteControlPanel> PanelWeakPtr;
};
