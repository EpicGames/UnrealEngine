// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCLogicPanelListBase.h"

#include "Commands/RemoteControlCommands.h"
#include "ScopedTransaction.h"
#include "SRCLogicPanelBase.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRemoteControlPanel.h"

#define LOCTEXT_NAMESPACE "SRCLogicPanelListBase"

void SRCLogicPanelListBase::Construct(const FArguments& InArgs, const TSharedRef<SRCLogicPanelBase>& InLogicParentPanel, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	LogicPanelWeakPtr = InLogicParentPanel;
	RemoteControlPanelWeakPtr = InPanel;
	CommandList = InPanel->GetCommandList();
}

TSharedPtr<SWidget> SRCLogicPanelListBase::GetContextMenuWidget()
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection*/true, CommandList);

	// Special menu options
	MenuBuilder.BeginSection("Advanced");
	AddSpecialContextMenuOptions(MenuBuilder);
	MenuBuilder.EndSection();

	// Generic options (based on UI Commands)
	MenuBuilder.BeginSection("Common");

	const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

	MenuBuilder.AddMenuEntry(Commands.CopyItem);

	FText PasteItemLabel = LOCTEXT("Paste", "Paste");
	if (TSharedPtr<SRemoteControlPanel> RemoteControlPanel = RemoteControlPanelWeakPtr.Pin())
	{
		if (TSharedPtr<SRCLogicPanelBase> LogicPanel = RemoteControlPanel->LogicClipboardItemSource)
		{
			const FText Suffix = LogicPanel->GetPasteItemMenuEntrySuffix();
			if (!Suffix.IsEmpty())
			{
				PasteItemLabel = FText::Format(FText::FromString("{0} ({1})"), PasteItemLabel, Suffix);
			}
		}
	}
	MenuBuilder.AddMenuEntry(Commands.PasteItem, NAME_None, PasteItemLabel);
	MenuBuilder.AddMenuEntry(Commands.DuplicateItem);
	MenuBuilder.AddMenuEntry(Commands.UpdateValue);
	MenuBuilder.AddMenuEntry(Commands.DeleteEntity);
	MenuBuilder.AddMenuEntry(LOCTEXT("DeleteAll", "Delete All"),
		LOCTEXT("ContextMenuEditTooltip", "Delete all the rows in this list"),
		FSlateIcon(),
		FUIAction(FExecuteAction::CreateSP(this, &SRCLogicPanelListBase::RequestDeleteAllItems),
			FCanExecuteAction::CreateSP(this, &SRCLogicPanelListBase::CanDeleteAllItems)));

	MenuBuilder.EndSection();

	TSharedPtr<SWidget> MenuWidget = MenuBuilder.MakeWidget();
	ContextMenuWidgetCached = MenuWidget;

	return MenuWidget;
}

bool SRCLogicPanelListBase::CanDeleteAllItems() const
{
	if (TSharedPtr<SRemoteControlPanel> RemoteControlPanel = RemoteControlPanelWeakPtr.Pin())
	{
		return !RemoteControlPanel->IsModeActive(ERCPanelMode::Live);
	}

	return false;
}

void SRCLogicPanelListBase::RequestDeleteAllItems()
{
	if (TSharedPtr<SRCLogicPanelBase> LogicParentPanel = LogicPanelWeakPtr.Pin())
	{
		LogicParentPanel->RequestDeleteAllItems();
	}
}

int32 SRCLogicPanelListBase::RemoveModel(const TSharedPtr<FRCLogicModeBase> InModel)
{
	if (!InModel.IsValid())
	{
		return 0;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveModel", "Remove Model"));

	const int32 RemovedCount = InModel->RemoveModel();

	if (RemovedCount == 0)
	{
		Transaction.Cancel();
		return 0;
	}

	UpdateEntityUsage();

	return RemovedCount;
}

void SRCLogicPanelListBase::UpdateEntityUsage()
{
	if (TSharedPtr<SRemoteControlPanel> RCPanel = GetRemoteControlPanel())
	{
		if (TSharedPtr<SRCPanelExposedEntitiesList> EntitiesList = RCPanel->GetEntityList())
		{
			EntitiesList->UpdateEntityUsage();
		}
	}
}

#undef LOCTEXT_NAMESPACE
