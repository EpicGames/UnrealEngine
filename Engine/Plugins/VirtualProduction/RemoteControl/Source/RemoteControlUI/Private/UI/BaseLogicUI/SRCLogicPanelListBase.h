// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RCLogicModeBase.h"
#include "SlateOptMacros.h"
#include "UI/Action/Bind/RCActionBindModel.h"
#include "UI/Action/Conditional/RCActionConditionalModel.h"
#include "UI/Action/RCActionModel.h"
#include "UI/Behaviour/RCBehaviourModel.h"
#include "UI/Controller/RCControllerModel.h"
#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class SRCLogicPanelBase;
class SRemoteControlPanel;

/*
* ~ SRCLogicPanelListBase ~
*
* Base UI Widget for Lists in Logic Panels.
* Can represent Controllers / Behaviours / Actions, etc.
*/
class REMOTECONTROLUI_API SRCLogicPanelListBase : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCLogicPanelListBase)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRCLogicPanelBase>& InLogicParentPanel, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Returns true if the underlying list is valid and empty. */
	virtual bool IsEmpty() const = 0;
	
	/** Returns number of items in the list. */
	virtual int32 Num() const = 0;

	/** The number of items currently selected in a Logic list panel */
	virtual int32 NumSelectedLogicItems() const = 0;

	/** Whether the List View currently has focus. */
	virtual bool IsListFocused() const = 0;

	/** Deletes currently selected items from the list view */
	virtual void DeleteSelectedPanelItems()  = 0;

	/** Returns the UI items currently selected by the user (if any). To be implemented per child panel */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() = 0;

	/** Provides a common entry point for adding Logic related data objects (UObjects) to their respective panels */
	virtual void AddNewLogicItem(UObject* InLogicItem) {}

	/** Builds the right-click context menu populated with generic actions (based on UI Commands) */
	TSharedPtr<SWidget> GetContextMenuWidget();

	/** Allows Logic panels to add special functionality to the Context Menu based on context */
	virtual void AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder) {};

	TSharedPtr<SRemoteControlPanel> GetRemoteControlPanel()
	{
		return RemoteControlPanelWeakPtr.Pin();
	}
	
	virtual void SelectSource_Execute() {}

private:

	/** Refreshes the list from the latest state of the model*/
	virtual void Reset() = 0;

	/** Handles broadcasting of a successful remove item operation.
	* This is handled uniquely by each type of child list widget*/
	virtual void BroadcastOnItemRemoved() = 0;

	/** Fetches the Remote Control preset associated with the parent panel */
	virtual URemoteControlPreset* GetPreset() = 0;

	/** Removes the given UI model item from the list of UI models for this panel list*/
	virtual int32 RemoveModel(const TSharedPtr<FRCLogicModeBase> InModel);

	/** Returns true when we can delete all items. */
	bool CanDeleteAllItems() const;
	
	/** Requests the parent logic panel to delete all items in this list*/
	void RequestDeleteAllItems();

protected:
	TWeakPtr<SWidget> ContextMenuWidgetCached;

	/**
	 * Helper function for handling common Delete Item functionality across all child panels (Actions/Behaviours/Controllers)
	 * Currently invoked from each Panel List child class with appropriate template class
	 * @return true if any items were deleted
	 */
	template<class T>
	bool DeleteItemsFromLogicPanel(TArray<TSharedPtr<T>>& ItemsSource, const TArray<TSharedPtr<T>>& SelectedItems)
	{
		bool bIsDeleted = false;
		for (const TSharedPtr<T> SelectedItem : SelectedItems)
		{
			if (SelectedItem.IsValid())
			{
				// Remove Model from Data Container
				const int32 RemoveCount = RemoveModel(SelectedItem);
				if (RemoveCount > 0)
				{
					bIsDeleted = true;

					// Remove View Model from UI List
					const int32 RemoveModelItemIndex = ItemsSource.IndexOfByPredicate([SelectedItem](TSharedPtr<T> InModel)
						{
							return SelectedItem == InModel;
						});

					if (RemoveModelItemIndex > INDEX_NONE)
					{
						ItemsSource.RemoveAt(RemoveModelItemIndex);
					}
				}
			}
		}

		if (bIsDeleted)
		{
			BroadcastOnItemRemoved();
			Reset();
		}
		return bIsDeleted;
	}

	/** Command list to build context menus for */
	TSharedPtr<FUICommandList> CommandList;

	/** The parent Remote Control Panel widget*/
	TWeakPtr<SRemoteControlPanel> RemoteControlPanelWeakPtr;

	/** The parent Logic Panel*/
	TWeakPtr<SRCLogicPanelBase> LogicPanelWeakPtr;

	virtual void UpdateEntityUsage();
};