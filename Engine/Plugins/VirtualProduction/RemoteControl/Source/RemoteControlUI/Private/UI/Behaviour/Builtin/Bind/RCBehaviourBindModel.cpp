// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBehaviourBindModel.h"

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "SRCBehaviourBind.h"
#include "UI/Action/Bind/RCActionBindModel.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCActionPanelList.h"

#define LOCTEXT_NAMESPACE "FRCBehaviourBindModel"

FRCBehaviourBindModel::FRCBehaviourBindModel(URCBehaviourBind* BindBehaviour, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCBehaviourModel(BindBehaviour, InRemoteControlPanel)
	, BindBehaviourWeakPtr(BindBehaviour)
{
}

bool FRCBehaviourBindModel::HasBehaviourDetailsWidget()
{
	return true;
}

TSharedRef<SWidget> FRCBehaviourBindModel::GetBehaviourDetailsWidget()
{
	return SAssignNew(BehaviourDetailsWidget, SRCBehaviourBind, SharedThis(this));
}

TSharedPtr<SRCLogicPanelListBase> FRCBehaviourBindModel::GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel)
{
	return SNew(SRCActionPanelList<FRCActionBindModel>, InActionPanel, SharedThis(this));
}

URCBehaviourBind* FRCBehaviourBindModel::GetBindBehaviour() const
{
	return Cast<URCBehaviourBind>(GetBehaviour());
}

#undef LOCTEXT_NAMESPACE