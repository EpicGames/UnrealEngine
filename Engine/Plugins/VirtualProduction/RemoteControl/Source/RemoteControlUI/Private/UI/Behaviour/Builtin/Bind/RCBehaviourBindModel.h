// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UI/Behaviour/RCBehaviourModel.h"

class SRCActionPanel;
class SRCBehaviourBind;
class SRCLogicPanelListBase;
class SWidget;
class URCBehaviourBind;

/*
* ~ FRCBehaviourBindModel ~
*
* UI model for Bind Behaviour

*/
class FRCBehaviourBindModel : public FRCBehaviourModel
{
public:
	FRCBehaviourBindModel(URCBehaviourBind* BindBehaviour, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	/** Returns true if this behaviour have a details widget or false if not*/
	virtual bool HasBehaviourDetailsWidget() override;

	/** Builds a Behaviour specific details widget for Bind Behaviour*/
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget() override;

	/** The Actions List widget to be used for Bind Behaviour */
	virtual TSharedPtr<SRCLogicPanelListBase> GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel);

	/** Fetches the Bind Behaviour UObject model*/
	URCBehaviourBind* GetBindBehaviour() const;

private:
	/** The Bind Behaviour (Data model) associated with us*/
	TWeakObjectPtr<URCBehaviourBind> BindBehaviourWeakPtr;

	/** Behaviour Details Widget for Bind Behaviour */
	TSharedPtr<SRCBehaviourBind> BehaviourDetailsWidget;
};