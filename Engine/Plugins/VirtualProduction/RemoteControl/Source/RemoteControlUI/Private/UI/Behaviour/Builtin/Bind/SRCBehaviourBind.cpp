// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourBind.h"

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "UI/Behaviour/Builtin/Bind/RCBehaviourBindModel.h"
#include "UI/SRemoteControlPanel.h"
#include "UI/Action/SRCActionPanel.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRCBehaviourBind"

void SRCBehaviourBind::Construct(const FArguments& InArgs, TSharedRef<FRCBehaviourBindModel> InBehaviourItem)
{
	BindBehaviourItemWeakPtr = InBehaviourItem;

	ChildSlot
	.Padding(8.f, 4.f)
	[
		SNew(SHorizontalBox)
		// Label
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(EVerticalAlignment::VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("AllowNumericInputLabel", "Allow numeric inputs as strings"))
		]

		// Checkbox
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(EHorizontalAlignment::HAlign_Fill)
		.VAlign(EVerticalAlignment::VAlign_Center)
		.Padding(FMargin(10.f))
		[
			SAssignNew(CheckboxAllowNumericInput, SCheckBox)
			.IsChecked(ECheckBoxState::Unchecked)
			.OnCheckStateChanged(this, &SRCBehaviourBind::OnAllowNumericCheckboxChanged)
		]
	];

	if (const URCBehaviourBind* BindBehaviour = InBehaviourItem->GetBindBehaviour())
	{
		CheckboxAllowNumericInput->SetIsChecked(BindBehaviour->AreNumericInputsAllowedAsStrings() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked);
	}
}

void SRCBehaviourBind::OnAllowNumericCheckboxChanged(ECheckBoxState NewState)
{
	if (TSharedPtr<const FRCBehaviourBindModel> BehaviourItem = BindBehaviourItemWeakPtr.Pin())
	{
		if (URCBehaviourBind* BindBehaviour = BehaviourItem->GetBindBehaviour())
		{
			// Update flag
			BindBehaviour->SetAllowNumericInputAsStrings((NewState == ECheckBoxState::Checked));
		}
	}
	
}

#undef LOCTEXT_NAMESPACE