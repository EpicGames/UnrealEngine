// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"

enum class ECheckBoxState : uint8;
class FRCBehaviourBindModel;
class SCheckBox;
class URCBehaviourBind;

/*
* ~ SRCBehaviourConditional ~
*
* Behaviour specific details panel for Bind Behaviour
* 
* Contains a checkbox which specifies whether string to numeric (and vice versa) conversions are desired
*/
class REMOTECONTROLUI_API SRCBehaviourBind : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourBind)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<FRCBehaviourBindModel> InBehaviourItem);

private:

	/** Called when the checkbox state changes for "Allow Numeric Input As Strings" */
	void OnAllowNumericCheckboxChanged(ECheckBoxState NewState);

	/** The Behaviour (UI model) associated with us*/
	TWeakPtr<FRCBehaviourBindModel> BindBehaviourItemWeakPtr;

	/** Checkbox for "Allow Numeric Input as Strings" */
	TSharedPtr<SCheckBox> CheckboxAllowNumericInput;
};
