// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBehaviourConditionalModel.h"

#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Controller/RCController.h"
#include "Modules/ModuleManager.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "PropertyEditorModule.h"
#include "RCVirtualProperty.h"
#include "SRCBehaviourConditional.h"
#include "UI/Action/Conditional/RCActionConditionalModel.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCActionPanelList.h"
#include "UI/RCUIHelpers.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "FRCBehaviourConditionalModel"

FRCBehaviourConditionalModel::FRCBehaviourConditionalModel(URCBehaviourConditional* ConditionalBehaviour)
	: FRCBehaviourModel(ConditionalBehaviour)
	, ConditionalBehaviourWeakPtr(ConditionalBehaviour)
{
	CreateComparandInputField();
}

URCBehaviourConditional* FRCBehaviourConditionalModel::GetConditionalBehaviour()
{
	return Cast<URCBehaviourConditional>(GetBehaviour());
}

URCAction* FRCBehaviourConditionalModel::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	URCAction* NewAction = nullptr;

	if (URCBehaviourConditional* ConditionalBehaviour = GetConditionalBehaviour())
	{
		NewAction = ConditionalBehaviour->AddConditionalAction(InRemoteControlField, Condition, ConditionalBehaviour->Comparand);

		OnActionAdded(NewAction);
	}

	return NewAction;
}

bool FRCBehaviourConditionalModel::HasBehaviourDetailsWidget()
{
	return true;
}

void FRCBehaviourConditionalModel::CreateComparandInputField()
{
	if (URCBehaviourConditional* ConditionalBehaviour = GetConditionalBehaviour())
	{
		if (URCController* Controller = ConditionalBehaviour->ControllerWeakPtr.Get())
		{
			TObjectPtr<URCVirtualPropertySelfContainer>& Comparand = ConditionalBehaviour->Comparand;
			URCVirtualPropertySelfContainer* PreviousComparand = Comparand;

			// Virtual Property for the Comparand
			Comparand = NewObject<URCVirtualPropertySelfContainer>(ConditionalBehaviour);

			if (PreviousComparand)
			{
				// Users typically enter multiple actions for a single condition so make it easier for them by retaining Condition value
				Comparand->DuplicatePropertyWithCopy(PreviousComparand);
			}
			else
			{
				Comparand->DuplicateProperty(FName("Comparand"), Controller->GetProperty());
			}

			// In case the Comparand is a Vector/Rotator we don't want the Lock widget
			if (FProperty* ComparandProperty = Comparand->GetProperty())
			{
				ComparandProperty->RemoveMetaData(TEXT("AllowPreserveRatio"));
			}

			// UI widget (via Property Generator)
			DetailTreeNodeWeakPtr = UE::RCUIHelpers::GetDetailTreeNodeForVirtualProperty(Comparand, PropertyRowGenerator);
		}
	}
}

TSharedRef<SWidget> FRCBehaviourConditionalModel::GetBehaviourDetailsWidget()
{
	return SAssignNew(BehaviourDetailsWidget, SRCBehaviourConditional, SharedThis(this));
}

void FRCBehaviourConditionalModel::OnActionAdded(URCAction* Action)
{
	if (URCBehaviourConditional* ConditionalBehaviour = GetConditionalBehaviour())
	{
		ConditionalBehaviour->OnActionAdded(Action, Condition, ConditionalBehaviour->Comparand);

		// Create a new Comparand for the user (the previous input field is already associated with the newly created action)
		CreateComparandInputField();

		BehaviourDetailsWidget->RefreshPropertyWidget();
	}
}

TSharedRef<SWidget> FRCBehaviourConditionalModel::GetComparandFieldWidget() const
{
	if (const TSharedPtr<IDetailTreeNode> DetailTreeNode = DetailTreeNodeWeakPtr.Pin())
	{
		if (TSharedPtr<IPropertyHandle> PropertyHandle = DetailTreeNode->CreatePropertyHandle())
		{
			if (URCBehaviour* Behaviour = GetBehaviour())
			{
				if (URCVirtualPropertyBase* Controller = Behaviour->ControllerWeakPtr.Get())
				{
					if (const TSharedPtr<SWidget>& CustomControllerWidget = IRemoteControlUIModule::Get().CreateCustomControllerWidget(Controller, PropertyHandle.ToSharedRef()))
					{
						return CustomControllerWidget.ToSharedRef();
					}
				}
			}
		}

		return UE::RCUIHelpers::GetGenericFieldWidget(DetailTreeNode);
	}

	return SNullWidget::NullWidget;
}

void FRCBehaviourConditionalModel::SetSelectedConditionType(const ERCBehaviourConditionType InCondition)
{
	Condition = InCondition;
}

TSharedPtr<SRCLogicPanelListBase> FRCBehaviourConditionalModel::GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel)
{
	return SNew(SRCActionPanelList<FRCActionConditionalModel>, InActionPanel, SharedThis(this));
}

#undef LOCTEXT_NAMESPACE