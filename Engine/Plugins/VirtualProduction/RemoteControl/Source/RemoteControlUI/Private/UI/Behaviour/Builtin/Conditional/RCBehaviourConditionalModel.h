// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UI/Behaviour/RCBehaviourModel.h"

enum class ERCBehaviourConditionType : uint8;
class IDetailTreeNode;
class IPropertyRowGenerator;
class SRCActionPanel;
class SRCBehaviourConditional;
class SRCLogicPanelListBase;
class SWidget;
class URCBehaviourConditional;
class URCVirtualPropertySelfContainer;

/*
* ~ FRCBehaviourConditionalModel ~
*
* UI model for Conditional Behaviour

*/
class FRCBehaviourConditionalModel : public FRCBehaviourModel
{
public:
	FRCBehaviourConditionalModel(URCBehaviourConditional* IsEqualBehaviour);

	/** Fetches the Conditional Behaviour UObject */
	URCBehaviourConditional* GetConditionalBehaviour();

	/** Add a Logic Action using a remote control field as input */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** Returns true if this behaviour have a details widget or false if not*/
	virtual bool HasBehaviourDetailsWidget() override;

	/** Builds a Behaviour specific details widget enabling the user to define Conditions*/
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget() override;

	/** Builds the Comparand input field's widget which enables the user to enter values to compare with*/
	TSharedRef<SWidget> GetComparandFieldWidget() const;

	/** Updates the condition type selected by the user for Conditional Behaviour */
	void SetSelectedConditionType(const ERCBehaviourConditionType InCondition);

	/** The Actions List widget to be used for Conditional Behaviour */	
	virtual TSharedPtr<SRCLogicPanelListBase> GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel);

	/** The condition type currently selected by the user */
	ERCBehaviourConditionType Condition;

protected:
	/** Invoked by parent class after an action has been added from UI*/
	virtual void OnActionAdded(URCAction* Action) override;

private:
	/** Builds a generic value widget of the Controller's type
	* This is used to store the Comparand's value as entered by the user*/
	void CreateComparandInputField();

	/** The Conditional Behaviour (Data model) associated with us*/
	TWeakObjectPtr<URCBehaviourConditional> ConditionalBehaviourWeakPtr;

	/** The row generator used to build a generic Value Widget*/
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Used to create a generic value widget for comparison based on the active Controller's type*/
	TWeakPtr<IDetailTreeNode> DetailTreeNodeWeakPtr;	

	/** Behaviour Details Widget (Conditions Widget in the case of this Behaviour) */
	TSharedPtr<SRCBehaviourConditional> BehaviourDetailsWidget;
};