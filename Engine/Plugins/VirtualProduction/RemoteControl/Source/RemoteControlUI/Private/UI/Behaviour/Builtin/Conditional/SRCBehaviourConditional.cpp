// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourConditional.h"

#include "Action/RCAction.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Controller/RCController.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "Styling/CoreStyle.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Behaviour/Builtin/Conditional/RCBehaviourConditionalModel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SRCBehaviourConditional"

class SPositiveActionButton;

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCBehaviourConditional::Construct(const FArguments& InArgs, TSharedRef<FRCBehaviourConditionalModel> InBehaviourItem)
{
	ConditionalBehaviourItemWeakPtr = InBehaviourItem;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	if (URCBehaviourConditional* Behaviour = Cast<URCBehaviourConditional>(InBehaviourItem->GetBehaviour()))
	{
		if (URCController* Controller = Behaviour->ControllerWeakPtr.Get())
		{
			auto AddCondition = [this](ERCBehaviourConditionType InConditionType)
			{
				Conditions.Add(MakeShared<ERCBehaviourConditionType>(InConditionType));
			};

			AddCondition(ERCBehaviourConditionType::IsEqual);

			if (Controller->IsNumericType())
			{
				AddCondition(ERCBehaviourConditionType::IsLesserThan);
				AddCondition(ERCBehaviourConditionType::IsGreaterThan);
				AddCondition(ERCBehaviourConditionType::IsLesserThanOrEqualTo);
				AddCondition(ERCBehaviourConditionType::IsGreaterThanOrEqualTo);
			}

			AddCondition(ERCBehaviourConditionType::Else);
		}
	}

	ChildSlot
	.Padding(8.f, 4.f)
	[
		SNew(SHorizontalBox)
		// Conditions Panel
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.FillWidth(0.5f)
		[
			SNew(SVerticalBox)
			// Conditions Panel Title
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.Padding(FMargin(3.f))
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConditionsPanelTitle", "Condition"))
				.Font(FRemoteControlPanelStyle::Get()->GetFontStyle("RemoteControlPanel.Actions.ValuePanelHeader"))
			]

			// Conditions List
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.Padding(FMargin(3.f))
			.AutoHeight()
			[
				SAssignNew(ListViewConditions, SListView<TSharedPtr<ERCBehaviourConditionType>>)
				.ListItemsSource(&Conditions)
				.Orientation(Orient_Horizontal) // Horizontal List
				.OnSelectionChanged(this, &SRCBehaviourConditional::OnConditionsListSelectionChanged)
				.OnGenerateRow(this, &SRCBehaviourConditional::OnGenerateWidgetForConditionsList)
				// @todo: Add dark background for list as per Figma
			]
		]

		// Vertical Separator
		+ SHorizontalBox::Slot()
		.Padding(FMargin(5.f, 0.f))
		.AutoWidth()
		[
			SNew(SSeparator)
			.Orientation(Orient_Vertical)
			.SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
			.Thickness(2.5f)
		]

		// Comparand Panel
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.FillWidth(0.5f)
		[
			SNew(SVerticalBox)
			// Comparand Label
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.Padding(FMargin(3.f))
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ConditionsPanelComparandLabel", "Input Value"))
				.Font(FRemoteControlPanelStyle::Get()->GetFontStyle("RemoteControlPanel.Actions.ValuePanelHeader"))
			]

			// Comparand Value Widget
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Center)
			.Padding(FMargin(3.f))
			.AutoHeight()
			[
				SAssignNew(ComparandFieldBoxWidget, SBox)
				[
					InBehaviourItem->GetComparandFieldWidget()
				]			
			]
		]
	];

	// Select equality condition by default
	ListViewConditions->SetSelection(Conditions[0], ESelectInfo::Direct);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCBehaviourConditional::RefreshPropertyWidget()
{
	if (ComparandFieldBoxWidget)
	{
		if (TSharedPtr<const FRCBehaviourConditionalModel> BehaviourItem = ConditionalBehaviourItemWeakPtr.Pin())
		{
			ComparandFieldBoxWidget->SetContent(BehaviourItem->GetComparandFieldWidget());
		}
	}
}

TSharedRef<ITableRow> SRCBehaviourConditional::OnGenerateWidgetForConditionsList(TSharedPtr<ERCBehaviourConditionType> InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	FText ConditionDisplayText;

	if (TSharedPtr<const FRCBehaviourConditionalModel> BehaviourItem = ConditionalBehaviourItemWeakPtr.Pin())
	{
		if (URCBehaviourConditional* ConditionalBehaviour = Cast<URCBehaviourConditional>(BehaviourItem->GetBehaviour()))
		{
			if (InItem.IsValid())
			{
				ConditionDisplayText = ConditionalBehaviour->GetConditionTypeAsText(*InItem.Get());
			}
		}
	}

	return SNew(STableRow<TSharedPtr<FText>>, OwnerTable)
		.Style(&RCPanelStyle->TableRowStyle)
		[
			SNew(SBox)
			.Padding(FMargin(6.f))
			[
				SNew(STextBlock).Text(ConditionDisplayText)
			]
			
		];
}

void SRCBehaviourConditional::OnConditionsListSelectionChanged(TSharedPtr<ERCBehaviourConditionType> InItem, ESelectInfo::Type)
{
	if (!ensure(InItem.IsValid()))
	{
		return;
	}

	const ERCBehaviourConditionType ConditionType = *InItem.Get();

	if (TSharedPtr<FRCBehaviourConditionalModel> BehaviourItem = ConditionalBehaviourItemWeakPtr.Pin())
	{
		BehaviourItem->Condition = ConditionType;
	}

	// Else Condition does not need to show the Comparand field
	const EVisibility bComparandFieldVisibility = (ConditionType == ERCBehaviourConditionType::Else) ? EVisibility::Hidden : EVisibility::Visible;
	ComparandFieldBoxWidget->SetVisibility(bComparandFieldVisibility);
}

#undef LOCTEXT_NAMESPACE
