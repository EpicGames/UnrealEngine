// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlField.h"
#include "Widgets/SCompoundWidget.h"

enum class ERCBehaviourConditionType : uint8;
class FRCBehaviourConditionalModel;
struct FRCPanelStyle;
class ITableRow;
class STableViewBase;
class SBox;
template <typename ItemType> class SListView;

/*
* ~ SRCBehaviourConditional ~
*
* Behaviour specific details panel for Conditional Behaviour
* 
* Provides users with the ability to definite Conditions on a per action basis
* Contains a Conditions list (for choosing a Condition) and a Comparand input field (for specifying the value with which to compare, per condition)
*/
class REMOTECONTROLUI_API SRCBehaviourConditional : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourConditional)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<FRCBehaviourConditionalModel> InBehaviourItem);

	/** Refreshes the Comparand input field.
	* This needs to be performed each time the user creates a new Action, because the previous Comparand field has been consumed
	*and so we create a new virtual property Comparand with whose memory the UI widget needs to be rebound.*/
	void RefreshPropertyWidget();

private:
	/** OnGenerateRow delegate for the Conditions List*/
	TSharedRef<ITableRow> OnGenerateWidgetForConditionsList(TSharedPtr<ERCBehaviourConditionType> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** OnSelectionChanged delegate for the Conditions List */
	void OnConditionsListSelectionChanged(TSharedPtr<ERCBehaviourConditionType> InItem, ESelectInfo::Type);

	/** List of possible Condition Types*/
	TArray<TSharedPtr<ERCBehaviourConditionType>> Conditions;

	/** The Behaviour (UI model) associated with us*/
	TWeakPtr<FRCBehaviourConditionalModel> ConditionalBehaviourItemWeakPtr;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Conditions List View widget - Enables the user to pick a Condition Type per Condition*/
	TSharedPtr <SListView<TSharedPtr<ERCBehaviourConditionType>>> ListViewConditions;

	/** Parent Box for the Comparand input field (which enables us to refresh it whenever required)*/
	TSharedPtr<SBox> ComparandFieldBoxWidget;
};
