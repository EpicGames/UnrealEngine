// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Behaviour/Builtin/Path/RCBehaviorSetAssetByPathModelNew.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"
#include "Controller/RCController.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/StyleColors.h"
#include "UI/Action/Path/RCActionSetAssetByPathModelNew.h"
#include "UI/Action/SRCActionPanelList.h"
#include "UI/Behaviour/Builtin/Path/SRCBehaviorSetAssetByPathNew.h"
#include "UI/Behaviour/Builtin/Path/SRCBehaviorSetAssetByPathNewElementRow.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/Customizations/RCAssetPathElementCustomization.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "RCSetAssetByPathBehaviorModelNew"

namespace UE::RemoteControl::UI::Private
{
	namespace SetAssetByPathBehaviorModelNew
	{
		constexpr const TCHAR* ValidImage = TEXT("SourceControl.StatusIcon.On");
		constexpr const TCHAR* ErrorImage = TEXT("Icons.Error.Solid");
	}
}

FRCSetAssetByPathBehaviorModelNew::FRCSetAssetByPathBehaviorModelNew(URCSetAssetByPathBehaviorNew* InSetAssetByPathBehavior, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCBehaviourModel(InSetAssetByPathBehavior, InRemoteControlPanel)
{
	SetAssetByPathBehaviorWeakPtr = InSetAssetByPathBehavior;
}

void FRCSetAssetByPathBehaviorModelNew::Initialize()
{
	PreviewPathWidget = SNew(SHorizontalBox);
	PathArrayWidget = SNew(SBox);

	ElementsView = SNew(SListView<TSharedPtr<FRCPathBehaviorElementRow>>)
		.ListItemsSource(&ElementItems)
		.SelectionMode(ESelectionMode::Single)
		.OnGenerateRow(this, &FRCSetAssetByPathBehaviorModelNew::OnGenerateWidgetForList);

	ValidationImage = SNew(SImage)
		.DesiredSizeOverride(FVector2D(12));

	FPropertyRowGeneratorArgs ArgsRowGeneratorArray;
	ArgsRowGeneratorArray.NotifyHook = this;
	PropertyRowGeneratorArray = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(ArgsRowGeneratorArray);

	PropertyRowGeneratorArray->RegisterInstancedCustomPropertyTypeLayout(
		FRCAssetPathElementNew::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateLambda(
			[ThisWeak = SharedThis(this).ToWeakPtr()]()
			{
				return FRCAssetPathElementCustomization::MakeInstance(ThisWeak.Pin());
			}
		)
	);

	if (URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = SetAssetByPathBehaviorWeakPtr.Get())
	{
		PropertyRowGeneratorArray->SetStructure(MakeShareable(new FStructOnScope(FRCSetAssetPathNew::StaticStruct(), (uint8*)&SetAssetByPathBehavior->PathStruct)));

		PropertyRowGeneratorArray->OnFinishedChangingProperties().AddLambda([this](const FPropertyChangedEvent& InEvent)
			{
				OnAssetPathFinishedChangingProperties(InEvent);
			});

		PropertyRowGeneratorArray->OnRowsRefreshed().AddLambda([this]()
			{
				RefreshPathAndPreview();
			});

		DetailTreeNodeWeakPtrArray.Empty();

		for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGeneratorArray->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			CategoryNode->GetChildren(Children);

			for (const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				DetailTreeNodeWeakPtrArray.Add(Child);
			}
		}
	}

	RefreshPathAndPreview();
}

bool FRCSetAssetByPathBehaviorModelNew::HasBehaviourDetailsWidget()
{
	return true;
}

TSharedRef<SWidget> FRCSetAssetByPathBehaviorModelNew::GetBehaviourDetailsWidget()
{
	return SNew(SRCBehaviorSetAssetByPathNew, SharedThis(this));
}

TSharedRef<SWidget> FRCSetAssetByPathBehaviorModelNew::GetPropertyWidget()
{
	TSharedRef<SVerticalBox> FieldWidget = SNew(SVerticalBox);

	FieldWidget->AddSlot()
		.Padding(FMargin(3.f, 3.f))
		.AutoHeight()
		[
			PathArrayWidget->AsShared()
		];

	FieldWidget->AddSlot()
		.Padding(FMargin(3.f, 8.f))
		.AutoHeight()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("PathPreview", "Path Preview"))
					.Font(DEFAULT_FONT("Regular", 8))
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(5.f, 0.f, 0.f, 0.f)
				[
					ValidationImage.ToSharedRef()
				]
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 5.f, 0.f, 0.f)
			[
				PreviewPathWidget->AsShared()
			]
		];
	
	return 	SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.FillWidth(1.0f)
		[
			FieldWidget
		];
}

void FRCSetAssetByPathBehaviorModelNew::NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel)
{
	// Update the preview when a controller change to always get the correct path
	RefreshPreview();
	RefreshValidity();
}

TSharedPtr<SRCLogicPanelListBase> FRCSetAssetByPathBehaviorModelNew::GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel)
{
	return SNew(SRCActionPanelList<FRCActionSetAssetByPathModelNew>, InActionPanel, SharedThis(this));
}

void FRCSetAssetByPathBehaviorModelNew::RefreshPreview() const
{
	PreviewPathWidget->ClearChildren();

	URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = SetAssetByPathBehaviorWeakPtr.Get();

	if (!SetAssetByPathBehavior)
	{
		PreviewPathWidget->AddSlot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Font(DEFAULT_FONT("Regular", 10))
				.Text(LOCTEXT("InvalidPath", "Invalid Path!"))
			];

		ValidationImage->SetImage(FAppStyle::Get().GetBrush(UE::RemoteControl::UI::Private::SetAssetByPathBehaviorModelNew::ErrorImage));
		ValidationImage->SetToolTipText(LOCTEXT("InvalidImageTooltip", "Path does not point to a valid asset!"));

		return;
	}

	bool bIsValidPath = false;

	const TArray<FRCSetAssetByPathBehaviorNewPathElement> PathElements = SetAssetByPathBehavior->GetPathElements();
	const bool bInternal = SetAssetByPathBehavior->bInternal;

	FString Path;
	
	if (bInternal)
	{
		PreviewPathWidget->AddSlot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Font(DEFAULT_FONT("Regular", 10))
				.Text(FText::FromString(UE::RemoteControl::Logic::SetAssetByPathNew::ContentFolder))
			];
	}

	TArray<TTuple<FName, FString>> ControllerValues;

	for (const FRCSetAssetByPathBehaviorNewPathElement& PathElement : PathElements)
	{
		if (!PathElement.ControllerName.IsNone())
		{
			if (PathElement.Controller)
			{
				ControllerValues.Add({PathElement.Controller->DisplayName, PathElement.Path});
			}
			else
			{
				ControllerValues.Add({PathElement.ControllerName, PathElement.Path});
			}			
		}

		if (PathElement.bIsError)
		{
			if (PathElement.Path.IsEmpty())
			{
				PreviewPathWidget->AddSlot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Font(DEFAULT_FONT("Regular", 10))
						.Text(LOCTEXT("NoControllerValue", "{Missing_Value}"))
						.ColorAndOpacity(FStyleColors::AccentGray.GetSpecifiedColor())
					];
			}
			else
			{
				PreviewPathWidget->AddSlot()
					.AutoWidth()
					[
						SNew(STextBlock)
						.AutoWrapText(true)
						.Font(DEFAULT_FONT("Regular", 10))
						.Text(FText::FromString(FString(TEXT("{")) + PathElement.Path + FString(TEXT("}"))))
						.ColorAndOpacity(FStyleColors::AccentGray.GetSpecifiedColor())
					];
			}
		}
		else
		{
			Path += PathElement.Path;

			PreviewPathWidget->AddSlot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.AutoWrapText(true)
					.Font(DEFAULT_FONT("Regular", 10))
					.Text(FText::FromString(PathElement.Path))
				];
		}
	}

	if (!bInternal)
	{
		bIsValidPath = FPaths::FileExists(Path);
	}
	else
	{
		// Specifying a path without the internal package name works, but cannot be found by the asset registry.
		// Double up the last path element if it does not contain a dot to complete the path.
		FAssetData AssetData;
		FString CorrectedPath = UE::RemoteControl::Logic::SetAssetByPathNew::ContentFolder + Path;

		if (!CorrectedPath.EndsWith(TEXT("/")))
		{
			const FString Filename = FPaths::GetCleanFilename(CorrectedPath);
			int32 DotIndex;

			if (!Filename.FindChar(TEXT('.'), DotIndex))
			{
				CorrectedPath += TEXT(".") + Filename;
			}
		}

		bIsValidPath = IAssetRegistry::Get()->TryGetAssetByObjectPath(CorrectedPath, AssetData) == UE::AssetRegistry::EExists::Exists;
	}

	ValidationImage->SetImage(FAppStyle::Get().GetBrush(
		bIsValidPath
			? UE::RemoteControl::UI::Private::SetAssetByPathBehaviorModelNew::ValidImage
			: UE::RemoteControl::UI::Private::SetAssetByPathBehaviorModelNew::ErrorImage
	));

	FText Tooltip = bIsValidPath
		? LOCTEXT("ValidImageTooltip", "Path points to a valid asset.")
		: LOCTEXT("InvalidImageTooltip", "Path does not point to a valid asset!");

	if (!ControllerValues.IsEmpty())
	{
		const FText ControllerValueFormat = LOCTEXT("ControllerValueFormat", "- {0}: {1}");
		TArray<FText> ControllerValueTexts;

		for (const TTuple<FName, FString>& ControllerValue : ControllerValues)
		{
			ControllerValueTexts.Add(
				FText::Format(ControllerValueFormat, FText::FromName(ControllerValue.Key), FText::FromString(ControllerValue.Value))
			);
		}

		Tooltip = FText::Format(
			LOCTEXT("ControllerValuesTooltipFormat", "{0}\n\nController Values:\n{1}"),
			Tooltip, 
			FText::Join(INVTEXT("\n"), ControllerValueTexts)
		);
	}

	ValidationImage->SetToolTipText(Tooltip);
}

void FRCSetAssetByPathBehaviorModelNew::RefreshValidity()
{
	bool bInternal = true;
	TArray<FRCSetAssetByPathBehaviorNewPathElement> PathStack;
	URemoteControlPreset* Preset = nullptr;

	if (URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = SetAssetByPathBehaviorWeakPtr.Get())
	{
		bInternal = SetAssetByPathBehavior->bInternal;
		PathStack = SetAssetByPathBehavior->GetPathElements();
	}

	if (const TSharedPtr<SRemoteControlPanel>& RCPanel = GetRemoteControlPanel())
	{
		Preset = RCPanel->GetPreset();
	}

	for (const TSharedPtr<FRCPathBehaviorElementRow>& ElementItem : ElementItems)
	{
		ElementItem->Validity = ERCPathBehaviorElementValidty::Unchecked;
	}

	if (PathStack.Num() != ElementItems.Num())
	{
		return;
	}

	if (bInternal)
	{
		RefreshValidity_Internal(PathStack);
	}
	else
	{
		RefreshValidity_External(PathStack);
	}

	for (int32 Index = 0; Index < PathStack.Num(); ++Index)
	{
		if (!PathStack[Index].ControllerName.IsNone() && !PathStack[Index].Controller)
		{
			ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidController;
		}		
		else if (PathStack[Index].Controller)
		{
			FString Value;
			const bool bIsStringController = PathStack[Index].Controller->GetValueString(Value);
			
			if (!bIsStringController)
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidController;
			}
			else if (Value.IsEmpty())
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::EmptyControllerValue;
			}
		}
	}
}

void FRCSetAssetByPathBehaviorModelNew::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (URCSetAssetByPathBehaviorNew* SetAssetByPath = SetAssetByPathBehaviorWeakPtr.Get())
	{
		SetAssetByPath->Execute();
	}
}

void FRCSetAssetByPathBehaviorModelNew::ReorderElementItems(int32 InDroppedOnIndex, EItemDropZone InDropZone, int32 InDroppedIndex)
{
	if (InDroppedOnIndex == InDroppedIndex)
	{
		return;
	}

	URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = SetAssetByPathBehaviorWeakPtr.Get();

	if (!SetAssetByPathBehavior)
	{
		return;
	}

	if (!SetAssetByPathBehavior->PathStruct.AssetPath.IsValidIndex(InDroppedOnIndex)
		|| !SetAssetByPathBehavior->PathStruct.AssetPath.IsValidIndex(InDroppedIndex))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderPath", "Reorder Path"));
	SetAssetByPathBehavior->Modify();

	const bool bIsAbove = InDropZone == EItemDropZone::AboveItem;
	TArray<FRCAssetPathElementNew> NewAssetPath;

	for (int32 Index = 0; Index < SetAssetByPathBehavior->PathStruct.AssetPath.Num(); ++Index)
	{
		if (Index == InDroppedOnIndex && bIsAbove)
		{
			NewAssetPath.Add(SetAssetByPathBehavior->PathStruct.AssetPath[InDroppedIndex]);
		}

		if (Index != InDroppedIndex)
		{
			NewAssetPath.Add(SetAssetByPathBehavior->PathStruct.AssetPath[Index]);
		}

		if (Index == InDroppedOnIndex && !bIsAbove)
		{
			NewAssetPath.Add(SetAssetByPathBehavior->PathStruct.AssetPath[InDroppedIndex]);
		}
	}

	SetAssetByPathBehavior->PathStruct.AssetPath = NewAssetPath;
	SetAssetByPathBehavior->Execute();

	RefreshPathAndPreview();
}

void FRCSetAssetByPathBehaviorModelNew::OnAssetPathFinishedChangingProperties(const FPropertyChangedEvent& InEvent)
{
	if (InEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		// if the actual PropertyName is the AssetPath than it means that we want to add a controller since we force this in the Customization
		if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FRCSetAssetPathNew, AssetPath))
		{
			const int32 Index = InEvent.GetArrayIndex(InEvent.GetMemberPropertyName().ToString());

			if (Index == INDEX_NONE)
			{
				return;
			}

			URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = SetAssetByPathBehaviorWeakPtr.Get();

			if (!SetAssetByPathBehavior)
			{
				return;
			}

			TArray<FRCAssetPathElementNew>& AssetPath = SetAssetByPathBehavior->PathStruct.AssetPath;

			if (!AssetPath.IsValidIndex(Index))
			{
				return;
			}

			FRCAssetPathElementNew& CurrentElement = AssetPath[Index];

			if (const TSharedPtr<SRemoteControlPanel>& RCPanel = GetRemoteControlPanel())
			{
				if (URemoteControlPreset* Preset = RCPanel->GetPreset())
				{
					URCVirtualPropertyInContainer* NewController = Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::String);
					
					if (NewController)
					{
						NewController->DisplayIndex = Preset->GetNextControllerDisplayIndex();

						FName NewControllerName = TEXT("DefaultInput");
						if (!CurrentElement.Path.IsEmpty())
						{
							NewControllerName = FName(CurrentElement.Path);
						}

						const FName NewUniqueName = Preset->SetControllerDisplayName(NewController->Id, NewControllerName);
						if (CurrentElement.Path != NewUniqueName.ToString())
						{
							// Assign to the path the actual new name of the controller
							CurrentElement.Path = NewUniqueName.ToString();
						}

						if (!Preset->NewControllerCategory.IsEmpty())
						{
							using namespace UE::RemoteControl::UI::Private;

							NewController->SetMetadataValue(FRCControllerPropertyInfo::CategoryName, Preset->NewControllerCategory);
						}

						RCPanel->OnControllerAdded.Broadcast(NewController->PropertyName);
					}
				}
			}
		
		}
		// Just refresh the preview without reconstructing the widget if we just set values
		RefreshPreview();
		RefreshValidity();
	}
	else
	{
		RefreshPathAndPreview();
	}
}

void FRCSetAssetByPathBehaviorModelNew::RegenerateWeakPtrInternal()
{
	DetailTreeNodeWeakPtrArray.Empty();

	for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGeneratorArray->GetRootTreeNodes())
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		CategoryNode->GetChildren(Children);

		for (const TSharedRef<IDetailTreeNode>& Child : Children)
		{
			DetailTreeNodeWeakPtrArray.Add(Child);
		}
	}
}

void FRCSetAssetByPathBehaviorModelNew::RefreshPathAndPreview()
{
	const TSharedPtr<SVerticalBox> FieldArrayWidget = SNew(SVerticalBox);

	RegenerateWeakPtrInternal();

	if (URCSetAssetByPathBehaviorNew* Behavior = Cast<URCSetAssetByPathBehaviorNew>(GetBehaviour()))
	{
		Behavior->RefreshPathArray();
	}

	TSharedPtr<SBox> Header;
	TSharedPtr<SVerticalBox> Rows;

	TSharedRef<SVerticalBox> Content = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(Header, SBox)
			.Padding(FMargin(3.0f, 2.0f))
		]
		+ SVerticalBox::Slot()
		.MaxHeight(150.f)
		[
			ElementsView.ToSharedRef()
		];
	
	ElementItems.Empty();

	for (const TSharedPtr<IDetailTreeNode>& DetailTreeNodeArray : DetailTreeNodeWeakPtrArray)
	{
		const TSharedPtr<IDetailTreeNode> PinnedNode = DetailTreeNodeArray;

		TArray<TSharedRef<IDetailTreeNode>> Children;
		PinnedNode->GetChildren(Children);
		Children.Insert(PinnedNode.ToSharedRef(), 0);
		
		for (int32 Counter = 0; Counter < Children.Num(); Counter++)
		{
			const FNodeWidgets NodeWidgets = Children[Counter]->CreateNodeWidgets();

			if (NodeWidgets.ValueWidget)
			{
				TSharedRef<SHorizontalBox> CurrentHorizontalBox = SNew(SHorizontalBox);

				// Header/Control row
				if (Counter == 0)
				{
					// Add NameWidget only for the ArrayProperty not for its entries
					CurrentHorizontalBox->AddSlot()
					[
						NodeWidgets.NameWidget.ToSharedRef()
					];

					CurrentHorizontalBox->AddSlot()
						.FillWidth(1.f)
						[
							NodeWidgets.ValueWidget.ToSharedRef()
						];

					Header->SetContent(CurrentHorizontalBox);
					continue;
				}

				CurrentHorizontalBox->AddSlot()
					.FillWidth(1.f)
					[
						NodeWidgets.ValueWidget.ToSharedRef()
					];

				ElementItems.Add(MakeShared<FRCPathBehaviorElementRow>(Counter - 1, CurrentHorizontalBox));
			}
			else if (NodeWidgets.WholeRowWidget)
			{
				ElementItems.Add(MakeShared<FRCPathBehaviorElementRow>(Counter - 1, NodeWidgets.WholeRowWidget.ToSharedRef()));
			}
		}
	}

	ElementsView->RequestListRefresh();

	RefreshPreview();
	PathArrayWidget->SetContent(Content);

	RefreshValidity();
}

TSharedRef<ITableRow> FRCSetAssetByPathBehaviorModelNew::OnGenerateWidgetForList(TSharedPtr<FRCPathBehaviorElementRow> InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SRCBehaviorSetAssetByPathNewElementRow, InOwnerTable, SharedThis(this), InItem);
}

void FRCSetAssetByPathBehaviorModelNew::RefreshValidity_Internal(TConstArrayView<FRCSetAssetByPathBehaviorNewPathElement> InPathStack)
{
	IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
	FAssetData AssetData;
	FString Path;

	TArray<FString> FullPaths;

	for (const FRCSetAssetByPathBehaviorNewPathElement& PathElement : InPathStack)
	{
		if (FullPaths.IsEmpty())
		{
			FullPaths.Add(UE::RemoteControl::Logic::SetAssetByPathNew::ContentFolder + PathElement.Path);
		}
		else
		{
			FullPaths.Add(FullPaths.Last() + PathElement.Path);
		}
	}

	const int32 LastElement = InPathStack.Num() - 1;

	for (int32 Index = LastElement; Index >= 0; --Index)
	{
		Path = FullPaths[Index];

		bool bValid = false;

		if (!Path.EndsWith(TEXT("/")))
		{
			// Specifying a path without the internal package name works, but cannot be found by the asset registry.
			// Double up the last path element if it does not contain a dot to complete the path.
			const FString Filename = FPaths::GetCleanFilename(Path);
			int32 DotIndex;

			if (!Filename.FindChar(TEXT('.'), DotIndex))
			{
				Path += TEXT(".") + Filename;
			}

			if (AssetRegistry->TryGetAssetByObjectPath(Path, AssetData) == UE::AssetRegistry::EExists::Exists)
			{
				bValid = true;
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::ValidAsset;
			}
		}

		if (!bValid)
		{
			if (AssetRegistry->PathExists(Path))
			{
				bValid = true;
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::ValidPath;
			}
		}

		// Invalid path, mark as invalid and check the previous one.
		if (!bValid)
		{
			if (Index == LastElement)
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidAsset;
			}
			else
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidPath;
				ElementItems[Index + 1]->Validity = ERCPathBehaviorElementValidty::Unknown;
			}

			continue;
		}

		// Found a valid path, so all paths here and previous must be valid.
		for (int32 UpdateIndex = 0; UpdateIndex < Index; ++UpdateIndex)
		{
			if (UpdateIndex == LastElement)
			{
				ElementItems[UpdateIndex]->Validity = ERCPathBehaviorElementValidty::ValidAsset;
			}
			else
			{
				ElementItems[UpdateIndex]->Validity = ERCPathBehaviorElementValidty::ValidPath;
			}
		}

		break;
	}
}

void FRCSetAssetByPathBehaviorModelNew::RefreshValidity_External(TConstArrayView<FRCSetAssetByPathBehaviorNewPathElement> InPathStack)
{
	FString Path;

	TArray<FString> FullPaths;

	for (const FRCSetAssetByPathBehaviorNewPathElement& PathElement : InPathStack)
	{
		if (FullPaths.IsEmpty())
		{
			FullPaths.Add(PathElement.Path);
		}
		else
		{
			FullPaths.Add(FullPaths.Last() + PathElement.Path);
		}
	}

	const int32 LastElement = InPathStack.Num() - 1;

	for (int32 Index = LastElement; Index >= 0; --Index)
	{
		Path = FullPaths[Index];

		// Only the last index can point to a file.
		bool bValid = (Index == LastElement) && FPaths::FileExists(Path);

		if (!bValid)
		{
			bValid = FPaths::DirectoryExists(Path);
		}

		// Invalid path, mark as invalid and check the previous one.
		if (!bValid)
		{
			if (Index == LastElement)
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidAsset;
			}
			else
			{
				ElementItems[Index]->Validity = ERCPathBehaviorElementValidty::InvalidPath;
				ElementItems[Index + 1]->Validity = ERCPathBehaviorElementValidty::Unknown;
			}

			continue;
		}

		// Found a valid path, so all paths here and previous must be valid.
		for (int32 UpdateIndex = 0; UpdateIndex <= Index; ++UpdateIndex)
		{
			if (UpdateIndex == LastElement)
			{
				ElementItems[UpdateIndex]->Validity = ERCPathBehaviorElementValidty::ValidAsset;
			}
			else
			{
				ElementItems[UpdateIndex]->Validity = ERCPathBehaviorElementValidty::ValidPath;
			}
		}

		break;
	}
}

#undef LOCTEXT_NAMESPACE
