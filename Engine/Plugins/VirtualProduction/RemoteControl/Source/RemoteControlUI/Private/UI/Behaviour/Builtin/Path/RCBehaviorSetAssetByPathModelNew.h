// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/NotifyHook.h"
#include "RemoteControlEntity.h"
#include "UI/Behaviour/RCBehaviourModel.h"

class FString;
class IDetailTreeNode;
class ITableRow;
class SBox;
class SHorizontalBox;
class SImage;
class STableViewBase;
class URCSetAssetByPathBehaviorNew;
enum class EItemDropZone;
struct FRCPathBehaviorElementRow;
struct FRCSetAssetByPathBehaviorNewPathElement;
template<typename T> class SListView;

/*
 * ~ FRCSetAssetByPathBehaviorModelNew ~
 *
 * Child Behaviour class representing the "Set Asset By Path" Behavior's UI model.
 *
 * Generates several Widgets where users can build a Root Path using strings or controller values.
 * This is assigned as a loaded asset to exposed properties defined by actions actions.
 */
class FRCSetAssetByPathBehaviorModelNew : public FRCBehaviourModel, public FNotifyHook
{
public:
	explicit FRCSetAssetByPathBehaviorModelNew(URCSetAssetByPathBehaviorNew* InSetAssetByPathBehavior, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	void Initialize();

	/** Returns true if this behavior have a details widget or false if not*/
	virtual bool HasBehaviourDetailsWidget() override;

	/** Builds a Behavior specific widget as required for the Set Asset By Path Behavior */
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget() override;

	/*
	 * Builds the Property Details Widget, including a generic expendable Array Widget and
	 * further two Text Widgets representing elements needed for the Path Behavior.
	 * All of them store the user input and use them to perform the SetAssetByPath Behavior.
	 */
	TSharedRef<SWidget> GetPropertyWidget();

	/** Called when a controller value changed */
	virtual void NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel) override;

	virtual TSharedPtr<SRCLogicPanelListBase> GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel) override;

	/** Refresh the PreviewPath */
	void RefreshPreview() const;

	/** Refresh the validity state of the path elements. */
	void RefreshValidity();

	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, class FEditPropertyChain* InPropertyThatChanged) override;

	void ReorderElementItems(int32 InDroppedOnIndex, EItemDropZone InDropZone, int32 InDroppedIndex);

private:
	/** Callback called when the AssetPath properties change to either refresh or reconstruct it updating the path values */
	void OnAssetPathFinishedChangingProperties(const FPropertyChangedEvent& InEvent);

	/** Reconstruct the PropertyRow Generator */
	void RegenerateWeakPtrInternal();
	
	/** Regenerates and creates a new PathArray Widget if changed */
	void RefreshPathAndPreview();

	/** Creates a row for the elements view. */
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCPathBehaviorElementRow> InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	void RefreshValidity_Internal(TConstArrayView<FRCSetAssetByPathBehaviorNewPathElement> InPathStack);
	
	void RefreshValidity_External(TConstArrayView<FRCSetAssetByPathBehaviorNewPathElement> InPathStack);

private:
	/** The SetAssetByPath Behaviour associated with this Model */
	TWeakObjectPtr<URCSetAssetByPathBehaviorNew> SetAssetByPathBehaviorWeakPtr;

	/** Pointer to the Widget holding the PathArray created for this behavior */
	TSharedPtr<SBox> PathArrayWidget;

	/** List view containing the path elements. */
	TSharedPtr<SListView<TSharedPtr<FRCPathBehaviorElementRow>>> ElementsView;

	/** Data for each row in the list view. */
	TArray<TSharedPtr<FRCPathBehaviorElementRow>> ElementItems;

	/** Pointer to the Preview Text Widget for the Path */
	TSharedPtr<SHorizontalBox> PreviewPathWidget;

	/** Image showing whether the path is valid or not. */
	TSharedPtr<SImage> ValidationImage;

	/** The row generator used to build the generic array value widget for the Path Array */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGeneratorArray;

	/** Used to create a generic Value Widget based on the Paths Available*/
	TArray<TSharedPtr<IDetailTreeNode>> DetailTreeNodeWeakPtrArray;
};
