// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBehaviourSetAssetByPathModel.h"

#include "Controller/RCController.h"
#include "Editor.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "Selection.h"
#include "SRCBehaviourSetAssetByPath.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FRCSetAssetByPathBehaviourModel"

FRCSetAssetByPathBehaviourModel::FRCSetAssetByPathBehaviourModel(URCSetAssetByPathBehaviour* InSetAssetByPathBehaviour, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCBehaviourModel(InSetAssetByPathBehaviour, InRemoteControlPanel)
	, SetAssetByPathBehaviourWeakPtr(InSetAssetByPathBehaviour)
{
	FPropertyRowGeneratorArgs ArgsRowGenerator;
	ArgsRowGenerator.bShouldShowHiddenProperties = true;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(ArgsRowGenerator);

	const FPropertyRowGeneratorArgs ArgsRowGeneratorArray;
	PropertyRowGeneratorArray = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(ArgsRowGeneratorArray);

	if (InSetAssetByPathBehaviour)
	{
		PropertyRowGenerator->SetStructure(InSetAssetByPathBehaviour->PropertyInContainer->CreateStructOnScope());
		DetailTreeNodeWeakPtr.Empty();
		for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGenerator->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			CategoryNode->GetChildren(Children);
			for (const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				DetailTreeNodeWeakPtr.Add(Child);
			}
		}

		// Secondary TArray Struct
		PropertyRowGeneratorArray->SetStructure(MakeShareable(new FStructOnScope(FRCSetAssetPath::StaticStruct(), (uint8*) &InSetAssetByPathBehaviour->PathStruct)));
		PropertyRowGeneratorArray->OnFinishedChangingProperties().AddLambda([this] (const FPropertyChangedEvent& InEvent)
		{
			OnAssetPathFinishedChangingProperties(InEvent);
		});
		PropertyRowGeneratorArray->OnRowsRefreshed().AddLambda([this]()
		{
			RefreshPathAndPreview();
		});
		DetailTreeNodeWeakPtrArray.Empty();
		for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGeneratorArray->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			CategoryNode->GetChildren(Children);
			for (const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				DetailTreeNodeWeakPtrArray.Add(Child);
			}
		}
	}

	PreviewPathWidget = SNew(STextBlock)
		.AutoWrapText(true);
	PathArrayWidget = SNew(SBox);

	Cast<URCSetAssetByPathBehaviour>(GetBehaviour())->UpdateTargetEntity();
	TWeakPtr<const FRemoteControlEntity> InitialSelected = Cast<URCSetAssetByPathBehaviour>(GetBehaviour())->GetTargetEntity();
	
	RefreshPathAndPreview();
	SelectorBox = GetSelectorWidget(InitialSelected);
}

bool FRCSetAssetByPathBehaviourModel::HasBehaviourDetailsWidget()
{
	return true;
}

TSharedRef<SWidget> FRCSetAssetByPathBehaviourModel::GetBehaviourDetailsWidget()
{
	return SNew(SRCBehaviourSetAssetByPath, SharedThis(this));
}

TSharedRef<SWidget> FRCSetAssetByPathBehaviourModel::GetPropertyWidget()
{
	TSharedRef<SVerticalBox> FieldWidget = SNew(SVerticalBox);
	
	const TSharedPtr<IDetailTreeNode> PinnedNodeDefault = DetailTreeNodeWeakPtr[0];

	const FNodeWidgets NodeWidgetsDefault = PinnedNodeDefault->CreateNodeWidgets();
	if (NodeWidgetsDefault.ValueWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SNew(STextBlock)
						.Text(FText::FromString("Target Exposed Property"))
				]
				+ SHorizontalBox::Slot()
				[
					NodeWidgetsDefault.NameWidget.ToSharedRef()
				]
			];
		
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				[
					SelectorBox.ToSharedRef()
				]
				+ SHorizontalBox::Slot()
				[
					NodeWidgetsDefault.ValueWidget.ToSharedRef()
				]
			];
	}
	else if (NodeWidgetsDefault.WholeRowWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.AutoHeight()
			[
				NodeWidgetsDefault.WholeRowWidget.ToSharedRef()
			];
	}

	FieldWidget->AddSlot()
		.Padding(FMargin(3.f, 3.f))
		.AutoHeight()
		[
			SNew(SBox)
			.MaxDesiredHeight(150.f)
			[
				SNew(SScrollBox)
				+ SScrollBox::Slot()
				[
					PathArrayWidget->AsShared()
				]
			]
		];

	FieldWidget->AddSlot()
		.Padding(FMargin(3.f, 3.f))
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromString("Preview Path: "))
				]
			+ SHorizontalBox::Slot()
				[
					PreviewPathWidget->AsShared()
				]
		];

	
	return 	SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.FillWidth(1.0f)
		[
			FieldWidget
		];
}

void FRCSetAssetByPathBehaviourModel::RegenerateWeakPtrInternal()
{
	DetailTreeNodeWeakPtrArray.Empty();
	for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGeneratorArray->GetRootTreeNodes())
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		CategoryNode->GetChildren(Children);
		for (const TSharedRef<IDetailTreeNode>& Child : Children)
		{
			DetailTreeNodeWeakPtrArray.Add(Child);
		}
	}

	URCSetAssetByPathBehaviour* SetAssetByPathBehaviour = SetAssetByPathBehaviourWeakPtr.Get();
	if (!SetAssetByPathBehaviour)
	{
		return;
	}

	URCController* RCController = SetAssetByPathBehaviour->ControllerWeakPtr.Get();
	if (!RCController)
	{
		return;
	}

	URemoteControlPreset* RemoteControlPreset =  RCController->PresetWeakPtr.Get();
	if (!RemoteControlPreset)
	{
		return;
	}

	ExposedEntities = RemoteControlPreset->GetExposedEntities<const FRemoteControlEntity>();
}

void FRCSetAssetByPathBehaviourModel::RefreshPathAndPreview()
{
	const TSharedPtr<SVerticalBox> FieldArrayWidget = SNew(SVerticalBox);

	RegenerateWeakPtrInternal();
	if (URCSetAssetByPathBehaviour* Behaviour = Cast<URCSetAssetByPathBehaviour>(GetBehaviour()))
	{
		Behaviour->RefreshPathArray();
	}
	
	for (const TSharedPtr<IDetailTreeNode>& DetailTreeNodeArray : DetailTreeNodeWeakPtrArray)
	{
		const TSharedPtr<IDetailTreeNode> PinnedNode = DetailTreeNodeArray;

		TArray<TSharedRef<IDetailTreeNode>> Children;
		PinnedNode->GetChildren(Children);
		Children.Insert(PinnedNode.ToSharedRef(), 0);
		
		for (uint8 Counter = 0; Counter < Children.Num(); Counter++)
		{
			const FNodeWidgets NodeWidgets = Children[Counter]->CreateNodeWidgets();
			if (NodeWidgets.ValueWidget)
			{
				TSharedPtr<SHorizontalBox> CurrentHorizontalBox = SNew(SHorizontalBox);
				if (Counter == 0)
				{
					// Add NameWidget only for the ArrayProperty not for its entries
					CurrentHorizontalBox->AddSlot()
					[
						NodeWidgets.NameWidget.ToSharedRef()
					];
				}
				CurrentHorizontalBox->AddSlot()
				.FillWidth(1.f)
				[
					NodeWidgets.ValueWidget.ToSharedRef()
				];

				FieldArrayWidget->AddSlot()
				.Padding(FMargin(3.0f, 2.0f))
				.AutoHeight()
				.VAlign(VAlign_Center)
				[
					CurrentHorizontalBox.ToSharedRef()
				];
			}
			else if (NodeWidgets.WholeRowWidget)
			{
				FieldArrayWidget->AddSlot()
					.Padding(FMargin(3.0f, 2.0f))
					.AutoHeight()
					[
						NodeWidgets.WholeRowWidget.ToSharedRef()
					];
			}
		}
	}

	RefreshPreview();
	PathArrayWidget->SetContent(FieldArrayWidget.ToSharedRef());
}

void FRCSetAssetByPathBehaviourModel::RefreshPreview() const
{
	FString PreviewText = TEXT("Path is invalid!");
	if (URCSetAssetByPathBehaviour* SetAssetByPathBehaviour = SetAssetByPathBehaviourWeakPtr.Get())
	{
		PreviewText = SetAssetByPathBehaviour->GetCurrentPath();
	}
	PreviewPathWidget->SetText(FText::FromString(PreviewText));
}

TSharedRef<SWidget> FRCSetAssetByPathBehaviourModel::GetSelectorWidget(TWeakPtr<const FRemoteControlEntity> InInitialSelected)
{
	return SNew(SComboBox<TWeakPtr<const FRemoteControlEntity>>)
		.OptionsSource(&ExposedEntities)
		.OnGenerateWidget_Lambda([this](TWeakPtr<const FRemoteControlEntity> InItem) -> TSharedRef<SWidget>
		{
			TSharedPtr<const FRemoteControlEntity> PinnedItem = InItem.Pin();
			if (!PinnedItem)
			{
				return SNullWidget::NullWidget;
			}

			return SNew(STextBlock).Text(FText::FromString(*PinnedItem->GetLabel().ToString()));
		})
		.OnComboBoxOpening_Lambda([this]()
		{
			const URCSetAssetByPathBehaviour* PathBehaviour = Cast<URCSetAssetByPathBehaviour>(GetBehaviour());
			if (!PathBehaviour)
			{
				return;
			}
			
			const URCController* Controller = PathBehaviour->ControllerWeakPtr.Get();
			if (!Controller)
			{
				return;
			}

			const URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get();
			if (!Preset)
			{
				return;
			}
			
			ExposedEntities = Preset->GetExposedEntities<const FRemoteControlEntity>();
		})
		.OnSelectionChanged_Lambda([this](TWeakPtr<const FRemoteControlEntity> InItem, ESelectInfo::Type SelectType)
		{
			if (TSharedPtr<const FRemoteControlEntity> Entity = InItem.Pin())
			{
				if (URCSetAssetByPathBehaviour* PathBehaviour = Cast<URCSetAssetByPathBehaviour>(GetBehaviour()))
				{
					PathBehaviour->SetTargetEntity(Entity);
				}
			}
		})
		.InitiallySelectedItem(InInitialSelected)
		.Content()
		[
			SNew(STextBlock)
				.Text_Raw(this, &FRCSetAssetByPathBehaviourModel::GetSelectedEntityText)
		];
}

void FRCSetAssetByPathBehaviourModel::NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel)
{
	// Update the preview when a controller change to always get the correct path
	RefreshPreview();
}

FText FRCSetAssetByPathBehaviourModel::GetSelectedEntityText() const
{
	FText ReturnText = FText::FromString(*FString("Nothing selected yet"));

	if (URCSetAssetByPathBehaviour* Behaviour = SetAssetByPathBehaviourWeakPtr.Get())
	{
		if (Behaviour->GetTargetEntity().Pin())
		{
			ReturnText = FText::FromString(*Behaviour->GetTargetEntity().Pin()->GetLabel().ToString());
		}
	}
	return ReturnText;
}

void FRCSetAssetByPathBehaviourModel::OnAssetPathFinishedChangingProperties(const FPropertyChangedEvent& InEvent)
{
	if (InEvent.ChangeType == EPropertyChangeType::ValueSet)
	{
		// if the actual PropertyName is the AssetPath than it means that we want to add a controller since we force this in the Customization
		if (InEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(FRCSetAssetPath, AssetPath))
		{
			const int32 Index = InEvent.GetArrayIndex(InEvent.GetMemberPropertyName().ToString());

			URCSetAssetByPathBehaviour* SetAssetByPathBehaviour = SetAssetByPathBehaviourWeakPtr.Get();
			if (!SetAssetByPathBehaviour || Index == INDEX_NONE)
			{
				return;
			}

			TArray<FRCAssetPathElement>& AssetPath = SetAssetByPathBehaviour->PathStruct.AssetPath;
			if (!AssetPath.IsValidIndex(Index))
			{
				return;
			}

			FRCAssetPathElement& CurrentElement = AssetPath[Index];
			if (const TSharedPtr<SRemoteControlPanel>& RCPanel = GetRemoteControlPanel())
			{
				if (URemoteControlPreset* Preset = RCPanel->GetPreset())
				{
					URCVirtualPropertyInContainer* NewController = Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::String);
					
					if (NewController)
					{
						NewController->DisplayIndex = Preset->GetNextControllerDisplayIndex();

						FName NewControllerName = TEXT("DefaultInput");
						if (!CurrentElement.Path.IsEmpty())
						{
							NewControllerName = FName(CurrentElement.Path);
						}

						const FName NewUniqueName = Preset->SetControllerDisplayName(NewController->Id, NewControllerName);
						if (CurrentElement.Path != NewUniqueName.ToString())
						{
							// Assign to the path the actual new name of the controller
							CurrentElement.Path = NewUniqueName.ToString();
						}

						if (!Preset->NewControllerCategory.IsEmpty())
						{
							using namespace UE::RemoteControl::UI::Private;

							NewController->SetMetadataValue(FRCControllerPropertyInfo::CategoryName, Preset->NewControllerCategory);
						}

						RCPanel->OnControllerAdded.Broadcast(NewController->PropertyName);
					}
				}
			}
		}
		// Just refresh the preview without reconstructing the widget if we just set values
		RefreshPreview();
	}
	else
	{
		RefreshPathAndPreview();
	}
}

#undef LOCTEXT_NAMESPACE
