// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "RemoteControlEntity.h"
#include "UI/Behaviour/RCBehaviourModel.h"

class IDetailTreeNode;
class SBox;

namespace SetAssetByPathModelHelper
{
	const FString InputToken = FString(TEXT("*INPUT "));
}

/*
 * ~ FRCSetAssetByPathModel ~
 *
 * Child Behaviour class representing the "Set Asset By Path" Behaviour's UI model.
 *
 * Generates several Widgets where users can enter the RootPath, Target Property, and Default Property as FStrings.
 * The values are then put together, getting a path towards a possible object.
 */
class FRCSetAssetByPathBehaviourModel : public FRCBehaviourModel
{
public:
	FRCSetAssetByPathBehaviourModel(URCSetAssetByPathBehaviour* SetAssetByPathBehaviour, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	/** Returns true if this behaviour have a details widget or false if not*/
	virtual bool HasBehaviourDetailsWidget() override;

	/** Builds a Behaviour specific widget as required for the Set Asset By Path Behaviour */
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget() override;

	/*
	 * Builds the Property Details Widget, including a generic expendable Array Widget and
	 * further two Text Widgets representing elements needed for the Path Behaviour.
	 * All of them store the user input and use them to perform the SetAssetByPath Behaviour.
	 */
	TSharedRef<SWidget> GetPropertyWidget();

	/** Builds the Combobox which the Behaviour will use to select and and apply it's Asset Setting on. */
	TSharedRef<SWidget> GetSelectorWidget(TWeakPtr<const FRemoteControlEntity> InInitialSelected);

	/** Called when a controller value changed */
	virtual void NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel) override;

	/** Refresh the PreviewPath */
	void RefreshPreview() const;

protected:
	/** Gets the currently selected Entities Name, or a failure Text in case it fails. */
	FText GetSelectedEntityText() const;

private:
	/** Callback called when the AssetPath properties change to either refresh or reconstruct it updating the path values */
	void OnAssetPathFinishedChangingProperties(const FPropertyChangedEvent& InEvent);

	/** Reconstruct the PropertyRow Generator */
	void RegenerateWeakPtrInternal();
	
	/** Regenerates and creates a new PathArray Widget if changed */
	void RefreshPathAndPreview();
private:
	/** The SetAssetByPath Behaviour associated with this Model */
	TWeakObjectPtr<URCSetAssetByPathBehaviour> SetAssetByPathBehaviourWeakPtr;

	/** Pointer to the Widget holding the PathArray created for this behaviour */
	TSharedPtr<SBox> PathArrayWidget;

	/** Pointer to the Preview Text Widget for the Path */
	TSharedPtr<STextBlock> PreviewPathWidget;

	/** POinter to the ComboBox Widget */
	TSharedPtr<SWidget> SelectorBox;

	/** The row generator used to build the generic value widgets */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;
	
	/** The row generator used to build the generic array value widget for the Path Array */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGeneratorArray;

	/** Used to create a generic Value Widget based on the active Controller's type*/
	TArray<TSharedPtr<IDetailTreeNode>> DetailTreeNodeWeakPtr;
	
	/** Used to create a generic Value Widget based on the Paths Available*/
	TArray<TSharedPtr<IDetailTreeNode>> DetailTreeNodeWeakPtrArray;
	
	/** Array of Exposed Entities from the Preset to help create a Widget to choose the Target */
	TArray<TWeakPtr<const FRemoteControlEntity>> ExposedEntities;
};
