// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviorSetAssetByPathNew.h"

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"
#include "IDetailTreeNode.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SPositiveActionButton.h"
#include "Styling/StyleColors.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviorSetAssetByPathModelNew.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRCBehaviorSetAssetByPathNew"

class SPositiveActionButton;

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCBehaviorSetAssetByPathNew::Construct(const FArguments& InArgs, TSharedRef<FRCSetAssetByPathBehaviorModelNew> InBehaviourItem)
{
	SetAssetByPathWeakPtr = InBehaviourItem;
	PathBehaviour = Cast<URCSetAssetByPathBehaviorNew>(InBehaviourItem->GetBehaviour());
	if (!PathBehaviour.IsValid())
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
		return;
	}

	const FText TooltipText = LOCTEXT("OnlyAffectsTextures", "Only affects texture assets.");

	const TSharedRef<SHorizontalBox> InternalExternalSwitchWidget = SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(3.0f,0.0f))
		.FillWidth(1.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.IsChecked(this, &SRCBehaviorSetAssetByPathNew::GetInternalExternalSwitchState, /* Internal */ true)
			.OnCheckStateChanged(this, &SRCBehaviorSetAssetByPathNew::OnInternalExternalSwitchStateChanged, /* Internal */ true)
			.ToolTipText(TooltipText)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("Internal", "Internal"))
			]
		]
	
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(3.0f,0.0f))
		.FillWidth(1.0f)
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.IsChecked(this, &SRCBehaviorSetAssetByPathNew::GetInternalExternalSwitchState, /* Internal */ false)
			.OnCheckStateChanged(this, &SRCBehaviorSetAssetByPathNew::OnInternalExternalSwitchStateChanged, /* Internal */ false)
			.ToolTipText(TooltipText)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("External", "External"))
			]
		];
	
	ChildSlot
	.Padding(8.f, 4.f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			InternalExternalSwitchWidget
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InBehaviourItem->GetPropertyWidget()
		]
	];
}

ECheckBoxState SRCBehaviorSetAssetByPathNew::GetInternalExternalSwitchState(bool bInIsInternal) const
{
	if (!PathBehaviour.IsValid())
	{
		return ECheckBoxState::Undetermined;
	}

	return PathBehaviour->bInternal == bInIsInternal
		? ECheckBoxState::Checked
		: ECheckBoxState::Unchecked;
}

void SRCBehaviorSetAssetByPathNew::OnInternalExternalSwitchStateChanged(ECheckBoxState InState, bool bInIsInternal)
{
	if (!PathBehaviour.IsValid())
	{
		return;
	}

	if (InState != ECheckBoxState::Checked)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("PathBehaviorInternalToggle", "Toggled Path Behavior Internal/External."));
	PathBehaviour->Modify();
	PathBehaviour->bInternal = bInIsInternal;

	if (const TSharedPtr<FRCSetAssetByPathBehaviorModelNew>& SetAssetPath = SetAssetByPathWeakPtr.Pin())
	{
		SetAssetPath->RefreshPreview();
		SetAssetPath->RefreshValidity();
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
