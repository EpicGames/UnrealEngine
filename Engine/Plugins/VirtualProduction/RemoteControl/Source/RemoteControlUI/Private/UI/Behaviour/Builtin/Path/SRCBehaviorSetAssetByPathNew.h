// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"

class FRCSetAssetByPathBehaviorModelNew;
class UClass;
class URCSetAssetByPathBehaviorNew;
enum class ECheckBoxState : uint8;

class SRCBehaviorSetAssetByPathNew : public  SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviorSetAssetByPathNew)
		{}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<FRCSetAssetByPathBehaviorModelNew> InBehaviourItem);

private:
	/** Determines the state of the internal/external buttons. */
	ECheckBoxState GetInternalExternalSwitchState(bool bInIsInternal) const;

	/** Called when the Internal/External button are pressed */
	void OnInternalExternalSwitchStateChanged(ECheckBoxState InState, bool bInIsInternal);

private:
	/** The Behaviour (UI model) associated with us */
	TWeakPtr<FRCSetAssetByPathBehaviorModelNew> SetAssetByPathWeakPtr;

	/** The PathBehaviour associated with us */
	TWeakObjectPtr<URCSetAssetByPathBehaviorNew> PathBehaviour;
};
