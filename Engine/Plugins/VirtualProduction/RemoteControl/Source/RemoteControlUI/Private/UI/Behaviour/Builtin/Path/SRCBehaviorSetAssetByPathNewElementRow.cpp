// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Behaviour/Builtin/Path/SRCBehaviorSetAssetByPathNewElementRow.h"

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Styles/SlateBrushTemplates.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviorSetAssetByPathModelNew.h"
#include "UI/RemoteControlPanelStyle.h"

#define LOCTEXT_NAMESPACE "SRCBehaviorPanelRow"

namespace UE::RemoteControl::UI::Private
{
	namespace BehaviorSetAssetByPathNewElementRow
	{
		constexpr const TCHAR* ValidImage = TEXT("SourceControl.StatusIcon.On");
		constexpr const TCHAR* WarningImage = TEXT("SourceControl.StatusIcon.Error");
		constexpr const TCHAR* ErrorImage = TEXT("Icons.Error.Solid");
		constexpr const TCHAR* NoImage = TEXT("Sequencer.Empty");
	}
}

/*
* ~ FRCBehaviorSetAssetByPathNewElementDragDrop ~
* Facilitates drag-drop operation
*/
class FRCBehaviorSetAssetByPathNewElementDragDrop final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRCBehaviorSetAssetByPathNewElementDragDrop, FDecoratedDragDropOp)

	explicit FRCBehaviorSetAssetByPathNewElementDragDrop(int32 InIndex)
		: Index(InIndex)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	int32 GetIndex() const
	{
		return Index;
	}

private:
	int32 Index;
};

void SRCBehaviorSetAssetByPathNewElementRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<FRCSetAssetByPathBehaviorModelNew> InSetAssetByPathBehaviorModelNew, RCBehaviorSetAssetByPathNewElementListItem InElementItem)
{
	ElementItem = InElementItem;
	SetAssetByPathBehaviorModelNewWeak = InSetAssetByPathBehaviorModelNew;

	SetCursor(EMouseCursor::GrabHand);

	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	STableRow::Construct(
		STableRow::FArguments()
		.Padding(0.f)
		.OnDragDetected(this, &SRCBehaviorSetAssetByPathNewElementRow::OnRowDragDetected)
		.OnCanAcceptDrop(this, &SRCBehaviorSetAssetByPathNewElementRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SRCBehaviorSetAssetByPathNewElementRow::OnAcceptDrop)
		.ShowWires(false)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.f, 0.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("VerticalBoxDragIndicatorShort"))
			]

			+ SHorizontalBox::Slot()
			[
				InElementItem->Widget
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()	
			.Padding(0.f, 0.f, 4.f, 0.f)		
			[
				SNew(SBox)
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(SImage)
					.Image(this, &SRCBehaviorSetAssetByPathNewElementRow::GetValidityImage)
					.ToolTipText(this, &SRCBehaviorSetAssetByPathNewElementRow::GetValidityToolTip)
					.DesiredSizeOverride(FVector2D(12))
				]
			]
		],
		InOwnerTableView
	);
}

int32 SRCBehaviorSetAssetByPathNewElementRow::GetElementIndex() const
{
	return ElementItem->Index;
}

TOptional<EItemDropZone> SRCBehaviorSetAssetByPathNewElementRow::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, RCBehaviorSetAssetByPathNewElementListItem InItem)
{
	if (!SetAssetByPathBehaviorModelNewWeak.IsValid() || !ElementItem.IsValid())
	{
		return {};
	}

	TSharedPtr<FRCBehaviorSetAssetByPathNewElementDragDrop> PathBehaviorElementDragDrop = InDragDropEvent.GetOperationAs<FRCBehaviorSetAssetByPathNewElementDragDrop>();

	if (!PathBehaviorElementDragDrop.IsValid())
	{
		return {};
	}

	if (ElementItem->Index == PathBehaviorElementDragDrop->GetIndex())
	{
		return {};
	}

	return InDropZone;
}

FReply SRCBehaviorSetAssetByPathNewElementRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, RCBehaviorSetAssetByPathNewElementListItem InItem)
{
	if (!ElementItem.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FRCSetAssetByPathBehaviorModelNew> SetAssetByPathBehaviorModelNew = SetAssetByPathBehaviorModelNewWeak.Pin();

	if (!SetAssetByPathBehaviorModelNew.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FRCBehaviorSetAssetByPathNewElementDragDrop> PathBehaviorElementDragDrop = InDragDropEvent.GetOperationAs<FRCBehaviorSetAssetByPathNewElementDragDrop>();

	if (!PathBehaviorElementDragDrop.IsValid())
	{
		return FReply::Handled();
	}

	if (ElementItem->Index == PathBehaviorElementDragDrop->GetIndex())
	{
		return FReply::Handled();
	}

	SetAssetByPathBehaviorModelNew->ReorderElementItems(ElementItem->Index, InDropZone, PathBehaviorElementDragDrop->GetIndex());

	return FReply::Handled();
}

FReply SRCBehaviorSetAssetByPathNewElementRow::OnRowDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TSharedPtr<FRCSetAssetByPathBehaviorModelNew> SetAssetByPathBehaviorModelNew = SetAssetByPathBehaviorModelNewWeak.Pin();

	if (!SetAssetByPathBehaviorModelNew.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Unhandled();
	}

	TSharedRef<FRCBehaviorSetAssetByPathNewElementDragDrop> PathBehaviorElementDragDrop = MakeShared<FRCBehaviorSetAssetByPathNewElementDragDrop>(ElementItem->Index);
	return FReply::Handled().BeginDragDrop(PathBehaviorElementDragDrop);
}

const FSlateBrush* SRCBehaviorSetAssetByPathNewElementRow::GetValidityImage() const
{
	using namespace UE::RemoteControl::UI::Private::BehaviorSetAssetByPathNewElementRow;

	if (ElementItem.IsValid())
	{
		switch (ElementItem->Validity)
		{
			case ERCPathBehaviorElementValidty::ValidPath:
			case ERCPathBehaviorElementValidty::ValidAsset:
				return FAppStyle::Get().GetBrush(ValidImage);

			case ERCPathBehaviorElementValidty::InvalidPath:
			case ERCPathBehaviorElementValidty::InvalidAsset:
			case ERCPathBehaviorElementValidty::InvalidController:
				return FAppStyle::Get().GetBrush(ErrorImage);

			case ERCPathBehaviorElementValidty::Unknown:
				return FAppStyle::Get().GetBrush(NoImage);

			case ERCPathBehaviorElementValidty::Unchecked:
			case ERCPathBehaviorElementValidty::EmptyControllerValue:
				return FAppStyle::Get().GetBrush(WarningImage);
		}
	}

	return FAppStyle::Get().GetBrush(ErrorImage);
}

FText SRCBehaviorSetAssetByPathNewElementRow::GetValidityToolTip() const
{
	if (!ElementItem.IsValid())
	{
		return LOCTEXT("Error", "Error");
	}

	switch (ElementItem->Validity)
	{
		case ERCPathBehaviorElementValidty::ValidPath:
			return LOCTEXT("ValidPathElement", "Points to a valid folder.");

		case ERCPathBehaviorElementValidty::ValidAsset:
			return LOCTEXT("ValidAssetElement", "Points to a valid asset.");

		case ERCPathBehaviorElementValidty::InvalidPath:
			return LOCTEXT("InvalidPathElement", "Does not point to a valid folder!");

		case ERCPathBehaviorElementValidty::InvalidAsset:
			return LOCTEXT("InvalidAssetElement", "Does not point to a valid asset!");

		case ERCPathBehaviorElementValidty::InvalidController:
			return LOCTEXT("InvalidControllerElement", "Invalid controller!");

		case ERCPathBehaviorElementValidty::EmptyControllerValue:
			return LOCTEXT("EmptyControllerValueElement", "Empty controller value!");

		default:
			return LOCTEXT("UnknownPathElement", "Unknown.");
	}
}

#undef LOCTEXT_NAMESPACE
