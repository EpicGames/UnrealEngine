// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/Optional.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FRCSetAssetByPathBehaviorModelNew;
struct FRCPathBehaviorElementRow;
struct FSlateBrush;

using RCBehaviorSetAssetByPathNewElementListItem = TSharedPtr<FRCPathBehaviorElementRow>;

enum ERCPathBehaviorElementValidty
{
	ValidPath,
	ValidAsset,
	InvalidPath,
	InvalidAsset,
	InvalidController,
	EmptyControllerValue,
	Unknown,
	Unchecked
};

struct FRCPathBehaviorElementRow
{
	int32 Index;
	TSharedRef<SWidget> Widget;
	ERCPathBehaviorElementValidty Validity = ERCPathBehaviorElementValidty::Unchecked;
};

class SRCBehaviorSetAssetByPathNewElementRow : public STableRow<RCBehaviorSetAssetByPathNewElementListItem>
{
	using FSuperRowType = STableRow<RCBehaviorSetAssetByPathNewElementListItem>;

public:
	SLATE_BEGIN_ARGS(SRCBehaviorSetAssetByPathNewElementRow) 
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		TSharedRef<FRCSetAssetByPathBehaviorModelNew> InSetAssetByPathBehaviorModelNew, RCBehaviorSetAssetByPathNewElementListItem InElementItem);

	int32 GetElementIndex() const;

private:
	/** Called to validate whether a drag drop event is accepted for a behavior item row */
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, RCBehaviorSetAssetByPathNewElementListItem InItem);

	/** Called when a drop happens on this behavior item row */
	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, RCBehaviorSetAssetByPathNewElementListItem InItem);

	FReply OnRowDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	const FSlateBrush* GetValidityImage() const;

	FText GetValidityToolTip() const;

	TWeakPtr<FRCSetAssetByPathBehaviorModelNew> SetAssetByPathBehaviorModelNewWeak;
	RCBehaviorSetAssetByPathNewElementListItem ElementItem;
};