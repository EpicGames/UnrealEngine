// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourSetAssetByPath.h"

#include "IDetailTreeNode.h"
#include "RemoteControlPreset.h"
#include "SPositiveActionButton.h"
#include "SlateOptMacros.h"
#include "Styling/StyleColors.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviourSetAssetByPathModel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SRCBehaviourSetAssetByPath"

class SPositiveActionButton;

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCBehaviourSetAssetByPath::Construct(const FArguments& InArgs, TSharedRef<FRCSetAssetByPathBehaviourModel> InBehaviourItem)
{
	SetAssetByPathWeakPtr = InBehaviourItem;
	PathBehaviour = Cast<URCSetAssetByPathBehaviour>(InBehaviourItem->GetBehaviour());
	if (!PathBehaviour.IsValid())
	{
		ChildSlot
		[
			SNullWidget::NullWidget
		];
		return;
	}

	const TSharedRef<SHorizontalBox> SelectedClassWidget = SNew(SHorizontalBox);
	const TSharedRef<SHorizontalBox> InternalExternalSwitchWidget = SNew(SHorizontalBox)
		.Visibility(this, &SRCBehaviourSetAssetByPath::GetInternalExternalVisibility);

	for (UClass* SupportedClass : PathBehaviour->GetSupportedClasses())
	{
		SelectedClassWidget->AddSlot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.0f,0.0f))
		.FillWidth(1.0f)
		.AutoWidth()
		[
			SNew(SButton)
			.Text(FText::FromString(SupportedClass->GetName()))
			.ButtonColorAndOpacity(this, &SRCBehaviourSetAssetByPath::GetSelectedClassWidgetColor, SupportedClass)
			.OnPressed(this, &SRCBehaviourSetAssetByPath::OnSelectedClassWidgetPressed, SupportedClass)
		];
	}
		
	InternalExternalSwitchWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(0.0f,0.0f))
	.FillWidth(1.0f)
	.AutoWidth()
	[
		SNew(SButton)
		.Text(FText::FromString(FString("Internal")))
		.ButtonColorAndOpacity(this, &SRCBehaviourSetAssetByPath::GetInternalWidgetColor)
		.OnPressed(this, &SRCBehaviourSetAssetByPath::OnInternalExternalSwitchWidgetPressed, true)
	];
	
	InternalExternalSwitchWidget->AddSlot()
	.HAlign(HAlign_Right)
	.VAlign(VAlign_Center)
	.Padding(FMargin(0.0f,0.0f))
	.FillWidth(1.0f)
	.AutoWidth()
	[
		SNew(SButton)
		.Text(FText::FromString(FString("External")))
		.ButtonColorAndOpacity(this, &SRCBehaviourSetAssetByPath::GetExternalWidgetColor)
		.OnPressed(this, &SRCBehaviourSetAssetByPath::OnInternalExternalSwitchWidgetPressed, false)
	];
	
	ChildSlot
	.Padding(8.f, 4.f)
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SelectedClassWidget
		]
		+ SVerticalBox::Slot()
		.Padding(0.f,12.f)
		[
			InternalExternalSwitchWidget
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			InBehaviourItem->GetPropertyWidget()
		]
	];
}

void SRCBehaviourSetAssetByPath::OnInternalExternalSwitchWidgetPressed(bool bInIsInternal) const
{
	if (!PathBehaviour.IsValid())
	{
		return;
	}

	PathBehaviour->bInternal = bInIsInternal;
	if (const TSharedPtr<const FRCSetAssetByPathBehaviourModel>& SetAssetPath = SetAssetByPathWeakPtr.Pin())
	{
		SetAssetPath->RefreshPreview();
	}
}

FSlateColor SRCBehaviourSetAssetByPath::GetExternalWidgetColor() const
{
	if (!PathBehaviour.IsValid())
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	return PathBehaviour->bInternal ? FAppStyle::Get().GetSlateColor("Colors.AccentWhite") : FAppStyle::Get().GetSlateColor("Colors.Highlight");
}

FSlateColor SRCBehaviourSetAssetByPath::GetInternalWidgetColor() const
{
	if (!PathBehaviour.IsValid())
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	return PathBehaviour->bInternal ? FAppStyle::Get().GetSlateColor("Colors.Highlight") : FAppStyle::Get().GetSlateColor("Colors.AccentWhite");
}

EVisibility SRCBehaviourSetAssetByPath::GetInternalExternalVisibility() const
{
	if (!PathBehaviour->AssetClass)
	{
		return EVisibility::Collapsed;
	}
	return PathBehaviour->AssetClass->GetFName() == TEXT("Texture") ? EVisibility::Visible : EVisibility::Collapsed;
}

FSlateColor SRCBehaviourSetAssetByPath::GetSelectedClassWidgetColor(UClass* InClass) const
{
	if (!PathBehaviour.IsValid())
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	return PathBehaviour->AssetClass == InClass ? FAppStyle::Get().GetSlateColor("Colors.Select") : FAppStyle::Get().GetSlateColor("Colors.White");
}

void SRCBehaviourSetAssetByPath::OnSelectedClassWidgetPressed(UClass* InClass) const
{
	if (!PathBehaviour.IsValid())
	{
		return;
	}
	PathBehaviour->AssetClass = InClass;
	OnInternalExternalSwitchWidgetPressed(true);
}


END_SLATE_FUNCTION_BUILD_OPTIMIZATION

#undef LOCTEXT_NAMESPACE
