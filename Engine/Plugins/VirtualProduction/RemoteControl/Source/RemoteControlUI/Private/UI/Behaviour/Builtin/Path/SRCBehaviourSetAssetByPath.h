// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"

class FRCSetAssetByPathBehaviourModel;
class UClass;
class URCSetAssetByPathBehaviour;

class REMOTECONTROLUI_API SRCBehaviourSetAssetByPath : public  SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourSetAssetByPath)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<FRCSetAssetByPathBehaviourModel> InBehaviourItem);

private:
	/** Called when the Internal/External button are pressed */
	void OnInternalExternalSwitchWidgetPressed(bool bInIsInternal) const;

	/** Return the color of the External widget */
	FSlateColor GetExternalWidgetColor() const;

	/** Return the color of the Internal widget */
	FSlateColor GetInternalWidgetColor() const;

	/** Return the visibility of the InternalExternal widget */
	EVisibility GetInternalExternalVisibility() const;

	/** Return the color of the SelectedClass widget */
	FSlateColor GetSelectedClassWidgetColor(UClass* InClass) const;

	/** Called when the SelectedClass buttons are pressed */
	void OnSelectedClassWidgetPressed(UClass* InClass) const;

private:
	/** The Behaviour (UI model) associated with us */
	TWeakPtr<const FRCSetAssetByPathBehaviourModel> SetAssetByPathWeakPtr;

	/** The PathBehaviour associated with us */
	TWeakObjectPtr<URCSetAssetByPathBehaviour> PathBehaviour;
};
