// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBehaviourRangeMapModel.h"

#include "Action/RCAction.h"
#include "Action/RCPropertyAction.h"
#include "Controller/RCController.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "SRCBehaviourRangeMap.h"
#include "UI/Action/RangeMap/RCActionRangeMapModel.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCActionPanelList.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FRCRangeMapBehaviourModel"

FRCRangeMapBehaviourModel::FRCRangeMapBehaviourModel(URCRangeMapBehaviour* RangeMapBehaviour)
	: FRCBehaviourModel(RangeMapBehaviour)
	, RangeMapBehaviourWeakPtr(RangeMapBehaviour)
{
	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties = true;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);

	if (RangeMapBehaviour)
	{
		PropertyRowGenerator->SetStructure(RangeMapBehaviour->PropertyContainer->CreateStructOnScope());

		for (const TSharedRef<IDetailTreeNode>& CategoryNode : PropertyRowGenerator->GetRootTreeNodes())
		{
			TArray<TSharedRef<IDetailTreeNode>> Children;
			CategoryNode->GetChildren(Children);
			for (const TSharedRef<IDetailTreeNode>& Child : Children)
			{
				DetailTreeNodeWeakPtrArray.Add(Child);
			}
		}
	}
}

bool FRCRangeMapBehaviourModel::HasBehaviourDetailsWidget()
{
	return true;
}

TSharedRef<SWidget> FRCRangeMapBehaviourModel::GetBehaviourDetailsWidget()
{
	return SNew(SRCBehaviourRangeMap, SharedThis(this));
}

TSharedRef<SWidget> FRCRangeMapBehaviourModel::GetPropertyWidget() const
{
	const TSharedRef<SVerticalBox> FieldWidget = SNew(SVerticalBox);
	
	auto DetailTreeItr = DetailTreeNodeWeakPtrArray.CreateConstIterator();

	TSharedPtr<IDetailTreeNode> DetailTreeMinValue = DetailTreeItr->Pin();
	TSharedPtr<IDetailTreeNode> DetailTreeMaxValue = (++DetailTreeItr)->Pin();
	FNodeWidgets MinInputWidget = DetailTreeMinValue->CreateNodeWidgets();
	FNodeWidgets MaxInputWidget = DetailTreeMaxValue->CreateNodeWidgets();

	if (MinInputWidget.ValueWidget && MaxInputWidget.ValueWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				CreateMinMaxWidget(DetailTreeMinValue, DetailTreeMaxValue)
			];
	}
	

	return 	SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Top)
		.FillWidth(1.0f)
		.AutoWidth()
		[
			FieldWidget
		];
}

void FRCRangeMapBehaviourModel::OnActionAdded(URCAction* Action)
{
	if (URCRangeMapBehaviour* RangeMapBehaviour = Cast<URCRangeMapBehaviour>(GetBehaviour()))
	{
		if (URCPropertyAction* PropertyAction = Cast<URCPropertyAction>(Action))
		{
			RangeMapBehaviour->OnActionAdded(Action,PropertyAction->PropertySelfContainer);
		}
	}
}

TSharedPtr<SRCLogicPanelListBase> FRCRangeMapBehaviourModel::GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel)
{
	return SNew(SRCActionPanelList<FRCActionRangeMapModel>, InActionPanel, SharedThis(this));
}

URCAction* FRCRangeMapBehaviourModel::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	URCAction* NewAction = nullptr;

	if (URCRangeMapBehaviour* RangeMapBehaviour = Cast<URCRangeMapBehaviour>(GetBehaviour()))
	{
		double StepValue;
		RangeMapBehaviour->PropertyContainer->GetVirtualProperty(FName("Input"))->GetValueDouble(StepValue);

		const FGuid FieldId = InRemoteControlField->GetId();
		if (const TSharedPtr<FRemoteControlProperty> RemoteProperty = RangeMapBehaviour->ControllerWeakPtr.Get()->PresetWeakPtr.Get()->GetExposedEntity<FRemoteControlProperty>(FieldId).Pin())
		{
			const TObjectPtr<URCVirtualPropertySelfContainer> VirtualPropertySelfContainer = NewObject<URCVirtualPropertySelfContainer>();
			VirtualPropertySelfContainer->DuplicateProperty(RemoteProperty->GetProperty()->GetFName(), RemoteProperty->GetProperty());
			
			NewAction = RangeMapBehaviour->AddAction(InRemoteControlField);

			OnActionAdded(NewAction);
		}
	}

	return NewAction;
}

TSharedRef<SWidget> FRCRangeMapBehaviourModel::CreateMinMaxWidget(TSharedPtr<IDetailTreeNode> MinInputDetailTree, TSharedPtr<IDetailTreeNode> MaxInputDetailTree) const
{
	const FNodeWidgets MinInputWidget = MinInputDetailTree->CreateNodeWidgets();
	const FNodeWidgets MaxInputWidget = MaxInputDetailTree->CreateNodeWidgets();
	
	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.Padding(FMargin(3.0f, 2.0f))
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				MinInputWidget.NameWidget.ToSharedRef()
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				MinInputWidget.ValueWidget.ToSharedRef()
			]
		]
		+SHorizontalBox::Slot()
		.Padding(FMargin(3.0f, 2.0f))
		.HAlign(HAlign_Left)
		.AutoWidth()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Left)
			.AutoHeight()
			[
				MaxInputWidget.NameWidget.ToSharedRef()
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(0.0f, 2.0f))
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				MaxInputWidget.ValueWidget.ToSharedRef()
			]
		];
}

#undef LOCTEXT_NAMESPACE
