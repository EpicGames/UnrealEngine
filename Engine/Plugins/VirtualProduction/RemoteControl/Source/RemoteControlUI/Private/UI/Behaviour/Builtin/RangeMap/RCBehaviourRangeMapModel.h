// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Behaviour/Builtin/RangeMap/RCRangeMapBehaviour.h"
#include "UI/Behaviour/RCBehaviourModel.h"

class IDetailTreeNode;
class IPropertyRowGenerator;
class SWidget;

/*
* ~ FRCRangeMapBehaviourModel ~
*
* Child Behaviour class representing the "Range Mapping" Behaviour's UI model
* 
* Generates a Property Widget where users can enter the Max, Min, Threshhold, and Step Values.
* The values of them are used for the interaction implementation of the properties it affects..
*/
class FRCRangeMapBehaviourModel : public FRCBehaviourModel
{
public:
	FRCRangeMapBehaviourModel(URCRangeMapBehaviour* RangeMapBehaviour);

	/** Returns true if this behaviour have a details widget or false if not*/
	virtual bool HasBehaviourDetailsWidget() override;

	/** Builds a Behaviour specific widget as required for Range Mapping behaviour*/
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget() override;

	/** Builds a generic Value Widget of the Controller's type
	* Use to store user input for performing the Range Mapping Behaviour Interactions */
	TSharedRef<SWidget> GetPropertyWidget() const;

	/** Add a Logic Action using a remote control field as input */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField) override;

	/** The Actions List Widget used in the RangeMap Behaviour */
	virtual TSharedPtr<SRCLogicPanelListBase> GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel) override;

protected:
	/** Invoked by parent class after an action has been added from UI*/
	virtual void OnActionAdded(URCAction* Action) override;

private:
	/** The Range Mapping Behaviour (Data model) associated with us*/
	TWeakObjectPtr<URCRangeMapBehaviour> RangeMapBehaviourWeakPtr;

	/** The row generator used to build a generic Value Widget*/
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** Used to create generic Value Widgets for lerping control of values */
	TArray<TWeakPtr<IDetailTreeNode>> DetailTreeNodeWeakPtrArray;

private:
	/** Creates the horizontal box containing box the Minimum and Maximum Range Widgets */
	TSharedRef<SWidget> CreateMinMaxWidget(TSharedPtr<IDetailTreeNode> MinInputDetailTree, TSharedPtr<IDetailTreeNode> MaxInputDetailTree) const;
};
