// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourRangeMap.h"

#include "SPositiveActionButton.h"
#include "SlateOptMacros.h"
#include "Styling/CoreStyle.h"
#include "UI/Behaviour/Builtin/RangeMap/RCBehaviourRangeMapModel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SSeparator.h"

#define LOCTEXT_NAMESPACE "SRCBehaviourRangeMap"

class SPositiveActionButton;

void SRCBehaviourRangeMap::Construct(const FArguments& InArgs, TSharedRef<const FRCRangeMapBehaviourModel> InBehaviourItem)
{
	RangeMapBehaviourItemWeakPtr = InBehaviourItem;

	ChildSlot
	.Padding(8.f, 4.f)
	[
		SNew(SHorizontalBox)

		// PropertyWidget
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(FMargin(5.f))
		[
			SNew(SHorizontalBox)
			// Value Widget
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				InBehaviourItem->GetPropertyWidget()
			]
		]
	];
}

#undef LOCTEXT_NAMESPACE