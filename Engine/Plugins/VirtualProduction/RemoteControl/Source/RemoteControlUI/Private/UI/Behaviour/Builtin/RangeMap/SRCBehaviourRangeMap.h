// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlField.h"
#include "Widgets/SCompoundWidget.h"

class FRCRangeMapBehaviourModel;

/*
 * ~ SRCBehaviourRangeMap
 * 
 * Behaviour specific details panel for Range Mapping Behaviour.
 * Contains Input Panels which will be used to calculate and work with in the Behaviour itself.
 */
class REMOTECONTROLUI_API SRCBehaviourRangeMap : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourRangeMap){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, TSharedRef<const FRCRangeMapBehaviourModel> InBehaviourItem);

private:
	/** The Behaviour (UI Model) associated with this. */
	TWeakPtr<const FRCRangeMapBehaviourModel> RangeMapBehaviourItemWeakPtr;
};
