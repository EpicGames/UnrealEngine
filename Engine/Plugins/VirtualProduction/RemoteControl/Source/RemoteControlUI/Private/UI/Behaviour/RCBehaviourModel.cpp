// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBehaviourModel.h"

#include "Behaviour/RCBehaviourNode.h"
#include "Controller/RCController.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/RCActionModel.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Action/SRCActionPanelList.h"
#include "UI/BaseLogicUI/RCLogicHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FRCBehaviourModel"

const FLazyName FRCBehaviourModel::BehaviorModelName = "BehaviorModel";

FRCBehaviourModel::FRCBehaviourModel(URCBehaviour* InBehaviour
	, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel /*= nullptr*/)
	: FRCLogicModeBase(InRemoteControlPanel)
	, BehaviourWeakPtr(InBehaviour)
{
	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.BehaviourPanel");

	if (InBehaviour)
	{
		// Determine whether the behavior node has any property that is instance-editable
		if (InBehaviour->BehaviourNodeClass)
		{
			for (FProperty* Property : TFieldRange<FProperty>(InBehaviour->BehaviourNodeClass))
			{
				if (Property->HasAllPropertyFlags(CPF_Edit) && !Property->HasAnyPropertyFlags(CPF_DisableEditOnInstance))
				{
					bDetailsEditableBehavior = true;
					break;
				}
			}
		}

		const FText BehaviourDisplayName = InBehaviour->GetDisplayName();

		SAssignNew(BehaviourTitleText, STextBlock)
			.Text(BehaviourDisplayName)
			.TextStyle(&RCPanelStyle->HeaderTextStyle);

		RefreshIsBehaviourEnabled(InBehaviour->bIsEnabled);
	}
}

URCAction* FRCBehaviourModel::AddAction()
{
	URCAction* NewAction = nullptr;
	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		NewAction = Behaviour->AddAction();
		OnActionAdded(NewAction);
	}
	return NewAction;
}

URCAction* FRCBehaviourModel::AddAction(FName InFieldId)
{
	URCAction* NewAction = nullptr;
	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		NewAction = Behaviour->AddAction(InFieldId);
		OnActionAdded(NewAction);
	}
	return NewAction;
}

URCAction* FRCBehaviourModel::AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField)
{
	URCAction* NewAction = nullptr;

	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		NewAction = Behaviour->AddAction(InRemoteControlField);

		OnActionAdded(NewAction);
	}

	return NewAction;
}

TSharedRef<SWidget> FRCBehaviourModel::GetWidget() const
{
	if (!ensure(BehaviourWeakPtr.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		// Behaviour name
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		.Padding(FMargin(8.f))
		[
			BehaviourTitleText.ToSharedRef()
		];
}

bool FRCBehaviourModel::HasBehaviourDetailsWidget()
{
	return bDetailsEditableBehavior;
}

TSharedRef<SWidget> FRCBehaviourModel::GetBehaviourDetailsWidget()
{
	if (!bDetailsEditableBehavior)
	{
		return SNullWidget::NullWidget;
	}

	URCBehaviour* Behavior = BehaviourWeakPtr.Get();
	if (!Behavior)
	{
		return SNullWidget::NullWidget;
	}

	URCBehaviourNode* BehaviorNode = Behavior->GetBehaviourNode();
	if (!BehaviorNode)
	{
		return SNullWidget::NullWidget;
	}

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Automatic;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bHideSelectionTip = true;
	DetailsViewArgs.bLockable = false;
	DetailsViewArgs.bSearchInitialKeyFocus = true;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bShowScrollBar = true;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(DetailsViewArgs);
	DetailsView->SetObject(BehaviorNode);

	return SNew(SBox)
		.MaxDesiredHeight(200.f)
		[
			DetailsView
		];
}

void FRCBehaviourModel::OnOverrideBlueprint() const
{
	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		UBlueprint* Blueprint = Behaviour->GetBlueprint();
		if (!Blueprint)
		{
			Blueprint = UE::RCLogicHelpers::CreateBlueprintWithDialog(Behaviour->BehaviourNodeClass, Behaviour->GetPackage(), UBlueprint::StaticClass(), UBlueprintGeneratedClass::StaticClass());
			Behaviour->SetOverrideBehaviourBlueprintClass(Blueprint);
		}

		UE::RCLogicHelpers::OpenBlueprintEditor(Blueprint);
	}
}

bool FRCBehaviourModel::IsBehaviourEnabled() const
{
	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		return Behaviour->bIsEnabled;
	}

	return false;
}

void FRCBehaviourModel::SetIsBehaviourEnabled(const bool bIsEnabled)
{
	if (URCBehaviour* Behaviour = BehaviourWeakPtr.Get())
	{
		Behaviour->bIsEnabled = bIsEnabled;

		RefreshIsBehaviourEnabled(bIsEnabled);
	}
}

void FRCBehaviourModel::RefreshIsBehaviourEnabled(const bool bIsEnabled)
{
	BehaviourTitleText->SetEnabled(bIsEnabled);
}

TSharedPtr<SRCLogicPanelListBase> FRCBehaviourModel::GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel)
{
	// Returns the default Actions List; child classes can override as required

	return SNew(SRCActionPanelList<FRCActionModel>, InActionPanel, SharedThis(this));
}

bool FRCBehaviourModel::SupportPropertyId() const
{
	if (const URCBehaviour* Behavior = BehaviourWeakPtr.Get())
	{
		return Behavior->SupportPropertyId();
	}
	return false;
}

URCBehaviour* FRCBehaviourModel::GetBehaviour() const
{
	return BehaviourWeakPtr.Get();
}

int32 FRCBehaviourModel::RemoveModel()
{
	URCBehaviour* Behavior = GetBehaviour();

	if (!Behavior)
	{
		return 0;
	}

	URCController* Controller = Behavior->ControllerWeakPtr.Get();

	if (!Controller)
	{
		return 0;
	}

	if (GUndo)
	{
		Controller->Modify();
	}

	return Controller->RemoveBehaviour(Behavior) ? 1 : 0;
}

#undef LOCTEXT_NAMESPACE