// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviour.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"

class IPropertyRowGenerator;
class STextBlock;
class SRCActionPanel;
class SRCLogicPanelListBase;
class SWidget;
class URCBehaviour;
struct FRCPanelStyle;

/*
* ~ FRCBehaviourModel ~
*
* UI model for representing a Behaviour
* Generates a row widget with Behaviour related metadata
*/
class FRCBehaviourModel : public FRCLogicModeBase
{
public:
	static const FLazyName BehaviorModelName;

	FRCBehaviourModel(URCBehaviour* InBehaviour, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel = nullptr);

	virtual FName GetModelType() const override
	{
		return BehaviorModelName;
	}

	virtual int32 RemoveModel() override;

	/** Add a Logic Action using as an identity action. */
	virtual URCAction* AddAction();

	/** Add a Logic Action using as an identity action. */
	virtual URCAction* AddAction(FName InFieldId);

	/** Add a Logic Action using a remote control field as input. */
	virtual URCAction* AddAction(const TSharedRef<const FRemoteControlField> InRemoteControlField);

	/** The widget to be rendered for this Property in the Behaviour panel
	 * Currently displays Behaviour Name metadata in the Behaviours Panel List
	 */
	virtual TSharedRef<SWidget> GetWidget() const override;

	/** Returns true if this behaviour have a details widget or false if not. */
	virtual bool HasBehaviourDetailsWidget();

	/** Builds a Behaviour specific widget that child Behaviour classes can implement as required. */
	virtual TSharedRef<SWidget> GetBehaviourDetailsWidget();

	/** Returns the Behaviour (Data model) associated with us. */
	URCBehaviour* GetBehaviour() const;

	/** Handling for user action to override this behaviour via new Blueprint class. */
	void OnOverrideBlueprint() const;

	/** Whether the underlying Behaviour is currently enabled. */
	bool IsBehaviourEnabled() const;

	/** Set the Enabled state of our underlying Behaviour. */
	void SetIsBehaviourEnabled(const bool bIsEnabled);

	/** Refreshes the UI widgets enabled state depending on whether the parent behaviour is currently enabled. */
	void RefreshIsBehaviourEnabled(const bool bIsEnabled);

	/** The Actions List widget to be used for this Behaviour.
	 * The default actions table is used if a behaviour doesn't specify it.
	 */
	virtual TSharedPtr<SRCLogicPanelListBase> GetActionsListWidget(TSharedRef<SRCActionPanel> InActionPanel);

	/** Whether or not the behaviour support PropertyId action. */
	bool SupportPropertyId() const;

	/** Called when a controller value changed */
	virtual void NotifyControllerValueChanged(TSharedPtr<class FRCControllerModel> InControllerModel) {}

protected:
	/** Invoked after an action has been added for this Behaviour in the actions panel. */
	virtual void OnActionAdded(URCAction* Action) {}

private:
	/** The Behaviour (Data model) associated with us. */
	TWeakObjectPtr<URCBehaviour> BehaviourWeakPtr;

	/** Text block widget for representing the Behaviour's title. */
	TSharedPtr<STextBlock> BehaviourTitleText;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Determines if the behavior is an object editable in details view */
	bool bDetailsEditableBehavior = false;
};