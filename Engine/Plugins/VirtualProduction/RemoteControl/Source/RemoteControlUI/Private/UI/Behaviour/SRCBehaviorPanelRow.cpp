// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Behaviour/SRCBehaviorPanelRow.h"

#include "Behaviour/RCBehaviour.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Styles/SlateBrushTemplates.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Behaviour/SRCBehaviourPanelList.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "SRCBehaviorPanelRow"

/*
* ~ FRCBehaviorPanelDragDrop ~
* Facilitates drag-drop operation
*/
class FRCBehaviorPanelDragDrop final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRCBehaviorPanelDragDrop, FDecoratedDragDropOp)

	explicit FRCBehaviorPanelDragDrop(const TArray<FRCBehaviorPanelListItem>& InItems)
		: Items(InItems)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	TArray<FRCBehaviorPanelListItem> GetItems() const
	{
		return Items;
	}

private:
	TArray<FRCBehaviorPanelListItem> Items;
};

void SRCBehaviorPanelRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, TSharedRef<SRCBehaviourPanelList> InBehaviorPanelList, FRCBehaviorPanelListItem InBehaviorItem)
{
	BehaviorItem = InBehaviorItem;
	BehaviorPanelListWeak = InBehaviorPanelList;

	SetCursor(EMouseCursor::GrabHand);

	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	STableRow::Construct(
		STableRow::FArguments()
		.Style(InArgs._Style)
		.Padding(0.f)
		.OnDragDetected(this, &SRCBehaviorPanelRow::OnRowDragDetected)
		.OnCanAcceptDrop(this, &SRCBehaviorPanelRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SRCBehaviorPanelRow::OnAcceptDrop)
		.ShowWires(false)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.f, 0.f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("VerticalBoxDragIndicatorShort"))
			]

			+ SHorizontalBox::Slot()
			[
				InBehaviorItem->GetWidget()
			]

			// Toggle Behaviour Button
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.f, 0.f)
			[
				SNew(SCheckBox)
				.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Toggle Behaviour")))
				.ToolTipText(LOCTEXT("EditModeTooltip", "Enable/Disable this Behaviour.\nWhen a behaviour is disabled its Actions will not be processed when the Controller value changes"))
				.HAlign(HAlign_Center)
				.ForegroundColor(FLinearColor::White)
				.Style(&RCPanelStyle->ToggleButtonStyle)
				.IsChecked(this, &SRCBehaviorPanelRow::IsBehaviourChecked)
				.OnCheckStateChanged(this, &SRCBehaviorPanelRow::OnToggleEnableBehaviour)
			]
		],
		InOwnerTableView
	);
}

FRCBehaviorPanelListItem SRCBehaviorPanelRow::GetBehavior() const
{
	return BehaviorItem;
}

TOptional<EItemDropZone> SRCBehaviorPanelRow::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCBehaviorPanelListItem InItem)
{
	if (!BehaviorPanelListWeak.IsValid() || !BehaviorItem.IsValid())
	{
		return {};
	}

	TSharedPtr<FRCBehaviorPanelDragDrop> BehaviorDragDrop = InDragDropEvent.GetOperationAs<FRCBehaviorPanelDragDrop>();

	if (!BehaviorDragDrop.IsValid())
	{
		return {};
	}

	const TArray<FRCBehaviorPanelListItem> Items = BehaviorDragDrop->GetItems();

	if (Items.IsEmpty())
	{
		return {};
	}

	if (Items.Num() == 1 && Items[0].Get() == BehaviorItem.Get())
	{
		return {};
	}

	return InDropZone;
}

FReply SRCBehaviorPanelRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCBehaviorPanelListItem InItem)
{
	if (!BehaviorItem.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<SRCBehaviourPanelList> BehaviorPanelList = BehaviorPanelListWeak.Pin();

	if (!BehaviorPanelList.IsValid())
	{
		return FReply::Handled();
	}

	TSharedPtr<FRCBehaviorPanelDragDrop> BehaviorDragDrop = InDragDropEvent.GetOperationAs<FRCBehaviorPanelDragDrop>();

	if (!BehaviorDragDrop.IsValid())
	{
		return FReply::Handled();
	}

	const TArray<FRCBehaviorPanelListItem> Items = BehaviorDragDrop->GetItems();

	if (Items.IsEmpty())
	{
		return FReply::Handled();
	}

	BehaviorPanelList->ReorderBehaviorItems(BehaviorItem.ToSharedRef(), InDropZone, Items);

	return FReply::Handled();
}

FReply SRCBehaviorPanelRow::OnRowDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	TSharedPtr<SRCBehaviourPanelList> BehaviorPanelList = BehaviorPanelListWeak.Pin();

	if (!BehaviorPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	if (!InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Unhandled();
	}

	TSharedRef<FRCBehaviorPanelDragDrop> BehaviorDragDrop = MakeShared<FRCBehaviorPanelDragDrop>(BehaviorPanelList->GetSelectedBehaviourItems());
	return FReply::Handled().BeginDragDrop(BehaviorDragDrop);
}

ECheckBoxState SRCBehaviorPanelRow::IsBehaviourChecked() const
{
	return BehaviorItem.IsValid() && BehaviorItem->IsBehaviourEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SRCBehaviorPanelRow::OnToggleEnableBehaviour(ECheckBoxState State)
{
	if (BehaviorItem.IsValid())
	{
		if (TSharedPtr<SRCBehaviourPanelList> BehaviorPanelList = BehaviorPanelListWeak.Pin())
		{
			BehaviorPanelList->SetIsBehaviourEnabled(BehaviorItem.ToSharedRef(), State == ECheckBoxState::Checked);
		}
	}
}

#undef LOCTEXT_NAMESPACE
