// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FRCBehaviourModel;
class SRCBehaviourPanelList;
class URemoteControlPreset;

using FRCBehaviorPanelListItem = TSharedPtr<FRCBehaviourModel>;

class SRCBehaviorPanelRow : public STableRow<FRCBehaviorPanelListItem>
{
	using FSuperRowType = STableRow<FRCBehaviorPanelListItem>;

public:
	SLATE_BEGIN_ARGS(SRCBehaviorPanelRow) 
		{}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		TSharedRef<SRCBehaviourPanelList> InBehaviorPanelList, FRCBehaviorPanelListItem InBehaviorItem);

	FRCBehaviorPanelListItem GetBehavior() const;

private:
	/** Called to validate whether a drag drop event is accepted for a behavior item row */
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCBehaviorPanelListItem InItem);

	/** Called when a drop happens on this behavior item row */
	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCBehaviorPanelListItem InItem);

	FReply OnRowDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Return whether or not the given behaviour is checked */
	ECheckBoxState IsBehaviourChecked() const;

	/** Executed when toggling the behaviour state */
	void OnToggleEnableBehaviour(ECheckBoxState State);

	TWeakPtr<SRCBehaviourPanelList> BehaviorPanelListWeak;
	FRCBehaviorPanelListItem BehaviorItem;
};