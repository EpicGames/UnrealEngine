// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCBehaviourPanel.h"
#include "SRCBehaviourPanelList.h"

#include "Behaviour/Builtin/Path/RCBehaviourSetAssetByPathNode.h"
#include "Behaviour/RCBehaviour.h"
#include "Behaviour/RCBehaviourBlueprintNode.h"
#include "Behaviour/RCBehaviourNode.h"
#include "Controller/RCController.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "Misc/MessageDialog.h"

#include "SlateOptMacros.h"
#include "Styling/RemoteControlStyles.h"

#include "UI/Controller/RCControllerModel.h"
#include "UI/Panels/SRCDockPanel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRemoteControlPanel.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "SRCBehaviourPanel"

TSharedRef<SBox> SRCBehaviourPanel::GetNoneSelectedWidget()
{
	return SNew(SBox)
		.Padding(10.f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoneSelected", "Select a controller\nto view its behaviors."))
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.Justification(ETextJustify::Center)
			.AutoWrapText(true)
		];
}

void SRCBehaviourPanel::Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	SRCLogicPanelBase::Construct(SRCLogicPanelBase::FArguments(), InPanel);

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	UpdateWrappedWidget();

	ChildSlot
		.Padding(RCPanelStyle->PanelPadding)
		[
			WrappedBoxWidget.ToSharedRef()
		];

	// Register delegates
	InPanel->OnControllerSelectionChanged.AddSP(this, &SRCBehaviourPanel::OnControllerSelectionChanged);
	InPanel->OnControllerValueChangedDelegate.AddSP(this, &SRCBehaviourPanel::OnControllerValueChanged);
}

void SRCBehaviourPanel::OnControllerSelectionChanged(TSharedPtr<FRCControllerModel> InControllerItem, ESelectInfo::Type InSelectInfo)
{
	SelectedControllerItemWeakPtr = InControllerItem;

	// Cache the selection here since UpdateWrappedWidget will recreate the BehaviorList
	bool bShouldRestoreSelection = false;
	TArray<TSharedPtr<FRCBehaviourModel>> CurrentSelectedItems;
	if (BehaviourPanelList.IsValid() && InSelectInfo == ESelectInfo::Direct)
	{
		CurrentSelectedItems = BehaviourPanelList->GetSelectedBehaviourItems();
		bShouldRestoreSelection = true;
	}

	UpdateWrappedWidget(InControllerItem);

	if (BehaviourPanelList.IsValid())
	{
		if (bShouldRestoreSelection)
		{
			BehaviourPanelList->SetSelection(CurrentSelectedItems);
		}
		else if (InSelectInfo != ESelectInfo::Direct)
		{
			BehaviourPanelList->SelectFirstItem();
		}
	}
}

void SRCBehaviourPanel::OnControllerValueChanged(TSharedPtr<FRCControllerModel> InController)
{
	BehaviourPanelList->NotifyControllerValueChanged(InController);
}

void SRCBehaviourPanel::UpdateWrappedWidget(const TSharedPtr<FRCControllerModel>& InControllerItem)
{
	// Create panel widgets if invalid
	if (!AreBehaviorPanelWidgetsValid())
	{
		CreateBehaviorPanelWidgets();
	}

	if (InControllerItem.IsValid() && BehaviourPanelList->GetControllerItem() != InControllerItem)
	{
		BehaviourPanelList->SetControllerItem(InControllerItem);
		AddNewBehaviorComboButton->SetMenuContent(GetBehaviourMenuContentWidget());
	}
	else
	{
		BehaviourPanelList->SetControllerItem(nullptr);
	}
}

TSharedRef<SWidget> SRCBehaviourPanel::GetBehaviourMenuContentWidget()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	if (const TSharedPtr<FRCControllerModel> ControllerItem = SelectedControllerItemWeakPtr.Pin())
	{
		if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
		{
			for (TObjectIterator<UClass> It; It; ++It)
			{
				UClass* Class = *It;

				if (Class == URCBehaviourSetAssetByPathNode::StaticClass())
				{
					continue;
				}

				if (Class->IsChildOf(URCBehaviourNode::StaticClass()))
				{
					const bool bIsClassInstantiatable = Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists | CLASS_Abstract) || FKismetEditorUtilities::IsClassABlueprintSkeleton(Class);
					if (bIsClassInstantiatable)
					{
						continue;
					}

					if (Class->IsInBlueprint())
					{
						if (Class->GetSuperClass() != URCBehaviourBlueprintNode::StaticClass())
						{
							continue;
						}
					}

					URCBehaviour* Behaviour = Controller->CreateBehaviourWithoutCheck(Class);
					if (!Behaviour)
					{
						continue;
					}

					FUIAction Action(FExecuteAction::CreateSP(this, &SRCBehaviourPanel::OnAddBehaviourClicked, Class));
					Action.CanExecuteAction.BindSP(this, &SRCBehaviourPanel::CanExecuteAddBehaviour, Class, Controller);

					MenuBuilder.AddMenuEntry(
						FText::Format(LOCTEXT("AddBehaviourNode", "{0}"), Behaviour->GetDisplayName()),
						FText::Format(LOCTEXT("AddBehaviourNodeTooltip", "{0}"), Behaviour->GetBehaviorDescription()),
						FSlateIcon(),
						Action);
				}
			}
		}
	}

	return MenuBuilder.MakeWidget();
}

void SRCBehaviourPanel::OnAddBehaviourClicked(UClass* InClass)
{
	if (const TSharedPtr<FRCControllerModel> ControllerItem = SelectedControllerItemWeakPtr.Pin())
	{
		if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
		{
			FScopedTransaction Transaction(LOCTEXT("AddBehaviour", "Add Behaviour"));
			Controller->Modify();

			URCBehaviour* NewBehaviour = Controller->AddBehaviour(InClass);

			if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
			{
				RemoteControlPanel->OnBehaviourAdded.Broadcast(NewBehaviour);
			}
		}
	}
}

bool SRCBehaviourPanel::CanExecuteAddBehaviour(UClass* InClass, URCController* InController) const
{
	if (!InClass || !InController)
	{
		return false;
	}
	const URCBehaviourNode* DefaultBehaviourNode = Cast<URCBehaviourNode>(InClass->GetDefaultObject());
	TObjectPtr<URCBehaviour> Behaviour = InController->CreateBehaviourWithoutCheck(InClass);
	if (!Behaviour)
	{
		return false;
	}
	return DefaultBehaviourNode->IsSupported(Behaviour);
}

FReply SRCBehaviourPanel::OnClickEmptyButton()
{
	if (const TSharedPtr<FRCControllerModel> ControllerItem = SelectedControllerItemWeakPtr.Pin())
	{
		if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
		{
			FScopedTransaction Transaction(LOCTEXT("EmptyBehaviours", "Empty Behaviours"));
			Controller->Modify();

			Controller->EmptyBehaviours();
		}
	}

	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		RemoteControlPanel->OnEmptyBehaviours.Broadcast();
	}

	return FReply::Handled();
}

void SRCBehaviourPanel::CreateBehaviorPanelWidgets()
{
	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel();

	// Behavior list
	if (!BehaviourPanelList.IsValid())
	{
		BehaviourPanelList = SNew(SRCBehaviourPanelList, SharedThis(this), nullptr, RemoteControlPanel.ToSharedRef());
	}

	// Behavior Dock Panel
	if (!BehaviourDockPanel.IsValid())
	{
		BehaviourDockPanel = SNew(SRCMinorPanel)
			.HeaderLabel(LOCTEXT("BehavioursLabel", "Behavior"))
			.EnableFooter(false)
			[
				BehaviourPanelList.ToSharedRef()
			];
	}

	// Add New Behaviour Button
	if (!AddNewBehaviorComboButton.IsValid())
	{
		AddNewBehaviorComboButton = SNew(SComboButton)
			.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Behavior")))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.ForegroundColor(FSlateColor::UseForeground())
			.CollapseMenuOnParentFocus(true)
			.HasDownArrow(false)
			.ContentPadding(FMargin(4.f, 2.f))
			.ButtonContent()
			[
				SNew(SBox)
				.WidthOverride(RCPanelStyle->IconSize.X)
				.HeightOverride(RCPanelStyle->IconSize.Y)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
				]
			]
			.MenuContent()
			[
				GetBehaviourMenuContentWidget()
			];

		BehaviourDockPanel->ClearHeaderToolbarItems();
		BehaviourDockPanel->AddHeaderToolbarItem(EToolbar::Left, AddNewBehaviorComboButton.ToSharedRef());
	}

	// Wrapped Box
	if (!WrappedBoxWidget.IsValid())
	{
		WrappedBoxWidget = SNew(SBox)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex_Lambda([this] () { return BehaviourPanelList->GetControllerItem().IsValid()? 0 : 1; })

			// Index 0 is for valid behavior
			+ SWidgetSwitcher::Slot()
			[
				BehaviourDockPanel.ToSharedRef()
			]

			// Index 1 is for invalid behavior
			+ SWidgetSwitcher::Slot()
			[
				GetNoneSelectedWidget()
			]
		];
	}
}

bool SRCBehaviourPanel::AreBehaviorPanelWidgetsValid() const
{
	return WrappedBoxWidget.IsValid() &&
			BehaviourPanelList.IsValid() &&
			BehaviourDockPanel.IsValid() &&
			AddNewBehaviorComboButton.IsValid();
}

bool SRCBehaviourPanel::IsListFocused() const
{
	return BehaviourPanelList.IsValid() && BehaviourPanelList->IsListFocused();
}

void SRCBehaviourPanel::DeleteSelectedPanelItems()
{
	if (BehaviourPanelList.IsValid())
	{
		BehaviourPanelList->DeleteSelectedPanelItems();
	}
}

void SRCBehaviourPanel::DuplicateBehaviour(URCBehaviour* InBehaviour)
{
	if (URCController* Controller = GetParentController())
	{
		URCBehaviour* NewBehaviour = URCController::DuplicateBehaviour(Controller, InBehaviour);

		if (BehaviourPanelList.IsValid())
		{
			BehaviourPanelList->AddNewLogicItem(NewBehaviour);
		}
	}
}

void SRCBehaviourPanel::DuplicateSelectedPanelItems()
{
	FScopedTransaction Transaction(LOCTEXT("DuplicateBehaviour", "Duplicate Behaviour"));

	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (const TSharedPtr<FRCBehaviourModel>& BehaviourItem = StaticCastSharedPtr<FRCBehaviourModel>(LogicItem))
		{
			DuplicateBehaviour(BehaviourItem->GetBehaviour());
		}
	}
}

void SRCBehaviourPanel::CopySelectedPanelItems()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		TArray<UObject*> ItemsToCopy;
		const TArray<TSharedPtr<FRCLogicModeBase>> LogicItems = GetSelectedLogicItems();
		ItemsToCopy.Reserve(LogicItems.Num());

		for (const TSharedPtr<FRCLogicModeBase>& LogicItem : LogicItems)
		{
			if (const TSharedPtr<FRCBehaviourModel>& SelectedBehaviourItem = StaticCastSharedPtr<FRCBehaviourModel>(LogicItem))
			{
				ItemsToCopy.Add(SelectedBehaviourItem->GetBehaviour());
			}
		}

		RemoteControlPanel->SetLogicClipboardItems(ItemsToCopy, SharedThis(this));
	}
}

void SRCBehaviourPanel::PasteItemsFromClipboard()
{
	FScopedTransaction Transaction(LOCTEXT("PasteBehaviour", "Paste Behaviour"));

	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		if (RemoteControlPanel->LogicClipboardItemSource == SharedThis(this))
		{
			for (UObject* LogicItem : RemoteControlPanel->GetLogicClipboardItems())
			{
				if(URCBehaviour* Behaviour = Cast<URCBehaviour>(LogicItem))
				{
					DuplicateBehaviour(Behaviour);
				}
			}
		}
	}
}

bool SRCBehaviourPanel::CanPasteClipboardItems(const TArrayView<const TObjectPtr<UObject>> InLogicClipboardItems) const
{
	if (InLogicClipboardItems.Num() > 0)
	{
		const URCBehaviour* LogicClipboardBehaviour = Cast<URCBehaviour>(InLogicClipboardItems[0]);

		const URCController* ControllerSource = LogicClipboardBehaviour->ControllerWeakPtr.Get();
		if (!ControllerSource)
		{
			return false;
		}

		const URCController* ControllerTarget = GetParentController();
		if (!ControllerTarget)
		{
			return false;
		}

		// Copy-Paste is only permitted between Controllers of the same type
		//
		return ControllerSource->GetValueType() == ControllerTarget->GetValueType();
	}

	return false;
}

FText SRCBehaviourPanel::GetPasteItemMenuEntrySuffix()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		// This function should only have been called if we were the source of the item copied.
		if (ensure(RemoteControlPanel->LogicClipboardItemSource == SharedThis(this)))
		{
			TArray<UObject*> LogicClipboardItems = RemoteControlPanel->GetLogicClipboardItems();

			if (LogicClipboardItems.Num() > 0)
			{
				if (URCBehaviour* Behaviour = Cast<URCBehaviour>(LogicClipboardItems[0]))
				{
					if (LogicClipboardItems.Num() > 1)
					{
						return FText::Format(LOCTEXT("BehaviourPanelPasteMenuMultiEntrySuffix", "Behaviour {0} and {1} other(s)"), Behaviour->GetDisplayName(), (LogicClipboardItems.Num() - 1));
					}
					return FText::Format(LOCTEXT("BehaviourPanelPasteMenuEntrySuffix", "Behaviour {0}"), Behaviour->GetDisplayName());
				}
			}
		}
	}

	return FText::GetEmpty();
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCBehaviourPanel::GetSelectedLogicItems() const
{
	if (BehaviourPanelList)
	{
		return BehaviourPanelList->GetSelectedLogicItems();
	}

	return {};
}

URCController* SRCBehaviourPanel::GetParentController() const
{
	if (const TSharedPtr<FRCControllerModel> ControllerItem = SelectedControllerItemWeakPtr.Pin())
	{
		return Cast<URCController>(ControllerItem->GetVirtualProperty());
	}

	return nullptr;
}

FReply SRCBehaviourPanel::RequestDeleteSelectedItem()
{
	if (!BehaviourPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = LOCTEXT("DeleteBehaviourlWarning", "Delete the selected Behaviours?");

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		DeleteSelectedPanelItems();
	}

	return FReply::Handled();
}

FReply SRCBehaviourPanel::RequestDeleteAllItems()
{
	if (!BehaviourPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = FText::Format(LOCTEXT("DeleteAllWarning", "You are about to delete {0} behaviors. Are you sure you want to proceed?"), BehaviourPanelList->Num());

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		return OnClickEmptyButton();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
