// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Behaviour/RCBehaviourNode.h"
#include "UI/BaseLogicUI/SRCLogicPanelBase.h"

struct FRCPanelStyle;
class FRCBehaviourModel;
class FRCControllerModel;
class SBox;
class SComboButton;
class SRCBehaviourPanel;
class SRCMinorPanel;
class URCController;

/*
* ~ SRCBehaviourPanel ~
*
* UI Widget for Behaviour Panel.
* Contains a header (Add/Remove/Empty) and List of Behaviours
*/
class REMOTECONTROLUI_API SRCBehaviourPanel : public SRCLogicPanelBase
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourPanel)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Whether the Behaviour list widget currently has focus. Used for Delete Item UI command */
	bool IsListFocused() const;

	/** Delete Items UI command implementation for this panel */
	virtual void DeleteSelectedPanelItems() override;

	/** "Duplicate Items" UI command implementation for Behaviour panel*/
	virtual void DuplicateSelectedPanelItems() override;

	/** "Copy Items" UI command implementation for Behaviour panel*/
	virtual void CopySelectedPanelItems() override;

	/** "Paste Items" UI command implementation for Behaviour panel*/
	virtual void PasteItemsFromClipboard() override;

	/** Whether a given clipboard items can be successfully pasted into this panel */
	virtual bool CanPasteClipboardItems(const TArrayView<const TObjectPtr<UObject>> InLogicClipboardItems) const override;

	/** Provides an item suffix for the Paste context menu to provide users with useful context on the nature of the item being pasted */
	virtual FText GetPasteItemMenuEntrySuffix() override;

	/** Returns the UI items currently selected by the user (if any). To be implemented per child panel */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() const override;

	/** Returns the parent Controller associated with this behaviour*/
	URCController* GetParentController() const;

protected:

	/** Warns user before deleting a selected panel item. */
	virtual FReply RequestDeleteSelectedItem() override;

	/** Warns user before deleting all items in a panel. */
	virtual FReply RequestDeleteAllItems() override;

private:
	/** Get a Helper widget for behavior details. */
	static TSharedRef<SBox> GetNoneSelectedWidget();

	/** Duplicates a given Behaviour object*/
	void DuplicateBehaviour(URCBehaviour* InBehaviour);
	
	/**
	* Controller list selection change listener.
	* Updates the list of behaviours from newly selected Controller
	*/
	void OnControllerSelectionChanged(TSharedPtr<FRCControllerModel> InControllerItem, ESelectInfo::Type InSelectInfo);

	/** Called when a controller value is changed */
	void OnControllerValueChanged(TSharedPtr<FRCControllerModel> InController);

	/* Rebuilds the Behaviour Panel for a newly selected Controller*/
	void UpdateWrappedWidget(const TSharedPtr<FRCControllerModel>& InControllerItem = nullptr);

	/** Builds a menu containing the list of all possible Behaviours */
	TSharedRef<SWidget> GetBehaviourMenuContentWidget();

	/** Handles click event for Add Behaviour button*/
	void OnAddBehaviourClicked(UClass* InClass);

	/** Check if the controller can add the behaviour requested */
	bool CanExecuteAddBehaviour(UClass* InClass, URCController* InController) const;

	/** Handles click event for "Empty" button; clears all Behaviours from the panel*/
	FReply OnClickEmptyButton();

	/** Check the validity of the panel widgets and if not valid will create them */
	void CreateBehaviorPanelWidgets();

	/** Return whether or not all panel widgets are valid */
	bool AreBehaviorPanelWidgetsValid() const;

private:
	/** The parent Controller that this Behaviour panel is associated with */
	TWeakPtr<FRCControllerModel> SelectedControllerItemWeakPtr = nullptr;
	
	/** Parent Box for the entire widget */
	TSharedPtr<SBox> WrappedBoxWidget;

	/** Widget representing List of Behaviours */
	TSharedPtr<class SRCBehaviourPanelList> BehaviourPanelList;

	/** ComboButton to add new behavior */
	TSharedPtr<SComboButton> AddNewBehaviorComboButton;

	/** Behavior Minor Panel */
	TSharedPtr<SRCMinorPanel> BehaviourDockPanel;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle = nullptr;
};
