// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Behaviour/SRCBehaviourPanelList.h"

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/Builtin/Conditional/RCBehaviourConditional.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviorNew.h"
#include "Controller/RCController.h"
#include "RCBehaviourModel.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "SlateOptMacros.h"
#include "SRCBehaviourPanel.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/SRCActionPanel.h"
#include "UI/Behaviour/Builtin/Bind/RCBehaviourBindModel.h"
#include "UI/Behaviour/Builtin/Conditional/RCBehaviourConditionalModel.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviorSetAssetByPathModelNew.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviourSetAssetByPathModel.h"
#include "UI/Behaviour/Builtin/RangeMap/RCBehaviourRangeMapModel.h"
#include "UI/Behaviour/SRCBehaviorPanelRow.h"
#include "UI/Controller/RCControllerModel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "RemoteControlPanelBehavioursList"

namespace FRemoteControlBehaviourColumns
{
	const FName Behaviours = TEXT("Behaviors");
}

void SRCBehaviourPanelList::Construct(const FArguments& InArgs, TSharedRef<SRCBehaviourPanel> InBehaviourPanel, TSharedPtr<FRCControllerModel> InControllerItem, const TSharedRef<SRemoteControlPanel> InRemoteControlPanel)
{
	SRCLogicPanelListBase::Construct(SRCLogicPanelListBase::FArguments(), InBehaviourPanel, InRemoteControlPanel);
	
	BehaviourPanelWeakPtr = InBehaviourPanel;
	ControllerItemWeakPtr = InControllerItem;
	
	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	ListView = SNew(SListView<TSharedPtr<FRCBehaviourModel>>)
		.ListItemsSource(&BehaviourItems)
		.OnSelectionChanged(this, &SRCBehaviourPanelList::OnTreeSelectionChanged)
		.OnGenerateRow(this, &SRCBehaviourPanelList::OnGenerateWidgetForList)
		.ListViewStyle(&RCPanelStyle->TableViewStyle)
		.OnContextMenuOpening(this, &SRCLogicPanelListBase::GetContextMenuWidget)
		.HeaderRow(
			SNew(SHeaderRow)
			.Style(&RCPanelStyle->HeaderRowStyle)

			+ SHeaderRow::Column(FRemoteControlBehaviourColumns::Behaviours)
			.DefaultLabel(LOCTEXT("RCBehaviourColumnHeader", "Behaviors"))
			.FillWidth(1.f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)
		);
	
	ChildSlot
	[
		ListView.ToSharedRef()
	];

	// Add delegates
	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = InBehaviourPanel->GetRemoteControlPanel();
	check(RemoteControlPanel)
	RemoteControlPanel->OnBehaviourAdded.AddSP(this, &SRCBehaviourPanelList::OnBehaviourAdded);
	RemoteControlPanel->OnEmptyBehaviours.AddSP(this, &SRCBehaviourPanelList::OnEmptyBehaviours);

	if (InControllerItem.IsValid())
	{
		if (URCController* Controller = Cast<URCController>(InControllerItem->GetVirtualProperty()))
		{
			Controller->OnBehaviourListModified.AddSP(this, &SRCBehaviourPanelList::OnBehaviourListModified);
		}
	}

	Reset();
}

bool SRCBehaviourPanelList::IsEmpty() const
{
	return BehaviourItems.IsEmpty();
}

int32 SRCBehaviourPanelList::Num() const
{
	return BehaviourItems.Num();
}

int32 SRCBehaviourPanelList::NumSelectedLogicItems() const
{
	return ListView->GetNumItemsSelected();
}

void SRCBehaviourPanelList::AddNewLogicItem(UObject* InLogicItem)
{
	AddBehaviourToList(Cast<URCBehaviour>(InLogicItem));

	RequestRefresh();
}

void SRCBehaviourPanelList::AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder)
{
	const FUIAction EnableAction(FExecuteAction::CreateSP(this, &SRCBehaviourPanelList::SetAreSelectedBehaviorsEnabled, true));
	const FUIAction DisableAction(FExecuteAction::CreateSP(this, &SRCBehaviourPanelList::SetAreSelectedBehaviorsEnabled, false));

	static const FText EnableLabel = LOCTEXT("LabelEnableBehaviour", "Enable");
	static const FText EnableTooltip = LOCTEXT("TooltipEnableBehaviour", "Enables the current behaviours. Restores functioning of Actions associated with these behaviours.");
	static const FText DisableLabel = LOCTEXT("LabelDisableBehaviour", "Disable");
	static const FText DisableTooltip = LOCTEXT("TooltipDisableBehaviour", "Disables the current behaviours. Actions will not be processed for these behaviours when the Controller value changes");

	MenuBuilder.AddMenuEntry(EnableLabel, EnableTooltip, FSlateIcon(), EnableAction);
	MenuBuilder.AddMenuEntry(DisableLabel, DisableTooltip, FSlateIcon(), DisableAction);
}

void SRCBehaviourPanelList::SelectFirstItem()
{
	if (!BehaviourItems.IsEmpty())
	{
		ListView->SetSelection(BehaviourItems[0]);
	}
}

void SRCBehaviourPanelList::SetSelection(TArray<TSharedPtr<FRCBehaviourModel>> InBehaviorToSelect)
{
	for (const TSharedPtr<FRCBehaviourModel>& BehaviorModel : InBehaviorToSelect)
	{
		if (BehaviorModel.IsValid())
		{
			const URCBehaviour* Behavior = BehaviorModel->GetBehaviour();

			const TSharedPtr<FRCBehaviourModel>* SelectedBehavior = BehaviourItems.FindByPredicate([&Behavior]
				(const TSharedPtr<FRCBehaviourModel>& InBehaviorModel)
				{
					if (!InBehaviorModel.IsValid())
					{
						return false;
					}

					const URCBehaviour* CurrentBehavior = InBehaviorModel->GetBehaviour();

					if (!Behavior || !CurrentBehavior)
					{
						return false;
					}

					// Use the Behavior Internal Id to check for uniqueness
					return Behavior->Id == CurrentBehavior->Id;
				});

			if (SelectedBehavior)
			{
				ListView->SetItemSelection(*SelectedBehavior, true);
			}
		}
	}
}

void SRCBehaviourPanelList::NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel)
{
	// Get the current behavior displaying its action panel
	if (const TSharedPtr<FRCBehaviourModel>& SelectedBehavior = GetSelectedBehaviourItem())
	{
		SelectedBehavior->NotifyControllerValueChanged(InControllerModel);
	}
}

TSharedPtr<FRCControllerModel> SRCBehaviourPanelList::GetControllerItem() const
{
	if (ControllerItemWeakPtr.IsValid())
	{
		return ControllerItemWeakPtr.Pin();
	}
	return nullptr;
}

void SRCBehaviourPanelList::SetControllerItem(const TSharedPtr<FRCControllerModel>& InNewControllerItem)
{
	// Remove old Controller callback
	const TSharedPtr<FRCControllerModel> CurrentController = GetControllerItem();

	if (CurrentController == InNewControllerItem)
	{
		return;
	}

	if (CurrentController.IsValid())
	{
		if (URCController* Controller = Cast<URCController>(CurrentController->GetVirtualProperty()))
		{
			Controller->OnBehaviourListModified.RemoveAll(this);
		}
	}

	ControllerItemWeakPtr = InNewControllerItem;

	// Add new Controller callback
	const TSharedPtr<FRCControllerModel> NewController = GetControllerItem();
	if (NewController.IsValid())
	{
		if (URCController* Controller = Cast<URCController>(NewController->GetVirtualProperty()))
		{
			Controller->OnBehaviourListModified.AddSP(this, &SRCBehaviourPanelList::OnBehaviourListModified);
		}
	}

	Reset();
}

void SRCBehaviourPanelList::SetAreSelectedBehaviorsEnabled(const bool bIsEnabled)
{
	// Disable all selected behaviour here
	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (const TSharedPtr<FRCBehaviourModel>& BehaviourModel = StaticCastSharedPtr<FRCBehaviourModel>(LogicItem))
		{
			BehaviourModel->SetIsBehaviourEnabled(bIsEnabled);
		}
	}

	// Disable only the ActionPanel of the currently selected behaviour.
	// When selecting a different behaviour the ActionPanel will be re-constructed,
	// it will get the current status of the behaviour (enabled or disabled) and will match it for the ActionPanel as well
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		if (const TSharedPtr<SRCActionPanel> ActionPanel = RemoteControlPanel->GetLogicActionPanel())
		{
			ActionPanel->RefreshIsBehaviourEnabled(bIsEnabled);
		}
	}
}

void SRCBehaviourPanelList::SetIsBehaviourEnabled(TSharedRef<FRCBehaviourModel> InBehaviorModel, const bool bIsEnabled)
{
	// Disable all selected behaviour here
	InBehaviorModel->SetIsBehaviourEnabled(bIsEnabled);

	// Disable only the ActionPanel of the currently selected behaviour.
	// When selecting a different behaviour the ActionPanel will be re-constructed,
	// it will get the current status of the behaviour (enabled or disabled) and will match it for the ActionPanel as well
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		if (const TSharedPtr<SRCActionPanel> ActionPanel = RemoteControlPanel->GetLogicActionPanel())
		{
			ActionPanel->RefreshIsBehaviourEnabled(bIsEnabled);
		}
	}
}

void SRCBehaviourPanelList::ReorderBehaviorItems(TSharedRef<FRCBehaviourModel> InDroppedOnModel, EItemDropZone InDropZone, TArray<TSharedPtr<FRCBehaviourModel>> InDroppedModels)
{
	TSharedPtr<FRCControllerModel> ControllerMoel = ControllerItemWeakPtr.Pin();

	if (!ControllerMoel.IsValid())
	{
		return;
	}

	URCController* Controller = Cast<URCController>(ControllerMoel->GetVirtualProperty());

	if (!Controller)
	{
		return;
	}

	URCBehaviour* DroppedOnBehavior = InDroppedOnModel->GetBehaviour();

	if (!DroppedOnBehavior)
	{
		return;
	}

	TArray<URCBehaviour*> Behaviors;
	Behaviors.Reserve(InDroppedModels.Num());

	for (const TSharedPtr<FRCBehaviourModel>& BehaviorModel : InDroppedModels)
	{
		if (URCBehaviour* Behavior = BehaviorModel->GetBehaviour())
		{
			Behaviors.Add(Behavior);
		}
	}

	FScopedTransaction Transaction(LOCTEXT("ReorderBehaviors", "Reorder Behaviors"));
	Controller->Modify();

	Controller->ReorderBehaviorItems(DroppedOnBehavior, InDropZone != EItemDropZone::AboveItem, Behaviors);
}

void SRCBehaviourPanelList::SetIsBehaviourEnabled(const TSharedPtr<FRCBehaviourModel>& InBehaviourModel, const bool bIsEnabled)
{
	// Disable the behaviour
	if (InBehaviourModel.IsValid())
	{
		InBehaviourModel->SetIsBehaviourEnabled(bIsEnabled);
	}

	// Disable the action panel
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		if (const TSharedPtr<SRCActionPanel> ActionPanel = RemoteControlPanel->GetLogicActionPanel())
		{
			if (InBehaviourModel == ActionPanel->GetSelectedBehaviourItem())
			{
				ActionPanel->RefreshIsBehaviourEnabled(bIsEnabled);
			}
		}
	}
}

void SRCBehaviourPanelList::AddBehaviourToList(URCBehaviour* InBehaviour)
{
	if (!InBehaviour || !BehaviourPanelWeakPtr.IsValid())
	{
		return;
	}

	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = BehaviourPanelWeakPtr.Pin()->GetRemoteControlPanel();

	if (URCBehaviourConditional* ConditionalBehaviour = Cast<URCBehaviourConditional>(InBehaviour))
	{
		BehaviourItems.Add(MakeShared<FRCBehaviourConditionalModel>(ConditionalBehaviour));
	}
	else if (URCSetAssetByPathBehaviour* SetAssetByPathBehaviour = Cast<URCSetAssetByPathBehaviour>(InBehaviour))
	{
		BehaviourItems.Add(MakeShared<FRCSetAssetByPathBehaviourModel>(SetAssetByPathBehaviour, RemoteControlPanel));
	}
	else if (URCBehaviourBind* BindBehaviour = Cast<URCBehaviourBind>(InBehaviour))
	{
		BehaviourItems.Add(MakeShared<FRCBehaviourBindModel>(BindBehaviour, RemoteControlPanel));
	}
	else if (URCRangeMapBehaviour* RangeMapBehaviour = Cast<URCRangeMapBehaviour>(InBehaviour))
	{
		BehaviourItems.Add(MakeShared<FRCRangeMapBehaviourModel>(RangeMapBehaviour));
	}
	else if (URCSetAssetByPathBehaviorNew* SetAssetByPathBehavior = Cast<URCSetAssetByPathBehaviorNew>(InBehaviour))
	{
		TSharedRef<FRCSetAssetByPathBehaviorModelNew> NewBehavior = MakeShared<FRCSetAssetByPathBehaviorModelNew>(SetAssetByPathBehavior, RemoteControlPanel);
		NewBehavior->Initialize();

		BehaviourItems.Add(NewBehavior);
	}
	else
	{
		BehaviourItems.Add(MakeShared<FRCBehaviourModel>(InBehaviour, RemoteControlPanel));
	}
}

void SRCBehaviourPanelList::Reset()
{
	const TArray<TSharedPtr<FRCBehaviourModel>> SelectedBehaviors = ListView->GetSelectedItems();

	BehaviourItems.Empty();

	if (TSharedPtr<FRCControllerModel> ControllerItem = ControllerItemWeakPtr.Pin())
	{
		if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
		{
			for (URCBehaviour* Behaviour : Controller->GetBehaviors())
			{
				AddBehaviourToList(Behaviour);
			}
		}
	}

	ListView->RebuildList();
	SetSelection(SelectedBehaviors);

	UpdateEntityUsage();
}

TSharedRef<ITableRow> SRCBehaviourPanelList::OnGenerateWidgetForList(TSharedPtr<FRCBehaviourModel> InItem,
	const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SRCBehaviorPanelRow, OwnerTable, SharedThis(this), InItem)
		.Style(&RCPanelStyle->TableRowStyle);
}

void SRCBehaviourPanelList::OnTreeSelectionChanged(TSharedPtr<FRCBehaviourModel> InItem, ESelectInfo::Type)
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = BehaviourPanelWeakPtr.Pin()->GetRemoteControlPanel())
	{
		if (InItem != SelectedBehaviourItemWeakPtr.Pin())
		{
			SelectedBehaviourItemWeakPtr = InItem;

			RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(InItem);
		}

		if (const TSharedPtr<FRCControllerModel> ControllerItem = ControllerItemWeakPtr.Pin())
		{
			ControllerItem->UpdateSelectedBehaviourModel(InItem);
		}
	}
}

void SRCBehaviourPanelList::OnBehaviourAdded(const URCBehaviour* InBehaviour)
{
	Reset();
}

void SRCBehaviourPanelList::OnEmptyBehaviours()
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = BehaviourPanelWeakPtr.Pin()->GetRemoteControlPanel())
	{
		RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(nullptr);	
	}
	
	Reset();
}

void SRCBehaviourPanelList::BroadcastOnItemRemoved()
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = BehaviourPanelWeakPtr.Pin()->GetRemoteControlPanel())
	{
		RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(nullptr);
	}
}

URemoteControlPreset* SRCBehaviourPanelList::GetPreset()
{
	if (BehaviourPanelWeakPtr.IsValid())
	{
		return BehaviourPanelWeakPtr.Pin()->GetPreset();
	}

	return nullptr;
}

void SRCBehaviourPanelList::OnBehaviourListModified()
{
	Reset();
}

bool SRCBehaviourPanelList::IsListFocused() const
{
	return ListView->HasAnyUserFocus().IsSet() || ContextMenuWidgetCached.IsValid();;
}

void SRCBehaviourPanelList::DeleteSelectedPanelItems()
{
	FScopedTransaction Transaction(LOCTEXT("DeleteSelectedItems", "Delete Selected Items"));
	if (!DeleteItemsFromLogicPanel<FRCBehaviourModel>(BehaviourItems, ListView->GetSelectedItems()))
	{
		Transaction.Cancel();
	}
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCBehaviourPanelList::GetSelectedLogicItems()
{
	TArray<TSharedPtr<FRCLogicModeBase>> SelectedValidLogicItems;
	if (ListView.IsValid())
	{
		TArray<TSharedPtr<FRCBehaviourModel>> AllSelectedLogicItems = ListView->GetSelectedItems();
		SelectedValidLogicItems.Reserve(AllSelectedLogicItems.Num());

		for (const TSharedPtr<FRCBehaviourModel>& LogicItem : AllSelectedLogicItems)
		{
			if (LogicItem.IsValid())
			{
				SelectedValidLogicItems.Add(LogicItem);
			}
		}
	}
	return SelectedValidLogicItems;
}

TArray<TSharedPtr<FRCBehaviourModel>> SRCBehaviourPanelList::GetSelectedBehaviourItems() const
{
	return ListView->GetSelectedItems();
}

void SRCBehaviourPanelList::RequestRefresh()
{
	ListView->RequestListRefresh();
}

#undef LOCTEXT_NAMESPACE