// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateTypes.h"
#include "UI/BaseLogicUI/SRCLogicPanelListBase.h"

struct FRCPanelStyle;
class URCBehaviour;
class FRCControllerModel;
class FRCBehaviourModel;
class ITableRow;
class ITableBase;
class SRCBehaviourPanel;
class SRemoteControlPanel;
class STableViewBase;
class URemoteControlPreset;
enum class EItemDropZone;
template <typename ItemType> class SListView;

/*
* ~ SRCBehaviourPanelList ~
*
* UI Widget for Behaviours List
* Used as part of the RC Logic Behaviour Panel.
*/
class REMOTECONTROLUI_API SRCBehaviourPanelList : public SRCLogicPanelListBase
{
public:
	SLATE_BEGIN_ARGS(SRCBehaviourPanelList)
	{
	}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRCBehaviourPanel> InBehaviourPanel, TSharedPtr<FRCControllerModel> InControllerItem, const TSharedRef<SRemoteControlPanel> InRemoteControlPanel);
	
	/** Returns true if the underlying list is valid and empty. */
	virtual bool IsEmpty() const override;
	
	/** Returns number of items in the list. */
	virtual int32 Num() const override;

	/** The number of Controllers currently selected */
	virtual int32 NumSelectedLogicItems() const override;

	/** Whether the Behaviours List View currently has focus. */
	virtual bool IsListFocused() const override;

	/** Deletes currently selected items from the list view */
	virtual void DeleteSelectedPanelItems() override;

	/** Returns the UI items currently selected by the user (if any). */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() override;

	/** Return the behavior currently selected and displaying its ActionPanel */
	TSharedPtr<FRCBehaviourModel> GetSelectedBehaviourItem()
	{
		return SelectedBehaviourItemWeakPtr.Pin();
	}

	/** Return the list of all behavior currently selected */
	TArray<TSharedPtr<FRCBehaviourModel>> GetSelectedBehaviourItems() const;

	void RequestRefresh();

	virtual void AddNewLogicItem(UObject* InLogicItem) override;

	/** Allows Logic panels to add special functionality to the Context Menu based on context */
	virtual void AddSpecialContextMenuOptions(FMenuBuilder& MenuBuilder) override;

	/** Will set the selection to the first item */
	void SelectFirstItem();

	/** Set the current selection to the given InBehaviorToSelect list */
	void SetSelection(TArray<TSharedPtr<FRCBehaviourModel>> InBehaviorToSelect);

	/** Called when a controller value changed */
	void NotifyControllerValueChanged(TSharedPtr<FRCControllerModel> InController);

	/** Return the controller item for this behavior */
	TSharedPtr<FRCControllerModel> GetControllerItem() const;
	
	/** Set a new controller item to update the behavior view based on the new controller */
	void SetControllerItem(const TSharedPtr<FRCControllerModel>& InNewControllerItem);

	/** Enables or Disables the currently selected behaviour */
	void SetAreSelectedBehaviorsEnabled(const bool bIsEnabled);

	/** Enables or Disables the given behaviour */
	void SetIsBehaviourEnabled(TSharedRef<FRCBehaviourModel> InBehaviorModel, const bool bIsEnabled);

	/** Rearranges the behaviors based on the dropped behaviours. */
	void ReorderBehaviorItems(TSharedRef<FRCBehaviourModel> InDroppedOnModel, EItemDropZone InDropZone, TArray<TSharedPtr<FRCBehaviourModel>> InDroppedModels);

private:

	/** Enables or Disables the passed behaviour */
	void SetIsBehaviourEnabled(const TSharedPtr<FRCBehaviourModel>& InBehaviourModel, const bool bIsEnabled);

	void AddBehaviourToList(URCBehaviour* InBehaviour);

	/** OnGenerateRow delegate for the Behaviours List View */
	TSharedRef<ITableRow> OnGenerateWidgetForList(TSharedPtr<FRCBehaviourModel> InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** OnSelectionChanged delegate for Behaviours List View */
	void OnTreeSelectionChanged(TSharedPtr<FRCBehaviourModel> InItem , ESelectInfo::Type);

	/** Responds to the selection of a newly created Behaviour. Resets UI state */
	void OnBehaviourAdded(const URCBehaviour* InBehaviour);

	/** Responds to the removal of all Behaviours. Rests UI state */
	void OnEmptyBehaviours();

	/** Refreshes the list from the latest state of the model */
	virtual void Reset() override;

	/** Handles broadcasting of a successful remove item operation. */
	virtual void BroadcastOnItemRemoved() override;

	/** Fetches the Remote Control preset associated with the parent panel */
	virtual URemoteControlPreset* GetPreset() override;

	void OnBehaviourListModified();

private:

	/** The parent Behaviour Panel widget */
	TWeakPtr<SRCBehaviourPanel> BehaviourPanelWeakPtr;

	/** The Controller (UI model) associated with us */
	TWeakPtr<FRCControllerModel> ControllerItemWeakPtr;

	/** The currently selected Behaviour item (UI model) */
	TWeakPtr<FRCBehaviourModel> SelectedBehaviourItemWeakPtr;

	/** List of Behaviours (UI models) active in this widget */
	TArray<TSharedPtr<FRCBehaviourModel>> BehaviourItems;

	/** List View widget for representing our Behaviours List */
	TSharedPtr<SListView<TSharedPtr<FRCBehaviourModel>>> ListView;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
};

