// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/CustomControllers/SCustomStatelessEventControllerWidget.h"

#include "Controller/RCController.h"
#include "RemoteControlPreset.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCustomTextureControllerWidget"

void SCustomStatelessEventControllerWidget::Construct(const FArguments& InArgs, URCController* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle)
{
	ControllerWeak = InController;

	ChildSlot
	[
		SNew(SButton)
		.OnClicked(this, &SCustomStatelessEventControllerWidget::OnButtonClicked)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Trigger", "Trigger"))
		]
	];
}

FReply SCustomStatelessEventControllerWidget::OnButtonClicked()
{
	if (URCController* Controller = ControllerWeak.Get())
	{
		Controller->ExecuteBehaviours(/* Pre change */ false);

		if (URemoteControlPreset* Preset = Controller->PresetWeakPtr.Get())
		{
			Preset->OnControllerModified().Broadcast(Preset, {Controller->Id});
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
