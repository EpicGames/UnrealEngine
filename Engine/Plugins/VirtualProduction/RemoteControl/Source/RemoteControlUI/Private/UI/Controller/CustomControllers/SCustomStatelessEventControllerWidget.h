// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/WeakObjectPtr.h"

class IPropertyHandle;
class URCController;

class SCustomStatelessEventControllerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomStatelessEventControllerWidget)
		{}
	SLATE_END_ARGS()

	/**
	 * Constructs this widget with InArgs
	 * @param InOriginalPropertyHandle Original PropertyHandle of this CustomWidget
	 */
	void Construct(const FArguments& InArgs, URCController* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle);

private:
	/** The controller associated with this widget. */
	TWeakObjectPtr<URCController> ControllerWeak;

	FReply OnButtonClicked();
};
