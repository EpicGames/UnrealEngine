// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/CustomControllers/SCustomStringListControllerListEditorWidget.h"

#include "Containers/UnrealString.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "Misc/CString.h"
#include "Styles/SlateBrushTemplates.h"
#include "Styling/AppStyle.h"
#include "UI/Controller/CustomControllers/SCustomStringListControllerWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCustomStringListControllerListEditorWidget"

class SCustomStringListControllerEnumEditorRowEditBox : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SCustomStringListControllerEnumEditorRowEditBox)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<SCustomStringListControllerListEditorWidget> InParentWidget, FName InEntry, bool bInIsInTable)
	{
		ParentWidgetWeak = InParentWidget;
		Entry = InEntry;
		bIsInTable = bInIsInTable;

		TSharedRef<SHorizontalBox> EditBox = SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.MinWidth(200.f)
			.MaxWidth(200.f)
			.Padding(5.f)
			[
				SNew(SEditableTextBox)
				.Padding(5.f)
				.Text(this, &SCustomStringListControllerEnumEditorRowEditBox::GetEntry)
				.HintText(bInIsInTable 
					? LOCTEXT("TableHint", "Edit Entry")
					: LOCTEXT("AddHint", "Add Entry")
				)
				.Justification(ETextJustify::Left)
				.RevertTextOnEscape(true)
				.SelectAllTextWhenFocused(true)
				.OnVerifyTextChanged(this, &SCustomStringListControllerEnumEditorRowEditBox::OnEntryVerify)
				.OnTextCommitted(this, &SCustomStringListControllerEnumEditorRowEditBox::OnEntryCommitted)
			];

		if (bIsInTable)
		{
			EditBox->AddSlot()
				.AutoWidth()
				.Padding(FMargin(0.f, 0.f, 5.f, 0.f))
				.VAlign(VAlign_Center)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
					.ContentPadding(5.f)
					.OnClicked(this, &SCustomStringListControllerEnumEditorRowEditBox::OnRemoveClicked)
					.IsEnabled(this, &SCustomStringListControllerEnumEditorRowEditBox::CanRemove)
					.ClickMethod(EButtonClickMethod::MouseDown)
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Delete"))
						.DesiredSizeOverride(FVector2D(16.f, 16.f))
					]
				];

			EditBox->AddSlot()
				.AutoWidth()
				.Padding(FMargin(0.f, 0.f, 10.f, 0.f))
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("VerticalBoxDragIndicatorShort"))
					.DesiredSizeOverride(FVector2D(5.f, 15.f))
				];
		}
		else
		{
			EditBox->AddSlot()
				.AutoWidth()
				.Padding(FMargin(5.f, 0.f, 0.f, 0.f))
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.PlusCircle"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				];
		}

		ChildSlot
		[
			EditBox
		];
	}

private:
	TWeakPtr<SCustomStringListControllerListEditorWidget> ParentWidgetWeak;
	FName Entry;
	bool bIsInTable;

	FText GetEntry() const
	{
		if (Entry.IsNone())
		{
			return FText::GetEmpty();
		}

		return FText::FromName(Entry);
	}

	bool OnEntryVerify(const FText& InNewValue, FText& OutError)
	{
		TSharedPtr<SCustomStringListControllerListEditorWidget> ParentWidget = ParentWidgetWeak.Pin();

		if (!ParentWidget.IsValid())
		{
			OutError = LOCTEXT("InternalError", "Internal Error");
			return false;
		}

		if (InNewValue.IsEmpty())
		{
			if (bIsInTable)
			{
				OutError = LOCTEXT("Missing", "Missing Entry");
				return false;
			}

			return true;
		}

		const FString NewValueString = InNewValue.ToString();

		if (NewValueString.Contains(TEXT(";")) || NewValueString.Contains(TEXT("\"")))
		{
			OutError = LOCTEXT("Delimiter", "Entries cannot contain ';' or '\"'.");
			return false;
		}
		
		const FName New = *NewValueString;

		if (Entry == New)
		{
			return true;
		}

		if (ParentWidget->HasEntry(New))
		{
			OutError = LOCTEXT("Duplicate", "Duplicate Entry");
			return false;
		}

		return true;
	}

	void OnEntryCommitted(const FText& InNew, ETextCommit::Type InCommitType)
	{
		TSharedPtr<SCustomStringListControllerListEditorWidget> ParentWidget = ParentWidgetWeak.Pin();

		if (!ParentWidget.IsValid())
		{
			return;
		}

		if (InNew.IsEmpty())
		{
			return;
		}

		const FName NewEntry = *InNew.ToString().TrimStartAndEnd();

		if (Entry == NewEntry)
		{
			return;
		}

		if (ParentWidget->HasEntry(NewEntry))
		{
			return;
		}

		if (bIsInTable)
		{
			ParentWidget->ReplaceEntry(Entry, NewEntry);
			Entry = NewEntry;
		}
		else
		{
			ParentWidget->AddEntry(NewEntry);
		}
	}

	FReply OnRemoveClicked()
	{
		if (TSharedPtr<SCustomStringListControllerListEditorWidget> ParentWidget = ParentWidgetWeak.Pin())
		{
			ParentWidget->RemoveEntry(Entry);
		}

		return FReply::Handled();
	}

	bool CanRemove() const
	{
		if (TSharedPtr<SCustomStringListControllerListEditorWidget> ParentWidget = ParentWidgetWeak.Pin())
		{
			return ParentWidget->GetEntries().Num() > 1;
		}

		return false;
	}
};

class FCustomStringListControllerEnumEditorRowDragDropOp : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FCustomStringListControllerEnumEditorRowDragDropOp, FDecoratedDragDropOp)

	FCustomStringListControllerEnumEditorRowDragDropOp(FName InEntry)
		: Entry(InEntry)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	virtual ~FCustomStringListControllerEnumEditorRowDragDropOp() = default;

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNew(STextBlock)
			.Text(FText::FromName(Entry));
	}

	FName GetEntry() const
	{
		return Entry;
	}

private:
	FName Entry;
};

class SCustomStringListControllerEnumEditorRow : public STableRow<FName>
{
public:
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		TSharedRef<SCustomStringListControllerListEditorWidget> InParentWidget, FName InEntry)
	{
		ParentWidgetWeak = InParentWidget;
		Entry = InEntry;

		SetCursor(EMouseCursor::GrabHand);
		
		STableRow<FName>::Construct(
			STableRow<FName>::FArguments()
			.OnDragDetected(this, &SCustomStringListControllerEnumEditorRow::OnDragDetected)
			.OnCanAcceptDrop(this, &SCustomStringListControllerEnumEditorRow::OnCanAcceptDrop)
			.OnAcceptDrop(this, &SCustomStringListControllerEnumEditorRow::OnAcceptDrop)
			.Content()
			[
				SNew(SCustomStringListControllerEnumEditorRowEditBox, InParentWidget, InEntry, /* In Table */ true)
			],
			InOwnerTableView);
	}

private:
	TWeakPtr<SCustomStringListControllerListEditorWidget> ParentWidgetWeak;
	FName Entry;

	FReply OnDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent)
	{
		if (!Entry.IsNone())
		{
			TSharedRef<FCustomStringListControllerEnumEditorRowDragDropOp> DragDropOp = MakeShared<FCustomStringListControllerEnumEditorRowDragDropOp>(Entry);
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}

		return FReply::Handled();
	}

	TOptional<EItemDropZone> OnCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FName InEntry)
	{
		const TSharedPtr<FCustomStringListControllerEnumEditorRowDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FCustomStringListControllerEnumEditorRowDragDropOp>();

		if (!DragDropOp.IsValid())
		{
			return {};
		}

		if (Entry.IsNone())
		{
			return {};
		}

		return InDropZone;
	}

	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FName InEntry)
	{
		const TSharedPtr<FCustomStringListControllerEnumEditorRowDragDropOp> DragDropOp = InDragDropEvent.GetOperationAs<FCustomStringListControllerEnumEditorRowDragDropOp>();

		if (!DragDropOp.IsValid())
		{
			return FReply::Handled();
		}

		TSharedPtr<SCustomStringListControllerListEditorWidget> ParentWidget = ParentWidgetWeak.Pin();

		if (!ParentWidget.IsValid())
		{
			return FReply::Handled();
		}

		ParentWidget->OnValidDrop(InEntry, DragDropOp->GetEntry(), InDropZone);

		return FReply::Handled();
	}
};

void SCustomStringListControllerListEditorWidget::OpenModalWindow(const TSharedRef<SCustomStringListControllerWidget>& InParentWidget, const TArray<FName>& InEntries)
{
	TSharedRef<SCustomStringListControllerListEditorWidget> EditorWidget = SNew(SCustomStringListControllerListEditorWidget, InParentWidget, InEntries);

	TSharedRef<SWindow> Container = SNew(SWindow)
		.SizingRule(ESizingRule::Autosized)
		.SupportsMinimize(false)
		.SupportsMaximize(false)
		.Title(LOCTEXT("ModalWindowTitle", "Edit String List"))
		[
			EditorWidget
		];
		
	FSlateApplication::Get().AddModalWindow(Container, InParentWidget);
}

void SCustomStringListControllerListEditorWidget::Construct(const FArguments& InArgs, const TSharedRef<SCustomStringListControllerWidget>& InParentWidget, const TArray<FName>& InEntries)
{
	ParentWidgetWeak = InParentWidget;
	Entries = InEntries;

	ListView = SNew(SListView<FName>)
		.ListItemsSource(&Entries)
		.OnGenerateRow(this, &SCustomStringListControllerListEditorWidget::OnGenerateRow)
		.SelectionMode(ESelectionMode::Single);

	TSharedPtr<SWidget> NewRowWidget;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(10.f, 10.f, 10.f, 10.f)
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			ListView.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		.Padding(10.f, 0.f, 10.f, 10.f)
		.HAlign(HAlign_Fill)
		.AutoHeight()
		[
			SAssignNew(NewRowWidget, SCustomStringListControllerEnumEditorRowEditBox, SharedThis(this), NAME_None, /* In Table */ false)
		]
		+ SVerticalBox::Slot()
		.Padding(10.f, 0.f, 10.f, 10.f)
		.AutoHeight()
		.HAlign(HAlign_Center)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(1.f, 0.f, 10.f, 0.f)
			[
				SNew(SButton)
				.OnClicked(this, &SCustomStringListControllerListEditorWidget::OnOkClicked)
				.ToolTipText(LOCTEXT("OkayToolTip", "Close this window and use these values for the enum."))
				.ClickMethod(EButtonClickMethod::MouseDown)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AcceptValues", "Ok"))
				]
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(10.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.OnClicked(this, &SCustomStringListControllerListEditorWidget::OnCancelClicked)
				.ToolTipText(LOCTEXT("CancelToolTip", "Renumber the entries sequentially from 0."))
				.ClickMethod(EButtonClickMethod::MouseDown)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("RejectValues", "Cancel"))
				]
			]
		]
	];
}

bool SCustomStringListControllerListEditorWidget::HasEntry(FName InEntry) const
{
	for (const FName& Entry : Entries)
	{
		if (Entry == InEntry)
		{
			return true;
		}
	}

	return false;
}

void SCustomStringListControllerListEditorWidget::OnValuesChanged()
{
	ListView->RequestListRefresh();
}

TConstArrayView<FName> SCustomStringListControllerListEditorWidget::GetEntries() const
{
	return Entries;
}

TSharedRef<ITableRow> SCustomStringListControllerListEditorWidget::OnGenerateRow(FName InEntry, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	return SNew(SCustomStringListControllerEnumEditorRow, InOwnerTableView, SharedThis(this), InEntry);
}

void SCustomStringListControllerListEditorWidget::UpdateStringList()
{
	if (Entries.IsEmpty())
	{
		return;
	}
	
	if (TSharedPtr<SCustomStringListControllerWidget> ParentWidget = ParentWidgetWeak.Pin())
	{
		ParentWidget->UpdateStringList(Entries);
	}
}

void SCustomStringListControllerListEditorWidget::CloseWindow()
{
	if (TSharedPtr<SWindow> CurrentWindow = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
	{
		CurrentWindow->RequestDestroyWindow();
	}
}

FReply SCustomStringListControllerListEditorWidget::OnOkClicked()
{
	UpdateStringList();
	CloseWindow();

	return FReply::Handled();
}

FReply SCustomStringListControllerListEditorWidget::OnCancelClicked()
{
	CloseWindow();

	return FReply::Handled();
}

void SCustomStringListControllerListEditorWidget::OnValidDrop(FName InDropTarget, FName InDropped, EItemDropZone InDropZone)
{
	if (InDropTarget == InDropped)
	{
		return;
	}

	int32 DroppedOnIndex = Entries.Find(InDropTarget);

	if (DroppedOnIndex == INDEX_NONE)
	{
		return;
	}

	const int32 DroppedIndex = Entries.Find(InDropped);

	if (DroppedIndex == INDEX_NONE || DroppedIndex == DroppedOnIndex)
	{
		return;
	}

	if (DroppedOnIndex >= DroppedIndex)
	{
		--DroppedOnIndex;
	}

	switch (InDropZone)
	{
		case EItemDropZone::BelowItem:
			Entries.RemoveAt(DroppedIndex);
			Entries.Insert(InDropped, DroppedOnIndex + 1);
			break;

		case EItemDropZone::OntoItem:
		case EItemDropZone::AboveItem:
			Entries.RemoveAt(DroppedIndex);
			Entries.Insert(InDropped, DroppedOnIndex);
			break;

		default:
			return;
	}

	OnValuesChanged();
}

void SCustomStringListControllerListEditorWidget::AddEntry(FName InEntry)
{
	Entries.Add(InEntry);

	OnValuesChanged();
}

void SCustomStringListControllerListEditorWidget::RemoveEntry(FName InEntry)
{
	if (Entries.Num() <= 1)
	{
		return;
	}

	Entries.Remove(InEntry);

	OnValuesChanged();
}

void SCustomStringListControllerListEditorWidget::ReplaceEntry(FName InFrom, FName InTo)
{
	for (int32 Index = 0; Index < Entries.Num(); ++Index)
	{
		if (Entries[Index] == InFrom)
		{
			Entries[Index] = InTo;
			return;
		}
	}
}

#undef LOCTEXT_NAMESPACE
