// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Views/SListView.h"

class SCustomStringListControllerWidget;
class SWindow;

class SCustomStringListControllerListEditorWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomStringListControllerListEditorWidget)
		{}
	SLATE_END_ARGS()

	static void OpenModalWindow(const TSharedRef<SCustomStringListControllerWidget>& InParentWidget, const TArray<FName>& InEntries);

	virtual ~SCustomStringListControllerListEditorWidget() override = default;

	void Construct(const FArguments& InArgs, const TSharedRef<SCustomStringListControllerWidget>& InParentWidget, const TArray<FName>& InEntries);

	bool HasEntry(FName InDescription) const;

	void AddEntry(FName InEntry);

	void RemoveEntry(FName InEntry);

	void ReplaceEntry(FName InFrom, FName InTo);

	void OnValuesChanged();

	TConstArrayView<FName> GetEntries() const;

	void OnValidDrop(FName InDropTarget, FName InDropped, EItemDropZone InDropZone);

private:
	TWeakPtr<SCustomStringListControllerWidget> ParentWidgetWeak;
	TSharedPtr<SListView<FName>> ListView;
	TArray<FName> Entries;

	TSharedRef<ITableRow> OnGenerateRow(FName InEntry, const TSharedRef<STableViewBase>& InOwnerTable);

	void UpdateStringList();

	void CloseWindow();

	FReply OnOkClicked();
		
	FReply OnCancelClicked();
};
