// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/CustomControllers/SCustomStringListControllerWidget.h"

#include "Controller/RCCustomControllerUtilities.h"
#include "PropertyHandle.h"
#include "RCVirtualProperty.h"
#include "ScopedTransaction.h"
#include "UI/Controller/CustomControllers/SCustomStringListControllerListEditorWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SCustomStringListControllerWidget"

void SCustomStringListControllerWidget::Construct(const FArguments& InArgs, URCVirtualPropertyBase* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle)
{
	ControllerWeak = InController;
	OriginalPropertyHandle = InOriginalPropertyHandle;

	using namespace UE::RCCustomControllers;
	
	const FString EntriesString = InController->GetMetadataValue(StringListControllerEntriesName);
	TArray<FString> EntriesArray;
	EntriesString.ParseIntoArray(EntriesArray, TEXT(";"));

	if (EntriesArray.IsEmpty())
	{
		EntriesArray.Add("New Entry 1");
	}

	Entries.Reserve(EntriesArray.Num());

	for (const FString& Entry : EntriesArray)
	{
		Entries.Add(*Entry);
	}

	FString CurrentValue;
	InOriginalPropertyHandle->GetValue(CurrentValue);

	if (CurrentValue.IsEmpty())
	{
		CurrentValue = Entries[0].ToString();
		InOriginalPropertyHandle->SetValue(CurrentValue);
	}

	StringSelector = SNew(SComboBox<FName>)
		.OptionsSource(&Entries)
		.InitiallySelectedItem(*CurrentValue)
		.OnGenerateWidget(this, &SCustomStringListControllerWidget::OnGenerateWidget)
		.OnSelectionChanged(this, &SCustomStringListControllerWidget::OnSelectionChanged)
		.Content()
		[
			SNew(STextBlock)
			.Text(this, &SCustomStringListControllerWidget::GetCurrentValueText)
		];

	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[
			StringSelector.ToSharedRef()
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.f, 0.f, 0.f, 0.f)
		[
			SNew(SButton)
			.OnClicked(this, &SCustomStringListControllerWidget::OnEditStringListClicked)
			.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.ContentPadding(2.f)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Toolbar.Details"))
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
			]
		]
	];
}

void SCustomStringListControllerWidget::UpdateStringList(const TArray<FName>& InNewEntries)
{
	if (InNewEntries.IsEmpty())
	{
		return;
	}

	URCVirtualPropertyBase* Controller = ControllerWeak.Get();

	if (!Controller)
	{
		return;
	}

	FString CurrentValue;
	OriginalPropertyHandle->GetValue(CurrentValue);

	// Separators
	int32 Length = InNewEntries.Num() - 1;

	for (const FName& Entry : InNewEntries)
	{
		Length += Entry.GetStringLength();
	}

	FString NewMeta;
	NewMeta.Reserve(Length);

	Entries.Empty(InNewEntries.Num());

	for (const FName& Entry : InNewEntries)
	{
		if (Entry.IsNone())
		{
			continue;
		}

		if (!NewMeta.IsEmpty())
		{
			NewMeta += TEXT(";");
		}

		NewMeta += Entry.ToString();

		Entries.Add(Entry);
	}

	FScopedTransaction Transaction(LOCTEXT("UpdateStringValues", "Update String Values"));

	Controller->Modify();

	TArray<UObject*> Outers;
	OriginalPropertyHandle->GetOuterObjects(Outers);

	if (!Outers.IsEmpty())
	{
		for (UObject* Outer : Outers)
		{
			Outer->Modify();
		}
	}

	using namespace UE::RCCustomControllers;

	Controller->SetMetadataValue(StringListControllerEntriesName, NewMeta);

	StringSelector->SetItemsSource(&Entries);

	FName CurrentValueName = *CurrentValue;

	if (!InNewEntries.Contains(CurrentValueName))
	{
		OriginalPropertyHandle->SetValue(InNewEntries[0].ToString());
	}
}

FReply SCustomStringListControllerWidget::OnEditStringListClicked()
{
	SCustomStringListControllerListEditorWidget::OpenModalWindow(SharedThis(this), Entries);

	return FReply::Handled();
}

TSharedRef<SWidget> SCustomStringListControllerWidget::OnGenerateWidget(FName InEntry)
{
	return SNew(STextBlock)
		.Text(FText::FromName(InEntry));
}

void SCustomStringListControllerWidget::OnSelectionChanged(FName InEntry, ESelectInfo::Type InSelectionType)
{
	if (InEntry.IsNone())
	{
		return;
	}

	TArray<UObject*> Outers;
	OriginalPropertyHandle->GetOuterObjects(Outers);

	if (!Outers.IsEmpty())
	{
		FScopedTransaction Transaction(LOCTEXT("SetStringValue", "Set String Value"));

		for (UObject* Outer : Outers)
		{
			Outer->Modify();
		}
	}
	
	OriginalPropertyHandle->SetValue(InEntry.ToString());
}

FText SCustomStringListControllerWidget::GetCurrentValueText() const
{
	FString CurrentValue;
	OriginalPropertyHandle->GetValue(CurrentValue);

	return FText::FromString(CurrentValue);
}

#undef LOCTEXT_NAMESPACE
