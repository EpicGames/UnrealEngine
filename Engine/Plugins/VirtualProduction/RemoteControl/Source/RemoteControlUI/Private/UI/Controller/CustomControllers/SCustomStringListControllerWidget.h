// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"

#include "Containers/Array.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Input/SComboBox.h"

class FText;
class IPropertyHandle;
class SBox;
class URCVirtualPropertyBase;
struct FPropertyChangedEvent;

class SCustomStringListControllerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomStringListControllerWidget)
		{}
	SLATE_END_ARGS()

	virtual ~SCustomStringListControllerWidget() override = default;

	/**
	 * Constructs this widget with InArgs
	 * @param InOriginalPropertyHandle Original PropertyHandle of this CustomWidget
	 */
	void Construct(const FArguments& InArgs, URCVirtualPropertyBase* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle);

	void UpdateStringList(const TArray<FName>& InNewEntries);

private:
	/** Original PropertyHandle, used to update correctly the Widget and the Controller */
	TWeakObjectPtr<URCVirtualPropertyBase> ControllerWeak;
	TSharedPtr<IPropertyHandle> OriginalPropertyHandle;
	TSharedPtr<SBox> PropertyWidgetContainer;
	TArray<FName> Entries;
	TSharedPtr<SComboBox<FName>> StringSelector;

	FReply OnEditStringListClicked();

	TSharedRef<SWidget> OnGenerateWidget(FName InEntry);

	void OnSelectionChanged(FName InEntry, ESelectInfo::Type InSelectionType);

	FText GetCurrentValueText() const;
};
