// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCustomTextureControllerWidget.h"
#include "Action/Bind/RCCustomBindActionUtilities.h"
#include "AssetThumbnail.h"
#include "Controller/RCController.h"
#include "EditorDirectories.h"
#include "Engine/Texture2D.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "RCVirtualProperty.h"
#include "SlateOptMacros.h"
#include "Styling/AppStyle.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Input/SFilePathPicker.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "ExternalTextureControllerWidget"

void SCustomTextureControllerWidget::Construct(const FArguments& InArgs, URCController* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle)
{
	if (!InOriginalPropertyHandle.IsValid())
	{
		return;
	}

	OriginalPropertyHandle = InOriginalPropertyHandle;

	FString ControllerString;
	OriginalPropertyHandle->GetValueAsFormattedString(ControllerString);
	bInternal = FPackageName::IsValidTextForLongPackageName(ControllerString);
	if (FPackageName::IsValidTextForLongPackageName(ControllerString))
	{
		bInternal = true;
		CurrentAssetPath = ControllerString;
	}
	else
	{
		bInternal = false;
		CurrentExternalPath = ControllerString;
	}

	ControllerTypes.Add(MakeShared<FString>(TEXT("External")));
	ControllerTypes.Add(MakeShared<FString>(TEXT("Asset")));
	
	ChildSlot
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SAssignNew(ThumbnailWidgetBox, SBox)
			.WidthOverride(64)
			.HeightOverride(64)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.05)
		[
			SNew(SSpacer)
		]
		+ SHorizontalBox::Slot()
		.FillWidth(0.6)
		[
			SAssignNew(ValueWidgetBox, SBox)
		]
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextComboBox)
			.OptionsSource(&ControllerTypes)
			.OnSelectionChanged(this, &SCustomTextureControllerWidget::OnControllerTypeChanged)
			.InitiallySelectedItem(ControllerTypes[bInternal])
		]
	];

	UpdateValueWidget();
	UpdateThumbnailWidget();
}

FString SCustomTextureControllerWidget::GetFilePath() const
{
	return CurrentExternalPath;
}

FString SCustomTextureControllerWidget::GetAssetPath() const
{
	return CurrentAssetPath;
}

FString SCustomTextureControllerWidget::GetAssetPathName() const
{
	if (Texture)
	{
		return Texture->GetPathName();
	}

	return TEXT("");
}

void SCustomTextureControllerWidget::OnControllerTypeChanged(TSharedPtr<FString, ESPMode::ThreadSafe> InString, ESelectInfo::Type InArg)
{
	const FString& Selection = *InString.Get();
	
	if (Selection == TEXT("Asset"))
	{
		bInternal = true;
	}
	if (Selection == TEXT("External"))
	{
		bInternal = false;
	}

	Texture = nullptr;
	
	UpdateValueWidget();
	UpdateThumbnailWidget();
	UpdateControllerValue();
}

void SCustomTextureControllerWidget::OnAssetSelected(const FAssetData& AssetData)
{
	UObject* TextureAsset = AssetData.GetAsset();
	if (TextureAsset != nullptr)
	{
		if (TextureAsset->IsA<UTexture2D>())
		{
			Texture = Cast<UTexture2D>(TextureAsset);
			CurrentAssetPath = AssetData.PackageName.ToString();
			UpdateControllerValue();
		}
	}
}

FString SCustomTextureControllerWidget::GetCurrentPath() const
{
	if (bInternal)
	{
		return GetAssetPath();
	}
	else
	{
		return GetFilePath();
	}
}

void SCustomTextureControllerWidget::UpdateControllerValue()
{
	const FString& Path = GetCurrentPath();

	if (OriginalPropertyHandle.IsValid())
	{
		OriginalPropertyHandle->SetValueFromFormattedString(Path);
	}
	RefreshThumbnailImage();
}

void SCustomTextureControllerWidget::HandleFilePathPickerPathPicked(const FString& InPickedPath)
{
	// The following code is adapted from FFilePathStructCustomization::HandleFilePathPickerPathPicked.
	FString FinalPath = InPickedPath;

	// Important: The result of this path conversion must be so that
	// FPackageName::IsValidTextForLongPackageName returns false.
	// For instance, it will be so if it doesn't start with a leading slash.
	
	auto ConvertPath = [](const FString& InAbsolutePath)
	{
		// "Content" doesn't have a token since it is the default base path.
		const FString ContentPath = FPaths::ConvertRelativePathToFull(FPaths::ProjectContentDir());
		if (FPaths::IsUnderDirectory(InAbsolutePath, ContentPath))
		{
			FString FinalPath = InAbsolutePath;
			if (FPaths::MakePathRelativeTo(FinalPath, *ContentPath))
			{
				return FinalPath;
			}
		}

		// Attempt to replace some other known paths with tokens.
		return InAbsolutePath.Replace(*FPaths::ConvertRelativePathToFull(FPaths::ProjectDir()), TEXT("{project_dir}"));
	};
	
	if (!InPickedPath.IsEmpty())
	{
		// if received path is relative, it is likely relative to the editor's exe.
		// We want to replace that path with tokens for known directories so that it can be replaced properly when cooked.
		if (FPaths::IsRelative(InPickedPath))
		{
			const FString AbsolutePickedPath = FPaths::ConvertRelativePathToFull(InPickedPath);
			if (FPaths::FileExists(AbsolutePickedPath))
			{
				FinalPath = ConvertPath(AbsolutePickedPath);
			}
		}
	}

	CurrentExternalPath = FinalPath;
	
	UpdateControllerValue();
	
	FEditorDirectories::Get().SetLastDirectory(ELastDirectory::GENERIC_OPEN, FPaths::GetPath(InPickedPath));
}

TSharedRef<SWidget> SCustomTextureControllerWidget::GetAssetThumbnailWidget()
{
	if (!Texture)
	{
		RefreshThumbnailImage();
	}
	
	const TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(Texture, 64, 64, UThumbnailManager::Get().GetSharedThumbnailPool());
	return AssetThumbnail->MakeThumbnailWidget();
}

TSharedRef<SWidget> SCustomTextureControllerWidget::GetExternalTextureValueWidget()
{
	static const FString FileTypeFilter = TEXT("Image files (*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.dds)|*.jpg; *.png; *.bmp; *.ico; *.exr; *.icns; *.jpeg; *.tga; *.hdr; *.dds");
	
	return SNew(SFilePathPicker)
			.BrowseButtonImage(FAppStyle::GetBrush("PropertyWindow.Button_Ellipsis"))
			.BrowseButtonStyle(FAppStyle::Get(), "HoverHintOnly")
			.BrowseButtonToolTip(LOCTEXT("FileButtonToolTipText", "Choose a file from this computer"))
			.BrowseDirectory(FEditorDirectories::Get().GetLastDirectory(ELastDirectory::GENERIC_OPEN))
			.BrowseTitle(LOCTEXT("PropertyEditorTitle", "File picker..."))
			.FilePath(this, &SCustomTextureControllerWidget::GetFilePath)
			.FileTypeFilter(FileTypeFilter)
			.OnPathPicked(this, &SCustomTextureControllerWidget::HandleFilePathPickerPathPicked);
}

TSharedRef<SWidget> SCustomTextureControllerWidget::GetAssetTextureValueWidget()
{
	return SNew(SObjectPropertyEntryBox)
			.AllowedClass(UTexture2D::StaticClass())
			.OnObjectChanged(this, &SCustomTextureControllerWidget::OnAssetSelected)
			.ObjectPath(this, &SCustomTextureControllerWidget::GetAssetPathName)
			.DisplayUseSelected(true)
			.DisplayBrowse(true);
}

void SCustomTextureControllerWidget::UpdateValueWidget()
{
	if (!ValueWidgetBox)
	{
		return;
	}
	
	if (bInternal)
	{
		ValueWidgetBox->SetContent(GetAssetTextureValueWidget());
	}
	else
	{
		ValueWidgetBox->SetContent(GetExternalTextureValueWidget());
	}
}

void SCustomTextureControllerWidget::UpdateThumbnailWidget()
{
	if (!ThumbnailWidgetBox)
	{
		return;
	}
	
	ThumbnailWidgetBox->SetContent(GetAssetThumbnailWidget());
}

void SCustomTextureControllerWidget::RefreshThumbnailImage()
{
	if (UTexture2D* LoadedTexture = UE::RCCustomBindActionUtilities::LoadTextureFromPath(GetCurrentPath()))
	{
		Texture = LoadedTexture;
		UpdateThumbnailWidget();
	}
}

#undef LOCTEXT_NAMESPACE
