// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"

class IPropertyHandle;
class URCController;
class UTexture2D;

class SCustomTextureControllerWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SCustomTextureControllerWidget)
		{
		}

	SLATE_END_ARGS()
	
	/**
	 * Constructs this widget with InArgs
	 * @param InOriginalPropertyHandle Original PropertyHandle of this CustomWidget
	 */
	void Construct(const FArguments& InArgs, URCController* InController, const TSharedPtr<IPropertyHandle>& InOriginalPropertyHandle);

private:
	/** Handle toggling Texture Controller type change: External vs. Internal */
	void OnControllerTypeChanged(TSharedPtr<FString, ESPMode::ThreadSafe> InString, ESelectInfo::Type InArg);

	/** Called when a Texture is selected in Asset mode */
	void OnAssetSelected(const FAssetData& AssetData);

	/** Sets the controller value starting from the current path set via this Widget */
	void UpdateControllerValue();

	/** Called to handle the file selection via OS window */
	void HandleFilePathPickerPathPicked(const FString& String);

	/** Refresh the value widget content. Useful when switching between External and Asset modes */
	void UpdateValueWidget();

	/** Refresh the thumbnail widget content */
	void UpdateThumbnailWidget();

	/** Loads the selected texture and updates the thumbnail Widget*/
	void RefreshThumbnailImage();

	/** Returns the current external file path */
	FString GetFilePath() const;

	/** Returns the current internal/asset file path */
	FString GetAssetPath() const;

	/** Returns the path name for the currently stored texture */
	FString GetAssetPathName() const;

	/** Returns current texture path value. Can be External or Asset, depending on current mode */
	FString GetCurrentPath() const;

	TSharedRef<SWidget> GetAssetThumbnailWidget();
	TSharedRef<SWidget> GetExternalTextureValueWidget();
	TSharedRef<SWidget> GetAssetTextureValueWidget();

	/** The texture selected by the user via the asset or external file path */
	TObjectPtr<UTexture2D> Texture;

	/** The widget containing the controller value */
	TSharedPtr<SBox> ValueWidgetBox;

	/** The widget containing the thumbnail for the currently selected Texture */
	TSharedPtr<SBox> ThumbnailWidgetBox;

	/** Current path for texture project asset */
	FString CurrentAssetPath;

	/** Current path pointing to external texture (e.g. png file) */
	FString CurrentExternalPath;

	/** Is the Texture controller currently used to select a texture asset (internal) or an external texture? */
	bool bInternal = false;

	/** List of available Texture Controller types, used by combo box selection. Supports Asset and External*/
	TArray<TSharedPtr<FString>> ControllerTypes;

	/** Original PropertyHandle, used to update correctly the Widget and the Controller */
	TSharedPtr<IPropertyHandle> OriginalPropertyHandle;
};
