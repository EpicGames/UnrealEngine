// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCCategoryModel.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "FRCCategoryModel"

const FLazyName FRCCategoryModel::CategoryModelName = "CategoryModel";

FRCCategoryModel::FRCCategoryModel(const FGuid& InCateogryId, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCLogicModeBase(InRemoteControlPanel)
	, Id(InCateogryId)
	, CachedIndex(INDEX_NONE)
{
}

TOptional<FRCVirtualPropertyCategory> FRCCategoryModel::GetCategory() const
{
	CheckCachedIndex();

	if (CachedIndex != INDEX_NONE)
	{
		return GetPreset()->GetControllerContainer()->Categories[CachedIndex];
	}

	return {};
}

void FRCCategoryModel::Initialize()
{
	SAssignNew(CategoryNameTextBox, SInlineEditableTextBlock)
		.Text(this, &FRCCategoryModel::GetCategoryDisplayName)
		.MultiLine(false)
		.OnTextCommitted(this, &FRCCategoryModel::OnCategoryNameCommitted);
}

TSharedRef<SWidget> FRCCategoryModel::GetWidget() const
{
	static const FMargin SlotMargin(10.0f, 2.0f);

	return SNew(SBox)
		.Padding(SlotMargin)
		[
			CategoryNameTextBox.ToSharedRef()
		];
}

int32 FRCCategoryModel::RemoveModel()
{
	TOptional<FRCVirtualPropertyCategory> Category = GetCategory();

	if (!Category.IsSet())
	{
		return 0;
	}

	URemoteControlPreset* Preset = GetPreset();

	if (!Preset)
	{
		return 0;
	}

	URCVirtualPropertyContainerBase* ControllerContainer = Preset->GetControllerContainer();

	if (!ControllerContainer)
	{
		return 0;
	}

	if (GUndo)
	{
		ControllerContainer->Modify();
	}

	for (int32 Index = 0; Index < ControllerContainer->Categories.Num(); ++Index)
	{
		if (ControllerContainer->Categories[Index].Id == Category->Id)
		{
			ControllerContainer->Categories.RemoveAt(Index);
			return 1;
		}
	}

	return 0;
}

TSharedRef<SWidget> FRCCategoryModel::GetNameWidget() const
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(10.f, 2.f)
		[
			CategoryNameTextBox.ToSharedRef()
		];
}

void FRCCategoryModel::OnCategoryNameCommitted(const FText& InNewCategoryName, ETextCommit::Type InCommitInfo)
{
	if (!InNewCategoryName.IsEmpty())
	{
		CheckCachedIndex();

		if (CachedIndex != INDEX_NONE)
		{
			// Validated by having a valid CachedIndex
			URCVirtualPropertyContainerBase* ControllerContainer = GetPreset()->GetControllerContainer();

			FScopedTransaction Transaction(LOCTEXT("RenameCategory", "Rename Category"));
			ControllerContainer->Modify();
			ControllerContainer->Categories[CachedIndex].Name = InNewCategoryName;
			CategoryNameTextBox->SetText(InNewCategoryName);
		}
	}
}

void FRCCategoryModel::CheckCachedIndex() const
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (URCVirtualPropertyContainerBase* ControllerContainer = Preset->GetControllerContainer())
		{
			if (CachedIndex != INDEX_NONE && ControllerContainer->Categories.IsValidIndex(CachedIndex) 
				&& ControllerContainer->Categories[CachedIndex].Id == Id)
			{
				return;
			}

			for (int32 Index = 0; Index < ControllerContainer->Categories.Num(); ++Index)
			{
				if (ControllerContainer->Categories[Index].Id == Id)
				{
					CachedIndex = Index;
					return;
				}
			}
		}
	}

	CachedIndex = INDEX_NONE;
}

void FRCCategoryModel::EnterNameEditingMode()
{	
	CategoryNameTextBox->EnterEditingMode();
}

FText FRCCategoryModel::GetCategoryDisplayName() const
{
	CheckCachedIndex();

	if (CachedIndex != INDEX_NONE)
	{
		return GetPreset()->GetControllerContainer()->Categories[CachedIndex].Name;
	}

	return FText::GetEmpty();
}

void FRCCategoryModel::SetDisplayIndex(int32 InDisplayIndex)
{
	CheckCachedIndex();

	if (CachedIndex != INDEX_NONE)
	{
		// Validated by having a valid CachedIndex
		URCVirtualPropertyContainerBase* ControllerContainer = GetPreset()->GetControllerContainer();

		if (GUndo)
		{
			ControllerContainer->Modify();
		}

		ControllerContainer->Categories[CachedIndex].DisplayIndex = InDisplayIndex;
	}
}

#undef LOCTEXT_NAMESPACE
