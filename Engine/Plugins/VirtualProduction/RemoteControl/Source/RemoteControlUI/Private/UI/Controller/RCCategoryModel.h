// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/BaseLogicUI/RCLogicModeBase.h"

#include "Misc/Optional.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

class SInlineEditableTextBlock;
class SRemoteControlPanel;
struct FRCVirtualPropertyCategory;

/*
* ~ FRCCategoryModel ~
*
* UI model for representing a Category
* Contains a row widget with Category Name
*/
class FRCCategoryModel : public FRCLogicModeBase
{
public:
	static const FLazyName CategoryModelName;

	DECLARE_MULTICAST_DELEGATE_OneParam(FOnValueChanged, TSharedPtr<FRCCategoryModel> /* InCategoryModel */);

	FRCCategoryModel(const FGuid& InCateogryId, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	virtual FName GetModelType() const override
	{
		return CategoryModelName;
	}

	TOptional<FRCVirtualPropertyCategory> GetCategory() const;

	/** Called right after object construction to initialize the model (and allow using SharedThis, etc) */
	void Initialize();

	/**
	 * The widget to be rendered for this Category
	 * Used to represent a single row when added to the Controllers Panel List
	 */
	virtual TSharedRef<SWidget> GetWidget() const override;

	virtual int32 RemoveModel() override;

	/** The widget showing the Name of this Category */
	TSharedRef<SWidget> GetNameWidget() const;

	/** Allows users to enter text into the Category Name Box */
	void EnterNameEditingMode();

	/** User-friendly Name of the underlying Category */
	FText GetCategoryDisplayName() const;

	/**Fetches the unique Id for this UI item */
	FGuid GetId() const { return Id; }

	/** Set the display index of the underlying category item */
	void SetDisplayIndex(int32 InDisplayIndex);

	/** Triggered when the Value this controller changes */
	FOnValueChanged OnValueChanged;

private:
	/** Text commit event for Category Name text box */
	void OnCategoryNameCommitted(const FText& InNewCategoryName, ETextCommit::Type InCommitInfo);

	/** Checks cached index fro validity and sets it if not. */
	void CheckCachedIndex() const;

	/** Category name - editable text box */
	TSharedPtr<SInlineEditableTextBlock> CategoryNameTextBox;

	/** Unique Id for this UI item */
	FGuid Id;

	/** Cached index for the category for faster lookup. */
	mutable int32 CachedIndex;
};
