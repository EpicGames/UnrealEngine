// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCControllerModel.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "CustomControllers/SCustomTextureControllerWidget.h"
#include "IDetailTreeNode.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "TypeTranslator/RCTypeTranslator.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "FRCControllerModel"

const FLazyName FRCControllerModel::ControllerModelName = "ControllerModel";

FRCControllerModel::FRCControllerModel(URCVirtualPropertyBase* InVirtualProperty, const TSharedRef<IDetailTreeNode>& InTreeNode, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel)
	: FRCLogicModeBase(InRemoteControlPanel)
	, VirtualPropertyWeakPtr(InVirtualProperty)
	, DetailTreeNodeWeakPtr(InTreeNode)
	, CurrentControlValueType(EPropertyBagPropertyType::None)
{
	Id = FGuid::NewGuid();
}

void FRCControllerModel::Initialize()
{
	URCVirtualPropertyBase* const VirtualProperty = VirtualPropertyWeakPtr.Get();
	const TSharedPtr<IDetailTreeNode> TreeNode = DetailTreeNodeWeakPtr.Pin();

	if (ensure(VirtualProperty && TreeNode.IsValid()))
	{
		if (VirtualProperty->DisplayName.IsNone())
		{
			VirtualProperty->DisplayName = VirtualProperty->PropertyName;
		}

		SAssignNew(ControllerNameTextBox, SEditableTextBox)
			.Text(this, &FRCControllerModel::GetControllerDisplayName)
			.RevertTextOnEscape(true)
			.SelectAllTextWhenFocused(true)
			.OnTextCommitted(this, &FRCControllerModel::OnControllerNameCommitted);

		SAssignNew(ControllerDescriptionTextBox, SInlineEditableTextBlock)
			.Text(this, &FRCControllerModel::GetControllerDescription)
			.MultiLine(true)
			.OnTextCommitted(this, &FRCControllerModel::OnControllerDescriptionCommitted);

		SAssignNew(ControllerFieldIdTextBox, SInlineEditableTextBlock)
			.Text(FText::FromName(VirtualProperty->FieldId))
			.OnTextCommitted(this, &FRCControllerModel::OnControllerFieldIdCommitted);

		if (const TSharedPtr<IPropertyHandle>& PropertyHandle = TreeNode->CreatePropertyHandle())
		{
			PropertyHandle->SetOnPropertyValueChangedWithData(TDelegate<void(const FPropertyChangedEvent&)>::CreateSP(this, &FRCControllerModel::OnPropertyValueChanged));
		}
	}
}

TSharedRef<SWidget> FRCControllerModel::GetWidget() const
{
	const FNodeWidgets NodeWidgets = DetailTreeNodeWeakPtr.Pin()->CreateNodeWidgets();

	const TSharedRef<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);
	if (VirtualPropertyWeakPtr.IsValid())
	{
		static const FMargin SlotMargin(10.0f, 2.0f);
		
		// If this is a custom controller, we will use its custom widget
		if (const TSharedPtr<SWidget>& CustomControllerWidget = IRemoteControlUIModule::Get().CreateCustomControllerWidget(VirtualPropertyWeakPtr.Get(), DetailTreeNodeWeakPtr.Pin()->CreatePropertyHandle()))
		{
			FieldWidget->AddSlot()
				.Padding(SlotMargin)
				[
					CustomControllerWidget.ToSharedRef()
				];
		}
		else if (NodeWidgets.ValueWidget)
		{
			FieldWidget->AddSlot()
				.Padding(SlotMargin)
				[
					NodeWidgets.ValueWidget.ToSharedRef()
				];
		}
		else if (NodeWidgets.WholeRowWidget)
		{
			FieldWidget->AddSlot()
				.Padding(SlotMargin)
				[
					NodeWidgets.WholeRowWidget.ToSharedRef()
				];
		}
	}

	return FieldWidget;
}

TSharedRef<SWidget> FRCControllerModel::GetNameWidget() const
{
	return SNew(SBox)
		.VAlign(VAlign_Center)
		.Padding(10.f, 2.f)
		[
			ControllerNameTextBox.ToSharedRef()
		];
}

TSharedRef<SWidget> FRCControllerModel::GetDescriptionWidget() const
{
	return SNew(SBox).Padding(10.f, 2.f)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.VAlign(VAlign_Center)
			[
				ControllerDescriptionTextBox.ToSharedRef()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RemoteControlDescriptionPlaceholder", "Double click to change description"))
				.IsEnabled(false)
				.Visibility(this, &FRCControllerModel::GetPlaceholderVisibility)
			]
		];
}

TSharedRef<SWidget> FRCControllerModel::GetFieldIdWidget() const
{
	return SNew(SBox).Padding(10.f, 2.f)
		[
			ControllerFieldIdTextBox.ToSharedRef()
		];
}

TSharedRef<SWidget> FRCControllerModel::GetTypeSelectionWidget()
{
	if (const URCVirtualPropertyBase* Controller = GetVirtualProperty())
	{
		const FName& FieldId = Controller->FieldId;
		const TArray<URCVirtualPropertyBase*>& Controllers = GetPreset()->GetControllersByFieldId(FieldId);
		
		if (Controllers.Num() > 1 && bIsMultiController)
		{
			const EPropertyBagPropertyType OptimalValueType = FRCTypeTranslator::GetOptimalValueType(GetPreset()->GetControllersTypesByFieldId(FieldId));
			CurrentControlValueType = OptimalValueType;

			return SNew(SBox).Padding(10.f, 2.f)
			[
				SNew(STextComboBox)
				.OptionsSource(&ControlledTypesAsStrings)
				.OnSelectionChanged(this, &FRCControllerModel::OnTextControlValueTypeChanged)
				.InitiallySelectedItem(ControlledTypesAsStrings[CurrentControlValueTypeIndex])
			];
		}
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> FRCControllerModel::GetControllerExtensionWidget(const FName& InColumnName) const
{
	FRCControllerExtensionWidgetsInfo ExtensionWidgetsInfo(GetVirtualProperty());
	
	IRemoteControlUIModule::Get().OnGenerateControllerExtensionsWidgets().Broadcast(ExtensionWidgetsInfo);

	const TMap<FName, TSharedRef<SWidget>>& WidgetsMap = ExtensionWidgetsInfo.CustomWidgetsMap;
	if (WidgetsMap.Contains(InColumnName))
	{
		return WidgetsMap[InColumnName];
	}
	
	return SNullWidget::NullWidget;
}

void FRCControllerModel::SetMultiController(bool bInIsMultiController)
{
	bIsMultiController = bInIsMultiController;
	
	InitControlledTypes();
}

URCVirtualPropertyBase* FRCControllerModel::GetVirtualProperty() const
{
	return VirtualPropertyWeakPtr.Get();
}

const FName FRCControllerModel::GetPropertyName() const
{
	if (const URCVirtualPropertyBase* VirtualProperty = GetVirtualProperty())
	{
		return VirtualProperty->PropertyName;
	}

	return NAME_None;
}

TSharedPtr<FRCBehaviourModel> FRCControllerModel::GetSelectedBehaviourModel() const
{
	return SelectedBehaviourModelWeakPtr.Pin();
}

void FRCControllerModel::UpdateSelectedBehaviourModel(TSharedPtr<FRCBehaviourModel> InModel)
{
	SelectedBehaviourModelWeakPtr = InModel;
}

void FRCControllerModel::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		if (const URCVirtualPropertyBase* Controller = GetVirtualProperty())
		{
			if (URemoteControlPreset* Preset = GetPreset())
			{
				// Cache controllers label again here since during Undo/Redo the cache map won't be changed
				Preset->CacheControllersLabels();
			}
			ControllerNameTextBox->SetText(FText::FromName(Controller->DisplayName));
			ControllerDescriptionTextBox->SetText(Controller->Description);
		}
	}
}

void FRCControllerModel::OnControllerNameCommitted(const FText& InNewControllerName, ETextCommit::Type InCommitInfo)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (const URCVirtualPropertyBase* Controller = GetVirtualProperty())
		{
			FScopedTransaction Transaction(LOCTEXT("RenameController", "Rename Controller"));
			const FName OldName = Controller->DisplayName;
			const FName AssignedLabel = Preset->SetControllerDisplayName(Controller->Id, FName(FText::TrimPrecedingAndTrailing(InNewControllerName).ToString()));
			ControllerNameTextBox->SetText(FText::FromName(AssignedLabel));
			Preset->OnControllerRenamed().Broadcast(Preset, OldName, AssignedLabel);
		}
	}
}

void FRCControllerModel::OnControllerDescriptionCommitted(const FText& InNewControllerDescription, ETextCommit::Type InCommitInfo)
{
	if (URCVirtualPropertyBase* Controller = GetVirtualProperty())
	{
		FScopedTransaction Transaction(LOCTEXT("ChangedControllerDescription", "Update controller description"));
		Controller->Modify();
		Controller->Description = InNewControllerDescription;
		ControllerDescriptionTextBox->SetText(InNewControllerDescription);
	}
}

void FRCControllerModel::OnControllerFieldIdCommitted(const FText& InNewControllerFieldId, ETextCommit::Type InCommitInfo)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (URCVirtualPropertyBase* Controller = GetVirtualProperty())
		{
			// todo: think about how to setup this Field Id change feature
			// We might need some delegate firing when a Field Id update occurs e.g
			// OnControllerFieldIdChangedFromPanel(FName OldFieldId, FName NewFieldId)
		}
	}
}

void FRCControllerModel::OnTextControlValueTypeChanged(TSharedPtr<FString, ESPMode::ThreadSafe> InControlValueTypeString, ESelectInfo::Type Arg)
{
	if (ControlledTypesAsStrings.Contains(InControlValueTypeString))
	{
		CurrentControlValueTypeIndex = ControlledTypesAsStrings.IndexOfByKey(InControlValueTypeString);
	}
	else
	{
		CurrentControlValueTypeIndex = 0;
	}

	const FString& Value = *InControlValueTypeString.Get();
	CurrentControlValueType = static_cast<EPropertyBagPropertyType>(StaticEnum<EPropertyBagPropertyType>()->GetValueByNameString(Value));
	
	if (OnValueTypeChanged.IsBound())
	{
		if (URCVirtualPropertyBase* Controller = GetVirtualProperty())
		{
			if (OnValueTypeChanged.IsBound())
			{
				OnValueTypeChanged.Broadcast(Controller, CurrentControlValueType);
			}
		}
	}
}

void FRCControllerModel::OnPropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{	
	if (OnValueChanged.IsBound())
	{
		OnValueChanged.Broadcast(StaticCastSharedRef<FRCControllerModel>(AsShared()));
	}
}

EVisibility FRCControllerModel::GetPlaceholderVisibility() const
{
	if (ControllerDescriptionTextBox.IsValid())
	{
		if (ControllerDescriptionTextBox->IsInEditMode())
		{
			return EVisibility::Collapsed;
		}
		return ControllerDescriptionTextBox->GetText().IsEmpty() ? EVisibility::Visible : EVisibility:: Collapsed;
	}
	return EVisibility::Collapsed;
}

void FRCControllerModel::InitControlledTypes()
{
	ControlledTypesAsStrings.Empty();
	
	if (const URCVirtualPropertyBase* Controller = GetVirtualProperty())
	{
		const TArray<EPropertyBagPropertyType>& ValueTypes = GetPreset()->GetControllersTypesByFieldId(Controller->FieldId);

		int32 Count = 0;
		TArray<FString> Types;
		for (const EPropertyBagPropertyType ValueType : ValueTypes)
		{
			const FText& TypeName = UEnum::GetDisplayValueAsText(ValueType);
			Types.AddUnique(TypeName.ToString());

			if (Controller->GetValueType() == ValueType)
			{
				CurrentControlValueTypeIndex = Count;
			}

			Count++;
		}

		for (const FString& TypeName : Types)
		{
			ControlledTypesAsStrings.Add(MakeShared<FString>(TypeName));
		}
	}
}

void FRCControllerModel::EnterDescriptionEditingMode()
{	
	ControllerDescriptionTextBox->EnterEditingMode();
}

FText FRCControllerModel::GetControllerDisplayName() const
{
	if (URCVirtualPropertyBase* Controller = GetVirtualProperty())
	{
		return FText::FromName(Controller->DisplayName);
	}
	return FText::GetEmpty();
}

FText FRCControllerModel::GetControllerDescription() const
{
	if (URCVirtualPropertyBase* Controller = GetVirtualProperty())
	{
		return Controller->Description;
	}
	return FText::GetEmpty();
}

int32 FRCControllerModel::RemoveModel()
{
	URCVirtualPropertyBase* Controller = GetVirtualProperty();

	if (!Controller)
	{
		return 0;
	}

	URemoteControlPreset* Preset = GetPreset();

	if (!Preset)
	{
		return 0;
	}

	if (GUndo)
	{
		Preset->Modify();
	}

	if (Preset->RemoveController(GetPropertyName()))
	{
		return 1;
	}

	return 0;
}

#undef LOCTEXT_NAMESPACE
