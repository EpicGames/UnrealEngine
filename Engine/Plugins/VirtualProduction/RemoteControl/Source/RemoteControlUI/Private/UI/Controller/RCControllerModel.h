// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorUndoClient.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"

enum class EPropertyBagPropertyType : uint8;
class FRCBehaviourModel;
class IDetailTreeNode;
class SEditableTextBox;
class SInlineEditableTextBlock;
class SRemoteControlPanel;
class URCBehaviour;
class URCControllerContainer;
class URCUserDefinedStruct;
class URCVirtualPropertyContainerBase;
class URCVirtualPropertyBase;

/*
* ~ FRCControllerModel ~
*
* UI model for representing a Controller
* Contains a row widget with Controller Name, Field Id and a Value Widget.
* If this is setup to be a MultiController, a Value Type selection Widget will be available as well.
*/
class FRCControllerModel : public FRCLogicModeBase, public FSelfRegisteringEditorUndoClient
{
public:
	static const FLazyName ControllerModelName;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnValueTypeChanged, URCVirtualPropertyBase* /* InController */, EPropertyBagPropertyType /* InValueType */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnValueChanged, TSharedPtr<FRCControllerModel> /* InControllerModel */);
	
	FRCControllerModel(URCVirtualPropertyBase* InVirtualProperty, const TSharedRef<IDetailTreeNode>& InTreeNode, const TSharedPtr<SRemoteControlPanel> InRemoteControlPanel);

	virtual FName GetModelType() const override
	{
		return ControllerModelName;
	}

	/** Called right after object construction to initialize the model (and allow using SharedThis, etc) */
	void Initialize();

	/**
	 * The widget to be rendered for this Controller
	 * Used to represent a single row when added to the Controllers Panel List
	 */
	virtual TSharedRef<SWidget> GetWidget() const override;

	virtual int32 RemoveModel() override;

	/** The widget showing the Name of this Controller */
	TSharedRef<SWidget> GetNameWidget() const;

	/** The widget showing the Description of this Controller */
	TSharedRef<SWidget> GetDescriptionWidget() const;

	/** The widget showing the Field Id of this Controller */
	TSharedRef<SWidget> GetFieldIdWidget() const;

	/** The widget showing the Value Type selection dropdown for this Controller (used for MultiControllers) */
	TSharedRef<SWidget> GetTypeSelectionWidget();

	/** The widget allowing controller columns customization */
	TSharedRef<SWidget> GetControllerExtensionWidget(const FName& InColumnName) const;
	
	/**
	 * Specify if this is a MultiController handling other hidden controllers.
	 * MultiControllers will setup and show a Type Selection Widget.
	 */
	void SetMultiController(bool bInIsMultiController);

	/** Returns the Controller (Data Model) associated with us*/
	virtual URCVirtualPropertyBase* GetVirtualProperty() const;
	
	/** Returns the Controller Name (from Virtual Property) represented by us*/
	virtual const FName GetPropertyName() const;

	/** Returns the currently selected Behaviour (UI model) */
	TSharedPtr<FRCBehaviourModel> GetSelectedBehaviourModel() const;

	/** Updates our internal record of the currently selected Behaviour (UI Model) */
	void UpdateSelectedBehaviourModel(TSharedPtr<FRCBehaviourModel> InModel);

	/** Allows users to enter text into the Controller Name Box */
	void EnterDescriptionEditingMode();

	/** User-friendly Name of the underlying Controller */
	FText GetControllerDisplayName() const;

	/** Description of the underlying Controller */
	FText GetControllerDescription() const;

	/**Fetches the unique Id for this UI item */
	FGuid GetId() const { return Id; }

	/** Triggered when the Value Type of this changes (for MultiControllers) */
	FOnValueTypeChanged OnValueTypeChanged;

	/** Triggered when the Value this controller changes */
	FOnValueChanged OnValueChanged;

private:
	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override { PostUndo(bSuccess); };
	// End of FEditorUndoClient

	/** Text commit event for Controller Name text box */
	void OnControllerNameCommitted(const FText& InNewControllerName, ETextCommit::Type InCommitInfo);

	/** Text commit event for Controller description text box */
	void OnControllerDescriptionCommitted(const FText& InNewControllerDescription, ETextCommit::Type InCommitInfo);

	/** Text commit event for Controller FieldId text box */
	void OnControllerFieldIdCommitted(const FText& InNewControllerFieldId, ETextCommit::Type InCommitInfo);

	/** Value Type change event, in case this is a MultiController */
	void OnTextControlValueTypeChanged(TSharedPtr<FString, ESPMode::ThreadSafe> InControlValueTypeString, ESelectInfo::Type Arg);

	/** Value change event */
	void OnPropertyValueChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	/** Get the current visibility of the placeholder TextBlock widget */
	EVisibility GetPlaceholderVisibility() const;

	/** Initializes the list of controlled types. Used by MultiControllers */
	void InitControlledTypes();

	/** Virtual Property (Data Model) represented by us*/
	TWeakObjectPtr<URCVirtualPropertyBase> VirtualPropertyWeakPtr;
	
	/** The row generator used to build a generic Value Widget*/
	TWeakPtr<IDetailTreeNode> DetailTreeNodeWeakPtr;

	/** The currently selected Behaviour (UI model) */
	TWeakPtr<FRCBehaviourModel>  SelectedBehaviourModelWeakPtr;

	/** Controller id - editable text box */
	TSharedPtr<SEditableTextBox> ControllerNameTextBox;

	/** Controller description - editable text box */
	TSharedPtr<SInlineEditableTextBlock> ControllerDescriptionTextBox;

	/** Controller Field Id - editable text box */
	TSharedPtr<SInlineEditableTextBlock> ControllerFieldIdTextBox;

	/** Holds a list of controlled types, represented as strings. Used to show the Value Type selection dropdown */
	TArray<TSharedPtr<FString>> ControlledTypesAsStrings;

	/** Index representing the currently selected Value Type within the ControlledTypesAsStrings array */
	int32 CurrentControlValueTypeIndex = 0;

	/** Current control value type */
	EPropertyBagPropertyType CurrentControlValueType;

	/** Is this a multi controller?*/
	bool bIsMultiController = false;

	/**Unique Id for this UI item*/
	FGuid Id;
};
