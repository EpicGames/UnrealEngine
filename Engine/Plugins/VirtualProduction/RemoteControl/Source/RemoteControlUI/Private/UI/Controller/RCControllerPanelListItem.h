// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Framework/Views/TableViewTypeTraits.h"
#include "HAL/Platform.h"

enum class ERCControllerPanelListItemType : uint8
{
	None,
	Category,
	Controller
};

struct FRCControllerPanelListItem
{
	ERCControllerPanelListItemType Type;
	int32 Index;

	operator bool() const
	{
		return Type != ERCControllerPanelListItemType::None;
	}

	bool operator==(const FRCControllerPanelListItem& InOther) const
	{
		return Type == InOther.Type
			&& Index == InOther.Index;
	}

	bool operator<(const FRCControllerPanelListItem& InOther) const
	{
		if (static_cast<uint8>(Type) < static_cast<uint8>(InOther.Type))
		{
			return true;
		}
		else if (static_cast<uint8>(Type) > static_cast<uint8>(InOther.Type))
		{
			return false;
		}

		return Index < InOther.Index;
	}

	friend uint32 GetTypeHash(const FRCControllerPanelListItem& InItem)
	{
		return static_cast<uint32>(InItem.Index) | (1 << (32 - static_cast<uint32>(InItem.Type)));
	}
};

namespace UE::RemoteControl::UI::Private
{
	namespace ControllerPanelListItem
	{
		const FRCControllerPanelListItem None = {ERCControllerPanelListItemType::None, INDEX_NONE};
	}
}

template <>
struct TIsValidListItem<FRCControllerPanelListItem>
{
	enum
	{
		Value = true
	};
};

/**
 * Functionality (e.g. setting the value, testing invalid value) specialized for FGroupItem.
 */
template <>
struct TListTypeTraits<FRCControllerPanelListItem>
{
public:
	typedef FRCControllerPanelListItem NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FRCControllerPanelListItem, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FRCControllerPanelListItem, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FRCControllerPanelListItem>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector& Collector,
		TArray<FRCControllerPanelListItem>& ItemsWithGeneratedWidgets,
		TSet<FRCControllerPanelListItem>& SelectedItems,
		TMap<const U*, FRCControllerPanelListItem>& WidgetToItemMap)
	{
	}

	static bool IsPtrValid(const FRCControllerPanelListItem& InValue)
	{
		return InValue.Type != ERCControllerPanelListItemType::None;
	}

	static void ResetPtr(FRCControllerPanelListItem& InValue)
	{
		InValue.Type = ERCControllerPanelListItemType::None;
		InValue.Index = INDEX_NONE;
	}

	static FRCControllerPanelListItem MakeNullPtr()
	{
		return UE::RemoteControl::UI::Private::ControllerPanelListItem::None;
	}

	static FRCControllerPanelListItem NullableItemTypeConvertToItemType(const FRCControllerPanelListItem& InValue)
	{
		return InValue;
	}

	static FString DebugDump(FRCControllerPanelListItem InValue)
	{
		return FString::FromInt(static_cast<int32>(InValue.Type)) + TEXT(": ") + FString::FromInt(InValue.Index);
	}

	class SerializerType
	{
	};
};
