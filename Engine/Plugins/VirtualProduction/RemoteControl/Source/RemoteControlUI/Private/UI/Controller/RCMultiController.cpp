// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCMultiController.h"
#include "Controller/RCController.h"
#include "StructUtils/PropertyBag.h"
#include "TypeTranslator/RCTypeTranslator.h"
#include "RCVirtualProperty.h"
#include "RemoteControlPreset.h"

FRCMultiController::FRCMultiController(const TWeakObjectPtr<URCVirtualPropertyBase>& InController)
{
	if (InController.IsValid())
	{
		MainControllerWeak = InController;
		ValueType = InController->GetValueType();
	}
}

void FRCMultiController::AddHandledController(URCVirtualPropertyBase* InController)
{
	if (MainControllerWeak.IsValid())
	{
		if (URCController* Controller = Cast<URCController>(InController))
		{
			HandledControllers.AddUnique(InController);
		}
	}
}

void FRCMultiController::SetMainController(URCVirtualPropertyBase* InController)
{
	MainControllerWeak = InController;
	ValueType = MainControllerWeak->GetValueType();
}

void FRCMultiController::UpdateHandledControllersValue()
{
	if (MainControllerWeak.IsValid())
	{
		if (const URCVirtualPropertyBase* MultiController = MainControllerWeak.Get())
		{
			FRCTypeTranslator::Get()->Translate(MultiController, HandledControllers);
		}
	}
}

FName FRCMultiController::GetFieldId() const
{
	if (MainControllerWeak.IsValid())
	{
		return MainControllerWeak->FieldId;
	}

	return NAME_None;	
}

FGuid FRCMultiController::GetMainControllerId() const
{
	if (MainControllerWeak.IsValid())
	{
		return MainControllerWeak->Id;
	}

	return FGuid();
}

bool FRCMultiControllersState::TryToAddAsMultiController(URCVirtualPropertyBase* InController)
{
	if (!InController)
	{
		return false;
	}

	if (!InController->PresetWeakPtr.IsValid())
	{
		return false;
	}
	
	const URemoteControlPreset* const Preset = InController->PresetWeakPtr.Get();
	const FName& FieldId = InController->FieldId;
	
	TArray<URCVirtualPropertyBase*> ControllersWithSameFieldId = Preset->GetControllersByFieldId(FieldId);
	if (ControllersWithSameFieldId.Num() > 1)
	{
		if (MultiControllersMap.Contains(FieldId))
		{
			// This Field Id has already been handled, no need to add a new MultiController for it
			return false;
		}
		
		// Retrieve currently selected value type for this Field Id
		const EPropertyBagPropertyType* FieldIdTypePtr = SelectedValueTypes.Find(FieldId);
		EPropertyBagPropertyType FieldIdType;
	
		if (!FieldIdTypePtr)
		{
			// If type is not set for this Field Id, let's set it to the optimal one
			FieldIdType = UE::MultiControllers::Public::GetOptimalValueTypeForFieldId(Preset, FieldId);
			SelectedValueTypes.Emplace(FieldId, FieldIdType);
		}
		else
		{
			// Otherwise, retrieve the currently selected value type
			FieldIdType = *FieldIdTypePtr;
		}

		// Retrieve value type of controller we are trying to add as MultiController
		const EPropertyBagPropertyType ControllerValueType = InController->GetValueType();
		const TArray<EPropertyBagPropertyType>& FieldIdControlledTypes = UE::MultiControllers::Public::GetFieldIdValueTypes(Preset, FieldId);
		if (!FieldIdControlledTypes.Contains(FieldIdType))
		{
			// If SelectedValueTypes contains a type which is not actually handled by any controller with this Field Id,
			// then it's outdated - ignore and remove misleading info
			SelectedValueTypes.Remove(FieldId);
		}
		else if (ControllerValueType != FieldIdType)
		{
			// This controller has a value type different from the selected one
			return false;
		}

		// We got here, so it means we can add a new MultiController
		FRCMultiController NewMultiController(InController);
		
		for (URCVirtualPropertyBase* ControllerToHandle : ControllersWithSameFieldId)
		{
			// Skip self (will have same type)
			if (ControllerToHandle->GetValueType() != InController->GetValueType())
			{
				NewMultiController.AddHandledController(ControllerToHandle);
			}
		}

		MultiControllersMap.Add(FieldId, NewMultiController);

		return true;
	}

	return false;
}

void FRCMultiControllersState::UpdateFieldIdValueType(const FName& InFieldId, EPropertyBagPropertyType InValueType)
{
	SelectedValueTypes.Emplace(InFieldId, InValueType);
	RemoveMultiController(InFieldId);
}

EPropertyBagPropertyType FRCMultiControllersState::GetCurrentType(const FName& InFieldId)
{
	if (const EPropertyBagPropertyType* const ValueTypePtr = SelectedValueTypes.Find(InFieldId))
	{
		return *ValueTypePtr;
	}

	return EPropertyBagPropertyType::None;
}

FRCMultiController FRCMultiControllersState::GetMultiController(const FName& InFieldId)
{
	if (const FRCMultiController* const MultiController = MultiControllersMap.Find(InFieldId))
	{
		return *MultiController;
	}

	return FRCMultiController(nullptr);
}

void FRCMultiControllersState::ResetMultiControllers()
{
	MultiControllersMap.Empty();
}

void FRCMultiControllersState::ResetSelectedValueTypes()
{
	SelectedValueTypes.Empty();
}

void FRCMultiControllersState::RemoveMultiController(const FName& InFieldId)
{
	if (MultiControllersMap.Contains(InFieldId))
	{		
		MultiControllersMap.Remove(InFieldId);
	}
}

EPropertyBagPropertyType UE::MultiControllers::Public::GetOptimalValueTypeForFieldId(const URemoteControlPreset* InPreset, const FName& InFieldId)
{
	EPropertyBagPropertyType OptimalType = EPropertyBagPropertyType::Bool;
	
	if (InPreset)
	{
		const TArray<EPropertyBagPropertyType>& ValueTypes = InPreset->GetControllersTypesByFieldId(InFieldId);
		OptimalType = FRCTypeTranslator::Get()->GetOptimalValueType(ValueTypes);
	}

	return OptimalType;
}

TArray<EPropertyBagPropertyType> UE::MultiControllers::Public::GetFieldIdValueTypes(const URemoteControlPreset* InPreset, const FName& InFieldId)
{
	if (!InPreset)
	{
		return {};
	}
	
	TArray<EPropertyBagPropertyType> ValueTypes;
	const TArray<URCVirtualPropertyBase*>& Controllers = InPreset->GetControllersByFieldId(InFieldId);
	
	for (const URCVirtualPropertyBase* Controller : Controllers)
	{
		ValueTypes.Add(Controller->GetValueType());
	}

	return ValueTypes;
}
