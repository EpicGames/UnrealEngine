// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "StructUtils/PropertyBag.h"

class URCBehaviour;
class URCVirtualPropertyBase;
class URemoteControlPreset;

/**
 * A MultiController keeps a reference to a Main Controller and to a set of Handled Controllers with the same Field Id.
 * A MultiController can try to use its MainController value to update the value of its Handled Controllers.
 * This is useful when Control Panel List is in MultiController Mode, and Controllers with duplicate Field Ids are hidden.
 */
struct FRCMultiController
{
public:
	FRCMultiController(const TWeakObjectPtr<URCVirtualPropertyBase>& InController);

	/** Sets the provided controller as the MainController, which will drive other Handled Controllers */
	void SetMainController(URCVirtualPropertyBase* InController);

	/** Adds a controller to the list of Handled Controllers managed by the MainController of this MultiController setup */
	void AddHandledController(URCVirtualPropertyBase* InController);

	/** Goes through each Handled Controller, and tries to update its value based on the current value of the MainController*/
	void UpdateHandledControllersValue();

	/** MultiController is considered valid when its MainController is valid */
	bool IsValid() const { return MainControllerWeak.IsValid(); }

	/** Returns the Field Id this MultiController is associated with */
	FName GetFieldId() const;

	/** Return the Id of the MainController */
	FGuid GetMainControllerId() const;

	/** Returns the value type of the MainController */
	EPropertyBagPropertyType GetValueType() const { return ValueType; }

private:
	TWeakObjectPtr<URCVirtualPropertyBase> MainControllerWeak;
	EPropertyBagPropertyType ValueType;
	TArray<TObjectPtr<URCVirtualPropertyBase>> HandledControllers;
};

/**
 * Holds and handles a Map of MultiControllers.
 * Also keeps reference of currently selected Value Type per each Field Id.
 */
struct FRCMultiControllersState
{
	bool TryToAddAsMultiController(URCVirtualPropertyBase* InController);
	
	EPropertyBagPropertyType GetCurrentType(const FName& InFieldId);
	FRCMultiController GetMultiController(const FName& InFieldId);

	/**
	 * Calling this function will overwrite the currently selected type for the specified field id.
	 * FRCMultiControllersState will then be in an invalid state, so it will require the hosting ControllerPanelList to call Reset()
	 */
	void UpdateFieldIdValueType(const FName& InFieldId, EPropertyBagPropertyType InValueType);

	/** Empties the internal map referencing the handled MultiControllers */
	void ResetMultiControllers();

	/** Empties the internal map used to store the currently selected value type for each Field Id */
	void ResetSelectedValueTypes();

	/** Removes the MultiController associated with the specified Field Id */	
	void RemoveMultiController(const FName& InFieldId);

private:
	/** Map of MultiControllers, indexed by their Field Id*/
	TMap<FName, FRCMultiController> MultiControllersMap;

	/** Used to store the currently chosen value type. This is stored outside of FRCMultiController, so that the info is kept between UI Refreshes */
	TMap<FName, EPropertyBagPropertyType> SelectedValueTypes;
};

namespace UE::MultiControllers::Public
{
	/** Returns the optimal value type for the specified Field Id */
	EPropertyBagPropertyType GetOptimalValueTypeForFieldId(const URemoteControlPreset* InPreset, const FName& InFieldId);

	/** Returns a list of all the Value Types currently associated to the specified Field Id */
	TArray<EPropertyBagPropertyType> GetFieldIdValueTypes(const URemoteControlPreset* InPreset, const FName& InFieldId);
}
