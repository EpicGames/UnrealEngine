// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCControllerPanel.h"

#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "Controller/RCStatelessEventController.h"
#include "Engine/Texture.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GraphEditorSettings.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Materials/MaterialInterface.h"
#include "Misc/MessageDialog.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RCControllerModel.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlCommon.h"
#include "RemoteControlLogicConfig.h"
#include "RemoteControlPreset.h"
#include "SlateOptMacros.h"
#include "SRCControllerPanelList.h"
#include "Styling/RemoteControlStyles.h"
#include "TypeSelector/SRCControllerTypeSelector.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"
#include "UI/Panels/SRCDockPanel.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRemoteControlPanel.h"
#include "UObject/UnrealType.h"
#include "UObject/UnrealTypePrivate.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Views/SListView.h"

#if WITH_EDITOR
#include "ScopedTransaction.h"
#endif

#define LOCTEXT_NAMESPACE "SRCControllerPanel"

namespace UE::RemoteControl::UI::Private
{
	const FLazyName FRCControllerPropertyInfo::CategoryName = TEXT("Category");
	const FLazyName FRCControllerPropertyInfo::InitialDescriptionName = TEXT("InitialDescription");
	const FString FRCControllerPropertyInfo::CategoryMeta = TEXT("Category");
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCControllerPanel::Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	SRCLogicPanelBase::Construct(SRCLogicPanelBase::FArguments(), InPanel);
	
	bIsInLiveMode = InArgs._LiveMode;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	// Controller Dock Panel
	TSharedPtr<SRCMinorPanel> ControllerDockPanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("ControllersLabel", "Controller"))
		.EnableFooter(false)
		[
			SAssignNew(ControllerPanelList, SRCControllerPanelList, SharedThis(this), InPanel)
		];

	// Add New Controller Button
	const TSharedRef<SWidget> AddNewControllerButton = SNew(SComboButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Controller")))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.IsEnabled_Lambda([this]() { return !bIsInLiveMode.Get(); })
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.CollapseMenuOnParentFocus(true)
		.HasDownArrow(false)
		.ContentPadding(FMargin(4.f, 2.f))
		.ButtonContent()
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle->IconSize.X)
			.HeightOverride(RCPanelStyle->IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
		]
		.OnGetMenuContent(this, &SRCControllerPanel::GetControllerMenuContentWidget);

	ControllerDockPanel->AddHeaderToolbarItem(EToolbar::Left, AddNewControllerButton);
	ControllerDockPanel->AddHeaderToolbarItem(EToolbar::Right, GetMultiControllerSwitchWidget());

	ChildSlot
		.Padding(RCPanelStyle->PanelPadding)
		[
			ControllerDockPanel.ToSharedRef()
		];
}

void SRCControllerPanel::Refresh()
{
	if (ControllerPanelList.IsValid())
	{
		ControllerPanelList->Reset();
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

UE::RemoteControl::UI::Private::FRCControllerPropertyInfo SRCControllerPanel::GetPropertyInfoForCustomType(const FName& InType)
{
	using namespace UE::RemoteControl::UI::Private;

	FRCControllerPropertyInfo PropertyInfo;
	const EPropertyBagPropertyType CustomControllerType = UE::RCCustomControllers::GetCustomControllerType(InType);

	if (CustomControllerType != EPropertyBagPropertyType::None)
	{
		PropertyInfo.Type = CustomControllerType;
		PropertyInfo.CustomMetaData = UE::RCCustomControllers::GetCustomControllerMetaData(InType.ToString());
	}

	return PropertyInfo;
}

static UObject* GetBaseStructForType(FName StructType)
{
	if (StructType == NAME_Vector)
	{
		return TBaseStructure<FVector>::Get();
	}
	else if (StructType == NAME_Vector2D)
	{
		return TBaseStructure<FVector2d>::Get();
	}
	else if (StructType == NAME_Color)
	{
		return TBaseStructure<FColor>::Get();
	}
	else if (StructType == NAME_Rotator)
	{
		return TBaseStructure<FRotator>::Get();
	}

	ensureMsgf(false, TEXT("Found unsupported struct type %s in config."), *StructType.ToString());

	return nullptr;
}

bool SRCControllerPanel::IsListFocused() const
{
	return ControllerPanelList->IsListFocused();
}

void SRCControllerPanel::DeleteSelectedPanelItems()
{
	if (ControllerPanelList)
	{
		ControllerPanelList->DeleteSelectedPanelItems();
	}
}

bool SRCControllerPanel::CanDuplicateItems() const
{
	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (LogicItem->GetModelType() == FRCControllerModel::ControllerModelName)
		{
			return true;
		}
	}

	return false;
}

void SRCControllerPanel::DuplicateSelectedPanelItems()
{
	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (LogicItem->GetModelType() == FRCControllerModel::ControllerModelName)
		{
			if (const TSharedPtr<FRCControllerModel>& ControllerItem = StaticCastSharedPtr<FRCControllerModel>(LogicItem))
			{
				if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
				{
					DuplicateController(Controller);
				}
			}
		}
	}
}

bool SRCControllerPanel::CanCopyItems() const
{
	for (const TSharedPtr<FRCLogicModeBase>& LogicItem : GetSelectedLogicItems())
	{
		if (LogicItem->GetModelType() == FRCControllerModel::ControllerModelName)
		{
			return true;
		}
	}

	return false;
}

void SRCControllerPanel::CopySelectedPanelItems()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		TArray<UObject*> ItemsToCopy;
		const TArray<TSharedPtr<FRCLogicModeBase>> LogicItems = GetSelectedLogicItems();
		ItemsToCopy.Reserve(LogicItems.Num());

		for (const TSharedPtr<FRCLogicModeBase>& LogicItem : LogicItems)
		{
			if (LogicItem->GetModelType() == FRCControllerModel::ControllerModelName)
			{
				if (const TSharedPtr<FRCControllerModel>& ControllerItem = StaticCastSharedPtr<FRCControllerModel>(LogicItem))
				{
					if (URCController* Controller = Cast<URCController>(ControllerItem->GetVirtualProperty()))
					{
						ItemsToCopy.Add(Controller);
					}
				}
			}
		}

		RemoteControlPanel->SetLogicClipboardItems(ItemsToCopy, SharedThis(this));
	}
}

void SRCControllerPanel::PasteItemsFromClipboard()
{
	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		if (RemoteControlPanel->LogicClipboardItemSource == SharedThis(this))
		{
			for (UObject* LogicClipboardItem : RemoteControlPanel->GetLogicClipboardItems())
			{
				if (URCController* Controller = Cast<URCController>(LogicClipboardItem))
				{
					DuplicateController(Controller);
				}
			}
		}
	}
}

FText SRCControllerPanel::GetPasteItemMenuEntrySuffix()
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		// This function should only have been called if we were the source of the item copied.
		if (ensure(RemoteControlPanel->LogicClipboardItemSource == SharedThis(this)))
		{
			TArray<UObject*> LogicClipboardItems = RemoteControlPanel->GetLogicClipboardItems();

			if (LogicClipboardItems.Num() > 0)
			{
				if (const URCController* Controller = Cast<URCController>(LogicClipboardItems[0]))
				{
					if (LogicClipboardItems.Num() > 1)
					{
						return FText::Format(LOCTEXT("ControllerPanelPasteMenuMultiEntrySuffix", "Controller {0} and {1} other(s)"), FText::FromName(Controller->DisplayName), (LogicClipboardItems.Num() - 1));
					}
					return FText::Format(LOCTEXT("ControllerPanelPasteMenuEntrySuffix", "Controller {0}"), FText::FromName(Controller->DisplayName));
				}
			}
		}
	}

	return FText::GetEmpty();
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCControllerPanel::GetSelectedLogicItems() const
{
	if (ControllerPanelList)
	{
		return ControllerPanelList->GetSelectedLogicItems();
	}

	return {};
}

TArray<TSharedPtr<FRCControllerModel>> SRCControllerPanel::GetSelectedControllers() const
{
	TArray<TSharedPtr<FRCLogicModeBase>> SelectedLogicModes = GetSelectedLogicItems();

	TArray<TSharedPtr<FRCControllerModel>> SelectedControllers;
	SelectedControllers.Reserve(SelectedLogicModes.Num());

	for (const TSharedPtr<FRCLogicModeBase>& LogicMode : SelectedLogicModes)
	{
		if (LogicMode.IsValid() && LogicMode->GetModelType() == FRCControllerModel::ControllerModelName)
		{
			SelectedControllers.Add(StaticCastSharedPtr<FRCControllerModel>(LogicMode));
		}
	}

	return SelectedControllers;
}

int32 SRCControllerPanel::NumControllerItems() const
{
	if (ControllerPanelList.IsValid())
	{
		return ControllerPanelList->NumControllerItems();
	}
	return INDEX_NONE;
}

void SRCControllerPanel::DuplicateController(URCController* InController)
{
	if (!ensure(InController))
	{
		return;
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (URCController* NewController = Cast<URCController>(Preset->DuplicateController(InController)))
		{
			NewController->SetDisplayIndex(Preset->GetNextControllerDisplayIndex());

			ControllerPanelList->RequestRefresh();
		}
	}
}

FReply SRCControllerPanel::RequestDeleteSelectedItem()
{
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = LOCTEXT("DeleteSelectedItemlWarning", "Delete the selected Controller?");

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		DeleteSelectedPanelItems();
	}

	return FReply::Handled();
}

FReply SRCControllerPanel::RequestDeleteAllItems()
{
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = FText::Format(LOCTEXT("DeleteAllWarning", "You are about to delete {0} controllers. Are you sure you want to proceed?"), ControllerPanelList->Num());

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		return OnClickEmptyButton();
	}

	return FReply::Handled();
}

void SRCControllerPanel::OnToggleMultiControllersMode(ECheckBoxState CheckBoxState)
{
	switch (CheckBoxState)
	{
		case ECheckBoxState::Unchecked:
			bIsMultiControllerMode = false;
			break;

		case ECheckBoxState::Checked:
			bIsMultiControllerMode = true;
			break;

		case ECheckBoxState::Undetermined:
			bIsMultiControllerMode = false;
			break;
	}

	if (ControllerPanelList.IsValid())
	{
		ControllerPanelList->SetMultiControllerMode(bIsMultiControllerMode);
	}
}

EVisibility SRCControllerPanel::GetMultiControllerSwitchVisibility() const
{
	if (const URemoteControlPreset* const Preset = GetPreset())
	{
		const int32 NoFieldIdControllersNum = Preset->GetControllersByFieldId(NAME_None).Num();

		// if no controller has a Field Id, let's hide the MultiController Switch
		if (NoFieldIdControllersNum == Preset->GetNumControllers())
		{
			return EVisibility::Collapsed;
			//todo: this could be smarter, e.g. showing the switch only if there are controllers with the same Field Id
		}
	}

	return EVisibility::Visible;
}

TSharedRef<SWidget> SRCControllerPanel::GetControllerMenuContentWidget() const
{
	using namespace UE::RemoteControl::UI::Private;

	constexpr  bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	MenuBuilder.BeginSection("Types", LOCTEXT("Types", "Types"));

	TArray<TPair<EPropertyBagPropertyType, FRCControllerPropertyInfo>> VirtualPropertyFieldClassNames;

	// See Config file: BaseRemoteControl.ini
	const URemoteControlLogicConfig* RCLogicConfig = GetDefault<URemoteControlLogicConfig>();
	for (const EPropertyBagPropertyType ControllerType : RCLogicConfig->SupportedControllerTypes)
	{
		if (ControllerType == EPropertyBagPropertyType::Struct)
		{
			for (const FName& StructType : RCLogicConfig->SupportedControllerStructTypes)
			{
				UObject* ValueTypeObject = GetBaseStructForType(StructType);
				if (ensure(ValueTypeObject))
				{
					FRCControllerPropertyInfo PropertyInfo(ValueTypeObject);
					VirtualPropertyFieldClassNames.Add({ ControllerType, PropertyInfo });
				}
			}
		}
		else if (ControllerType == EPropertyBagPropertyType::Object)
		{
			for (const FName& ObjectType : RCLogicConfig->SupportedControllerObjectClassPaths)
			{			
				FSoftClassPath ClassPath(ObjectType.ToString());
				UObject* ValueTypeObject = ClassPath.TryLoad();
				
				if (ensure(ValueTypeObject))
				{
					FRCControllerPropertyInfo PropertyInfo(ValueTypeObject);
					VirtualPropertyFieldClassNames.Add({ ControllerType, PropertyInfo });
				}	
			}
		}
		else if (ControllerType == EPropertyBagPropertyType::Enum)
		{
			// Handled later
			continue;
		}
		else
		{
			VirtualPropertyFieldClassNames.Add({ ControllerType, nullptr });
		}
	}

	for (const FName& CustomType : RCLogicConfig->SupportedControllerCustomTypes)
	{
		FRCControllerPropertyInfo PropertyInfo = GetPropertyInfoForCustomType(CustomType);
		VirtualPropertyFieldClassNames.Add({ PropertyInfo.Type, PropertyInfo });
	}

	// Generate a menu from the list of supported Controllers
	for (const TPair<EPropertyBagPropertyType, FRCControllerPropertyInfo>& Pair : VirtualPropertyFieldClassNames)
	{
		const EPropertyBagPropertyType Type = Pair.Key;
		const FRCControllerPropertyInfo& PropertyInfo = Pair.Value;
		UObject* ValueTypeObject = PropertyInfo.ValueTypeObject;
		
		// Display Name
		FName DefaultName;

		const FName& CustomNameKey = UE::RCCustomControllers::CustomControllerNameKey;
		
		// Check if this is a custom property with a dedicated name
		if (PropertyInfo.CustomMetaData.Contains(CustomNameKey))
		{
			DefaultName = FName(PropertyInfo.CustomMetaData[CustomNameKey]);
		}
		else
		{
			DefaultName = URCVirtualPropertyBase::GetVirtualPropertyTypeDisplayName(Type, ValueTypeObject);
		}

		// Type Color
		FLinearColor TypeColor = FColor::White;

		// Generate a transient virtual property for deducing type color:
		FInstancedPropertyBag Bag;
		Bag.AddProperty(DefaultName, Type, ValueTypeObject);
		const FPropertyBagPropertyDesc* BagPropertyDesc = Bag.FindPropertyDescByName(DefaultName);
		if (!ensure(BagPropertyDesc))
		{
			continue;
		}

		// Variable Color Bar
		TSharedRef<SWidget> MenuItemWidget = 
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				UE::RCUIHelpers::GetTypeColorWidget(BagPropertyDesc->CachedProperty)
			]
		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(FText::FromName(DefaultName))
			];

		FUIAction Action(FExecuteAction::CreateSP(this, &SRCControllerPanel::OnAddControllerClicked, Type, PropertyInfo));
		MenuBuilder.AddMenuEntry(Action, MenuItemWidget);
	}

	using namespace UE::RCCustomControllers;

	const UGraphEditorSettings* GraphEditorSettings = GetDefault<UGraphEditorSettings>();

	// Event
	{
		const FText ButtonName = LOCTEXT("StatelessEvent", "Event");

		TSharedRef<SWidget> MenuItemWidget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				UE::RCUIHelpers::GetTypeColorWidget(GraphEditorSettings->DelegatePinTypeColor, ButtonName)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(ButtonName)
			];

		FRCControllerPropertyInfo ButtonPropertyInfo;
		ButtonPropertyInfo.Type = EPropertyBagPropertyType::Struct;
		ButtonPropertyInfo.ValueTypeObject = FRCStatelessEventController::StaticStruct();
		ButtonPropertyInfo.CustomMetaData.Add(CustomControllerNameKey, CustomStatelessEventControllerName);

		FUIAction Action(FExecuteAction::CreateSP(this, &SRCControllerPanel::OnAddControllerClicked, ButtonPropertyInfo.Type, ButtonPropertyInfo));
		MenuBuilder.AddMenuEntry(Action, MenuItemWidget);
	}

	// All enums
	{
		const FText EnumsName = LOCTEXT("Enums", "Enums");

		{
			TSharedRef<SWidget> MenuItemWidget =
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Fill)
				[
					UE::RCUIHelpers::GetTypeColorWidget(GraphEditorSettings->BytePinTypeColor, EnumsName)
				]
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(FMargin(6.f, 0.f))
				[
					SNew(STextBlock)
					.Text(EnumsName)
				];

			FNewMenuDelegate Delegate = FNewMenuDelegate::CreateSP(this, &SRCControllerPanel::GetControllerMenuEnumContentWidget);
			MenuBuilder.AddSubMenu(MenuItemWidget, Delegate);
		}
	}

	// String List
	{
		const FText StringListName = LOCTEXT("StringList", "String List");

		TSharedRef<SWidget> MenuItemWidget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				UE::RCUIHelpers::GetTypeColorWidget(GraphEditorSettings->StringPinTypeColor, StringListName)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(StringListName)
			];

		using namespace UE::RCCustomControllers;

		FRCControllerPropertyInfo PropertyInfo;
		PropertyInfo.Type = EPropertyBagPropertyType::String;
		PropertyInfo.CustomMetaData.Add(CustomControllerNameKey, CustomStringListControllerName);
		PropertyInfo.CustomMetaData.Add(StringListControllerEntriesName, "New Entry");

		FUIAction Action(FExecuteAction::CreateSP(this, &SRCControllerPanel::OnAddControllerClicked, PropertyInfo.Type, PropertyInfo));
		MenuBuilder.AddMenuEntry(Action, MenuItemWidget);
	}

	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Other", LOCTEXT("Other", "Other"));

	// Category
	{
		const FText CategoryName = LOCTEXT("Category", "Category");

		TSharedRef<SWidget> MenuItemWidget =
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Fill)
			[
				UE::RCUIHelpers::GetTypeColorWidget(GraphEditorSettings->Int64PinTypeColor, CategoryName)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(FMargin(6.f, 0.f))
			[
				SNew(STextBlock)
				.Text(CategoryName)
			];

		FUIAction Action(FExecuteAction::CreateSP(this, &SRCControllerPanel::OnAddCategoryClicked));
		MenuBuilder.AddMenuEntry(Action, MenuItemWidget);
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRCControllerPanel::GetControllerMenuStructContentWidget(FMenuBuilder& InMenuBuilder) const
{
	// 99% of this is producing the menu items, not this list.
	TArray<UScriptStruct*> Structs;
	Structs.Reserve(1000); // Just start out with a big number

	const URemoteControlLogicConfig* RCLogicConfig = GetDefault<URemoteControlLogicConfig>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (UScriptStruct* Struct : TObjectRange<UScriptStruct>())
	{
		if (Struct->HasAnyFlags(RF_ClassDefaultObject | RF_MirroredGarbage | RF_NewerVersionExists))
		{
			continue;
		}

		// Don't add the ones explicitly added to the base list.
		if (RCLogicConfig->SupportedControllerStructTypes.Contains(Struct->GetFName()))
		{
			continue;
		}

		bool bHasEditableField = false;

		for (FProperty* Property : TFieldRange<FProperty>(Struct, EFieldIterationFlags::IncludeSuper))
		{
			if (Property->HasAllPropertyFlags(CPF_Edit | CPF_BlueprintVisible))
			{
				bHasEditableField = true;
				break;
			}
		}

		if (!bHasEditableField)
		{
			continue;
		}

		Structs.Add(Struct);
	}

	Structs.Sort(
		[](const UScriptStruct& A, const UScriptStruct& B)
		{
			return A.GetName() < B.GetName();
		}
	);

	using namespace UE::RemoteControl::UI::Private;

	TArray<TSharedPtr<FRCControllerPropertyInfo>> Types;
	Types.Reserve(Structs.Num());

	for (UScriptStruct* Struct : Structs)
	{
		TSharedPtr<FRCControllerPropertyInfo>& Type = Types.Add_GetRef(MakeShared<FRCControllerPropertyInfo>(Struct));
		Type->Type = EPropertyBagPropertyType::Struct;
	}

	InMenuBuilder.AddWidget(
		SNew(SRCControllerTypeSelector, Types)
		.OnTypeSelected(FRCControllerTypeBase::FOnTypeSelected::CreateSP(this, &SRCControllerPanel::OnAddControllerSelected)),
		FText::GetEmpty(),
		/* No Indent */ false
	);
}

void SRCControllerPanel::GetControllerMenuEnumContentWidget(FMenuBuilder& InMenuBuilder) const
{
	// 99% of this is producing the menu items, not this list.
	TArray<UEnum*> Enums;
	Enums.Reserve(5000); // Just start out with a big number

	const URemoteControlLogicConfig* RCLogicConfig = GetDefault<URemoteControlLogicConfig>();

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	for (UEnum* Enum : TObjectRange<UEnum>())
	{
		if (Enum->HasAnyFlags(RF_ClassDefaultObject | RF_MirroredGarbage | RF_NewerVersionExists))
		{
			continue;
		}

		Enums.Add(Enum);
	}

	Enums.Sort(
		[](const UEnum& A, const UEnum& B)
		{
			return A.GetName() < B.GetName();
		}
	);

	using namespace UE::RemoteControl::UI::Private;

	TArray<TSharedPtr<FRCControllerPropertyInfo>> Types;
	Types.Reserve(Enums.Num());

	for (UEnum* Enum : Enums)
	{
		TSharedPtr<FRCControllerPropertyInfo>& Type = Types.Add_GetRef(MakeShared<FRCControllerPropertyInfo>(Enum));
		Type->Type = EPropertyBagPropertyType::Enum;
	}

	InMenuBuilder.AddWidget(
		SNew(SRCControllerTypeSelector, Types)
		.OnTypeSelected(FRCControllerTypeBase::FOnTypeSelected::CreateSP(this, &SRCControllerPanel::OnAddControllerSelected)),
		FText::GetEmpty(),
		/* No Indent */ false
	);
}

TSharedRef<SWidget> SRCControllerPanel::GetMultiControllerSwitchWidget()
{
	const FText& SwitchTooltipText = LOCTEXT("MultiControllerModeSwitchTooltip", "Enable/Disable MultiController Mode.\nWhen enabled, the Controllers list will only show one Controller per Field Id.");

	return
		SNew(SCheckBox)
		.Visibility(this, &SRCControllerPanel::GetMultiControllerSwitchVisibility)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Toggle MultiController Mode")))
		.ToolTipText(SwitchTooltipText)
		.HAlign(HAlign_Center)
		.Style(&RCPanelStyle->ToggleButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.IsChecked_Lambda([this]() { return bIsMultiControllerMode ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		.OnCheckStateChanged(this, &SRCControllerPanel::OnToggleMultiControllersMode)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MultiControllersModeSwitchLabel", "Multi"))
			.Justification(ETextJustify::Center)
			.TextStyle(&RCPanelStyle->PanelTextStyle)
		];
}

void SRCControllerPanel::OnAddControllerClicked(const EPropertyBagPropertyType InValueType, const UE::RemoteControl::UI::Private::FRCControllerPropertyInfo InControllerPropertyInfo) const
{
	OnAddControllerClicked(InValueType, InControllerPropertyInfo, URCController::StaticClass());
}

void SRCControllerPanel::OnAddControllerClicked(const EPropertyBagPropertyType InValueType, const UE::RemoteControl::UI::Private::FRCControllerPropertyInfo InControllerPropertyInfo, TSubclassOf<URCController> InControllerClass) const
{
	// Add to the asset
	URemoteControlPreset* Preset = GetPreset();

	if (!Preset)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddController", "Add Controller"));
	Preset->Modify();

	using namespace UE::RemoteControl::UI::Private;

	URCVirtualPropertyInContainer* NewVirtualProperty = Preset->AddController(URCController::StaticClass(), InValueType, InControllerPropertyInfo.ValueTypeObject, NAME_None);

	if (!NewVirtualProperty)
	{
		return;
	}

	if (!Preset->NewControllerCategory.IsEmpty())
	{
		NewVirtualProperty->SetMetadataValue(FRCControllerPropertyInfo::CategoryName, Preset->NewControllerCategory);
	}

	// Copy over the custom metadata of this property
	// Initial description should not be copied
	for (const TPair<FName, FString>& Pair : InControllerPropertyInfo.CustomMetaData)
	{
		if (Pair.Key != FRCControllerPropertyInfo::InitialDescriptionName)
		{
			NewVirtualProperty->SetMetadataValue(Pair.Key, Pair.Value);
		}
	}

	// Additional logic for Custom Controllers naming
	if (UE::RCCustomControllers::IsCustomController(NewVirtualProperty))
	{
		const FName& NewControllerName = UE::RCCustomControllers::GetUniqueNameForController(NewVirtualProperty);
		NewVirtualProperty->DisplayName = NewControllerName;
	}

	if (const FString* InitialDescription = InControllerPropertyInfo.CustomMetaData.Find(FRCControllerPropertyInfo::InitialDescriptionName))
	{
		if (!InitialDescription->IsEmpty())
		{
			NewVirtualProperty->Description = FText::FromString(*InitialDescription);
		}
	}

	NewVirtualProperty->DisplayIndex = Preset->GetNextControllerDisplayIndex();

	// Refresh list
	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel();
	check(RemoteControlPanel);
	RemoteControlPanel->OnControllerAdded.Broadcast(NewVirtualProperty->PropertyName);
}

void SRCControllerPanel::OnAddCategoryClicked() const
{
	// Add to the asset
	URemoteControlPreset* Preset = GetPreset();

	if (!Preset)
	{
		return;
	}

	URCVirtualPropertyContainerBase* ControllerContainer = Preset->GetControllerContainer();

	if (!ControllerContainer)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddCategory", "Add Category"));
	ControllerContainer->Modify();

	FRCVirtualPropertyCategory& NewCategory = ControllerContainer->Categories.AddDefaulted_GetRef();
	NewCategory.Id = FGuid::NewGuid();
	NewCategory.Name = LOCTEXT("NewCategory", "New Category");
	
	NewCategory.DisplayIndex = Preset->GetNextControllerDisplayIndex();

	// Refresh list
	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel();
	check(RemoteControlPanel);
	RemoteControlPanel->OnControllerAdded.Broadcast(*NewCategory.Id.ToString());
}

void SRCControllerPanel::OnAddControllerSelected(TSharedPtr<UE::RemoteControl::UI::Private::FRCControllerPropertyInfo> InControllerPropertyInfo) const
{
	OnAddControllerClicked(InControllerPropertyInfo->Type, *InControllerPropertyInfo);
}

void SRCControllerPanel::EnterRenameMode()
{
	if (ControllerPanelList)
	{
		ControllerPanelList->EnterRenameMode();
	}
}

FReply SRCControllerPanel::OnClickEmptyButton()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		FScopedTransaction Transaction(LOCTEXT("EmptyControllers", "Empty Controllers"));
		Preset->Modify();

		Preset->ResetControllers();

		if (URCVirtualPropertyContainerBase* ControllerContainer = Preset->GetControllerContainer())
		{
			ControllerContainer->Categories.Empty();
		}
	}

	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = GetRemoteControlPanel())
	{
		RemoteControlPanel->OnEmptyControllers.Broadcast();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
