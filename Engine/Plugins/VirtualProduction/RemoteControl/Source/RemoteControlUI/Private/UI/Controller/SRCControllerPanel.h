// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "UI/BaseLogicUI/SRCLogicPanelBase.h"

class FMenuBuilder;
class FRCControllerModel;
class SRCControllerPanel;
class URCController;
enum class ECheckBoxState : uint8;
enum class EPropertyBagPropertyType : uint8;
struct FRCPanelStyle;

namespace UE::RemoteControl::UI::Private
{
/**
 * Helper struct used when generating Controllers.
 * Wraps the TypeObject and the Bag Type.
 *
 * Additionally, some facultative MetaData can be added to create a Custom Controller extending the initial type
 * e.g. External Texture custom controller is a specialized String controller
 * See RCCustomControllerUtilities.h for helper functions related to custom controllers.
 */
struct FRCControllerPropertyInfo
{
	static const FLazyName CategoryName;
	static const FLazyName InitialDescriptionName;
	static const FString CategoryMeta;

	FRCControllerPropertyInfo() {}
	FRCControllerPropertyInfo(UObject* InValueTypeObject) : ValueTypeObject(InValueTypeObject) {}

	/** The type of the controller which will be created starting from this FRCControllerPropertyInfo */
	EPropertyBagPropertyType Type = EPropertyBagPropertyType::Name;

	/** Object defining an Enum, Struct, or Class */
	UObject* ValueTypeObject = nullptr;

	/** MetaData Map which can be used to pass additional settings or information to be used when customizing a Custom Controller */
	TMap<FName, FString> CustomMetaData;
};

}

/*
* ~ SRCControllerPanel ~
*
* UI Widget for Controller Panel.
* Contains a header (Add/Remove/Empty) and List of Controllers
*/
class REMOTECONTROLUI_API SRCControllerPanel : public SRCLogicPanelBase
{
public:
	SLATE_BEGIN_ARGS(SRCControllerPanel)
	{}

		SLATE_ATTRIBUTE(bool, LiveMode)

	SLATE_END_ARGS()

	static UE::RemoteControl::UI::Private::FRCControllerPropertyInfo GetPropertyInfoForCustomType(const FName& InType);

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Called to refresh the controller list */
	void Refresh();

	/** Whether the Controller list widget currently has focus. Used for Delete Item UI command */
	bool IsListFocused() const;

	/** Delete Items UI command implementation for this panel */
	virtual void DeleteSelectedPanelItems() override;

	/** Checks to make sure selected items are duplicatable */
	virtual bool CanDuplicateItems() const override;

	/** "Duplicate Items" UI command implementation for Controller panel*/
	virtual void DuplicateSelectedPanelItems() override;

	/** Checks to make sure selected items are copyable. */
	virtual bool CanCopyItems() const override;

	/** "Copy Items" UI command implementation for Controller panel*/
	virtual void CopySelectedPanelItems() override;

	/** "Paste Items" UI command implementation for Controller panel*/
	virtual void PasteItemsFromClipboard() override;

	/** Provides an item suffix for the Paste context menu to provide users with useful context on the nature of the item being pasted */
	virtual FText GetPasteItemMenuEntrySuffix() override;

	/** Returns the UI items currently selected by the user (if any). To be implemented per child panel*/
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() const override;

	/** Get the currently selected controller models. */
	TArray<TSharedPtr<FRCControllerModel>> GetSelectedControllers() const;

	/** Retrieve the current number of Controllers */
	int32 NumControllerItems() const;

	void EnterRenameMode();

protected:
	/** Warns user before deleting a selected panel item. */
	virtual FReply RequestDeleteSelectedItem() override;

	/** Warns user before deleting all items in a panel. */
	virtual FReply RequestDeleteAllItems() override;

private:
	/** Duplicates a given Controller object*/
	void DuplicateController(URCController* InController);

	/** Builds a menu containing the list of all possible Controllers*/
	TSharedRef<SWidget> GetControllerMenuContentWidget() const;

	/** Builds a menu containing the list of all possible structs on which to base a Controller. */
	void GetControllerMenuStructContentWidget(FMenuBuilder& InMenuBuilder) const;

	/** Builds a menu containing the list of all possible enums on which to base a Controller. */
	void GetControllerMenuEnumContentWidget(FMenuBuilder& InMenuBuilder) const;

	TSharedRef<SWidget> GetMultiControllerSwitchWidget();

	/** Handles click event for Add Controller button */
	void OnAddControllerClicked(const EPropertyBagPropertyType InValueType, const UE::RemoteControl::UI::Private::FRCControllerPropertyInfo InControllerPropertyInfo) const;
	void OnAddControllerClicked(const EPropertyBagPropertyType InValueType, const UE::RemoteControl::UI::Private::FRCControllerPropertyInfo InControllerPropertyInfo, TSubclassOf<URCController> InControllerClass) const;

	/** Handles adding a new category */
	void OnAddCategoryClicked() const;

	/** Handles selection of controller type in submenus. */
	void OnAddControllerSelected(TSharedPtr<UE::RemoteControl::UI::Private::FRCControllerPropertyInfo> InControllerPropertyInfo) const;

	/** Handles checkbox changes event for the MultiController mode selection widget */
	void OnToggleMultiControllersMode(ECheckBoxState CheckBoxState);

	/** Whether to show the MultiController switch or not */
	EVisibility GetMultiControllerSwitchVisibility() const;

	/** Handles click event for Empty button; clears all controllers from the panel*/
	FReply OnClickEmptyButton();

private:

	/** Widget representing List of Controllers */
	TSharedPtr<class SRCControllerPanelList> ControllerPanelList;

	/** Whether the panel is in live mode. */
	TAttribute<bool> bIsInLiveMode;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Is the Panel showing a list with MultiControllers handling (and hiding) controllers with duplicate Field Ids?*/
	bool bIsMultiControllerMode = false;
};
