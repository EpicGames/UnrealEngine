// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCControllerPanelCategoryRow.h"
#include "Controller/RCController.h"
#include "Controller/RCControllerUtilities.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "SRCControllerPanelList.h"
#include "UI/Controller/RCCategoryModel.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/RCUIHelpers.h"
#include "UI/SRCPanelExposedEntity.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SExpanderArrow.h"

#define LOCTEXT_NAMESPACE "SRCControllerPanelCategoryRow"

void SRCControllerPanelCategoryRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
	TSharedRef<SRCControllerPanelList> InControllerPanelList, FRCControllerPanelListItem InCategoryItem)
{
	CategoryItem = InCategoryItem;
	ControllerPanelListWeak = InControllerPanelList.ToWeakPtr();

	SetCursor(EMouseCursor::GrabHand);

	TSharedPtr<FRCCategoryModel> CategoryModel = GetCategory();

	STableRow::Construct(
		STableRow::FArguments()
		.Style(InArgs._Style)
		.Padding(0.f)
		.OnCanAcceptDrop(this, &SRCControllerPanelCategoryRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SRCControllerPanelCategoryRow::OnAcceptDrop)
		.ShowWires(true)
		.Content()
		[
			SNew(SBox)
			.Padding(4.5f)
			[
				CategoryModel.IsValid() 
					? CategoryModel->GetNameWidget()
					: SNullWidget::NullWidget
			]
		],
		InOwnerTableView
	);
}

TSharedPtr<FRCCategoryModel> SRCControllerPanelCategoryRow::GetCategory() const
{
	if (TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin())
	{
		return ControllerPanelList->FindCategoryItemByListItem(CategoryItem);
	}

	return {};
}

TOptional<EItemDropZone> SRCControllerPanelCategoryRow::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem)
{
	TSharedPtr<FRCCategoryModel> CategoryModel = GetCategory();
	if (!CategoryModel.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	// Fetch the Exposed Entities
	URemoteControlPreset* const Preset = ControllerPanelList->GetPreset();
	if (!Preset)
	{
		return TOptional<EItemDropZone>();
	}

	// Dragging Categorys onto Categorys (Reordering). Only allow Above/Below re-ordering (no-onto support)
	if (TSharedPtr<FRCControllerPanelDragDrop> ControllerDragDrop = InDragDropEvent.GetOperationAs<FRCControllerPanelDragDrop>())
	{
		if (!ControllerDragDrop->GetItems().Contains(CategoryItem))
		{
			return InDropZone;
		}
	}

	if (TSharedPtr<FExposedEntityDragDrop> ExposedEntityDragDrop = InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
	{
		return InDropZone;
	}

	return TOptional<EItemDropZone>();
}

FReply SRCControllerPanelCategoryRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem)
{
	TSharedPtr<FRCCategoryModel> CategoryModel = GetCategory();
	if (!CategoryModel.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	ControllerPanelList->bIsAnyControllerItemEligibleForDragDrop = false;

	URemoteControlPreset* Preset = ControllerPanelList->GetPreset();
	if (!Preset)
	{
		return FReply::Unhandled();
	}

	if (TSharedPtr<FRCControllerPanelDragDrop> ControllerDragDrop = InDragDropEvent.GetOperationAs<FRCControllerPanelDragDrop>())
	{
		FScopedTransaction Transaction(LOCTEXT("ReorderControllers", "Reorder Controllers"));

		if (!ControllerPanelList->ReorderItems(ControllerDragDrop->GetItems(), CategoryItem, InDropZone))
		{
			Transaction.Cancel();
		}

		return FReply::Handled();
	}

	if (TSharedPtr<FExposedEntityDragDrop> ExposedEntityDragDrop = InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
	{
		AddControllerFromEntities(*ExposedEntityDragDrop, InDropZone);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SRCControllerPanelCategoryRow::AddControllerFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, EItemDropZone InDropZone)
{
	TSharedPtr<FRCCategoryModel> CategoryModel = GetCategory();
	if (!CategoryModel.IsValid())
	{
		return;
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return;
	}

	URemoteControlPreset* Preset = ControllerPanelList->GetPreset();
	if (!Preset)
	{
		return;
	}

	const FString OldDefault = Preset->NewControllerCategory;

	if (InDropZone == EItemDropZone::OntoItem)
	{
		Preset->NewControllerCategory = CategoryModel->GetId().ToString();
	}

	FScopedTransaction Transaction(LOCTEXT("AutoBindEntities", "Auto bind entities to controllers"));

	const TArray<FGuid>& SelectedFieldIds = InExposedEntityDragDrop.GetSelectedFieldsId();

	TArray<URCController*, TInlineAllocator<1>> CreatedControllers;
	TArray<TSharedRef<const FRemoteControlProperty>, TInlineAllocator<1>> SourcePropertyEntities;

	CreatedControllers.Reserve(SelectedFieldIds.Num());
	SourcePropertyEntities.Reserve(SelectedFieldIds.Num());

	for (const FGuid& ExposedEntityId : SelectedFieldIds)
	{
		if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
		{
			if (URCController* NewController = UE::RCUIHelpers::CreateControllerFromEntity(Preset, RemoteControlProperty))
			{
				CreatedControllers.Emplace(NewController);
				SourcePropertyEntities.Add(RemoteControlProperty.ToSharedRef());
			}
		}
	}

	Preset->NewControllerCategory = OldDefault;

	if (CreatedControllers.IsEmpty())
	{
		Transaction.Cancel();
		return;
	}

	// Refresh panel list so that the controller models are created
	ControllerPanelList->Reset();

	// Reorder the controller items to match the target drop zone
	{
		const FRCControllerPanelListItem NewControllerItem = ControllerPanelList->FindListItemByCategoryItem(CategoryModel.ToSharedRef());
		const TArray<TSharedPtr<FRCControllerModel>> ControllersToMove = ControllerPanelList->FindControllerItemsByObject(CreatedControllers);

		TArray<FRCControllerPanelListItem> ControllerItemsToMove;
		ControllerItemsToMove.Reserve(CreatedControllers.Num());

		for (const TSharedPtr<FRCControllerModel>& ControllerToMove : ControllersToMove)
		{
			if (ControllerToMove.IsValid())
			{
				if (const FRCControllerPanelListItem ControllerModelItem = ControllerPanelList->FindListItemByControllerItem(ControllerToMove.ToSharedRef()))
				{
					ControllerItemsToMove.Add(ControllerModelItem);
				}
			}
		}

		ControllerPanelList->ReorderItems(ControllerItemsToMove, NewControllerItem, InDropZone);
	}

	check(CreatedControllers.Num() == SourcePropertyEntities.Num());

	// Create Bind Behaviour and Bind to the property
	for (int32 Index = 0; Index < CreatedControllers.Num(); ++Index)
	{
		constexpr bool bExecuteBind = true;
		ControllerPanelList->CreateBindBehaviourAndAssignTo(CreatedControllers[Index], SourcePropertyEntities[Index], bExecuteBind);
	}
}

FReply SRCControllerPanelCategoryRow::OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (FSuperRowType::OnDragDetected(InMyGeometry, InMouseEvent).IsEventHandled())
	{
		return FReply::Handled();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedRef<FRCControllerPanelDragDrop> ControllerDragDrop = MakeShared<FRCControllerPanelDragDrop>(ControllerPanelList->GetSelectedItems());
		return FReply::Handled().BeginDragDrop(ControllerDragDrop);
	}
	return FReply::Unhandled();
}

FReply SRCControllerPanelCategoryRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (TSharedPtr<FRCCategoryModel> CategoryModel = GetCategory())
	{
		CategoryModel->EnterNameEditingMode();
		return FReply::Handled();
	}

	return FSuperRowType::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

#undef LOCTEXT_NAMESPACE
