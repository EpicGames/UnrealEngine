// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Optional.h"
#include "RCVirtualPropertyContainer.h"
#include "UI/Controller/RCControllerPanelListItem.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FExposedEntityDragDrop;
class FRCCategoryModel;
class SRCControllerPanelList;
class URemoteControlPreset;

class SRCControllerPanelCategoryRow : public STableRow<FRCControllerPanelListItem>
{
	using FSuperRowType = STableRow<FRCControllerPanelListItem>;

public:
	SLATE_BEGIN_ARGS(SRCControllerPanelCategoryRow) {}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		TSharedRef<SRCControllerPanelList> InControllerPanelList, FRCControllerPanelListItem InCategoryItem);

	TSharedPtr<FRCCategoryModel> GetCategory() const;

private:
	/** Called to validate whether a drag drop event is accepted for a controller item row */
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem);

	/** Called when a drop happens on this controller item row */
	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem);

	void AddControllerFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, EItemDropZone InDropZone);

	//~ Begin SWidget
	virtual FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TWeakPtr<SRCControllerPanelList> ControllerPanelListWeak;
	FRCControllerPanelListItem CategoryItem;
};