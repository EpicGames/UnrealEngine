// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCControllerPanelControllerRow.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Controller/RCController.h"
#include "Controller/RCControllerUtilities.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "RemoteControlPreset.h"
#include "SRCControllerPanelList.h"
#include "ScopedTransaction.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/RCUIHelpers.h"
#include "UI/SRCPanelExposedEntity.h"

#define LOCTEXT_NAMESPACE "SRCControllerPanelControllerRow"

const FLazyName FRCControllerColumns::Wires = TEXT("Wires");
const FLazyName FRCControllerColumns::TypeColor = TEXT("TypeColor");
const FLazyName FRCControllerColumns::ControllerId = TEXT("Controller Id");
const FLazyName FRCControllerColumns::Description = TEXT("Controller Description");
const FLazyName FRCControllerColumns::Value = TEXT("Controller Value");
const FLazyName FRCControllerColumns::FieldId = TEXT("Controller Field Id");
const FLazyName FRCControllerColumns::ValueTypeSelection = TEXT("Value Type Selection");

void SRCControllerPanelControllerRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
	TSharedRef<SRCControllerPanelList> InControllerPanelList, FRCControllerPanelListItem InControllerItem)
{
	ControllerItem = InControllerItem;
	ControllerPanelListWeak = InControllerPanelList.ToWeakPtr();

	SetCursor(EMouseCursor::GrabHand);

	FSuperRowType::Construct(FSuperRowType::FArguments()
		.Style(InArgs._Style)
		.Padding(0.f)
		.OnCanAcceptDrop(this, &SRCControllerPanelControllerRow::CanAcceptDrop)
		.OnAcceptDrop(this, &SRCControllerPanelControllerRow::OnAcceptDrop)
		.ShowWires(true)
		, InOwnerTableView);
}

TSharedRef<SWidget> SRCControllerPanelControllerRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();

	if (!ensure(ControllerModel.IsValid()))
	{
		return SNullWidget::NullWidget;
	}

	if (InColumnName == FRCControllerColumns::Wires)
	{
		return SNew(SExpanderArrow, SharedThis(this))
			.ShouldDrawWires(true);
	}

	TSharedPtr<SWidget> RowWidget;

	if (InColumnName == FRCControllerColumns::TypeColor)
	{
		RowWidget = GetTypeColorWidget();
	}
	else if (InColumnName == FRCControllerColumns::FieldId)
	{
		RowWidget = ControllerModel->GetFieldIdWidget();
	}
	else if (InColumnName == FRCControllerColumns::ValueTypeSelection)
	{
		RowWidget = ControllerModel->GetTypeSelectionWidget();
	}
	else if (InColumnName == FRCControllerColumns::ControllerId)
	{
		RowWidget = ControllerModel->GetNameWidget();
	}
	else if (InColumnName == FRCControllerColumns::Description)
	{
		RowWidget = ControllerModel->GetDescriptionWidget();
	}
	else if (InColumnName == FRCControllerColumns::Value)
	{
		RowWidget = ControllerModel->GetWidget();
	}
	else if (const TSharedPtr<SRCControllerPanelList> ControlPanelList = ControllerPanelListWeak.Pin())
	{
		if (ControlPanelList->GetCustomColumns().Contains(InColumnName))
		{
			RowWidget = ControllerModel->GetControllerExtensionWidget(InColumnName);
		}
	}

	if (!RowWidget.IsValid() || RowWidget == SNullWidget::NullWidget)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SBox)
		.Padding(0.f, 4.5f)
		[
			RowWidget.ToSharedRef()
		];
}

TSharedPtr<FRCControllerModel> SRCControllerPanelControllerRow::GetControllerModel() const
{
	if (TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin())
	{
		return ControllerPanelList->FindControllerItemByListItem(ControllerItem);
	}

	return nullptr;
}

TOptional<EItemDropZone> SRCControllerPanelControllerRow::CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem)
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();
	if (!ControllerModel.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	// Fetch the Exposed Entities
	URemoteControlPreset* const Preset = ControllerPanelList->GetPreset();
	if (!Preset)
	{
		return TOptional<EItemDropZone>();
	}

	// Dragging Controllers onto Controllers (Reordering). Only allow Above/Below re-ordering (no-onto support)
	if (TSharedPtr<FRCControllerPanelDragDrop> ControllerDragDrop = InDragDropEvent.GetOperationAs<FRCControllerPanelDragDrop>())
	{
		if (InDropZone == EItemDropZone::OntoItem)
		{
			return TOptional<EItemDropZone>();
		}
		
		if (!ControllerDragDrop->GetItems().Contains(ControllerItem))
		{
			return InDropZone;
		}

		return TOptional<EItemDropZone>();
	}

	if (TSharedPtr<FExposedEntityDragDrop> EntityDragDrop = InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
	{
		// Support when adding entities onto this controller, ensure that at least one entity can be added via bind action 
		if (InDropZone == EItemDropZone::OntoItem)
		{
			for (const FGuid& ExposedEntityId : EntityDragDrop->GetSelectedFieldsId())
			{
				if (TSharedPtr<const FRemoteControlField> RemoteControlField = Preset->GetExposedEntity<FRemoteControlField>(ExposedEntityId).Pin())
				{
					if (URCController* Controller = Cast<URCController>(ControllerModel->GetVirtualProperty()))
					{
						const bool bAllowNumericInputAsStrings = true;
						if (URCBehaviourBind::CanHaveActionForField(Controller, RemoteControlField.ToSharedRef(), bAllowNumericInputAsStrings))
						{
							ControllerPanelList->bIsAnyControllerItemEligibleForDragDrop = true;
							return InDropZone;
						}
					}
				}
			}
		}
		// When adding entities above/below controllers (i.e. create a new controller above/below this one), at least one has to be supported
		else
		{
			for (const FGuid& ExposedEntityId : EntityDragDrop->GetSelectedFieldsId())
			{
				if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
				{
					if (UE::RCControllers::CanCreateControllerFromEntity(RemoteControlProperty))
					{
						ControllerPanelList->bIsAnyControllerItemEligibleForDragDrop = true;
						return InDropZone;
					}
				}
			}
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SRCControllerPanelControllerRow::OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem)
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();
	if (!ControllerModel.IsValid())
	{
		return FReply::Unhandled();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	ControllerPanelList->bIsAnyControllerItemEligibleForDragDrop = false;

	URemoteControlPreset* Preset = ControllerPanelList->GetPreset();
	if (!Preset)
	{
		return FReply::Unhandled();
	}

	if (TSharedPtr<FRCControllerPanelDragDrop> ControllerDragDrop = InDragDropEvent.GetOperationAs<FRCControllerPanelDragDrop>())
	{
		FScopedTransaction Transaction(LOCTEXT("ReorderControllers", "Reorder Controllers"));

		if (!ControllerPanelList->ReorderItems(ControllerDragDrop->GetItems(), ControllerItem, InDropZone))
		{
			Transaction.Cancel();
		}

		return FReply::Handled();
	}

	if (TSharedPtr<FExposedEntityDragDrop> ExposedEntityDragDrop = InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
	{
		const FDragDropContext DragDropContext
			{
				.Preset = Preset,
				.ControllerPanelList = ControllerPanelList,
				.Item = InItem,
				.DropZone = InDropZone,
			};

		if (InDropZone == EItemDropZone::OntoItem)
		{
			CreateBindBehaviorsFromEntities(*ExposedEntityDragDrop, DragDropContext);
		}
		else
		{
			CreateControllersFromEntities(*ExposedEntityDragDrop, DragDropContext);
		}
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SRCControllerPanelControllerRow::CreateBindBehaviorsFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, const FDragDropContext& InContext)
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();
	if (!ControllerModel.IsValid())
	{
		return;
	}
	
	const TArray<FGuid>& DroppedFieldIds = InExposedEntityDragDrop.GetSelectedFieldsId();

	if (!ControllerModel.IsValid() || DroppedFieldIds.IsEmpty())
	{
		return;
	}

	URCController* const Controller = Cast<URCController>(ControllerModel->GetVirtualProperty());
	if (!Controller)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("BindPropertiesToController", "Bind properties to Controller"));

	bool bModified = false;

	// Update controller description if empty to match the first dragged property
	if (Controller->Description.IsEmpty())
	{
		if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = InContext.Preset->GetExposedEntity<FRemoteControlProperty>(DroppedFieldIds[0]).Pin())
		{
			Controller->Modify();
			Controller->Description = UE::RCUIHelpers::GenerateControllerDescriptionFromEntity(RemoteControlProperty);
			bModified = true;
		}
	}

	for (const FGuid& ExposedEntityId : DroppedFieldIds)
	{
		if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = InContext.Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
		{
			InContext.ControllerPanelList->CreateBindBehaviourAndAssignTo(Controller, RemoteControlProperty.ToSharedRef(), true);
			bModified = true;
		}
	}

	if (!bModified)
	{
		Transaction.Cancel();
	}
}

void SRCControllerPanelControllerRow::CreateControllersFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, const FDragDropContext& InContext)
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();
	if (!ControllerModel.IsValid())
	{
		return;
	}
	
	const FString OldDefault = InContext.Preset->NewControllerCategory;

	if (URCVirtualPropertyBase* Controller = ControllerModel->GetVirtualProperty())
	{
		using namespace UE::RemoteControl::UI::Private;
		InContext.Preset->NewControllerCategory = Controller->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);
	}

	FScopedTransaction Transaction(LOCTEXT("AutoBindEntities", "Auto bind entities to controllers"));

	const TArray<FGuid>& SelectedFieldIds = InExposedEntityDragDrop.GetSelectedFieldsId();

	TArray<URCController*, TInlineAllocator<1>> CreatedControllers;
	TArray<TSharedRef<const FRemoteControlProperty>, TInlineAllocator<1>> SourcePropertyEntities;

	CreatedControllers.Reserve(SelectedFieldIds.Num());
	SourcePropertyEntities.Reserve(SelectedFieldIds.Num());

	for (const FGuid& ExposedEntityId : SelectedFieldIds)
	{
		if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = InContext.Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
		{
			if (URCController* NewController = UE::RCUIHelpers::CreateControllerFromEntity(InContext.Preset, RemoteControlProperty))
			{
				CreatedControllers.Emplace(NewController);
				SourcePropertyEntities.Add(RemoteControlProperty.ToSharedRef());
			}
		}
	}

	InContext.Preset->NewControllerCategory = OldDefault;

	if (CreatedControllers.IsEmpty())
	{
		Transaction.Cancel();
		return;
	}

	// Refresh panel list so that the controller models are created
	InContext.ControllerPanelList->Reset();

	// Reorder the controller items to match the target drop zone
	{
		const FRCControllerPanelListItem NewControllerItem = InContext.ControllerPanelList->FindListItemByControllerItem(ControllerModel.ToSharedRef());
		const TArray<TSharedPtr<FRCControllerModel>> ControllersToMove = InContext.ControllerPanelList->FindControllerItemsByObject(CreatedControllers);

		TArray<FRCControllerPanelListItem> ControllerItemsToMove;
		ControllerItemsToMove.Reserve(CreatedControllers.Num());

		for (const TSharedPtr<FRCControllerModel>& ControllerToMove : ControllersToMove)
		{
			if (ControllerToMove.IsValid())
			{
				if (const FRCControllerPanelListItem ControllerModelItem = InContext.ControllerPanelList->FindListItemByControllerItem(ControllerToMove.ToSharedRef()))
				{
					ControllerItemsToMove.Add(ControllerModelItem);
				}
			}
		}

		InContext.ControllerPanelList->ReorderItems(ControllerItemsToMove, NewControllerItem, InContext.DropZone);
	}

	check(CreatedControllers.Num() == SourcePropertyEntities.Num());

	// Create Bind Behaviour and Bind to the property
	for (int32 Index = 0; Index < CreatedControllers.Num(); ++Index)
	{
		constexpr bool bExecuteBind = true;
		InContext.ControllerPanelList->CreateBindBehaviourAndAssignTo(CreatedControllers[Index], SourcePropertyEntities[Index], bExecuteBind);
	}
}

TSharedRef<SWidget> SRCControllerPanelControllerRow::GetTypeColorWidget()
{
	TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel();

	if (!ControllerModel.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	URCController* Controller = Cast<URCController>(ControllerModel->GetVirtualProperty());
	
	if (!Controller)
	{
		return SNullWidget::NullWidget;
	}

	using namespace UE::RCCustomControllers;
	using namespace UE::RemoteControl::UI::Private;

	if (const FProperty* Property = Controller->GetProperty())
	{
		return UE::RCUIHelpers::GetTypeColorWidget(Property);
	}

	return SNullWidget::NullWidget;
}

FReply SRCControllerPanelControllerRow::OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (FSuperRowType::OnDragDetected(InMyGeometry, InMouseEvent).IsEventHandled())
	{
		return FReply::Handled();
	}

	const TSharedPtr<SRCControllerPanelList> ControllerPanelList = ControllerPanelListWeak.Pin();
	if (!ControllerPanelList.IsValid())
	{
		return FReply::Unhandled();
	}

	if (InMouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		TSharedRef<FRCControllerPanelDragDrop> ControllerDragDrop = MakeShared<FRCControllerPanelDragDrop>(ControllerPanelList->GetSelectedItems());
		return FReply::Handled().BeginDragDrop(ControllerDragDrop);
	}
	return FReply::Unhandled();
}

FReply SRCControllerPanelControllerRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (TSharedPtr<FRCControllerModel> ControllerModel = GetControllerModel())
	{
		ControllerModel->EnterDescriptionEditingMode();
		return FReply::Handled();
	}
	return FSuperRowType::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

#undef LOCTEXT_NAMESPACE
