// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Controller/RCControllerPanelListItem.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FExposedEntityDragDrop;
class FRCControllerModel;
class SRCControllerPanelList;
class URemoteControlPreset;

struct FRCControllerColumns
{
	static const FLazyName Wires;
	static const FLazyName TypeColor;
	static const FLazyName ControllerId;
	static const FLazyName Description;
	static const FLazyName Value;
	static const FLazyName FieldId;
	static const FLazyName ValueTypeSelection;
};

class SRCControllerPanelControllerRow : public SMultiColumnTableRow<FRCControllerPanelListItem>
{
public:
	SLATE_BEGIN_ARGS(SRCControllerPanelControllerRow) {}
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView,
		TSharedRef<SRCControllerPanelList> InControllerPanelList, FRCControllerPanelListItem InControllerItem);

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;

	TSharedPtr<FRCControllerModel> GetControllerModel() const;

private:
	/** Called to validate whether a drag drop event is accepted for a controller item row */
	TOptional<EItemDropZone> CanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem);

	/** Called when a drop happens on this controller item row */
	FReply OnAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, FRCControllerPanelListItem InItem);

	struct FDragDropContext
	{
		URemoteControlPreset* Preset;
		const TSharedPtr<SRCControllerPanelList>& ControllerPanelList;
		const FRCControllerPanelListItem& Item;
		EItemDropZone DropZone;
	};

	/** Creates a bind behavior on the existing controller items for all the given entities found in the drag drop */
	void CreateBindBehaviorsFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, const FDragDropContext& InContext);

	/** Creates new controllers next to this item row (above/below depending on drop zone) and creates a bind behavior for each new controller for all the given entities found in the drag drop */
	void CreateControllersFromEntities(const FExposedEntityDragDrop& InExposedEntityDragDrop, const FDragDropContext& InContext);

	TSharedRef<SWidget> GetTypeColorWidget();

	//~ Begin SWidget
	virtual FReply OnDragDetected(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	//~ End SWidget

	TWeakPtr<SRCControllerPanelList> ControllerPanelListWeak;
	FRCControllerPanelListItem ControllerItem;
};