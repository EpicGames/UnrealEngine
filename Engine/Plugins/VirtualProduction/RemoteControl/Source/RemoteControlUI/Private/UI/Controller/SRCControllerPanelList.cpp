// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCControllerPanelList.h"

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBindNode.h"
#include "Commands/RemoteControlCommands.h"
#include "Controller/RCController.h"
#include "Controller/RCControllerContainer.h"
#include "Controller/RCControllerUtilities.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "IRemoteControlModule.h"
#include "Materials/MaterialInterface.h"
#include "RCCategoryModel.h"
#include "RCControllerModel.h"
#include "RCMultiController.h"
#include "RCVirtualProperty.h"
#include "RCVirtualPropertyContainer.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "SDropTarget.h"
#include "SlateOptMacros.h"
#include "SRCControllerPanelCategoryRow.h"
#include "SRCControllerPanelControllerRow.h"
#include "SRCControllerPanel.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Action/SRCActionPanelList.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/RCUIHelpers.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelDragHandle.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRCPanelExposedEntity.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SRCControllerPanelList"

void SRCControllerPanelList::Construct(const FArguments& InArgs, const TSharedRef<SRCControllerPanel> InControllerPanel, const TSharedRef<SRemoteControlPanel> InRemoteControlPanel)
{
	SRCLogicPanelListBase::Construct(SRCLogicPanelListBase::FArguments(), InControllerPanel, InRemoteControlPanel);
	
	ControllerPanelWeakPtr = InControllerPanel;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.LogicControllersPanel");

	TreeView = SNew(STreeView<FRCControllerPanelListItem>)
		.TreeItemsSource(&RootItems)
		.OnGetChildren(this, &SRCControllerPanelList::OnGetChildren)
		.OnSelectionChanged(this, &SRCControllerPanelList::OnTreeSelectionChanged)
		.OnGenerateRow(this, &SRCControllerPanelList::OnGenerateWidgetForTree)
		.SelectionMode(ESelectionMode::Multi)
		.OnContextMenuOpening(this, &SRCLogicPanelListBase::GetContextMenuWidget)
		.HeaderRow(
			SAssignNew(ControllersHeaderRow, SHeaderRow)
			.Style(&RCPanelStyle->HeaderRowStyle)

			+ SHeaderRow::Column(FRCControllerColumns::Wires)
			.DefaultLabel(FText())
			.FixedWidth(20)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SHeaderRow::Column(FRCControllerColumns::TypeColor)
			.DefaultLabel(FText())
			.FixedWidth(20)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SHeaderRow::Column(FRCControllerColumns::ControllerId)
			.DefaultLabel(LOCTEXT("ControllerIdColumnName", "Controller Id"))
			.FillWidth(0.2f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SHeaderRow::Column(FRCControllerColumns::Description)
			.DefaultLabel(LOCTEXT("ControllerNameColumnDescription", "Description"))
			.FillWidth(0.35f)

			+ SHeaderRow::Column(FRCControllerColumns::Value)
			.DefaultLabel(LOCTEXT("ControllerValueColumnName", "Input"))
			.FillWidth(0.45f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)
		);

	ChildSlot
	[
		SNew(SDropTarget)
		.VerticalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.VerticalDash"))
		.HorizontalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.HorizontalDash"))
		.OnDropped_Lambda([this](const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) { return SRCControllerPanelList::OnControllerTreeViewDragDrop(InDragDropEvent.GetOperation()); })
		.OnAllowDrop(this, &SRCControllerPanelList::OnAllowDrop)
		.OnIsRecognized(this, &SRCControllerPanelList::OnAllowDrop)
		[
			TreeView.ToSharedRef()
		]
	];

	// Add delegates
	if (const URemoteControlPreset* Preset = ControllerPanelWeakPtr.Pin()->GetPreset())
	{
		// Refresh list
		const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanelWeakPtr.Pin()->GetRemoteControlPanel();
		check(RemoteControlPanel)
		RemoteControlPanel->OnControllerAdded.AddSP(this, &SRCControllerPanelList::OnControllerAdded);
		RemoteControlPanel->OnEmptyControllers.AddSP(this, &SRCControllerPanelList::OnEmptyControllers);

		Preset->OnVirtualPropertyContainerModified().AddSP(this, &SRCControllerPanelList::OnControllerContainerModified);
	}

	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties = true;
	Args.NotifyHook = this;
	PropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);

	Reset();

	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->NewControllerCategory = FString();
	}
}

bool SRCControllerPanelList::IsEmpty() const
{
	return ControllerItems.IsEmpty();
}

int32 SRCControllerPanelList::Num() const
{
	return NumControllerItems();
}

int32 SRCControllerPanelList::NumSelectedLogicItems() const
{
	return TreeView->GetNumItemsSelected();
}

void SRCControllerPanelList::Reset()
{
	for (const TSharedPtr<FRCControllerModel>& ControllerModel : ControllerItems)
	{
		if (ControllerModel)
		{
			ControllerModel->OnValueTypeChanged.RemoveAll(this);
		}
	}

	// Cache Controller Selection
	const TArray<FRCControllerPanelListItem> SelectedItem = TreeView->GetSelectedItems();

	ControllerItems.Empty();

	check(ControllerPanelWeakPtr.IsValid());

	TSharedPtr<SRCControllerPanel> ControllerPanel = ControllerPanelWeakPtr.Pin();
	TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanel->GetRemoteControlPanel();

	URemoteControlPreset* const Preset = ControllerPanel->GetPreset();
	if (!Preset)
	{
		PropertyRowGenerator->SetStructure(TSharedPtr<FStructOnScope>());
		return;
	}

	PropertyRowGenerator->SetStructure(Preset->GetControllerContainerStructOnScope());
	if (!PropertyRowGenerator->OnFinishedChangingProperties().IsBoundToObject(this))
	{
		PropertyRowGenerator->OnFinishedChangingProperties().AddSP(this, &SRCControllerPanelList::OnFinishedChangingProperties);
	}

	// Generator should be moved to separate class
	TArray<TSharedRef<IDetailTreeNode>> RootTreeNodes = PropertyRowGenerator->GetRootTreeNodes();

	MultiControllers.ResetMultiControllers();

	bool bShowFieldIdsColumn = false;
	
	for (const TSharedRef<IDetailTreeNode>& CategoryNode : RootTreeNodes)
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		CategoryNode->GetChildren(Children);

		ControllerItems.Reserve(Children.Num());

		for (TSharedRef<IDetailTreeNode>& Child : Children)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = Child->CreatePropertyHandle();

			if (!ChildHandle.IsValid())
			{
				continue;
			}

			FProperty* Property = ChildHandle->GetProperty();
			check(Property);

			if (URCVirtualPropertyBase* Controller = Preset->GetController(Property->GetFName()))
			{
				bool bIsVisible = true;
				bool bIsMultiController = false;

				const FName& FieldId = Controller->FieldId;

				if (FieldId != NAME_None)
				{
					// there's at least one Field Id set, let's show their column
					bShowFieldIdsColumn = true;
				}

				// MultiController Mode: only showing one Controller per Field Id
				if (bIsInMultiControllerMode && Preset->GetControllersByFieldId(FieldId).Num() > 1)
				{
					bIsMultiController = MultiControllers.TryToAddAsMultiController(Controller);
					bIsVisible = bIsMultiController;
				}

				if (bIsVisible)
				{
					const TSharedRef<FRCControllerModel> ControllerModel = MakeShared<FRCControllerModel>(Controller, Child, RemoteControlPanel);
					ControllerModel->Initialize();
					ControllerModel->OnValueChanged.AddSP(this, &SRCControllerPanelList::OnControllerValueChanged, bIsMultiController);
					if (bIsMultiController)
					{
						ControllerModel->SetMultiController(bIsMultiController);
						ControllerModel->OnValueTypeChanged.AddSP(this, &SRCControllerPanelList::OnControllerValueTypeChanged);
					}
					ControllerItems.Add(ControllerModel);
				}
			}
		}
	}

	// sort by Field Id
	if (bIsInMultiControllerMode)
	{
		Algo::Sort(ControllerItems, [](const TSharedPtr<FRCControllerModel>& A, const TSharedPtr<FRCControllerModel>& B)
			{
				if (A.IsValid() && B.IsValid())
				{
					const URCVirtualPropertyBase* ControllerA = A->GetVirtualProperty();
					const URCVirtualPropertyBase* ControllerB = B->GetVirtualProperty();

					if (ControllerA && ControllerB)
					{
						return ControllerA->FieldId.FastLess(ControllerB->FieldId);
					}
				}

				return false;
			});
	}
	else
	{
		Algo::Sort(ControllerItems, [](const TSharedPtr<FRCControllerModel>& A, const TSharedPtr<FRCControllerModel>& B)
			{
				if (A.IsValid() && B.IsValid())
				{
					const URCVirtualPropertyBase* ControllerA = A->GetVirtualProperty();
					const URCVirtualPropertyBase* ControllerB = B->GetVirtualProperty();

					if (ControllerA && ControllerB)
					{
						return ControllerA->DisplayIndex < ControllerB->DisplayIndex;
					}
				}

				return B.IsValid();
			});
	}

	using namespace UE::RemoteControl::UI::Private;

	TSet<FString> ValidCategories;

	CategoryItems.Empty();

	if (URCVirtualPropertyContainerBase* ControllerContainer = Preset->GetControllerContainer())
	{
		CategoryItems.Reserve(ControllerContainer->Categories.Num());

		for (const FRCVirtualPropertyCategory& Category : ControllerContainer->Categories)
		{
			CategoryItems.Add(MakeShared<FRCCategoryModel>(Category.Id, RemoteControlPanel));
			CategoryItems.Last()->Initialize();
			
			ValidCategories.Add(Category.Id.ToString());
		}

		Algo::Sort(
			CategoryItems, 
			[](const TSharedPtr<FRCCategoryModel>& A, const TSharedPtr<FRCCategoryModel>& B)
			{
				return A->GetCategory()->DisplayIndex < B->GetCategory()->DisplayIndex;
			});
	}

	ShowFieldIdHeaderColumn(bShowFieldIdsColumn);
	ShowValueTypeHeaderColumn(bIsInMultiControllerMode);

	// Handle custom additional columns
	CustomColumns.Empty();
	IRemoteControlUIModule::Get().OnAddControllerExtensionColumn().Broadcast(CustomColumns);
	for (const FName& ColumnName : CustomColumns)
	{
		if (ControllersHeaderRow.IsValid())
		{
			const bool bColumnIsGenerated = ControllersHeaderRow->IsColumnGenerated(ColumnName);
			if (!bColumnIsGenerated)
			{
				ControllersHeaderRow->AddColumn(
					SHeaderRow::FColumn::FArguments()
					.ColumnId(ColumnName)
					.DefaultLabel(FText::FromName(ColumnName))
					.FillWidth(0.2f)
					.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)
				);
			}
		}
	}

	for (const TSharedPtr<FRCControllerModel>& ControllerModel : ControllerItems)
	{
		if (ControllerModel.IsValid())
		{
			if (URCVirtualPropertyBase* VirtualProperty = ControllerModel->GetVirtualProperty())
			{
				const FString ParentCategoryName = VirtualProperty->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);

				if (!ParentCategoryName.IsEmpty() && !ValidCategories.Contains(ParentCategoryName))
				{
					VirtualProperty->RemoveMetadataValue(FRCControllerPropertyInfo::CategoryName);
				}
			}
		}
	}

	UpdateRootItems();

	TreeView->RebuildList();

	// Restore Controller Selection
	for (const FRCControllerPanelListItem& Item : SelectedItem)
	{
		if (Item.Type == ERCControllerPanelListItemType::Controller && ControllerItems.IsValidIndex(Item.Index))
		{
			const TSharedPtr<FRCControllerModel> ControllerModel = ControllerItems[Item.Index];

			if (ControllerModel.IsValid())
			{
				const FName SelectedControllerName = ControllerModel->GetPropertyName();

				const TSharedPtr<FRCControllerModel>* SelectedController = ControllerItems.FindByPredicate([&SelectedControllerName]
				(const TSharedPtr<FRCControllerModel>& InControllerModel)
					{
						// Internal PropertyName is unique, so we use that
						return InControllerModel.IsValid() && SelectedControllerName == InControllerModel->GetPropertyName();
					});

				if (SelectedController)
				{
					TreeView->SetItemSelection(Item, true);
				}
			}
		}
	}

	UpdateEntityUsage();
}

void SRCControllerPanelList::UpdateRootItems()
{
	using namespace UE::RemoteControl::UI::Private;

	RootItems.Empty(CategoryItems.Num() + ControllerItems.Num());

	for (int32 Index = 0; Index < CategoryItems.Num(); ++Index)
	{
		RootItems.Emplace(
			ERCControllerPanelListItemType::Category,
			Index
		);
	}

	for (int32 Index = 0; Index < ControllerItems.Num(); ++Index)
	{
		if (ControllerItems[Index].IsValid())
		{
			if (URCVirtualPropertyBase* VirtualProperty = ControllerItems[Index]->GetVirtualProperty())
			{
				if (VirtualProperty->GetMetadataValue(FRCControllerPropertyInfo::CategoryName).IsEmpty())
				{
					RootItems.Emplace(
						ERCControllerPanelListItemType::Controller,
						Index
					);
				}
			}
		}
	}

	auto GetDisplayIndex = [this](const FRCControllerPanelListItem& InItem) -> int32
		{
			switch (InItem.Type)
			{
				default:
					return INDEX_NONE;

				case ERCControllerPanelListItemType::Category:
					if (CategoryItems.IsValidIndex(InItem.Index) && CategoryItems[InItem.Index].IsValid())
					{
						if (TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[InItem.Index]->GetCategory())
						{
							return Category->DisplayIndex;
						}
					}

					return INDEX_NONE;

				case ERCControllerPanelListItemType::Controller:
					if (ControllerItems.IsValidIndex(InItem.Index) && ControllerItems[InItem.Index].IsValid())
					{
						if (URCVirtualPropertyBase* Controller = ControllerItems[InItem.Index]->GetVirtualProperty())
						{
							return Controller->DisplayIndex;
						}					
					}

					return INDEX_NONE;
			}
		};

	Algo::Sort(
		RootItems,
		[&GetDisplayIndex](const FRCControllerPanelListItem& A, const FRCControllerPanelListItem& B)
		{
			return GetDisplayIndex(A) < GetDisplayIndex(B);
	});
}

void SRCControllerPanelList::OnItemRemoved(const FRCControllerPanelListItem& InRemovedItem)
{
	// Remove category and put all items into the base list
	if (InRemovedItem.Type != ERCControllerPanelListItemType::Category)
	{
		return;
	}

	if (!CategoryItems.IsValidIndex(InRemovedItem.Index) || !CategoryItems[InRemovedItem.Index].IsValid())
	{
		return;
	}

	TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[InRemovedItem.Index]->GetCategory();

	if (!Category.IsSet())
	{
		return;
	}

	const FString CategoryPropertyName = Category->Id.ToString();

	// Shift all display indices higher than the deleted index down by 1
	for (int32 ControllerIndex = 0; ControllerIndex < ControllerItems.Num(); ControllerIndex++)
	{
		if (ensure(ControllerItems[ControllerIndex]))
		{
			if (URCVirtualPropertyBase* Controller = ControllerItems[ControllerIndex]->GetVirtualProperty())
			{
				using namespace UE::RemoteControl::UI::Private;

				if (Controller->GetMetadataValue(FRCControllerPropertyInfo::CategoryName) == CategoryPropertyName)
				{
					if (GUndo)
					{
						Controller->Modify();
					}

					Controller->RemoveMetadataValue(FRCControllerPropertyInfo::CategoryName);
				}
			}
		}
	}

	UpdateRootItems();
}

TSharedRef<ITableRow> SRCControllerPanelList::OnGenerateWidgetForTree(FRCControllerPanelListItem InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	switch (InItem.Type)
	{
		default:
			break;

		case ERCControllerPanelListItemType::Category:
			if (CategoryItems.IsValidIndex(InItem.Index) && CategoryItems[InItem.Index].IsValid())
			{
				return SNew(SRCControllerPanelCategoryRow, OwnerTable, SharedThis(this), InItem)
					.Style(&RCPanelStyle->TableRowStyle);
			}
			break;

		case ERCControllerPanelListItemType::Controller:
			if (ControllerItems.IsValidIndex(InItem.Index) && ControllerItems[InItem.Index].IsValid())
			{
				return SNew(SRCControllerPanelControllerRow, OwnerTable, SharedThis(this), InItem)
					.Style(&RCPanelStyle->TableRowStyle);
			}
			break;
	}

	return SNew(STableRow<FRCControllerPanelListItem>, TreeView.ToSharedRef());
}

bool SRCControllerPanelList::IsItemCategoryExpanded(FRCControllerPanelListItem InItem) const
{
	using namespace UE::RemoteControl::UI::Private;

	if (InItem.Type != ERCControllerPanelListItemType::Category)
	{
		return true;
	}

	if (TSharedPtr<ITableRow> Row = TreeView->WidgetFromItem(InItem))
	{
		return !Row->DoesItemHaveChildren() || Row->IsItemExpanded();
	}

	return false;
}

void SRCControllerPanelList::OnGetChildren(FRCControllerPanelListItem InItem, TArray<FRCControllerPanelListItem>& OutChildren)
{
	using namespace UE::RemoteControl::UI::Private;

	if (InItem.Type != ERCControllerPanelListItemType::Category)
	{
		return;
	}

	if (!CategoryItems.IsValidIndex(InItem.Index) || !CategoryItems[InItem.Index].IsValid())
	{
		return;
	}

	TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[InItem.Index]->GetCategory();

	if (!Category.IsSet())
	{
		return;
	}

	const FString CategoryPropertyName = Category->Id.ToString();

	for (int32 Index = 0; Index < ControllerItems.Num(); ++Index)
	{
		if (ControllerItems[Index].IsValid())
		{
			if (URCVirtualPropertyBase* VirtualProperty = ControllerItems[Index]->GetVirtualProperty())
			{
				if (VirtualProperty->GetMetadataValue(FRCControllerPropertyInfo::CategoryName) == CategoryPropertyName)
				{
					OutChildren.Emplace(
						ERCControllerPanelListItemType::Controller,
						Index
					);
				}
			}
		}
	}
}

void SRCControllerPanelList::OnTreeSelectionChanged(FRCControllerPanelListItem InItem, ESelectInfo::Type InSelectInfo)
{
	TSharedPtr<SRCControllerPanel> ControllerPanel = ControllerPanelWeakPtr.Pin();

	if (!ControllerPanel.IsValid())
	{
		return;
	}

	const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanel->GetRemoteControlPanel();

	if (!RemoteControlPanel.IsValid())
	{
		return;
	}

	TSharedPtr<FRCControllerModel> ControllerModel;

	if (InItem.Type == ERCControllerPanelListItemType::Controller
		&& ControllerItems.IsValidIndex(InItem.Index)
		&& ControllerItems[InItem.Index].IsValid())
	{
		ControllerModel = ControllerItems[InItem.Index].ToSharedRef();
	}

	// Tree selection gets updated and notifies with the first element in the selection set.
	// Controller Behavior / Actions only support viewing one behavior at a time
	if (ControllerModel != SelectedControllerItemWeakPtr)
	{
		SelectedControllerItemWeakPtr = ControllerModel;
		RemoteControlPanel->OnControllerSelectionChanged.Broadcast(ControllerModel, InSelectInfo);
		RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(ControllerModel ? ControllerModel->GetSelectedBehaviourModel() : nullptr);
	}

	URemoteControlPreset* Preset = RemoteControlPanel->GetPreset();

	if (!Preset)
	{
		return;
	}

	TArray<TSharedPtr<FRCLogicModeBase>> SelectedItems = GetSelectedLogicItems();
	FString NewCategory = FString();

	for (const TSharedPtr<FRCLogicModeBase>& SelectedItem : SelectedItems)
	{
		if (!SelectedItem.IsValid())
		{
			continue;
		}
		
		if (SelectedItem->GetModelType() == FRCCategoryModel::CategoryModelName)
		{
			if (TOptional<FRCVirtualPropertyCategory> Category = StaticCastSharedPtr<FRCCategoryModel>(SelectedItem)->GetCategory())
			{
				// We have encountered our first category, set it.
				if (NewCategory.IsEmpty())
				{
					NewCategory = Category->Id.ToString();
				}
				// We have more than 1 category selected, reset value.
				else
				{
					NewCategory = FString();
					break;
				}
			}
		}
		else if (SelectedItem->GetModelType() == FRCControllerModel::ControllerModelName)
		{
			if (URCVirtualPropertyBase* Controller = StaticCastSharedPtr<FRCControllerModel>(SelectedItem)->GetVirtualProperty())
			{
				using namespace UE::RemoteControl::UI::Private;
				const FString ControllerCategory = Controller->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);

				// Root item selected, no default category.
				if (ControllerCategory.IsEmpty())
				{
					NewCategory = FString();
					break;
				}
				// A controller in a category sets the new default.
				else if (NewCategory.IsEmpty())
				{
					NewCategory = ControllerCategory;
				}
				// If it's the same, we do nothing. However a multiple selected categories means no default.
				else if (NewCategory != ControllerCategory)
				{
					NewCategory = FString();
					break;
				}
			}
		}
	}

	Preset->NewControllerCategory = NewCategory;
}

void SRCControllerPanelList::SelectController(URCController* InController)
{
	for (int32 Index = 0; Index < ControllerItems.Num(); ++Index)
	{
		if (!ensure(ControllerItems[Index].IsValid()))
		{
			continue;
		}

		if (ControllerItems[Index]->GetVirtualProperty() == InController)
		{
			TreeView->SetSelection({
				ERCControllerPanelListItemType::Controller,
				Index
			});
		}
	}
}

void SRCControllerPanelList::OnControllerAdded(const FName& InNewPropertyName)
{
	Reset();
}

void SRCControllerPanelList::OnNotifyPreChangeProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->OnNotifyPreChangeVirtualProperty(PropertyChangedEvent);
	}
}

void SRCControllerPanelList::OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->OnModifyController(PropertyChangedEvent);
	}
}

void SRCControllerPanelList::OnControllerValueTypeChanged(URCVirtualPropertyBase* InController, EPropertyBagPropertyType InValueType)
{	
	if (InController)
	{		
		MultiControllers.UpdateFieldIdValueType(InController->FieldId, InValueType);
		Reset();

		// todo: do we also want to refresh controllers values after type change?
	}
}

void SRCControllerPanelList::OnControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel, bool bInIsMultiController)
{
	if (bInIsMultiController)
	{
		if (const URCVirtualPropertyBase* Controller = InControllerModel->GetVirtualProperty())
		{
			FRCMultiController MultiController = MultiControllers.GetMultiController(Controller->FieldId);

			if (MultiController.IsValid())
			{
				MultiController.UpdateHandledControllersValue();
			}
		}
	}

	if (const TSharedPtr<SRemoteControlPanel>& RemoteControlPanel = GetRemoteControlPanel())
	{
		RemoteControlPanel->OnControllerValueChangedDelegate.Broadcast(InControllerModel);
	}
}


void SRCControllerPanelList::OnEmptyControllers()
{
	if (TSharedPtr< SRCControllerPanel> ControllerPanel = ControllerPanelWeakPtr.Pin())
	{
		if (TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanel->GetRemoteControlPanel())
		{
			RemoteControlPanel->OnControllerSelectionChanged.Broadcast(nullptr, ESelectInfo::Direct);
			RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(nullptr);
		}

		Reset();
	}
}

void SRCControllerPanelList::OnControllerContainerModified()
{
	Reset();
}

void SRCControllerPanelList::BroadcastOnItemRemoved()
{
	if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanelWeakPtr.Pin()->GetRemoteControlPanel())
	{
		RemoteControlPanel->OnControllerSelectionChanged.Broadcast(nullptr, ESelectInfo::Direct);
		RemoteControlPanel->OnBehaviourSelectionChanged.Broadcast(nullptr);
	}
}

URemoteControlPreset* SRCControllerPanelList::GetPreset()
{
	if (ControllerPanelWeakPtr.IsValid())
	{
		return ControllerPanelWeakPtr.Pin()->GetPreset();
	}

	return nullptr;	
}

bool SRCControllerPanelList::IsListFocused() const
{
	return TreeView->HasAnyUserFocus().IsSet() || ContextMenuWidgetCached.IsValid();
}

void SRCControllerPanelList::DeleteSelectedPanelItems()
{
	FScopedTransaction Transaction(LOCTEXT("DeleteSelectedItems", "Delete Selected Items"));
	TArray<FRCControllerPanelListItem> SelectedItems = TreeView->GetSelectedItems();

	TArray<TSharedPtr<FRCCategoryModel>> SelectedCategories;
	SelectedCategories.Reserve(SelectedItems.Num());

	TArray<int32> SelectedCategoryIndices;
	SelectedCategoryIndices.Reserve(SelectedItems.Num());

	TArray<TSharedPtr<FRCControllerModel>> SelectedControllers;
	SelectedControllers.Reserve(SelectedItems.Num());

	TArray<int32> SelectedControllerIndices;
	SelectedControllerIndices.Reserve(SelectedItems.Num());

	TSet<FString> CategoryNames;

	for (const FRCControllerPanelListItem& Item : SelectedItems)
	{
		switch (Item.Type)
		{
			case ERCControllerPanelListItemType::Category:
				if (CategoryItems.IsValidIndex(Item.Index) && CategoryItems[Item.Index].IsValid())
				{
					SelectedCategories.Add(CategoryItems[Item.Index]);
					SelectedCategoryIndices.Add(Item.Index);
					CategoryNames.Add(CategoryItems[Item.Index]->GetId().ToString());
				}
				break;

			case ERCControllerPanelListItemType::Controller:
				if (ControllerItems.IsValidIndex(Item.Index) && ControllerItems[Item.Index].IsValid())
				{
					SelectedControllers.Add(ControllerItems[Item.Index]);
					SelectedControllerIndices.Add(Item.Index);
				}
				break;
		}
	}

	// Also remove controllers in deleted categories.
	for (int32 ControllerIndex = 0; ControllerIndex < ControllerItems.Num(); ++ControllerIndex)
	{
		if (!SelectedControllerIndices.Contains(ControllerIndex))
		{
			if (ControllerItems[ControllerIndex].IsValid())
			{
				if (URCVirtualPropertyBase* Controller = ControllerItems[ControllerIndex]->GetVirtualProperty())
				{
					using namespace UE::RemoteControl::UI::Private;
					const FString Category = Controller->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);

					if (!Category.IsEmpty() && CategoryNames.Contains(Category))
					{
						SelectedControllers.Add(ControllerItems[ControllerIndex]);
						SelectedControllerIndices.Add(ControllerIndex);
					}
				}
			}
		}
	}

	const bool bDeletedCategories = DeleteItemsFromLogicPanel<FRCCategoryModel>(CategoryItems, SelectedCategories);

	if (bDeletedCategories)
	{
		 CategoryNames.Reserve(SelectedCategoryIndices.Num());

		for (int32 SelectedIndex = 0; SelectedIndex < SelectedCategoryIndices.Num(); ++SelectedIndex)
		{
			OnItemRemoved({
				ERCControllerPanelListItemType::Category,
				SelectedCategoryIndices[SelectedIndex]
			});
		}
	}

	const bool bDeletedControllers = DeleteItemsFromLogicPanel<FRCControllerModel>(ControllerItems, SelectedControllers);

	if (bDeletedControllers)
	{
		for (int32 Index = 0; Index < SelectedControllerIndices.Num(); ++Index)
		{
			OnItemRemoved({
				ERCControllerPanelListItemType::Controller,
				SelectedControllerIndices[Index]
			});
		}

		UpdateEntityUsage();
	}

	if (!bDeletedCategories && !bDeletedControllers)
	{
		Transaction.Cancel();
	}
}

TArray<FRCControllerPanelListItem> SRCControllerPanelList::GetSelectedItems() const
{
	return TreeView->GetSelectedItems();
}

TArray<TSharedPtr<FRCControllerModel>> SRCControllerPanelList::GetSelectedControllers() const
{
	TArray<FRCControllerPanelListItem> SelectedItems = TreeView->GetSelectedItems();

	TArray<TSharedPtr<FRCControllerModel>> SelectedControllers;
	SelectedControllers.Reserve(SelectedItems.Num());

	for (const FRCControllerPanelListItem& Item : SelectedItems)
	{
		if (Item.Type != ERCControllerPanelListItemType::Controller)
		{
			continue;
		}

		if (!ControllerItems.IsValidIndex(Item.Index) || !ControllerItems[Item.Index].IsValid())
		{
			continue;
		}

		SelectedControllers.Add(ControllerItems[Item.Index]);
	}

	return SelectedControllers;
}

TArray<TSharedPtr<FRCCategoryModel>> SRCControllerPanelList::GetSelectedCategories() const
{
	TArray<FRCControllerPanelListItem> SelectedItems = TreeView->GetSelectedItems();

	TArray<TSharedPtr<FRCCategoryModel>> SelectedCategories;
	SelectedCategories.Reserve(SelectedItems.Num());

	for (const FRCControllerPanelListItem& Item : SelectedItems)
	{
		if (Item.Type != ERCControllerPanelListItemType::Category)
		{
			continue;
		}

		if (!CategoryItems.IsValidIndex(Item.Index) || !CategoryItems[Item.Index].IsValid())
		{
			continue;
		}

		SelectedCategories.Add(CategoryItems[Item.Index]);
	}

	return SelectedCategories;
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCControllerPanelList::GetSelectedLogicItems()
{
	TArray<TSharedPtr<FRCLogicModeBase>> Items;
	Items.Append(GetSelectedCategories());
	Items.Append(GetSelectedControllers());

	return Items;
}

void SRCControllerPanelList::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange)
{
	// If a Vector is modified and the Z value changes, the sub property (corresponding to Z) gets notified to us.
	// However for Controllers we are actually interested in the parent Struct property (corresponding to the Vector) as the Virtual Property is associated with Struct's FProperty (rather than its X/Y/Z components)
	// For this reason the "Active Member Node" is extracted below from the child property
	if (FEditPropertyChain::TDoubleLinkedListNode* ActiveMemberNode = PropertyAboutToChange->GetActiveMemberNode())
	{
		FPropertyChangedEvent PropertyChangedEvent(ActiveMemberNode->GetValue());

		OnNotifyPreChangeProperties(PropertyChangedEvent);
	}
}

void SRCControllerPanelList::EnterRenameMode()
{
	if (TSharedPtr<FRCControllerModel> SelectedItem = SelectedControllerItemWeakPtr.Pin())
	{
		SelectedItem->EnterDescriptionEditingMode();
	}
}

TSharedPtr<FRCCategoryModel>  SRCControllerPanelList::FindCategoryItemByListItem(const FRCControllerPanelListItem& InItem) const
{
	if (InItem.Type == ERCControllerPanelListItemType::Category)
	{
		if (CategoryItems.IsValidIndex(InItem.Index))
		{
			return CategoryItems[InItem.Index];
		}
	}

	return nullptr;
}

TSharedPtr<FRCControllerModel> SRCControllerPanelList::FindControllerItemByListItem(const FRCControllerPanelListItem& InItem) const
{
	if (InItem.Type == ERCControllerPanelListItemType::Controller && ControllerItems.IsValidIndex(InItem.Index))
	{
		return ControllerItems[InItem.Index];
	}

	return nullptr;
}

TSharedPtr<FRCControllerModel> SRCControllerPanelList::FindControllerItemById(const FGuid& InId) const
{
	for (TSharedPtr<FRCControllerModel> ControllerItem : ControllerItems)
	{
		if (ControllerItem && ControllerItem->GetId() == InId)
		{
			return ControllerItem;
		}
	}

	return nullptr;
}

TSharedPtr<FRCControllerModel> SRCControllerPanelList::FindControllerItemByPropertyName(const FName& InPropertyName) const
{
	if (InPropertyName.IsNone())
	{
		return nullptr;
	}

	for (TSharedPtr<FRCControllerModel> ControllerItem : ControllerItems)
	{
		if (ControllerItem && ControllerItem->GetPropertyName() == InPropertyName)
		{
			return ControllerItem;
		}
	}

	return nullptr;
}

TArray<TSharedPtr<FRCControllerModel>> SRCControllerPanelList::FindControllerItemsById(TConstArrayView<FGuid> InIds)
{
	TArray<TSharedPtr<FRCControllerModel>> FoundControllerItems;
	FoundControllerItems.Reserve(InIds.Num());

	for (const TSharedPtr<FRCControllerModel>& ControllerItem : ControllerItems)
	{
		if (ControllerItem.IsValid() && InIds.Contains(ControllerItem->GetId()))
		{
			FoundControllerItems.Add(ControllerItem);
		}
	}

	return FoundControllerItems;
}

TArray<TSharedPtr<FRCControllerModel>> SRCControllerPanelList::FindControllerItemsByObject(TConstArrayView<URCController*> InControllers)
{
	TArray<TSharedPtr<FRCControllerModel>> FoundControllerItems;
	FoundControllerItems.Reserve(InControllers.Num());

	for (const TSharedPtr<FRCControllerModel>& ControllerItem : ControllerItems)
	{
		if (ControllerItem.IsValid() && InControllers.Contains(ControllerItem->GetVirtualProperty()))
		{
			FoundControllerItems.Add(ControllerItem);
		}
	}

	return FoundControllerItems;
}

FRCControllerPanelListItem SRCControllerPanelList::FindListItemByControllerItem(const TSharedRef<FRCControllerModel>& InModel) const
{
	URCVirtualPropertyBase* InController = InModel->GetVirtualProperty();

	for (int32 Index = 0; Index < ControllerItems.Num(); ++Index)
	{
		if (ControllerItems[Index].IsValid() && ControllerItems[Index]->GetVirtualProperty() == InController)
		{
			return {ERCControllerPanelListItemType::Controller, Index};
		}
	}

	return UE::RemoteControl::UI::Private::ControllerPanelListItem::None;
}

FRCControllerPanelListItem SRCControllerPanelList::FindListItemByCategoryItem(const TSharedRef<FRCCategoryModel>& InModel) const
{
	const FGuid InCategoryId = InModel->GetId();

	for (int32 Index = 0; Index < CategoryItems.Num(); ++Index)
	{
		if (CategoryItems[Index].IsValid() && CategoryItems[Index]->GetId() == InCategoryId)
		{
			return {ERCControllerPanelListItemType::Category, Index};
		}
	}

	return UE::RemoteControl::UI::Private::ControllerPanelListItem::None;
}

bool SRCControllerPanelList::ReorderItems(TConstArrayView<FRCControllerPanelListItem> InItemsToMove, FRCControllerPanelListItem InDroppedOnItem, EItemDropZone InDropZone)
{
	if (InItemsToMove.Contains(InDroppedOnItem))
	{
		return false;
	}

	// Check we have a valid drop target
	switch (InDroppedOnItem.Type)
	{
		case ERCControllerPanelListItemType::None:
			return false;

		case ERCControllerPanelListItemType::Category:
			if (!CategoryItems.IsValidIndex(InDroppedOnItem.Index) || !CategoryItems[InDroppedOnItem.Index].IsValid())
			{
				return false;
			}
			break;

		case ERCControllerPanelListItemType::Controller:
			if (!ControllerItems.IsValidIndex(InDroppedOnItem.Index) || !ControllerItems[InDroppedOnItem.Index].IsValid())
			{
				return false;
			}
			break;
	}

	using namespace UE::RemoteControl::UI::Private;

	// If we reorder onto a category, find the last item of the category and drop it below that.
	if (InDroppedOnItem.Type == ERCControllerPanelListItemType::Category && InDropZone == EItemDropZone::OntoItem)
	{
		if (TSharedPtr<FRCCategoryModel> DroppedOnCategory = FindCategoryItemByListItem(InDroppedOnItem))
		{
			if (TOptional<FRCVirtualPropertyCategory> Category = DroppedOnCategory->GetCategory())
			{
				const FString CategoryId = Category->Id.ToString();
				int32 MaxControllerIndex = INDEX_NONE;

				for (int32 Index = 0; Index < ControllerItems.Num(); ++Index)
				{
					if (ControllerItems[Index].IsValid())
					{
						if (URCVirtualPropertyBase* Controller = ControllerItems[Index]->GetVirtualProperty())
						{
							const FString ControllerCategory = Controller->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);

							if (ControllerCategory == CategoryId)
							{
								MaxControllerIndex = FMath::Max(MaxControllerIndex, Index);
							}
						}
					}
				}

				if (MaxControllerIndex != INDEX_NONE)
				{
					InDroppedOnItem = {ERCControllerPanelListItemType::Controller, MaxControllerIndex};
					InDropZone = EItemDropZone::BelowItem;
				}
			}
		}
	}

	// Rebuild list assuming that everything is already in the correct display order
	TMap<FName, TArray<FRCControllerPanelListItem>> Tree;
	Tree.Add(NAME_None);

	for (const TSharedPtr<FRCCategoryModel>& CategoryModel : CategoryItems)
	{
		if (TOptional<FRCVirtualPropertyCategory> Category = CategoryModel->GetCategory())
		{
			Tree.Add(*Category->Id.ToString());
		}
	}

	auto AddDroppedItems = [this, &InItemsToMove, &Tree](FName InCategoryName = NAME_None)
		{
			for (const FRCControllerPanelListItem& DroppedItem : InItemsToMove)
			{
				// Force categories into the root list
				if (DroppedItem.Type == ERCControllerPanelListItemType::Category)
				{
					Tree[NAME_None].Add(DroppedItem);
				}
				else
				{
					if (!Tree.Contains(InCategoryName))
					{
						InCategoryName = NAME_None;
					}

					Tree[InCategoryName].Add(DroppedItem);

					if (URCVirtualPropertyBase* Controller = ControllerItems[DroppedItem.Index]->GetVirtualProperty())
					{
						using namespace UE::RemoteControl::UI::Private;

						if (InCategoryName.IsNone())
						{
							Controller->RemoveMetadataValue(FRCControllerPropertyInfo::CategoryName);
						}
						else
						{
							Controller->SetMetadataValue(FRCControllerPropertyInfo::CategoryName, InCategoryName.ToString());
						}
					}
				}
			}
		};

	int32 CategoryIndex = 0;
	int32 ControllerIndex = 0;

	auto GetNextItem = [this, &InItemsToMove, &CategoryIndex, &ControllerIndex]() -> FRCControllerPanelListItem
		{
			int32 NextCategoryIndex = INDEX_NONE;
			int32 NextControllerIndex = INDEX_NONE;
				
			// We may possibly have invalid or missing categories
			while (CategoryIndex < CategoryItems.Num())
			{
				const FRCControllerPanelListItem CategoryItem = {ERCControllerPanelListItemType::Category, CategoryIndex};

				if (CategoryItems[CategoryIndex].IsValid() && !InItemsToMove.Contains(CategoryItem))
				{
					if (TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[CategoryIndex]->GetCategory())
					{
						NextCategoryIndex = Category->DisplayIndex;
						break;
					}
				}

				++CategoryIndex;
			}

			// We may possibly have invalid or missing controllers
			while (ControllerIndex < ControllerItems.Num())
			{
				const FRCControllerPanelListItem ControllerItem = {ERCControllerPanelListItemType::Controller, ControllerIndex};

				if (ControllerItems[ControllerIndex].IsValid() && !InItemsToMove.Contains(ControllerItem))
				{
					if (URCVirtualPropertyBase* Controller = ControllerItems[ControllerIndex]->GetVirtualProperty())
					{
						NextControllerIndex = Controller->DisplayIndex;
						break;
					}
				}

				++ControllerIndex;
			}

			if (NextCategoryIndex != INDEX_NONE && NextControllerIndex != INDEX_NONE)
			{
				if (NextCategoryIndex <= NextControllerIndex)
				{
					return {ERCControllerPanelListItemType::Category, CategoryIndex++};
				}
				else
				{
					return {ERCControllerPanelListItemType::Controller, ControllerIndex++};
				}
			}
			else if (NextCategoryIndex != INDEX_NONE)
			{
				return {ERCControllerPanelListItemType::Category, CategoryIndex++};
			}
			else if (NextControllerIndex != INDEX_NONE)
			{
				return {ERCControllerPanelListItemType::Controller, ControllerIndex++};
			}
			else
			{
				return UE::RemoteControl::UI::Private::ControllerPanelListItem::None;
			}
		};

	FRCControllerPanelListItem NextItem = GetNextItem();

	while (NextItem.Type != ERCControllerPanelListItemType::None)
	{
		const bool bIsDroppedOnItem = (NextItem == InDroppedOnItem);
		FName ItemCategory = NAME_None;

		if (NextItem.Type == ERCControllerPanelListItemType::Controller)
		{
			ItemCategory = *ControllerItems[NextItem.Index]->GetVirtualProperty()->GetMetadataValue(FRCControllerPropertyInfo::CategoryName);
		}

		// Make sure the category exists
		TArray<FRCControllerPanelListItem>* CategoryArray = Tree.Find(ItemCategory);

		if (!CategoryArray)
		{
			ItemCategory = NAME_None;
			CategoryArray = Tree.Find(ItemCategory);
		}

		if (bIsDroppedOnItem && InDropZone == EItemDropZone::AboveItem)
		{
			AddDroppedItems(ItemCategory);
		}

		if (!InItemsToMove.Contains(NextItem))
		{
			CategoryArray->Add(NextItem);
		}

		if (bIsDroppedOnItem)
		{
			if (InDropZone == EItemDropZone::OntoItem)
			{
				if (NextItem.Type == ERCControllerPanelListItemType::Category)
				{
					AddDroppedItems(*CategoryItems[NextItem.Index]->GetCategory()->Id.ToString());
				}
				else
				{
					AddDroppedItems(ItemCategory);
				}
			}
			else if (InDropZone == EItemDropZone::BelowItem)
			{
				AddDroppedItems(ItemCategory);
			}
		}

		NextItem = GetNextItem();
	}

	auto GetDisplayIndex = [this](const FRCControllerPanelListItem& InItem) -> int32
		{
			switch (InItem.Type)
			{
				default:
					break;

				case ERCControllerPanelListItemType::Category:
					if (CategoryItems.IsValidIndex(InItem.Index) && CategoryItems[InItem.Index].IsValid())
					{

					}
					break;

				case ERCControllerPanelListItemType::Controller:
					if (ControllerItems.IsValidIndex(InItem.Index) && ControllerItems[InItem.Index].IsValid())
					{
						if (URCVirtualPropertyBase* Controller = ControllerItems[InItem.Index]->GetVirtualProperty())
						{
							return Controller->DisplayIndex;
						}
					}
					break;
			}

			return INDEX_NONE;
		};

	auto SetDisplayIndex = [this](const FRCControllerPanelListItem& InItem, int32 InDisplayIndex)
		{
			switch (InItem.Type)
			{
				default:
					return;

				case ERCControllerPanelListItemType::Category:
					if (CategoryItems.IsValidIndex(InItem.Index) && CategoryItems[InItem.Index].IsValid())
					{
						if (TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[InItem.Index]->GetCategory())
						{
							// Already has Modify() called
							CategoryItems[InItem.Index]->SetDisplayIndex(InDisplayIndex);
						}
					}
					return;

				case ERCControllerPanelListItemType::Controller:
					if (ControllerItems.IsValidIndex(InItem.Index) && ControllerItems[InItem.Index].IsValid())
					{
						if (URCVirtualPropertyBase* Controller = ControllerItems[InItem.Index]->GetVirtualProperty())
						{
							if (GUndo)
							{
								Controller->Modify();
							}

							Controller->DisplayIndex = InDisplayIndex;
						}
					}
					return;
			}
		};

	int32 DisplayIndex = 0;
	RootItems = Tree[NAME_None];
	
	for (const FRCControllerPanelListItem& RootItem : RootItems)
	{
		SetDisplayIndex(RootItem, DisplayIndex);
		++DisplayIndex;

		if (RootItem.Type == ERCControllerPanelListItemType::Category)
		{
			if (TOptional<FRCVirtualPropertyCategory> Category = CategoryItems[RootItem.Index]->GetCategory())
			{
				const FName CategoryName = *Category->Id.ToString();

				for (const FRCControllerPanelListItem& CategoryItem : Tree[CategoryName])
				{
					SetDisplayIndex(CategoryItem, DisplayIndex);
					++DisplayIndex;
				}
			}
		}
	}

	Algo::Sort(
		CategoryItems,
		[](const TSharedPtr<FRCCategoryModel>& A, const TSharedPtr<FRCCategoryModel>& B)
		{
			int32 DisplayIndexA = INDEX_NONE;
			int32 DisplayIndexB = INDEX_NONE;

			if (A.IsValid())
			{
				if (TOptional<FRCVirtualPropertyCategory> Category = A->GetCategory())
				{
					DisplayIndexA = Category->DisplayIndex;
				}
			}

			if (B.IsValid())
			{
				if (TOptional<FRCVirtualPropertyCategory> Category = B->GetCategory())
				{
					DisplayIndexB = Category->DisplayIndex;
				}
			}

			return DisplayIndexA < DisplayIndexB;
		}
	);

	Algo::Sort(
		ControllerItems,
		[](const TSharedPtr<FRCControllerModel>& A, const TSharedPtr<FRCControllerModel>& B)
		{
			int32 DisplayIndexA = INDEX_NONE;
			int32 DisplayIndexB = INDEX_NONE;

			if (A.IsValid())
			{
				if (URCVirtualPropertyBase* Controller = A->GetVirtualProperty())
				{
					DisplayIndexA = Controller->DisplayIndex;
				}
			}

			if (B.IsValid())
			{
				if (URCVirtualPropertyBase* Controller = B->GetVirtualProperty())
				{
					DisplayIndexB = Controller->DisplayIndex;
				}
			}

			return DisplayIndexA < DisplayIndexB;
		}
	);

	Reset();

	return true;
}

static TSharedPtr<FExposedEntityDragDrop> GetExposedEntityDragDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	if (DragDropOperation)
	{
		if (DragDropOperation->IsOfType<FExposedEntityDragDrop>())
		{
			return StaticCastSharedPtr<FExposedEntityDragDrop>(DragDropOperation);
		}
	}

	return nullptr;
}

bool SRCControllerPanelList::IsEntitySupported(const FGuid ExposedEntityId)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (const TSharedPtr<const FRemoteControlProperty>& RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
		{
			return UE::RCControllers::CanCreateControllerFromEntity(RemoteControlProperty);
		}
	}

	return false;
}

bool SRCControllerPanelList::OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	if (IsTreeViewHovered())
	{
		bIsAnyControllerItemEligibleForDragDrop = false;
	}
	// Ensures that this drop target is visually disabled whenever the user is attempting to drop onto an existing Controller (rather than the ListView's empty space)
	else if (bIsAnyControllerItemEligibleForDragDrop)
	{
		return false;
	}

	if (TSharedPtr<FExposedEntityDragDrop> DragDropOp = GetExposedEntityDragDrop(DragDropOperation))
	{
		// Fetch the Exposed Entity
		const TArray<FGuid>& ExposedEntitiesIds = DragDropOp->GetSelectedFieldsId();

		// Check if Entity is supported by controllers
		// Allow drop if at least one entity is supported.
		for (const FGuid& EntityId : ExposedEntitiesIds)
		{
			if (IsEntitySupported(EntityId))
			{
				return true;
			}
		}
	}
	return false;
}

FReply SRCControllerPanelList::OnControllerTreeViewDragDrop(TSharedPtr<FDragDropOperation> InDragDropOperation)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return FReply::Handled();
	}

	if (TSharedPtr<FExposedEntityDragDrop> DragDropOp = GetExposedEntityDragDrop(InDragDropOperation))
	{
		FScopedTransaction Transaction(LOCTEXT("AutoBindEntities", "Auto bind entities to controllers"));
		const FString OldDefault = Preset->NewControllerCategory;
		Preset->NewControllerCategory = FString();

		bool bModified = false;
		for (const FGuid& ExposedEntityId : DragDropOp->GetSelectedFieldsId())
		{
			if (TSharedPtr<const FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(ExposedEntityId).Pin())
			{
				bModified |= CreateAutoBindForProperty(RemoteControlProperty);
			}
		}

		Preset->NewControllerCategory = OldDefault;

		if (!bModified)
		{
			Transaction.Cancel();
		}
	}

	return FReply::Handled();
}

bool SRCControllerPanelList::CreateAutoBindForProperty(TSharedPtr<const FRemoteControlProperty> InRemoteControlProperty)
{
	if (URCController* NewController = UE::RCUIHelpers::CreateControllerFromEntity(GetPreset(), InRemoteControlProperty))
	{
		// Refresh UI
		Reset();

		// Create Bind Behaviour and Bind to the property
		constexpr bool bExecuteBind = true;
		CreateBindBehaviourAndAssignTo(NewController, InRemoteControlProperty.ToSharedRef(), bExecuteBind);
		return true;
	}
	return false;
}

void SRCControllerPanelList::CreateBindBehaviourAndAssignTo(URCController* Controller, TSharedRef<const FRemoteControlProperty> InRemoteControlProperty, const bool bExecuteBind)
{
	const URCBehaviourBind* BindBehaviour = nullptr;

	bool bRequiresNumericConversion = false;
	if (!URCBehaviourBind::CanHaveActionForField(Controller, InRemoteControlProperty, false))
	{
		if (URCBehaviourBind::CanHaveActionForField(Controller, InRemoteControlProperty, true))
		{
			bRequiresNumericConversion = true;
		}
		else
		{
			ensureAlwaysMsgf(false, TEXT("Incompatible property provided for Auto Bind!"));
			return;
		}
	}

	for (const URCBehaviour* Behaviour : Controller->GetBehaviors())
	{
		if (Behaviour && Behaviour->IsA(URCBehaviourBind::StaticClass()))
		{
			BindBehaviour = Cast<URCBehaviourBind>(Behaviour);

			// In case numeric conversion is required we might have multiple Bind behaviours with different settings,
			// so we do not break in case there is at least one Bind behaviour with a matching clause. If not, we will create a new Bind behaviour with the requried setting (see below)
			if (!bRequiresNumericConversion || BindBehaviour->AreNumericInputsAllowedAsStrings())
			{
				break;
			}
		}
	}

	if (BindBehaviour)
	{
		if (bRequiresNumericConversion && !BindBehaviour->AreNumericInputsAllowedAsStrings())
		{
			// If the requested Bind operation requires numeric conversion but the existing Bind behaviour doesn't support this, then we prefer creating a new Bind behaviour to facilitate this operation.
			// This allows the the user to successfully perform the Auto Bind as desired without disrupting the setting on the existing Bind behaviour
			BindBehaviour = nullptr;
		}
		
	}

	if (!BindBehaviour)
	{
		Controller->Modify();

		URCBehaviourBind* NewBindBehaviour = Cast<URCBehaviourBind>(Controller->AddBehaviour(URCBehaviourBindNode::StaticClass()));

		// If this is a new Bind Behaviour and we are attempting to link unrelated (but compatible types), via numeric conversion, then we apply the bAllowNumericInputAsStrings flag
		NewBindBehaviour->SetAllowNumericInputAsStrings(bRequiresNumericConversion);

		BindBehaviour = NewBindBehaviour;

		// Broadcast new behaviour
		if (const TSharedPtr<SRCControllerPanel> ControllerPanel = ControllerPanelWeakPtr.Pin())
		{
			if (const TSharedPtr<SRemoteControlPanel> RemoteControlPanel = ControllerPanel->GetRemoteControlPanel())
			{
				RemoteControlPanel->OnBehaviourAdded.Broadcast(BindBehaviour);
			}
		}
	}

	if (ensure(BindBehaviour))
	{
		URCBehaviourBind* BindBehaviourMutable = const_cast<URCBehaviourBind*>(BindBehaviour);

		BindBehaviourMutable->Modify();
		URCAction* BindAction = BindBehaviourMutable->AddPropertyBindAction(InRemoteControlProperty);

		if (bExecuteBind)
		{
			BindAction->Execute();
		}
	}

	// Update the UI selection to provide the user visual feedback indicating that a Bind behaviour has been created/updated
	SelectController(Controller);
}

bool SRCControllerPanelList::IsTreeViewHovered()
{
	return TreeView->IsDirectlyHovered();
}

void SRCControllerPanelList::ShowValueTypeHeaderColumn(bool bInShowColumn)
{
	if (ControllersHeaderRow.IsValid())
	{
		const bool bColumnIsGenerated = ControllersHeaderRow->IsColumnGenerated(FRCControllerColumns::ValueTypeSelection);
		if (bInShowColumn)
		{
			if (!bColumnIsGenerated)
			{
				ControllersHeaderRow->AddColumn(
					SHeaderRow::FColumn::FArguments()
					.ColumnId(FRCControllerColumns::ValueTypeSelection)
					.DefaultLabel(LOCTEXT("ControllerValueTypeColumnName", "Value Type"))
					.FillWidth(0.2f)
					.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)
				);
			}
		}
		else if (bColumnIsGenerated)
		{
			ControllersHeaderRow->RemoveColumn(FRCControllerColumns::ValueTypeSelection);
		}
	}
}

void SRCControllerPanelList::ShowFieldIdHeaderColumn(bool bInShowColumn)
{	
	if (ControllersHeaderRow.IsValid())
	{
		const bool bColumnIsGenerated = ControllersHeaderRow->IsColumnGenerated(FRCControllerColumns::FieldId);
		if (bInShowColumn)
		{
			if (!bColumnIsGenerated)
			{
				ControllersHeaderRow->InsertColumn(
					SHeaderRow::FColumn::FArguments()
					.ColumnId(FRCControllerColumns::FieldId)
					.DefaultLabel(LOCTEXT("ControllerNameColumnFieldId", "Field Id"))
					.FillWidth(0.2f)
					.HeaderContentPadding(RCPanelStyle->HeaderRowPadding),
					2);
			}
		}
		else if (bColumnIsGenerated)
		{
			ControllersHeaderRow->RemoveColumn(FRCControllerColumns::FieldId);
		}
	}
}

void SRCControllerPanelList::AddColumn(const FName& InColumnName)
{
	CustomColumns.AddUnique(InColumnName);
}

void SRCControllerPanelList::SetMultiControllerMode(bool bIsUniqueModeOn)
{
	bIsInMultiControllerMode = bIsUniqueModeOn;
	Reset();
}

#undef LOCTEXT_NAMESPACE
