// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Misc/NotifyHook.h"
#include "Misc/Optional.h"
#include "RCMultiController.h"
#include "RCVirtualPropertyContainer.h"
#include "UI/BaseLogicUI/SRCLogicPanelListBase.h"
#include "UI/Controller/RCControllerPanelListItem.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Layout/SBorder.h"

class FDragDropOperation;
class FEditPropertyChain;
class FRCCategoryModel;
class FRCControllerModel;
class FRCLogicModeBase;
class IPropertyRowGenerator;
class ITableBase;
class ITableRow;
class SDropTarget;
class SRCControllerPanelControllerRow;
class SRCControllerPanel;
class SRemoteControlPanel;
class STableViewBase;
class URCController;
class URemoteControlPreset;
enum class EItemDropZone;
struct FRCPanelStyle;
template <typename ItemType> class STreeView;

/*
* ~ FRCControllerPanelDragDrop ~
* Facilitates drag-drop operation
*/
class FRCControllerPanelDragDrop final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FRCControllerDragDropOp, FDecoratedDragDropOp)

	explicit FRCControllerPanelDragDrop(const TArray<FRCControllerPanelListItem>& InItems)
		: Items(InItems)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;
	}

	TArray<FRCControllerPanelListItem> GetItems() const
	{
		return Items;
	}

private:
	TArray<FRCControllerPanelListItem> Items;
};

/*
* ~ SRCControllerPanelList ~
*
* UI Widget for Controllers List
* Used as part of the RC Logic Actions Panel.
*/
class REMOTECONTROLUI_API SRCControllerPanelList : public SRCLogicPanelListBase, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SRCControllerPanelList)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SRCControllerPanel> InControllerPanel, const TSharedRef<SRemoteControlPanel> InRemoteControlPanel);

	/** Returns true if the underlying list is valid and empty. */
	virtual bool IsEmpty() const override;

	/** Returns number of items in the list. */
	virtual int32 Num() const override;

	/** The number of Controllers currently selected */
	virtual int32 NumSelectedLogicItems() const override;

	/** Whether the Controllers List View currently has focus. */
	virtual bool IsListFocused() const override;

	/** Deletes currently selected items from the list view */
	virtual void DeleteSelectedPanelItems() override;

	/** Returns the list items selected by the user (if any). */
	TArray<FRCControllerPanelListItem> GetSelectedItems() const;

	/** Returns the controllers selected by the user (if any). */
	TArray<TSharedPtr<FRCControllerModel>> GetSelectedControllers() const;

	/** Returns the categories selected by the user (if any). */
	TArray<TSharedPtr<FRCCategoryModel>> GetSelectedCategories() const;

	/** Returns the UI items currently selected by the user (if any). */
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() override;

	/** FNotifyHook Interface Begin */
	virtual void NotifyPreChange(FEditPropertyChain* PropertyAboutToChange) override;
	/** FNotifyHook Interface End */

	void EnterRenameMode();

	int32 NumControllerItems() const
	{
		return ControllerItems.Num() + CategoryItems.Num();
	}

	/** Finds a category by list item */
	TSharedPtr<FRCCategoryModel> FindCategoryItemByListItem(const FRCControllerPanelListItem& InItem) const;

	/** Finds a Controller UI model by list item */
	TSharedPtr<FRCControllerModel> FindControllerItemByListItem(const FRCControllerPanelListItem& InItem) const;

	/** Finds a Controller UI model by unique Id */
	TSharedPtr<FRCControllerModel> FindControllerItemById(const FGuid& InId) const;

	/** Finds a Controller UI model by unique Id */
	TSharedPtr<FRCControllerModel> FindControllerItemByPropertyName(const FName& InPropertyName) const;

	/** Finds the Controller UI models by unique Id */
	TArray<TSharedPtr<FRCControllerModel>> FindControllerItemsById(TConstArrayView<FGuid> InIds);

	/** Finds the Controller UI models by their controllers */
	TArray<TSharedPtr<FRCControllerModel>> FindControllerItemsByObject(TConstArrayView<URCController*> InControllers);

	/** Finds the item by controller UI model */
	FRCControllerPanelListItem FindListItemByControllerItem(const TSharedRef<FRCControllerModel>& InModel) const;

	/** Finds the item by category id */
	FRCControllerPanelListItem FindListItemByCategoryItem(const TSharedRef<FRCCategoryModel>& InModel) const;

	/**
	 * Called to moves the given controller items to the given target index, pushing all other rows below
	 * returns true if the operation was successful
	 */
	bool ReorderItems(TConstArrayView<FRCControllerPanelListItem> InItemsToMove, FRCControllerPanelListItem InDroppedOnItem, EItemDropZone InDropZone);

	/** Requests the panel to refresh its contents from the latest list of Controllers */
	void RequestRefresh()
	{
		Reset();
	}

	/** Drag-Drop validation delegate for the Controllers Panel List */
	bool OnAllowDrop(TSharedPtr<FDragDropOperation> DragDropOperation);

	/** Drag-Drop action delegate for the Controllers Panel List */
	FReply OnControllerTreeViewDragDrop(TSharedPtr<FDragDropOperation> InDragDropOperation);

	/** Fetches the Remote Control preset associated with the parent panel */
	virtual URemoteControlPreset* GetPreset() override;

	/** Creates a Bind Behaviour for the given Controller and binds the given remote control property to it */
	void CreateBindBehaviourAndAssignTo(URCController* Controller, TSharedRef<const FRemoteControlProperty> InRemoteControlProperty, const bool bExecuteBind);

	/** Whether the user's cursor is directly hovered over the List View */
	bool IsTreeViewHovered();

	/** Enable/disable MultiController Mode */
	void SetMultiControllerMode(bool bIsUniqueModeOn);

	/** Return the current list of custom columns names, if any */
	TConstArrayView<FName> GetCustomColumns() const { return CustomColumns; } 

	/** Flag that facilitates usage of two mutually exclusive drag-drop zones within a single panel 
	* 
	* The first drag-drop zone is empty panel space for "Bind To New Controller"
	* The second drag-drop zone is the Controller name widget for "Bind To Existing Controller"
	* 
	* This flag is set if any Controller has active drag-drop focus, in which case we disable the first drag-drop zone. This is purely for visual clarity*/
	bool bIsAnyControllerItemEligibleForDragDrop = false;

	/** OnGenerateRow delegate for the Actions List View */
	TSharedRef<ITableRow> OnGenerateWidgetForTree(FRCControllerPanelListItem InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/** Returns the row for a given item. */
	bool IsItemCategoryExpanded(FRCControllerPanelListItem InItem) const;

	/** Gets the children for categories. */
	void OnGetChildren(FRCControllerPanelListItem InItem, TArray<FRCControllerPanelListItem>& OutChildren);
	
	/** OnSelectionChanged delegate for Actions List View */
	void OnTreeSelectionChanged(FRCControllerPanelListItem InItem , ESelectInfo::Type);

	/** Selects the Controller UI item corresponding to a given Controller UObject */
	void SelectController(URCController* InController);

	/** Responds to the selection of a newly created Controller. Resets UI state */
	void OnControllerAdded(const FName& InNewPropertyName);
	
	/** Responds to the removal of all Controllers. Resets UI state */
	void OnEmptyControllers();

	void OnControllerContainerModified();

	/** Pre Change listener for Controllers, propagated via FNotifyHook associated with PropertyRowGenerator
	* Invoked while the user is scrubbing float or Vector sliders in the UI
	*/
	void OnNotifyPreChangeProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Change listener for Controllers. Bound to the PropertyRowGenerator's delegate
	* This is propagated to the corresponding Controller model (Virtual Property) for evaluating all associated Behaviours.
	*/
	void OnFinishedChangingProperties(const FPropertyChangedEvent& PropertyChangedEvent);

	/** Called when a MultiController Value Type changes. */
	void OnControllerValueTypeChanged(URCVirtualPropertyBase* InController, EPropertyBagPropertyType InValueType);

	/**
	 * Called when a Controller Value changes.
	 * Currently used to update handled controllers, in case of MultiControllers.
	 */
	void OnControllerValueChanged(TSharedPtr<FRCControllerModel> InControllerModel, bool bInIsMultiController);

	/** Creates a new Controller for the given Remote Control Property and also binds to it. Returns true if operation succeeded, false if nothing happened */
	bool CreateAutoBindForProperty(TSharedPtr<const FRemoteControlProperty> InRemoteControlProperty);

	/** The row generator used to represent each Controller as a row, when used with STreeView */
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	/** The currently selected Controller item (UI model) */
	TWeakPtr<FRCControllerModel> SelectedControllerItemWeakPtr = nullptr;
	
	/** The parent Controller Panel widget */
	TWeakPtr<SRCControllerPanel> ControllerPanelWeakPtr;

	/** List of Category (UI model) active in this widget */
	TArray<TSharedPtr<FRCCategoryModel>> CategoryItems;

	/** List of Controllers (UI model) active in this widget */
	TArray<TSharedPtr<FRCControllerModel>> ControllerItems;

	/** List of Controllers (UI model) that are in the root category */
	TArray<FRCControllerPanelListItem> RootItems;

	/** Tree View widget for representing our Controllers List */
	TSharedPtr<STreeView<FRCControllerPanelListItem>> TreeView;
	
	/** Refreshes the list from the latest state of the model */
	virtual void Reset() override;

	/** Updates the root item set from ControllerItems */
	void UpdateRootItems();

	/** Called to update metadata when things are removed. */
	void OnItemRemoved(const FRCControllerPanelListItem& InRemovedItem);

	/** Handles broadcasting of a successful remove item operation. */
	virtual void BroadcastOnItemRemoved() override;

	/** Set the visibility of the Tree header the Value Type Column used for MultiControllers */
	void ShowValueTypeHeaderColumn(bool bInShowColumn);

	/** Set the visibility of the Tree header the Field Id Column */
	void ShowFieldIdHeaderColumn(bool bInShowColumn);

	/** Add a custom column */
	void AddColumn(const FName& InColumnName);
	
	/** Checks whether the provided Entity can be used to create a Controller */
	bool IsEntitySupported(FGuid ExposedEntityId);

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Keeps track of current MultiControllers */
	FRCMultiControllersState MultiControllers;
	
	bool bIsInMultiControllerMode = false;

	/** Storing the header so we can add/remove columns */
	TSharedPtr<SHeaderRow> ControllersHeaderRow;

	/** List of custom columns names */
	TArray<FName> CustomColumns;
};
