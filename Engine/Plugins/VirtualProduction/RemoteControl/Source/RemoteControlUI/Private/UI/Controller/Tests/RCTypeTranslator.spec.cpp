// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "RemoteControlPreset.h"
#include "UI/Controller/TypeTranslator/RCTypeTranslator.h"
#include "UObject/StrongObjectPtr.h"

BEGIN_DEFINE_SPEC(
	FRCTypeTranslatorSpec,
	"Plugins.RemoteControlUI.RCTypeTranslator",
	EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::ProductFilter)
	TStrongObjectPtr<URemoteControlPreset> Preset;
	TStrongObjectPtr<URCVirtualPropertyBase> BoolVirtualProperty;
	TStrongObjectPtr<URCVirtualPropertyBase> StringVirtualProperty;
END_DEFINE_SPEC(FRCTypeTranslatorSpec)

void FRCTypeTranslatorSpec::Define()
{
	//todo: Add tests for other conversions
	
	BeforeEach([this]
	{
		// Init preset
		Preset.Reset(NewObject<URemoteControlPreset>());

		// Create controllers (these could actually be URCVirtualPropertyBase)
		BoolVirtualProperty.Reset(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::Bool, nullptr, TEXT("BoolController")));
		StringVirtualProperty.Reset(Preset->AddController(URCController::StaticClass(), EPropertyBagPropertyType::String, nullptr, TEXT("StringController")));
	});

	Describe(TEXT("From String"), [this]()
	{
		Describe(TEXT("To Bool"), [this]()
		{
			Describe(TEXT("String is Empty"), [this]()
			{
				It("Bool should be set to false", [this]()
				{
					const FString& SourceValue = TEXT("");
					constexpr bool ExpectedResult = false;
					bool Result;
					StringVirtualProperty->SetValueString(SourceValue);
					FRCTypeTranslator::Get()->Translate(StringVirtualProperty.Get(), {BoolVirtualProperty.Get()});					
					BoolVirtualProperty->GetValueBool(Result);
					TestEqual(TEXT("Values should match"), Result, ExpectedResult);
				});
			});

			Describe(TEXT("String is 0"), [this]()
			{
				It("Bool should be set to false", [this]()
				{
					const FString& SourceValue = TEXT("0");
					constexpr bool ExpectedResult = false;
					bool Result;
					StringVirtualProperty->SetValueString(SourceValue);
					FRCTypeTranslator::Get()->Translate(StringVirtualProperty.Get(), {BoolVirtualProperty.Get()});
					BoolVirtualProperty->GetValueBool(Result);
					TestEqual(TEXT("Values should match"), Result, ExpectedResult);
				});
			});

			Describe(TEXT("String is 123"), [this]()
			{
				It("Bool should be set to true", [this]()
				{
					const FString& SourceValue = TEXT("123");
					constexpr bool ExpectedResult = true;
					bool Result;
					StringVirtualProperty->SetValueString(SourceValue);
					FRCTypeTranslator::Get()->Translate(StringVirtualProperty.Get(), {BoolVirtualProperty.Get()});
					BoolVirtualProperty->GetValueBool(Result);
					TestEqual(TEXT("Values should match"), Result, ExpectedResult);
				});
			});
		});
	});
}
