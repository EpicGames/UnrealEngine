// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"

namespace UE::RemoteControl::UI::Private
{

struct FRCControllerPropertyInfo;

class FRCControllerTypeBase
{
public:
	using ItemType = TSharedPtr<FRCControllerPropertyInfo>;

	DECLARE_DELEGATE_OneParam(FOnTypeSelected, ItemType)
};

} // UE::RemoteControl::UI::Private
