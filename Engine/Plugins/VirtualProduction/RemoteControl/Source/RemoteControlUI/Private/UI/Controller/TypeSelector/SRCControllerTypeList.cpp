// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/TypeSelector/SRCControllerTypeList.h"

#include "Framework/Application/SlateApplication.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "UI/Controller/TypeSelector/SRCControllerTypeListEntry.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::RemoteControl::UI::Private
{

void SRCControllerTypeList::Construct(const FArguments& InArgs, const TArray<ItemType>& InTypes)
{
	Types = InTypes;
	Filter = FString();
	OnTypeSelectedDelegate = InArgs._OnTypeSelected;

	Super::Construct(
		Super::FArguments()
		.OnGenerateRow(this, &SRCControllerTypeList::GenerateRow)
		.ListItemsSource(&Types)
		.OnSelectionChanged(this, &SRCControllerTypeList::OnTypeSelected)
		.SelectionMode(ESelectionMode::Single)
	);
}

void SRCControllerTypeList::OnFilterTextChanged(const FText& InFilterText)
{
	const FString NewFilter = InFilterText.ToString();

	if (!Filter.IsEmpty() && NewFilter.StartsWith(Filter))
	{
		UpdateItemsBasedOnFilter(NewFilter, /* Re-Filter */ true);
	}
	else
	{
		UpdateItemsBasedOnFilter(NewFilter, /* Re-Filter */ false);
	}

	Filter = NewFilter;
}

TSharedRef<ITableRow> SRCControllerTypeList::GenerateRow(ItemType InItem, const TSharedRef<STableViewBase>& InOwnerTable)
{
	return SNew(SRCControllerTypeListEntry, InOwnerTable, InItem);
}

void SRCControllerTypeList::UpdateItemsBasedOnFilter(const FString& InFilter, bool bInReFilter)
{
	if (InFilter.IsEmpty())
	{
		SetItemsSource(&Types);
		FilteredTypes.Empty();
		return;
	}

	TArray<ItemType>& UnfilteredTypes = bInReFilter
		? FilteredTypes
		: Types;

	TArray<ItemType> NewTypes;
	NewTypes.Reserve(UnfilteredTypes.Num());

	for (const ItemType& Type : UnfilteredTypes)
	{
		switch (Type->Type)
		{
			case EPropertyBagPropertyType::Enum:
			{
				if (UEnum* Enum = Cast<UEnum>(Type->ValueTypeObject))
				{
					if (Enum->GetDisplayNameText().ToString().Contains(InFilter, ESearchCase::IgnoreCase))
					{
						NewTypes.Add(Type);
					}
				}
				break;
			}

			case EPropertyBagPropertyType::Struct:
			{
				if (UScriptStruct* Struct = Cast<UScriptStruct>(Type->ValueTypeObject))
				{
					if (Struct->GetDisplayNameText().ToString().Contains(InFilter, ESearchCase::IgnoreCase))
					{
						NewTypes.Add(Type);
					}
				}
				break;
			}
		}
	}

	FilteredTypes = NewTypes;
	ClearItemsSource();
	SetItemsSource(&FilteredTypes);
}

void SRCControllerTypeList::OnTypeSelected(ItemType InItem, ESelectInfo::Type InSelectInfo)
{
	FSlateApplication::Get().DismissAllMenus();

	if (OnTypeSelectedDelegate.IsBound())
	{
		OnTypeSelectedDelegate.Execute(InItem);
	}
}

} // UE::RemoteControl::UI::Private
