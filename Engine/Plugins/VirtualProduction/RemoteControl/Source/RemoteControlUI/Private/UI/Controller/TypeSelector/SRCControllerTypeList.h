// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Controller/TypeSelector/RCControllerTypeBase.h"
#include "Widgets/Views/SListView.h"

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Framework/SlateDelegates.h"
#include "Internationalization/Text.h"

namespace UE::RemoteControl::UI::Private
{

struct FRCControllerPropertyInfo;

class SRCControllerTypeList : public SListView<TSharedPtr<FRCControllerPropertyInfo>>, public FRCControllerTypeBase
{	
	using Super = SListView<ItemType>;

public:
	SLATE_BEGIN_ARGS(SRCControllerTypeList)
		{}
		SLATE_ARGUMENT(FOnTypeSelected, OnTypeSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<ItemType>& InTypes);

	void OnFilterTextChanged(const FText& InFilterText);

private:
	TArray<ItemType> Types;
	TArray<ItemType> FilteredTypes;
	FString Filter;
	FOnTypeSelected OnTypeSelectedDelegate;

	TSharedRef<ITableRow> GenerateRow(ItemType InItem, const TSharedRef<STableViewBase>& InOwnerTable);

	/** Setting bReFilter to true will filter the already filtered list. */
	void UpdateItemsBasedOnFilter(const FString& InFilter, bool bInReFilter);

	void OnTypeSelected(ItemType InItem, ESelectInfo::Type InSelectInfo);
};

} // UE::RemoteControl::UI::Private
