// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/TypeSelector/SRCControllerTypeListEntry.h"

#include "UI/Controller/SRCControllerPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::RemoteControl::UI::Private
{

void SRCControllerTypeListEntry::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, ItemType InItem)
{
	Super::Construct(Super::FArguments(), InOwnerTable);

	switch (InItem->Type)
	{
		case EPropertyBagPropertyType::Enum:
		{
			if (UEnum* Enum = Cast<UEnum>(InItem->ValueTypeObject))
			{
				ChildSlot
				[
					SNew(STextBlock)
					.Text(Enum->GetDisplayNameText())
				];
			}
			break;
		}

		case EPropertyBagPropertyType::Struct:
		{
			if (UScriptStruct* Struct = Cast<UScriptStruct>(InItem->ValueTypeObject))
			{
				ChildSlot
				[
					SNew(STextBlock)
					.Text(Struct->GetDisplayNameText())
				];
			}
			break;
		}
	}
}

}
