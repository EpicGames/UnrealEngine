// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Controller/TypeSelector/RCControllerTypeBase.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::RemoteControl::UI::Private
{

struct FRCControllerPropertyInfo;

class SRCControllerTypeListEntry : public STableRow<TSharedPtr<FRCControllerPropertyInfo>>, public FRCControllerTypeBase
{
	using Super = STableRow<ItemType>;

public:
	SLATE_BEGIN_ARGS(SRCControllerTypeListEntry)
		{}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, ItemType InItem);
};

} // UE::RemoteControl::UI::Private
