// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Controller/TypeSelector/SRCControllerTypeSelector.h"

#include "UI/Controller/TypeSelector/SRCControllerTypeList.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

namespace UE::RemoteControl::UI::Private
{

void SRCControllerTypeSelector::Construct(const FArguments& InArgs, const TArray<ItemType>& InTypes)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f, 4.f, 4.f, 4.f)
		[
			SNew(SSearchBox)
			.OnTextChanged(this, &SRCControllerTypeSelector::OnFilterTextCommitted, ETextCommit::Default)
			.OnTextCommitted(this, &SRCControllerTypeSelector::OnFilterTextCommitted)
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f, 0.f, 4.f, 4.f)
		[
			SNew(SBox)
			.WidthOverride(300.f)
			.HeightOverride(300.f)
			[
				SAssignNew(TypeList, SRCControllerTypeList, InTypes)
				.OnTypeSelected(InArgs._OnTypeSelected)
			]
		]
	];
}

void SRCControllerTypeSelector::OnFilterTextCommitted(const FText& InNewText, ETextCommit::Type InCommitInfo)
{
	if (InCommitInfo != ETextCommit::OnUserMovedFocus)
	{
		TypeList->OnFilterTextChanged(InNewText);
	}
}

} // UE::RemoteControl::UI::Private
