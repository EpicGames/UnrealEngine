// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Controller/TypeSelector/RCControllerTypeBase.h"
#include "Widgets/SCompoundWidget.h"

class FText;

namespace UE::RemoteControl::UI::Private
{

class SRCControllerTypeList;

class SRCControllerTypeSelector : public SCompoundWidget, public FRCControllerTypeBase
{
public:
	SLATE_BEGIN_ARGS(SRCControllerTypeSelector)
		{}
		SLATE_ARGUMENT(FOnTypeSelected, OnTypeSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<ItemType>& InTypes);

private:
	TSharedPtr<SRCControllerTypeList> TypeList;

	void OnFilterTextCommitted(const FText& InNewText, ETextCommit::Type InCommitInfo);
};

} // UE::RemoteControl::UI::Private
