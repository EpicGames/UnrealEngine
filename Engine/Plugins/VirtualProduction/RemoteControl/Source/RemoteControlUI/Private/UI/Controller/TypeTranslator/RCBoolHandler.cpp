// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCBoolHandler.h"
#include "RCVirtualProperty.h"

void FRCBoolHandler::Apply(const bool& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCBoolHandler::Apply(const uint8& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, static_cast<bool>(InValue));
}

void FRCBoolHandler::Apply(const double& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, static_cast<bool>(InValue));
}

void FRCBoolHandler::Apply(const float& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, static_cast<bool>(InValue));
}

void FRCBoolHandler::Apply(const int32& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, static_cast<bool>(InValue));
}

void FRCBoolHandler::Apply(const int64& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, static_cast<bool>(InValue));
}

void FRCBoolHandler::Apply(const FName& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCBoolHandler::Apply(const FString& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	if (InVirtualProperty)
	{
		if (InValue.IsEmpty() || InValue.Equals(TEXT("0")))
		{
			InVirtualProperty->SetValueBool(false);
		}
		else
		{
			InVirtualProperty->SetValueBool(true);
		}

		OnControllerPropertyModified(InVirtualProperty);
	}
}

void FRCBoolHandler::Apply(const FText& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCBoolHandler::UpdateValue(URCVirtualPropertyBase* InVirtualProperty, const bool InValue)
{
	if (InVirtualProperty)
	{
		InVirtualProperty->SetValueBool(InValue);
		OnControllerPropertyModified(InVirtualProperty);
	}
}
