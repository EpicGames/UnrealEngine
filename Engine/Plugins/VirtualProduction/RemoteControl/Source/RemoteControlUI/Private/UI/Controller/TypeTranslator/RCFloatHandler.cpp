// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCFloatHandler.h"
#include "RCVirtualProperty.h"

void FRCFloatHandler::Apply(const bool& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const uint8& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const double& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const float& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const int32& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const int64& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCFloatHandler::Apply(const FName& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCFloatHandler::Apply(const FString& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const float OutValue = FCString::Atof(*InValue);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCFloatHandler::Apply(const FText& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCFloatHandler::UpdateValue(URCVirtualPropertyBase* InVirtualProperty, const float InValue)
{
	if (InVirtualProperty)
	{
		InVirtualProperty->SetValueFloat(InValue);
		OnControllerPropertyModified(InVirtualProperty);
	}
}
