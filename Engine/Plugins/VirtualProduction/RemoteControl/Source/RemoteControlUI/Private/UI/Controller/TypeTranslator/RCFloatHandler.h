// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCTypeHandler.h"

class FRCFloatHandler : public FRCTypeHandler
{
public:
	virtual void Apply(const bool& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const uint8& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const double& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const float& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const int32& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const int64& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const FName& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const FString& InValue, URCVirtualPropertyBase* InVirtualProperty) override;
	virtual void Apply(const FText& InValue, URCVirtualPropertyBase* InVirtualProperty) override;

protected:
	static void UpdateValue(URCVirtualPropertyBase* InVirtualProperty, const float InValue);
	~FRCFloatHandler() = default;
};
