// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCStringHandler.h"
#include "RCVirtualProperty.h"

void FRCStringHandler::Apply(const bool& InValue, URCVirtualPropertyBase* InVirtualProperty)
{	
	const FString& OutValue = InValue ? TEXT("true") : TEXT("false");
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const uint8& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const FString& OutValue = FString::SanitizeFloat(InValue, 0);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const double& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const FString& OutValue = FString::SanitizeFloat(InValue);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const float& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const FString& OutValue = FString::SanitizeFloat(InValue);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const int32& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const FString& OutValue = FString::SanitizeFloat(InValue, 0);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const int64& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	const FString& OutValue = FString::SanitizeFloat(InValue, 0);
	UpdateValue(InVirtualProperty, OutValue);
}

void FRCStringHandler::Apply(const FName& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCStringHandler::Apply(const FString& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	UpdateValue(InVirtualProperty, InValue);
}

void FRCStringHandler::Apply(const FText& InValue, URCVirtualPropertyBase* InVirtualProperty)
{
	Apply(InValue.ToString(), InVirtualProperty);
}

void FRCStringHandler::UpdateValue(URCVirtualPropertyBase* InVirtualProperty, const FString& InValue)
{
	if (InVirtualProperty)
	{
		InVirtualProperty->SetValueString(InValue);
		OnControllerPropertyModified(InVirtualProperty);
	}
}
