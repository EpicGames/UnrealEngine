// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCTypeHandler.h"

class FRCStringHandler : public FRCTypeHandler
{
public:
	virtual void Apply(const bool& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const uint8& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const double& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const float& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const int32& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const int64& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const FName& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const FString& InValue, URCVirtualPropertyBase* InController) override;
	virtual void Apply(const FText& InValue, URCVirtualPropertyBase* InController) override;

protected:
	static void UpdateValue(URCVirtualPropertyBase* InVirtualProperty, const FString& InValue);
	~FRCStringHandler() = default;
};
