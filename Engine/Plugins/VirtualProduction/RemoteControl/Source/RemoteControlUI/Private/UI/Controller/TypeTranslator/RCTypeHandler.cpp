// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCTypeHandler.h"
#include "Controller/RCController.h"

void FRCTypeHandler::OnControllerPropertyModified(URCVirtualPropertyBase* InVirtualProperty)
{
	if (URCController* Controller = Cast<URCController>(InVirtualProperty))
	{
		Controller->OnModifyPropertyValue();
	}
}
