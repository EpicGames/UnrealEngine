// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Controller/RCController.h"

class FRCTypeHandler
{
public:
	static void OnControllerPropertyModified(URCVirtualPropertyBase* InVirtualProperty);
	
	virtual void Apply(const bool& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const uint8& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const double& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const float& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const int32& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const int64& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const FName& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const FString& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;
	virtual void Apply(const FText& InValue, URCVirtualPropertyBase* InVirtualProperty) = 0;

protected:
	~FRCTypeHandler() = default;
};
