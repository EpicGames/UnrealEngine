// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCTypeTranslator.h"
#include "Misc/EnumRange.h"
#include "StructUtils/PropertyBag.h"
#include "RCBoolHandler.h"
#include "RCFloatHandler.h"
#include "RCStringHandler.h"

// Needed to nicely cycle through Value Types
ENUM_RANGE_BY_FIRST_AND_LAST(EPropertyBagPropertyType, EPropertyBagPropertyType::Bool, EPropertyBagPropertyType::SoftObject);

FRCTypeTranslator::FRCTypeTranslator()
{
	for (const EPropertyBagPropertyType RCValueType : TEnumRange<EPropertyBagPropertyType>())
	{
		FRCTypeHandler* TypeHandler = CreateTypeHandler(RCValueType);
		TypeHandlers.Emplace(RCValueType, TypeHandler);
	}
}

FRCTypeTranslator* FRCTypeTranslator::Get()
{
	if (!TypeTranslatorSingleton)
	{
		TypeTranslatorSingleton = new FRCTypeTranslator();
	}

	return TypeTranslatorSingleton;
}

void FRCTypeTranslator::Translate(const URCVirtualPropertyBase* InSourceVirtualProperty, const TArray<URCVirtualPropertyBase*>& InTargetVirtualProperties)
{
	if (InSourceVirtualProperty)
	{
		const EPropertyBagPropertyType SourceValueType = InSourceVirtualProperty->GetValueType();

		// Switch could have been an option, but if/else seems more readable,
		// especially since each case is defining a type variable and that would lead to a lot of brackets
		if (SourceValueType == EPropertyBagPropertyType::Bool)
		{
			bool Value;
			InSourceVirtualProperty->GetValueBool(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Byte)
		{
			uint8 Value;
			InSourceVirtualProperty->GetValueByte(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Double)
		{
			double Value;
			InSourceVirtualProperty->GetValueDouble(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Float)
		{
			float Value;
			InSourceVirtualProperty->GetValueFloat(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Int32)
		{
			int32 Value;
			InSourceVirtualProperty->GetValueInt32(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Int64)
		{
			int64 Value;
			InSourceVirtualProperty->GetValueInt64(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Name)
		{
			FName Value;
			InSourceVirtualProperty->GetValueName(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::String)
		{
			FString Value;
			InSourceVirtualProperty->GetValueString(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Text)
		{
			FText Value;
			InSourceVirtualProperty->GetValueText(Value);
			Apply(Value, InTargetVirtualProperties);
		}
		else if (SourceValueType == EPropertyBagPropertyType::Struct)
		{
			// todo: Work out structs (e.g. vector, rotator, color, etc)
		}

		// todo: Handle other types if needed, or remove/collapse types
	}
}

EPropertyBagPropertyType FRCTypeTranslator::GetOptimalValueType(const TArray<EPropertyBagPropertyType>& ValueTypes)
{
	// todo: Improve this function, it is currently just going in order!
	EPropertyBagPropertyType OptimalType = EPropertyBagPropertyType::Bool;
	for (const EPropertyBagPropertyType ValueType : ValueTypes)
	{
		if (OptimalType < ValueType)
		{
			OptimalType = ValueType;
		}
	}

	return OptimalType;
}

FRCTypeHandler* FRCTypeTranslator::CreateTypeHandler(EPropertyBagPropertyType InValueType)
{
	// todo: Implement missing types
	switch (InValueType)
	{
		case EPropertyBagPropertyType::None:
			break;

		case EPropertyBagPropertyType::Bool:
			return new FRCBoolHandler();

		case EPropertyBagPropertyType::Byte:
			break;
			
		case EPropertyBagPropertyType::Int32:
			break;
			
		case EPropertyBagPropertyType::Int64:
			break;
			
		case EPropertyBagPropertyType::Float:
			return new FRCFloatHandler();
			
		case EPropertyBagPropertyType::Double:
			break;
			
		case EPropertyBagPropertyType::Name:
			break;
			
		case EPropertyBagPropertyType::String:
			return new FRCStringHandler();
			
		case EPropertyBagPropertyType::Text:
			break;
			
		case EPropertyBagPropertyType::Struct:
			break;
				
		default: ;
	}

	return nullptr;
}
