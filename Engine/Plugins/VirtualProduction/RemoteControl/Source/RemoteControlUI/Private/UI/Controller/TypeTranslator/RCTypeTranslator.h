// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StructUtils/PropertyBag.h"
#include "RCTypeHandler.h"
#include "Controller/RCController.h"

class FRCTypeTranslator
{
public:
	static FRCTypeTranslator* Get();

	/** 
	 * Conversion function: tries to use the Source Virtual Property current value to update the specified Target Virtual Properties values.
	 * Particularly useful for MultiControllers.
	 *
	 * @param InSourceVirtualProperty the virtual property providing the source value
	 * @param InTargetVirtualProperties the properties which will be updated using the source value
	 */
	static void Translate(const URCVirtualPropertyBase* InSourceVirtualProperty, const TArray<URCVirtualPropertyBase*>& InTargetVirtualProperties);

	/** Returns the "optimal" value type among the specified list */
	static EPropertyBagPropertyType GetOptimalValueType(const TArray<EPropertyBagPropertyType>& ValueTypes);
	
private:
	template<typename InValueType>
	static void Apply(const InValueType& Value, const TArray<URCVirtualPropertyBase*>& InTargetVirtualProperties)
	{
		for (URCVirtualPropertyBase* TargetVirtualProperty : InTargetVirtualProperties)
		{
			if (!TargetVirtualProperty)
			{
				continue;
			}

			if (FRCTypeHandler** TypeHandlerPtr = TypeHandlers.Find(TargetVirtualProperty->GetValueType()))
			{
				if (FRCTypeHandler* TypeHandler = *TypeHandlerPtr)
				{
					TypeHandler->Apply(Value, TargetVirtualProperty);
				}
			}
		}
	}

	static FRCTypeHandler* CreateTypeHandler(EPropertyBagPropertyType InValueType);

	FRCTypeTranslator();
	
	inline static FRCTypeTranslator* TypeTranslatorSingleton = nullptr;
	inline static TMap<EPropertyBagPropertyType, FRCTypeHandler*> TypeHandlers;
};
