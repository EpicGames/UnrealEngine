// Copyright Epic Games, Inc. All Rights Reserved.

#include "FPassphraseCustomization.h"

#include "DetailWidgetRow.h"
#include "RemoteControlSettings.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"

void FPassphraseCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<SEditableTextBox> IdentifierBox;
	TSharedPtr<SEditableTextBox> PassphraseBox;
	TSharedPtr<SButton> EditButton;
	TSharedPtr<SCheckBox> CheckBox;
	
	TSharedPtr<IPropertyHandle> IdentifierHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCPassphrase, Identifier));
	TSharedPtr<IPropertyHandle> PassphraseHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCPassphrase, Passphrase));

	HeaderRow.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.AutoWidth()
		[
			SAssignNew(IdentifierBox, SEditableTextBox)
			.IsReadOnly(true)
			.Text_Lambda([this, IdentifierHandle]()
			{
				FString StringValue;
				IdentifierHandle->GetValue(StringValue);
				
				FText OutText = FText::FromString(StringValue);
				return OutText;
			})
			.OnTextCommitted_Lambda([this, IdentifierHandle](const FText& Text, ETextCommit::Type CommitType)
			{
				IdentifierHandle->SetValue(Text.ToString());
			})
		]
	];

	HeaderRow.ValueContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Left)
		[
			SAssignNew(PassphraseBox, SEditableTextBox)
			.IsReadOnly(true)
			.IsPassword(true)
			.HintText(FText::FromString("Passphrase"))
			.SelectAllTextWhenFocused(true)
			.Text_Lambda([this, PassphraseHandle]()
			{
				FString OutString;
				PassphraseHandle->GetValue(OutString);
				
				return FText::FromString(OutString);
			})
			.OnTextCommitted_Lambda([this, PassphraseHandle](const FText& Text, ETextCommit::Type CommitType)
			{
				// Only Commits using Enter
				if (ETextCommit::OnEnter == CommitType)
				{
					PassphraseHandle->SetValue(FMD5::HashAnsiString(*Text.ToString()));
				}
			})
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Right)
		[
			SAssignNew(CheckBox, SCheckBox)
			.ToolTipText(FText::FromString("Unhide Password"))
			.IsChecked(ECheckBoxState::Unchecked)
			.OnCheckStateChanged_Lambda([this, PassphraseBox](ECheckBoxState CheckState)
			{
				PassphraseBox->SetIsPassword(CheckState == ECheckBoxState::Checked ? false : true);
			})
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[
			SAssignNew(EditButton, SButton)
			.Text(FText::FromString("Edit"))
			.OnClicked_Lambda([PassphraseBox, IdentifierBox]()
			{
				IdentifierBox->SetIsReadOnly(!IdentifierBox->IsReadOnly());
				PassphraseBox->SetIsReadOnly(!PassphraseBox->IsReadOnly());
				return FReply::Handled();
			})
		]
	];
}
