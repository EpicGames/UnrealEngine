// Copyright Epic Games, Inc. All Rights Reserved.

#include "NetworkAddressCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "RemoteControlSettings.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"

#define LOCTEXT_NAMESPACE "NetworkAddressCustomization"

TSharedRef<IPropertyTypeCustomization> FNetworkAddressCustomization::MakeInstance()
{
	// Create the instance and returned a SharedRef
	return MakeShareable(new FNetworkAddressCustomization());
}

void FNetworkAddressCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// First we need to retrieve every Property handles
	TSharedPtr<IPropertyHandle> ClassAPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCNetworkAddress, ClassA));
	TSharedPtr<IPropertyHandle> ClassBPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCNetworkAddress, ClassB));
	TSharedPtr<IPropertyHandle> ClassCPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCNetworkAddress, ClassC));
	TSharedPtr<IPropertyHandle> ClassDPropertyHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCNetworkAddress, ClassD));

	// Check them before using them
	check(ClassAPropertyHandle.IsValid()
		&& ClassBPropertyHandle.IsValid()
		&& ClassCPropertyHandle.IsValid()
		&& ClassDPropertyHandle.IsValid());

	constexpr bool bUseAllottedWidth = true;

	const float IconSizeSquared = 4;

	const TSharedRef<SWidget> DotIconWidget = SNew(SBox)
		.HeightOverride(IconSizeSquared)
		.WidthOverride(IconSizeSquared)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.FilledCircle"))
		];

	HeaderRow
		.NameContent()
		[
			StructPropertyHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		[
			SNew(SWrapBox)
			.UseAllottedSize(bUseAllottedWidth)

			+ SWrapBox::Slot()
			.Padding(2.f, 0.f)
			[
				ClassAPropertyHandle->CreatePropertyValueWidget()
			]
		
			+ SWrapBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				DotIconWidget
			]

			+ SWrapBox::Slot()
			.Padding(2.f, 0.f)
			[
				ClassBPropertyHandle->CreatePropertyValueWidget()
			]
			
			+ SWrapBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				DotIconWidget
			]

			+ SWrapBox::Slot()
			.Padding(2.f, 0.f)
			[
				ClassCPropertyHandle->CreatePropertyValueWidget()
			]
			
			+ SWrapBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2.f, 2.f)
			[
				DotIconWidget
			]

			+ SWrapBox::Slot()
			.Padding(2.f, 0.f)
			[
				ClassDPropertyHandle->CreatePropertyValueWidget()
			]
		];
}

void FNetworkAddressCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

#undef LOCTEXT_NAMESPACE
