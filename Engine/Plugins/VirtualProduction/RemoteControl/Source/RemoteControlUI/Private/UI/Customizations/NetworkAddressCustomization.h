// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"

class FDetailWidgetRow;
class IDetailChildrenBuilder;

/**
 * 
 */
class FNetworkAddressCustomization : public IPropertyTypeCustomization
{
public:

	/**
	 * It is just a convenient helpers which will be used to register our customization.
	 * When the propertyEditor module find our FRCNetworkAddress property,
	 * it will use this static method to instanciate our customization object.
	 */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ BEGIN : IPropertyTypeCustomization Interface
	
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	
	//~ END : IPropertyTypeCustomization Interface
};
