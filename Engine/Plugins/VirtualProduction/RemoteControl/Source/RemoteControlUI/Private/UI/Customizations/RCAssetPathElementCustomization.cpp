// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCAssetPathElementCustomization.h"

#include "Behaviour/Builtin/Path/RCSetAssetByPathBehaviour.h"
#include "DetailLayoutBuilder.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "IRemoteControlPropertyHandle.h"
#include "PropertyCustomizationHelpers.h"
#include "UI/Behaviour/Builtin/Path/RCBehaviorSetAssetByPathModelNew.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRCPanelExposedEntitiesList.h"
#include "UI/SRCPanelExposedField.h"
#include "UI/SRCPanelTreeNode.h"
#include "UI/SRemoteControlPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RCAssetPathElementCustomization"

class SRCAssetPathSelectorButton : public SButton
{
public:
	SLATE_BEGIN_ARGS(SRCAssetPathSelectorButton)
		{}
		SLATE_EVENT(FOnGetContent, OnGetMenuContent)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedRef<FRCAssetPathElementCustomization> InCustomization)
	{
		CustomizationWeak = InCustomization;
		OnGetMenuContent = InArgs._OnGetMenuContent;

		const TSharedRef<SToolTip> GetAssetPathToolTipWidget = SNew(SToolTip)
			.Text(LOCTEXT("RCGetAssetPathButton_Tooltip", 
				"Get the path of selected assets"
				"\n\n"
				"- Left click will attempt to get the first selected asset in the content browser and then the first valid selected exposed entity."
				"\n"
				"- Right click will open a menu allowing you to specify where to look, overriding the default priority."
			));

		SButton::Construct(
			SButton::FArguments()
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.OnClicked(this, &SRCAssetPathSelectorButton::OnLeftClicked)
			.ToolTip(GetAssetPathToolTipWidget)
			.IsFocusable(false)
			.ContentPadding(0)
			.Content()
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Use"))
				.ColorAndOpacity(FSlateColor::UseForeground())
			]
		);
	}

	virtual FReply OnMouseButtonDown(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (!IsEnabled())
		{
			return FReply::Unhandled();
		}

		if (InMouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
		{
			if (OnGetMenuContent.IsBound())
			{
				if (TSharedPtr<SWidget> MenuContent = OnGetMenuContent.Execute())
				{
					FSlateApplication& SlateApplication = FSlateApplication::Get();

					FWidgetPath MyWidgetPath;
					SlateApplication.GeneratePathToWidgetUnchecked(AsShared(), MyWidgetPath);

					const FVector2D CursorLocation = SlateApplication.GetCursorPos();

					SlateApplication.PushMenu(AsShared(), MyWidgetPath, MenuContent.ToSharedRef(), CursorLocation + FVector2D(16), FPopupTransitionEffect::ContextMenu);
				}
			}
				
			return FReply::Handled();
		}

		return SButton::OnMouseButtonDown(InMyGeometry, InMouseEvent);
	}

protected:
	TWeakPtr<FRCAssetPathElementCustomization> CustomizationWeak;
	FOnGetContent OnGetMenuContent;

	FReply OnLeftClicked()
	{
		if (TSharedPtr<FRCAssetPathElementCustomization> Customization = CustomizationWeak.Pin())
		{
			return Customization->OnGetAssetFromSelectionClicked();
		}

		return FReply::Unhandled();
	}
};

TSharedRef<IPropertyTypeCustomization> FRCAssetPathElementCustomization::MakeInstance()
{
	return MakeShared<FRCAssetPathElementCustomization>(nullptr);
}

TSharedRef<IPropertyTypeCustomization> FRCAssetPathElementCustomization::MakeInstance(TSharedPtr<FRCSetAssetByPathBehaviorModelNew> InPathBehaviorModelNew)
{
	return MakeShared<FRCAssetPathElementCustomization>(InPathBehaviorModelNew);
}

FRCAssetPathElementCustomization::FRCAssetPathElementCustomization(TSharedPtr<FRCSetAssetByPathBehaviorModelNew> InPathBehaviorModelNew)
	: PathBehaviorModelNewWeak(InPathBehaviorModelNew)
{

}

void FRCAssetPathElementCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
	ArrayEntryHandle = InPropertyHandle;
	PropertyUtilities = InCustomizationUtils.GetPropertyUtilities();
	IsInputHandle = ArrayEntryHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCAssetPathElement, bIsInput));
	PathHandle = ArrayEntryHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FRCAssetPathElement, Path));

	if (!IsInputHandle.IsValid() || !PathHandle.IsValid())
	{
		return;
	}

	PathHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateSP(this, &FRCAssetPathElementCustomization::OnPathChanged));

	const TSharedRef<SToolTip> CreateControllerToolTipWidget = SNew(SToolTip)
		.Text(LOCTEXT("RCCreateController_Tooltip", "Create a controller for the given RC Input path entry"));

	InHeaderRow.NameContent()
		[
			InPropertyHandle->CreatePropertyNameWidget()
		];

	InHeaderRow.ValueContent()
	[
		SNew(SHorizontalBox)
        // RC Input CheckBox
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(5.f, 0.f)
		.VAlign(VAlign_Center)
		[
			SNew(SCheckBox)
			.Style(&FRemoteControlPanelStyle::Get()->GetWidgetStyle<FCheckBoxStyle>("RemoteControlPathBehaviour.AssetCheckBox"))
			.IsChecked(this, &FRCAssetPathElementCustomization::IsChecked)
			.OnCheckStateChanged(this, &FRCAssetPathElementCustomization::OnCheckStateChanged)
			.IsFocusable(false)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RCInputButtonAssetPath", "Controller"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]

		// Path String
		+SHorizontalBox::Slot()
		.FillWidth(1.f)
		.Padding(5.f, 0.f)
		.VAlign(VAlign_Center)
		[
			PathHandle->CreatePropertyValueWidget(false)
		]

		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SWidgetSwitcher)
			.WidgetIndex(this, &FRCAssetPathElementCustomization::OnGetWidgetSwitcherIndex)

			// [0] Get Current Selected Asset Path Button
			+ SWidgetSwitcher::Slot()
			[
				SNew(SRCAssetPathSelectorButton, SharedThis(this))
				.OnGetMenuContent(FOnGetContent::CreateSP(this, &FRCAssetPathElementCustomization::GetPathSelectorMenuContent))
			]

			// [1] Create Controller button
			+ SWidgetSwitcher::Slot()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.IsEnabled(this, &FRCAssetPathElementCustomization::IsCreateControllerButtonEnabled)
				.OnClicked(this, &FRCAssetPathElementCustomization::OnCreateControllerButtonClicked)
				.ToolTip(CreateControllerToolTipWidget)
				.IsFocusable(false)
				.ContentPadding(0)
				.Content()
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
	];
}

void FRCAssetPathElementCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils)
{
}

void FRCAssetPathElementCustomization::OnPathChanged()
{
	if (!PathHandle.IsValid() || !IsInputHandle.IsValid())
	{
		return;
	}

	bool bIsRCInput = false;
	IsInputHandle->GetValue(bIsRCInput);
}

ECheckBoxState FRCAssetPathElementCustomization::IsChecked() const
{
	ECheckBoxState ReturnValue = ECheckBoxState::Undetermined;
	if (IsInputHandle.IsValid())
	{
		bool bIsInput;
		const FPropertyAccess::Result Result = IsInputHandle->GetValue(bIsInput);
		if (Result == FPropertyAccess::Success)
		{
			ReturnValue = bIsInput ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
	}
	return ReturnValue;
}

void FRCAssetPathElementCustomization::OnCheckStateChanged(ECheckBoxState InNewState)
{
	if (!IsInputHandle.IsValid())
	{
		return;
	}
	const bool bIsInput = InNewState == ECheckBoxState::Checked;
	IsInputHandle->SetValue(bIsInput);
}

FReply FRCAssetPathElementCustomization::OnGetAssetFromSelectionClicked()
{
	SetAssetFromPath(GetSelectedAssetPath());
	
	return FReply::Handled();
}

bool FRCAssetPathElementCustomization::IsCreateControllerButtonEnabled() const
{
	FString Value;
	PathHandle->GetValue(Value);

	if (Value.IsEmpty())
	{
		return true;
	}

	TSharedPtr<FRCSetAssetByPathBehaviorModelNew> PathBehaviorModelNew = PathBehaviorModelNewWeak.Pin();

	if (!PathBehaviorModelNew.IsValid())
	{
		return false;
	}

	URemoteControlPreset* Preset = PathBehaviorModelNew->GetPreset();

	if (!Preset)
	{
		return false;
	}

	return !Preset->GetControllerByDisplayName(*Value);
}

FReply FRCAssetPathElementCustomization::OnCreateControllerButtonClicked()
{
	if (PropertyUtilities.IsValid() && ArrayEntryHandle.IsValid())
	{
		FPropertyChangedEvent Event(ArrayEntryHandle->GetProperty(), EPropertyChangeType::ValueSet);
		Event.MemberProperty = ArrayEntryHandle->GetProperty();

		TMap<FString, int32> ArrayIndexPerObject;
		ArrayIndexPerObject.Add(Event.GetMemberPropertyName().ToString(), ArrayEntryHandle->GetArrayIndex());

		Event.SetArrayIndexPerObject(MakeArrayView(&ArrayIndexPerObject, 1));
		Event.ObjectIteratorIndex = 0;
		PropertyUtilities->NotifyFinishedChangingProperties(Event);
	}
	return FReply::Handled();
}

int32 FRCAssetPathElementCustomization::OnGetWidgetSwitcherIndex() const
{
	return IsChecked() == ECheckBoxState::Checked ? 1 : 0;
}

FString FRCAssetPathElementCustomization::GetSelectedAssetPath() const
{
	FString AssetPath = GetSelectedAssetPath_ContentBrowser();

	if (!AssetPath.IsEmpty())
	{
		return AssetPath;
	}

	return GetSelectedAssetPath_EntityList();
}

FString FRCAssetPathElementCustomization::GetSelectedAssetPath_ContentBrowser() const
{
	TArray<FAssetData> AssetData;
	GEditor->GetContentBrowserSelections(AssetData);

	if (AssetData.Num() > 0)
	{
		const UObject* SelectedAsset = AssetData[0].GetAsset();
		return SelectedAsset->GetPathName();
	}

	return FString();
}

FString FRCAssetPathElementCustomization::GetSelectedAssetPath_EntityList() const
{
	TSharedPtr<FRCSetAssetByPathBehaviorModelNew> PathBehaviorModel = PathBehaviorModelNewWeak.Pin();

	if (!PathBehaviorModel.IsValid())
	{
		return FString();
	}

	TSharedPtr<SRemoteControlPanel> RCPanel = PathBehaviorModel->GetRemoteControlPanel();

	if (!RCPanel.IsValid())
	{
		return FString();
	}

	TSharedPtr<SRCPanelExposedEntitiesList> EntityList = RCPanel->GetEntityList();

	if (!EntityList.IsValid())
	{
		return FString();
	}

	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedEntities = EntityList->GetSelectedEntities();

	for (const TSharedPtr<SRCPanelTreeNode>& SelectedEntity : SelectedEntities)
	{
		if (!SelectedEntity.IsValid())
		{
			continue;
		}

		if (SelectedEntity->GetRCType() != SRCPanelTreeNode::Field)
		{
			continue;
		}

		TSharedPtr<FRemoteControlField> RCField = StaticCastSharedPtr<SRCPanelExposedField>(SelectedEntity)->GetRemoteControlField().Pin();

		if (!RCField.IsValid() || RCField->FieldType != EExposedFieldType::Property || !RCField->IsBound())
		{
			continue;
		}

		TSharedPtr<IRemoteControlPropertyHandle> PropertyHandle = StaticCastSharedPtr<FRemoteControlProperty>(RCField)->GetPropertyHandle();

		if (!PropertyHandle.IsValid())
		{
			continue;
		}

		UObject* Object = nullptr;

		if (PropertyHandle->GetValue(Object))
		{
			if (Object && Object->IsAsset())
			{
				return Object->GetPathName();
			}
		}
	}

	return FString();
}

TSharedRef<SWidget> FRCAssetPathElementCustomization::GetPathSelectorMenuContent()
{
	FMenuBuilder MenuBuilder(/* Should close after selection */ true, /* Command list */ nullptr);

	MenuBuilder.BeginSection("SelectSource", LOCTEXT("SelectSource", "Select Source"));

	FUIAction ContentBrowserSource(
		FExecuteAction::CreateSP(this, &FRCAssetPathElementCustomization::SetAssetPathFromContentBrowser)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("ContentBrowser", "Content Browser"),
		LOCTEXT("ContentBrowserTooltip", "Take the asset path from the selected assets in the content browser."),
		FSlateIcon(),
		ContentBrowserSource
	);

	FUIAction EntityListSource(
		FExecuteAction::CreateSP(this, &FRCAssetPathElementCustomization::SetAssetPathFromEntityList)
	);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("EntityList", "Entity List"),
		LOCTEXT("EntityListTooltip", "Take the asset path from the first valid selected exposed entity."),
		FSlateIcon(),
		EntityListSource
	);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FRCAssetPathElementCustomization::SetAssetPathFromContentBrowser()
{
	SetAssetFromPath(GetSelectedAssetPath_ContentBrowser());
}

void FRCAssetPathElementCustomization::SetAssetPathFromEntityList()
{
	SetAssetFromPath(GetSelectedAssetPath_EntityList());
}

void FRCAssetPathElementCustomization::SetAssetFromPath(FString InPath)
{
	if (InPath.IsEmpty())
	{
		return;
	}

	// Remove the initial Game
	InPath.RemoveFromStart("/Game/");

	// Clear it, in case it is an already used one.
	// Use the first one in the Array
	int32 IndexOfLast;

	// Remove anything after the last /
	InPath.FindLastChar('/', IndexOfLast);

	if (IndexOfLast != INDEX_NONE)
	{
		if (IndexOfLast != (InPath.Len() - 1))
		{
			// Keep the slash
			InPath.RemoveAt(IndexOfLast + 1, InPath.Len() - IndexOfLast - 1);
		}
	}
	else
	{
		// if the Index is -1 then it means that we are selecting an asset already in the topmost folder
		// So we clear the string since it will just contains the AssetName
		InPath.Empty();
	}

	if (IsInputHandle.IsValid())
	{
		IsInputHandle->SetValue(false);
	}
	if (PathHandle.IsValid())
	{
		PathHandle->SetValue(InPath);
	}
}

#undef LOCTEXT_NAMESPACE
