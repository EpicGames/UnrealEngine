// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Styling/SlateTypes.h"

class FDetailWidgetRow;
class FRCSetAssetByPathBehaviorModelNew;
class IDetailChildrenBuilder;
class IPropertyHandle;

class FRCAssetPathElementCustomization : public IPropertyTypeCustomization
{
	friend class SRCAssetPathSelectorButton;

public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TSharedPtr<FRCSetAssetByPathBehaviorModelNew> InPathBehaviorModelNew);

	FRCAssetPathElementCustomization(TSharedPtr<FRCSetAssetByPathBehaviorModelNew> InPathBehaviorModelNew);

	//~ Begin IPropertyTypeCustomization
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> InPropertyHandle, FDetailWidgetRow& InHeaderRow, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InPropertyHandle, IDetailChildrenBuilder& InChildBuilder, IPropertyTypeCustomizationUtils& InCustomizationUtils) override;
	//~ End IPropertyTypeCustomization

private:
	/** Called when the path change */
	void OnPathChanged();

	/** Returns if the CheckBox is checked or not */
	ECheckBoxState IsChecked() const;

	/** Callback called when you click on the CheckBox to enable/disable it */
	void OnCheckStateChanged(ECheckBoxState InNewState);

	/** Callback called when you click on the arrow button next to the entries to retrieve the path of the selected Asset */
	FReply OnGetAssetFromSelectionClicked();

	/** Only lets you click the button if the path is empty. */
	bool IsCreateControllerButtonEnabled() const;

	/** Callback called when you click on the plus button next to the entries to create a controller associated with the current RC Input entry */
	FReply OnCreateControllerButtonClicked();

	/** Retrieve the current widget switcher index */
	int32 OnGetWidgetSwitcherIndex() const;

	FString GetSelectedAssetPath() const;

	FString GetSelectedAssetPath_ContentBrowser() const;

	FString GetSelectedAssetPath_EntityList() const;

	TSharedRef<SWidget> GetPathSelectorMenuContent();

	void SetAssetPathFromContentBrowser();

	void SetAssetPathFromEntityList();

	void SetAssetFromPath(FString InPath);

private:
	TWeakPtr<FRCSetAssetByPathBehaviorModelNew> PathBehaviorModelNewWeak;
	TSharedPtr<SWidget> PathWidget;
	TSharedPtr<IPropertyUtilities> PropertyUtilities;
	TSharedPtr<IPropertyHandle> ArrayEntryHandle;
	TSharedPtr<IPropertyHandle> IsInputHandle;
	TSharedPtr<IPropertyHandle> PathHandle;
};
