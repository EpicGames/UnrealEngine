// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlEntityCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDocumentation.h"
#include "IPropertyRowGenerator.h"
#include "RemoteControlBinding.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "FRemoteControlEntityCustomization"

namespace RemoteControlEntityCustomizationUtils
{
	/** Create a string : string detail row.  */
	void CreateDefaultWidget(const TPair<FString, FString>& Entry, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder, bool bReadOnly = false)
	{
		const FSlateFontInfo FontInfo = LayoutBuilder.GetDetailFont();
		FDetailWidgetRow& Row = CategoryBuilder.AddCustomRow(FText::FromString(Entry.Key))
        .NameContent()
        [
			SNew(STextBlock)
            .Font(FontInfo)
            .Text(FText::FromString(Entry.Key))
        ]
        .ValueContent()
        [
	        SNew(SEditableTextBox)
        	.Font(FontInfo)
        	.IsReadOnly(bReadOnly)
        	.Text(FText::FromString(Entry.Value))
        ];
	}
}

FRemoteControlEntityCustomization::FRemoteControlEntityCustomization()
{
	MetadataCustomizations.Add(FName("Min"), FOnCustomizeMetadataEntry::CreateRaw(this, &FRemoteControlEntityCustomization::CreateRangeWidget));
	
	// Hide the Max metadata since it is handled by Min.
	MetadataCustomizations.Add(FName("Max"), FOnCustomizeMetadataEntry::CreateLambda([](URemoteControlPreset*, const FGuid&, IDetailLayoutBuilder&, IDetailCategoryBuilder&){}));
}

void FRemoteControlEntityCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TSharedPtr<FStructOnScope>> Structs;
	DetailBuilder.GetStructsBeingCustomized(Structs);
	
	// At the moment multiple entity selection is not enabled in the preset,
	// If that is eventually enabled this ensure will be triggered.
	if (ensure(Structs.Num() == 1 && Structs[0]))
	{
		DisplayedEntity = Structs[0];
	}

	IDetailCategoryBuilder& EntityCategory = DetailBuilder.EditCategory("RemoteControlEntity");

	// Hide all properties by default.
	TArray<TSharedRef<IPropertyHandle>> Handles;
	EntityCategory.GetDefaultProperties(Handles);
	
	for (const TSharedRef<IPropertyHandle>& Handle : Handles)
	{
		DetailBuilder.HideProperty(Handle);
	}
	
	// Then re-add the ones we care about in the right order.
	GeneratePropertyRows(DetailBuilder, EntityCategory);
}

TSharedRef<IDetailCustomization> FRemoteControlEntityCustomization::MakeInstance()
{
	return MakeShared<FRemoteControlEntityCustomization>();
}

void FRemoteControlEntityCustomization::CreateRangeWidget(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder)
{
	const FSlateFontInfo FontInfo = LayoutBuilder.GetDetailFont();
	FText ToolTipText = LOCTEXT("ToolTip", "Maximum and minimum values hints for external applications.");
		
	CategoryBuilder.AddCustomRow( LOCTEXT("ValueRangeLabel", "Value Range") )
    .NameContent()
    [
        SNew(STextBlock)
        .Text(LOCTEXT("RangeLabel", "Range"))
        .ToolTipText(ToolTipText)
        .Font(FontInfo)
    ]
    .ValueContent()
    [
        SNew(SHorizontalBox)
       .ToolTipText(ToolTipText)
       +SHorizontalBox::Slot()
       .FillWidth(1)
       [
           SNew(SEditableTextBox)
           .Text(this, &FRemoteControlEntityCustomization::GetMetadataValue, FRemoteControlProperty::MetadataKey_Min)
           .OnTextCommitted_Raw(this, &FRemoteControlEntityCustomization::OnMetadataKeyCommitted, FRemoteControlProperty::MetadataKey_Min)
           .Font(FontInfo)
       ]
       +SHorizontalBox::Slot()
       .AutoWidth()
       [
           SNew(STextBlock)
           .Text(INVTEXT(" .. "))
           .Font(FontInfo)
       ]
       +SHorizontalBox::Slot()
       .FillWidth(1)
       [
			SNew(SEditableTextBox)
			.Text(this, &FRemoteControlEntityCustomization::GetMetadataValue, FRemoteControlProperty::MetadataKey_Max)
			.OnTextCommitted_Raw(this, &FRemoteControlEntityCustomization::OnMetadataKeyCommitted, FRemoteControlProperty::MetadataKey_Max)
			.Font(FontInfo)
       ]
    ];
}

void FRemoteControlEntityCustomization::CreateBindingWidget(const FRemoteControlEntity* RCEntity, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder) const
{
	const FSlateFontInfo FontInfo = LayoutBuilder.GetDetailFont();
	FText ToolTipText = LOCTEXT("RCBindingToolTip", "The object this is currently bound to.");

	FText PathNameText;

	if (ensure(RCEntity))
	{
		if (UObject* Obj = RCEntity->GetBoundObject())
		{
			PathNameText = FText::FromString(Obj->GetPathName());
		}
		else
		{
			FString BindingPath = RCEntity->GetLastBindingPath().ToString();
			if (!BindingPath.IsEmpty())
			{
				PathNameText = FText::Format(LOCTEXT("InvalidBindingPath", "INVALID_BINDING: {0}"), FText::FromString(BindingPath));
			}
		}
	}

	if (PathNameText.IsEmpty())
	{
		PathNameText = LOCTEXT("NoBindingInformation", "INVALID_BINDING: - This entity has no binding information.");
	}

	CategoryBuilder.AddCustomRow(LOCTEXT("BindingLabel", "Binding"))
		.NameContent()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BindingLabel", "Binding"))
			.ToolTipText(ToolTipText)
			.Font(FontInfo)
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			.ToolTipText(ToolTipText)
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			[
				SNew(SEditableTextBox)
				.IsReadOnly(true)
				.Text(PathNameText)
				.Font(FontInfo)
			]
		];
}

void FRemoteControlEntityCustomization::OnMetadataKeyCommitted(const FText& Text, ETextCommit::Type Type, FName MetadataKey)
{
	if (FRemoteControlEntity* Entity = GetEntityPtr())
	{
		check(Entity->GetOwner());
		FScopedTransaction Transaction(LOCTEXT("ModifiedMetadataTransaction", "Modified exposed entity metadata."));
		Entity->GetOwner()->Modify();
		Entity->SetMetadataValue(MetadataKey, Text.ToString());
	}
}

FRemoteControlEntity* FRemoteControlEntityCustomization::GetEntityPtr()
{
	if (!ensure(DisplayedEntity && DisplayedEntity->IsValid()))
    {
    	return nullptr;
    }

    return reinterpret_cast<FRemoteControlEntity*>(DisplayedEntity->GetStructMemory());
}

const FRemoteControlEntity* FRemoteControlEntityCustomization::GetEntityPtr() const
{
	if (!ensure(DisplayedEntity && DisplayedEntity->IsValid()))
	{
		return nullptr;
	}

	return reinterpret_cast<FRemoteControlEntity*>(DisplayedEntity->GetStructMemory());
}

FText FRemoteControlEntityCustomization::GetMetadataValue(FName Key) const
{
	FText Value;
	
	if (const FRemoteControlEntity* Entity = GetEntityPtr())
	{
		if (const FString* FoundValue = Entity->GetMetadata().Find(Key))
		{
			Value = FText::FromString(*FoundValue);
		}
	}
	return Value;
}

void FRemoteControlEntityCustomization::GeneratePropertyRows(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder)
{
	auto AddPropertyRow = [&DetailBuilder, &CategoryBuilder](const FString&& Name, const FString& Value, bool bReadOnly)
	{
		RemoteControlEntityCustomizationUtils::CreateDefaultWidget(TPair<FString, FString>{Name, Value}, DetailBuilder, CategoryBuilder, bReadOnly);
	};
	
	if (DisplayedEntity && DisplayedEntity->IsValid())
	{
		if (FRemoteControlEntity* Entity = GetEntityPtr())
		{
			// Add RCEntity properties.
			AddPropertyRow(TEXT("Label"), Entity->GetLabel().ToString(), /*bReadOnly=*/true);
			AddPropertyRow(TEXT("Id"), Entity->GetId().ToString(), /*bReadOnly=*/true);

			// Add RCField properties.
			if (DisplayedEntity->GetStruct()->IsChildOf(FRemoteControlField::StaticStruct()))
			{
				FName FieldName = static_cast<FRemoteControlField*>(Entity)->FieldName;
				AddPropertyRow(TEXT("FieldName"), FieldName.ToString(), /*bReadOnly=*/true);
			}

			CreateBindingWidget(Entity, DetailBuilder, CategoryBuilder);

			if (URemoteControlPreset* Preset = Entity->GetOwner())
			{
				// Add Entity metadata rows.
				for (const TPair<FName, FString>& Entry : Entity->GetMetadata())
				{
					if (FOnCustomizeMetadataEntry* Handler = MetadataCustomizations.Find(Entry.Key))
					{
						Handler->ExecuteIfBound(Preset, Entity->GetId(), DetailBuilder, CategoryBuilder);
					}
					else if (const FOnCustomizeMetadataEntry* ExternalHandler = FRemoteControlUIModule::Get().GetEntityMetadataCustomizations().Find(Entry.Key))
					{
						ExternalHandler->ExecuteIfBound(Preset, Entity->GetId(), DetailBuilder, CategoryBuilder);
					}
					else
					{
						// Fallback on a default representation. 
						AddPropertyRow(Entry.Key.ToString(), Entry.Value, /*bReadOnly=*/false);
					}
				}
			}
		}
	}
}


#undef LOCTEXT_NAMESPACE

