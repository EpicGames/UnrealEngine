// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IDetailCustomization.h"

#include "CoreMinimal.h"
#include "PropertyHandle.h"
#include "RemoteControlUIModule.h"

struct FRemoteControlEntity;
class FStructOnScope;
class IDetailCategoryBuilder;

/**
 * Implements a details panel customization for remote control entities.
 */
class FRemoteControlEntityCustomization : public IDetailCustomization
{
public:
	FRemoteControlEntityCustomization();
	//~ Begin IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ Begin IPropertyTypeCustomization interface

	/** Makes a new instance of this detail layout class for a specific detail view requesting it. */
	static TSharedRef<IDetailCustomization> MakeInstance();

private:
	/** Create a widget that handles modifying the min and max properties. */
	void CreateRangeWidget(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder);

	/** Create a widget that displays what's the entity currently bound to. */
	void CreateBindingWidget(const FRemoteControlEntity* RCEntity, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder) const;

	/** Handler called upon modifying the metadata of an exposed entity. */
	void OnMetadataKeyCommitted(const FText& Text, ETextCommit::Type Type, FName MetadataKey);

	/** Get a pointer to the entity currently being displayed in the details view. */
	FRemoteControlEntity* GetEntityPtr();
	
	/** Get a const pointer to the entity currently being displayed in the details view. */
	const FRemoteControlEntity* GetEntityPtr() const;
	
	/** Get a value from the metadata map of the entity. */
	FText GetMetadataValue(FName Key) const;

	/** Generate property row widgets */
	void GeneratePropertyRows(IDetailLayoutBuilder& DetailBuilder, IDetailCategoryBuilder& CategoryBuilder);

private:
	/** Entity currently being displayed. */
	TSharedPtr<FStructOnScope> DisplayedEntity;
	
	/** Map of metadata key to initializer delegate. */
	TMap<FName, FOnCustomizeMetadataEntry> MetadataCustomizations;
};
