// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Filters/CustomRCFilterData.h"

#include "AssetTypeCategories.h"
#include "AssetToolsModule.h"

#include "Filters/FilterBase.h"

#include "IAssetTypeActions.h"

#include "Styling/AppStyle.h"
#include "Styling/SlateIconFinder.h"

#include "UObject/Field.h"
#include "UObject/Object.h"

FSlateIcon FCustomTypeFilterData::EmptyIcon;
FSlateIcon FCustomTypeFilterData::EntityIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.VariableList.TypeIcon");
FSlateIcon FCustomTypeFilterData::ArrayIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.VariableList.ArrayTypeIcon");

// TODO : Request for combined map icon svg(s) if possible, for now ignore the key image and show only the value image.
FSlateIcon FCustomTypeFilterData::MapIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.VariableList.MapValueTypeIcon");

FSlateIcon FCustomTypeFilterData::SetIcon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Kismet.VariableList.SetTypeIcon");

void FCustomTypeFilterData::AddCategory(TSharedPtr<FFilterCategory> InCategory)
{
	Categories.Add(InCategory);
}

void FCustomTypeFilterData::BuildBackendFilter(FRCFilter& OutFilter)
{
	if (AssetTypeActions.IsValid() && AssetTypeActions.Pin()->CanFilter())
	{
		OutFilter.AddTypeFilter(AssetTypeActions.Pin()->GetSupportedClass());
	}
	else if (Class) // If there is no AssetTypeAction for this filter, simply add the class to the FRCFilter
	{
		OutFilter.AddTypeFilter(Class);
	}
	else if (!CustomTypeName.IsNone()) // It must be a custom type structure.
	{
		OutFilter.AddTypeFilter(CustomTypeName);
	}
	else // It must be basic property.
	{
		OutFilter.AddTypeFilter(EntityType);
	}
}

TSharedPtr<IAssetTypeActions> FCustomTypeFilterData::GetAssetTypeActions() const
{
	return AssetTypeActions.Pin();
}

TArray<TSharedPtr<FFilterCategory>> FCustomTypeFilterData::GetCategories() const
{
	return Categories;
}

UClass* FCustomTypeFilterData::GetClass() const
{
	if (TSharedPtr<IAssetTypeActions> AssetTypeActionsPin = AssetTypeActions.Pin())
	{
		return AssetTypeActionsPin->GetSupportedClass();
	}

	return Class;
}

FFieldClass* FCustomTypeFilterData::GetEntityType() const
{
	return EntityType;
}

EEntityTypeCategories::Type FCustomTypeFilterData::GetEntityTypeCategory() const
{
	return EntityTypeCategory;
}

FLinearColor FCustomTypeFilterData::GetColor() const
{
	if (TSharedPtr<IAssetTypeActions> AssetTypeActionsPin = AssetTypeActions.Pin())
	{
		return AssetTypeActionsPin->GetTypeColor();
	}

	if (EntityTypeCategory & EEntityTypeCategories::Containers)
	{
		return FLinearColor::White;
	}

	return Color;
}

FText FCustomTypeFilterData::GetName() const
{
	if (!FilterDisplayName.IsEmpty())
	{
		return FilterDisplayName;
	}

	if (TSharedPtr<IAssetTypeActions> AssetTypeActionsPin = AssetTypeActions.Pin())
	{
		return AssetTypeActionsPin->GetName();
	}

	if (Class)
	{
		return Class->GetDisplayNameText();
	}

	if (!CustomTypeName.IsNone())
	{
		return FText::FromName(CustomTypeName);
	}

	return GetProperEntityDisplayName();
}

FString FCustomTypeFilterData::GetFilterName() const
{
	if (TSharedPtr<IAssetTypeActions> AssetTypeActionsPin = AssetTypeActions.Pin())
	{
		return AssetTypeActionsPin->GetFilterName().ToString();
	}
	
	if (Class)
	{
		return Class->GetFName().ToString();
	}

	if (!CustomTypeName.IsNone())
	{
		return CustomTypeName.ToString();
	}

	return EntityType->GetFName().ToString();
}

FSlateIcon FCustomTypeFilterData::GetSlateIcon() const
{
	if (EntityTypeCategory & EEntityTypeCategories::Assets)
	{
		return FSlateIconFinder::FindIconForClass(GetClass());
	}

	return GetProperEntityIcon();
}

void FCustomTypeFilterData::ResetBackendFilter(FRCFilter& OutFilter)
{
	if (AssetTypeActions.IsValid())
	{
		if (AssetTypeActions.Pin()->CanFilter())
		{
			OutFilter.RemoveTypeFilter(AssetTypeActions.Pin()->GetSupportedClass());
		}
	}
	else if (Class) // If there is no AssetTypeAction for this filter, simply add the class to the FRCFilter
	{
		OutFilter.RemoveTypeFilter(Class);
	}
	else if (!CustomTypeName.IsNone()) // It must be a custom type structure.
	{
		OutFilter.RemoveTypeFilter(CustomTypeName);
	}
	else // It must be basic property.
	{
		OutFilter.RemoveTypeFilter(EntityType);
	}
}

FText FCustomTypeFilterData::GetProperEntityDisplayName() const
{
	FString EntityTypeName = EntityType->GetName();

	EntityTypeName.RemoveFromEnd("Property");

	if (EntityTypeName.Contains("Bool"))
	{
		EntityTypeName = "Boolean";
	}
	else if (EntityTypeName.Contains("Int"))
	{
		EntityTypeName = EntityTypeName.Replace(TEXT("Int"), TEXT("Integer"));
	}
	else if (EntityTypeName.Contains("Str"))
	{
		EntityTypeName = "String";
	}

	return FText::FromString(EntityTypeName);
}

FSlateIcon FCustomTypeFilterData::GetProperEntityIcon() const
{
	FSlateIcon ProperEntityIcon = EmptyIcon;

	if (EntityTypeCategory & EEntityTypeCategories::Containers)
	{
		if (EntityType->IsChildOf(FArrayProperty::StaticClass()))
		{
			ProperEntityIcon = ArrayIcon;
		}
		else if (EntityType->IsChildOf(FSetProperty::StaticClass()))
		{
			ProperEntityIcon = SetIcon;
		}
		else if (EntityType->IsChildOf(FMapProperty::StaticClass()))
		{
			ProperEntityIcon = MapIcon;
		}
	}
	else if (EntityType->IsChildOf(FProperty::StaticClass()))
	{
		ProperEntityIcon = EntityIcon;
	}

	return ProperEntityIcon;
}
