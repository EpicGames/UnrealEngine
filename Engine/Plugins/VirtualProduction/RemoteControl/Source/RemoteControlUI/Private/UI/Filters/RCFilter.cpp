// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Filters/RCFilter.h"

#include "UI/SRCPanelExposedField.h"
#include "UI/SRCPanelTreeNode.h"

#include "UObject/Field.h"
#include "UObject/UnrealType.h"

void FRCFilter::AddTypeFilter(FFieldClass* InEntityTypeFilter)
{
	EntityTypeFilters.Add(InEntityTypeFilter);
}

void FRCFilter::AddTypeFilter(const FName InCustomTypeFilter)
{
	CustomTypeFilters.Add(InCustomTypeFilter);
}

void FRCFilter::AddTypeFilter(UClass* InAssetTypeFilter)
{
	AssetTypeFilters.Add(InAssetTypeFilter);
}

bool FRCFilter::DoesPassFilters(FEntityFilterType InEntityItem)
{
	const SRCPanelTreeNode::ENodeType EntityRCType = InEntityItem->GetRCType();

	// Do not process Field Child.
	if (EntityRCType == SRCPanelTreeNode::FieldChild)
	{
		return false;
	}

	// Return early if the RC type of entity is Actor.
	if (EntityRCType == SRCPanelTreeNode::Actor)
	{
		return CustomTypeFilters.Contains(RemoteControlTypes::NAME_RCActors);
	}

	if (EntityRCType == SRCPanelTreeNode::Field)
	{
		const TSharedRef<const SRCPanelExposedField> ExposedField = StaticCastSharedRef<const SRCPanelExposedField>(InEntityItem);

		// Return early if the field type of entity is Function.
		if (ExposedField->GetFieldType() == EExposedFieldType::Function)	
		{
			return CustomTypeFilters.Contains(RemoteControlTypes::NAME_RCFunctions);
		}

		if (TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(ExposedField->GetRemoteControlField().Pin()))
		{
			if (FProperty* Property = RCProperty->GetProperty())
			{
				if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
				{
					return AssetTypeFilters.Contains(ObjectProperty->PropertyClass);
				}

				if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					// TODO : Implement filters for Custom Struct Classes (as of now this is tailored for Primary Types specifically).
					return CustomTypeFilters.Contains(StructProperty->Struct->GetFName());
				}

				return EntityTypeFilters.Contains(Property->GetClass());
			}
		}
	}

	return false;
}

bool FRCFilter::HasAnyActiveFilters()
{
	return CustomTypeFilters.Num() || EntityTypeFilters.Num() || AssetTypeFilters.Num();
}

void FRCFilter::RemoveTypeFilter(FFieldClass* InEntityTypeFilter)
{
	if (EntityTypeFilters.Contains(InEntityTypeFilter))
	{
		EntityTypeFilters.Remove(InEntityTypeFilter);
	}
}

void FRCFilter::RemoveTypeFilter(const FName& InCustomTypeFilter)
{
	if (CustomTypeFilters.Contains(InCustomTypeFilter))
	{
		CustomTypeFilters.Remove(InCustomTypeFilter);
	}
}

void FRCFilter::RemoveTypeFilter(UClass* InAssetTypeFilter)
{
	if (AssetTypeFilters.Contains(InAssetTypeFilter))
	{
		AssetTypeFilters.Remove(InAssetTypeFilter);
	}
}
