// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetToolsModule.h"
#include "AssetTypeCategories.h"
#include "EdGraphSchema_K2.h"

#include "Filters/SBasicFilterBar.h"

#include "IAssetTypeActions.h"

#include "UI/Filters/CustomRCFilterData.h"
#include "UI/Filters/RCFilter.h"

#include "SRCFilterBar.generated.h"

#define LOCTEXT_NAMESPACE "RCFilterBar"

/** Delegate that subclasses can use to specify classes to not include in this filter
 * Returning false for a class will prevent it from showing up in the add filter dropdown
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnFilterEntityType, FFieldClass* /* Entity Type */);

/* Delegate used by SRCFilterBar to populate the add filter menu */
DECLARE_DELEGATE_ThreeParams(FOnPopulateAddRCFilterMenu, UToolMenu*, TSharedPtr<FFilterCategory>, FOnFilterEntityType)

/** ToolMenuContext that is used to create the Add Filter Menu */
UCLASS()
class REMOTECONTROLUI_API URCFilterBarContext : public UObject
{
	GENERATED_BODY()

public:
	TSharedPtr<FFilterCategory> MenuExpansion;
	FOnPopulateAddRCFilterMenu PopulateFilterMenu;
};

/**
 * A custom filter bar to handle Remote Control Filtering System.
 */
template<typename FilterType>
class SRCFilterBar : public SBasicFilterBar<FilterType>
{
public:

	using FOnFilterChanged = typename SBasicFilterBar<FilterType>::FOnFilterChanged;

	SLATE_BEGIN_ARGS(SRCFilterBar)
		: _UseDefaultEntityFilters(true)
	{}

		/** Delegate for when filters have changed */
		SLATE_EVENT(SRCFilterBar<FilterType>::FOnFilterChanged, OnFilterChanged)

		/** Delegate to extend the Add Filter dropdown */
		SLATE_EVENT(FOnExtendAddFilterMenu, OnExtendAddFilterMenu)

		/** Initial List of Custom Filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT(TArray<TSharedRef<FFilterBase<FilterType>>>, CustomFilters)

		/** Initial List of Custom Class filters that will be added to the AddFilter Menu */
		SLATE_ARGUMENT(TArray<TSharedRef<FCustomTypeFilterData>>, CustomTypeFilters)

		/** Whether the filter bar should provide the default RC filters */
		SLATE_ARGUMENT(bool, UseDefaultEntityFilters)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs)
	{
		bUseDefaultEntityFilters = InArgs._UseDefaultEntityFilters;
		CustomTypeFilters = InArgs._CustomTypeFilters;
		
		typename SBasicFilterBar<FilterType>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._CustomFilters = InArgs._CustomFilters;
		Args._OnExtendAddFilterMenu = InArgs._OnExtendAddFilterMenu;
		
		SBasicFilterBar<FilterType>::Construct(Args);
		
		CreateEntityTypeFilters();
	}

protected:

	/**
	 * A custom data model to represent each type of filters we have.
	 */
	class FEntityFilter
	{
	public:

		FEntityFilter(TSharedRef<FCustomTypeFilterData> InCustomTypeFilter, FOnFilterChanged InOnFilterChanged)
			: bEnabled(false)
			, CustomTypeFilter(InCustomTypeFilter)
			, OnFilterChanged(InOnFilterChanged)
		{
		}

		/** Returns contribution of this filter to the combined filter */
		void GetBackendFilter(FRCFilter& OutFilter) const
		{
			if (CustomTypeFilter)
			{
				if (bEnabled)
				{
					CustomTypeFilter->BuildBackendFilter(OutFilter);
				}
				else
				{
					CustomTypeFilter->ResetBackendFilter(OutFilter);
				}
			}
		}

		/** Gets the asset type actions associated with this filter */
		const TSharedPtr<FCustomTypeFilterData>& GetCustomTypeFilterData() const
		{
			return CustomTypeFilter;
		}

		/** Returns the display name for this filter */
		FText GetFilterDisplayName() const
		{
			if (CustomTypeFilter.IsValid())
			{
				return CustomTypeFilter->GetName();
			}

			return LOCTEXT("NoneText", "None");
		}

		/** Returns the actual name of this filter. */
		FString GetFilterName() const
		{
			if (CustomTypeFilter.IsValid())
			{
				return CustomTypeFilter->GetFilterName();
			}

			return "None";
		}

		/** Returns true if this filter contributes to the combined filter */
		bool IsEnabled() const
		{
			return bEnabled;
		}

		/** Sets whether or not this filter is applied to the combined filter */
		void SetEnabled(bool InEnabled, bool InExecuteOnFilterChanged = true)
		{
			if (InEnabled != bEnabled)
			{
				bEnabled = InEnabled;

				if (InExecuteOnFilterChanged)
				{
					OnFilterChanged.ExecuteIfBound();
				}
			}
		}

	protected:

		/** Whether this filter is active or not. */
		bool bEnabled;

		/** View model for this filter describing the nature of this filter. */
		TSharedPtr<FCustomTypeFilterData> CustomTypeFilter;

		/** Invoked when the filter toggled */
		FOnFilterChanged OnFilterChanged;
	};

public:

	/** Use this function to get an FRCFilter that represents all the Asset & Entity Type filters that are currently active. */
	FRCFilter GetCombinedBackendFilter() const
	{
		FRCFilter CombinedFilter;

		// Add all selected filters
		for (int32 FilterIdx = 0; FilterIdx < EntityFilters.Num(); ++FilterIdx)
		{
			const TSharedPtr<FEntityFilter> EntityFilter = EntityFilters[FilterIdx];

			if (EntityFilter.IsValid())
			{
				EntityFilter->GetBackendFilter(CombinedFilter);
			}
		}

		return CombinedFilter;
	}

	/** Remove all filters from the filter bar, while disabling any active ones */
	virtual void RemoveAllFilters() override
	{
		EntityFilters.Empty();
		
		SBasicFilterBar<FilterType>::RemoveAllFilters();
		
		this->OnFilterChanged.ExecuteIfBound();
	}

protected:

	/** Add an RC Filter to the list of filters, making it "Active" but not enabled */
	TSharedRef<FEntityFilter> AddEntityFilter(const TSharedPtr<FCustomTypeFilterData>& CustomTypeFilter)
	{
		TSharedRef<FEntityFilter> NewFilter = MakeShared<FEntityFilter>(CustomTypeFilter.ToSharedRef(), this->OnFilterChanged);

		// Add it to our list of just EntityFilters
		EntityFilters.Add(NewFilter);

		return NewFilter;
	}
	
	/** Returns true if any filters are applied */
	bool HasAnyFilters() const
	{
		return EntityFilters.Num() > 0;
	}

	/** Attempts to remove the given filter from list. */
	void RemoveEntityFilter(const TSharedPtr<FCustomTypeFilterData>& CustomTypeFilter, bool ExecuteOnFilterChanged = true)
	{
		TSharedPtr<FEntityFilter> FilterToRemove;

		for (const TSharedPtr<FEntityFilter>& EntityFilter : EntityFilters)
		{
			const TSharedPtr<FCustomTypeFilterData>& EntityFilterData = EntityFilter->GetCustomTypeFilterData();

			if (EntityFilterData == CustomTypeFilter)
			{
				FilterToRemove = EntityFilter;
				break;
			}
		}

		if (FilterToRemove.IsValid())
		{
			EntityFilters.Remove(FilterToRemove.ToSharedRef());

			if (ExecuteOnFilterChanged)
			{
				this->OnFilterChanged.ExecuteIfBound();
			}
		}
	}

	/** Called when reset filters option is pressed */
	void OnResetFilters()
	{
		RemoveAllFilters();
	}

	/** Create the default set of EClassCastFlags Filters provided with the widget if requested */
	void CreateEntityTypeFilters()
	{
		if (!bUseDefaultEntityFilters)
		{
			return;
		}

		TypeFilterCategories.Empty();

		// Add the Basic property types category

		// Entity Type Categories
		TypeFilterCategories.Add(EEntityTypeCategories::Core
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Core", "Core"), LOCTEXT("CoreFilterTooltip", "Filter in entities of core types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::Strings
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Strings", "Strings"), LOCTEXT("StringsFilterTooltip", "Filter in entities of string types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::Structs
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Structs", "Structures"), LOCTEXT("StructsFilterTooltip", "Filter in entities of structure types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::Objects
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Objects", "Objects"), LOCTEXT("ObjectsFilterTooltip", "Filter in entities of objects types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::Containers
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Containers", "Containers"), LOCTEXT("ContainersFilterTooltip", "Filter in entities of container types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::References
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_References", "References"), LOCTEXT("ReferencesFilterTooltip", "Filter in entities of reference types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::UserDefined
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_UserDefined", "Custom"), LOCTEXT("UserDefinedFilterTooltip", "Filter in entities of custom types.")))
		);
		TypeFilterCategories.Add(EEntityTypeCategories::Primary
			, MakeShareable(new FFilterCategory(LOCTEXT("TypeFilter_Primary", "All"), LOCTEXT("PrimaryFilterTooltip", "Filter in entities of unique types.")))
		);

		auto GetTypeDisplayText =
			[](FName InCategory, FName InSubCategory = NAME_None)->FText
			{
				constexpr bool bForMenu = true;
				return UEdGraphSchema_K2::GetCategoryText(InCategory, InSubCategory, bForMenu);
			};

		// Core Type Filters.
		PopulateFilterData(EEntityTypeCategories::Core, FBoolProperty::StaticClass() , GetTypeDisplayText(UEdGraphSchema_K2::PC_Boolean));
		PopulateFilterData(EEntityTypeCategories::Core, FByteProperty::StaticClass() , GetTypeDisplayText(UEdGraphSchema_K2::PC_Byte));
		PopulateFilterData(EEntityTypeCategories::Core, FEnumProperty::StaticClass() , GetTypeDisplayText(UEdGraphSchema_K2::PC_Enum));
		PopulateFilterData(EEntityTypeCategories::Core, FFloatProperty::StaticClass(), GetTypeDisplayText(UEdGraphSchema_K2::PC_Real, UEdGraphSchema_K2::PC_Float));
		PopulateFilterData(EEntityTypeCategories::Core, FIntProperty::StaticClass()  , GetTypeDisplayText(UEdGraphSchema_K2::PC_Int));
		PopulateFilterData(EEntityTypeCategories::Core, FInt64Property::StaticClass(), GetTypeDisplayText(UEdGraphSchema_K2::PC_Int64));

		// String Type Filters.
		PopulateFilterData(EEntityTypeCategories::Strings, FNameProperty::StaticClass(), GetTypeDisplayText(UEdGraphSchema_K2::PC_Name));
		PopulateFilterData(EEntityTypeCategories::Strings, FStrProperty::StaticClass() , GetTypeDisplayText(UEdGraphSchema_K2::PC_String));
		PopulateFilterData(EEntityTypeCategories::Strings, FTextProperty::StaticClass(), GetTypeDisplayText(UEdGraphSchema_K2::PC_Text));

		// Primary Type Filters.
		PopulateFilterData(EEntityTypeCategories::Primary, FStructProperty::StaticClass(), TBaseStructure<FVector>::Get()->GetDisplayNameText(), NAME_Vector);
		PopulateFilterData(EEntityTypeCategories::Primary, FStructProperty::StaticClass(), TBaseStructure<FRotator>::Get()->GetDisplayNameText(), NAME_Rotator);
		PopulateFilterData(EEntityTypeCategories::Primary, FStructProperty::StaticClass(), TBaseStructure<FColor>::Get()->GetDisplayNameText(), NAME_Color);
		PopulateFilterData(EEntityTypeCategories::Primary, FStructProperty::StaticClass(), TBaseStructure<FTransform>::Get()->GetDisplayNameText(), NAME_Transform);
		PopulateFilterData(EEntityTypeCategories::Primary, FProperty::StaticClass(), LOCTEXT("RCActorsDisplayName", "Exposed Actors"), RemoteControlTypes::NAME_RCActors);
		PopulateFilterData(EEntityTypeCategories::Primary, FProperty::StaticClass(), LOCTEXT("RCFunctionsDisplayName", "Exposed Functions"), RemoteControlTypes::NAME_RCFunctions);

		// Container Type Filters.
		PopulateFilterData(EEntityTypeCategories::Containers, FArrayProperty::StaticClass(), LOCTEXT("ArrayDisplayName", "Array"));
		PopulateFilterData(EEntityTypeCategories::Containers, FMapProperty::StaticClass()  , LOCTEXT("MapDisplayName", "Map"));
		PopulateFilterData(EEntityTypeCategories::Containers, FSetProperty::StaticClass()  , LOCTEXT("SetDisplayName", "Set"));

		// TODO : Add support to Custom Classes.
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));

		AssetFilterCategories.Empty();

		// Add the Basic category
		AssetFilterCategories.Add(EAssetTypeCategories::Basic, MakeShareable(new FFilterCategory(LOCTEXT("BasicFilter", "Basic"), LOCTEXT("BasicFilterTooltip", "Filter by basic assets."))));
		
		// Add the Material category
		AssetFilterCategories.Add(EAssetTypeCategories::Materials, MakeShareable(new FFilterCategory(LOCTEXT("MaterialFilter", "Materials"), LOCTEXT("MaterialsFilterTooltip", "Filter by material assets."))));

		// Get the browser type maps
		TArray<TWeakPtr<IAssetTypeActions>> AssetTypeActionsList;
		AssetToolsModule.Get().GetAssetTypeActionsList(AssetTypeActionsList);

		// Sort the list
		struct FCompareIAssetTypeActions
		{
			FORCEINLINE bool operator()(const TWeakPtr<IAssetTypeActions>& A, const TWeakPtr<IAssetTypeActions>& B) const
			{
				return A.Pin()->GetName().CompareTo(B.Pin()->GetName()) == -1;
			}
		};

		AssetTypeActionsList.Sort(FCompareIAssetTypeActions());

		const TSharedRef<FPathPermissionList>& AssetClassPermissionList = AssetToolsModule.Get().GetAssetClassPathPermissionList(EAssetClassAction::CreateAsset);

		// For every asset type, convert it to an FCustomTypeFilterData and add it to the list
		for (int32 ClassIdx = 0; ClassIdx < AssetTypeActionsList.Num(); ++ClassIdx)
		{
			const TWeakPtr<IAssetTypeActions>& WeakTypeActions = AssetTypeActionsList[ClassIdx];
			if (WeakTypeActions.IsValid())
			{
				TSharedPtr<IAssetTypeActions> TypeActions = WeakTypeActions.Pin();
				if (ensure(TypeActions.IsValid()) && TypeActions->CanFilter())
				{
					UClass* SupportedClass = TypeActions->GetSupportedClass();
					if ((!SupportedClass || AssetClassPermissionList->PassesFilter(SupportedClass->GetClassPathName().ToString())) &&
						(TypeActions->GetCategories() & (EAssetTypeCategories::Basic | EAssetTypeCategories::Materials)) &&
						!AssetTypesToIgnore.Contains(TypeActions->GetName().ToString()))
					{
						// Convert the AssetTypeAction to an FCustomTypeFilterData and add it to our list
						PopulateFilterData(TypeActions);
					}
				}
			}
		}

		// Do a second pass through all the CustomTypeFilters with TypeFilterCategories to update their categories
		UpdateEntityTypeCategories();
	}

	/** Normalizes the CustomTypeFilters to their respective categories. */
	void UpdateEntityTypeCategories()
	{
		for (const TSharedRef<FCustomTypeFilterData>& CustomTypeFilter : CustomTypeFilters)
		{
			for (TMap<EEntityTypeCategories::Type, TSharedPtr<FFilterCategory>>::TIterator MenuIt = TypeFilterCategories.CreateIterator(); MenuIt; ++MenuIt)
			{
				if (MenuIt.Key() & CustomTypeFilter->GetEntityTypeCategory())
				{
					CustomTypeFilter->AddCategory(MenuIt.Value());
				}
			}

			for (TMap<EAssetTypeCategories::Type, TSharedPtr<FFilterCategory>>::TIterator MenuIt = AssetFilterCategories.CreateIterator(); MenuIt; ++MenuIt)
			{
				if (TSharedPtr<IAssetTypeActions> AssetTypeActions = CustomTypeFilter->GetAssetTypeActions())
				{
					if (MenuIt.Key() & AssetTypeActions->GetCategories())
					{
						CustomTypeFilter->AddCategory(MenuIt.Value());
					}
				}
			}
		}
	}
	
	/* Filter Dropdown Related Functionality */

	/** Handler for when the add filter menu is populated by a category */
	void CreateFiltersMenuCategory(FToolMenuSection& Section, const TArray<TSharedPtr<FCustomTypeFilterData>> CustomTypeFilterDatas) const
	{
		for (int32 EntityFilterIdx = 0; EntityFilterIdx < CustomTypeFilterDatas.Num(); ++EntityFilterIdx)
		{
			const TSharedPtr<FCustomTypeFilterData>& CustomTypeFilterData = CustomTypeFilterDatas[EntityFilterIdx];

			const FText& LabelText = CustomTypeFilterData->GetName();

			Section.AddMenuEntry(
				NAME_None,
				LabelText,
				FText::Format(LOCTEXT("FilterByTooltipPrefix", "Filter in entities of type {0}."), LabelText),
				CustomTypeFilterData->GetSlateIcon(),
				FUIAction(
					FExecuteAction::CreateSP(const_cast<SRCFilterBar<FilterType>*>(this), &SRCFilterBar<FilterType>::FilterByTypeClicked, CustomTypeFilterData),
					FCanExecuteAction(),
					FIsActionChecked::CreateSP(this, &SRCFilterBar<FilterType>::IsEntityTypeInUse, CustomTypeFilterData)),
				EUserInterfaceActionType::ToggleButton
			);
		}
	}

	/** Handler for when the add filter menu is populated by a category */
	void CreateFiltersMenuCategory(UToolMenu* InMenu, const TArray<TSharedPtr<FCustomTypeFilterData>> CustomTypeFilterDatas) const
	{
		CreateFiltersMenuCategory(InMenu->AddSection("Section"), CustomTypeFilterDatas);
	}

	/** Handler for when the add filter button was clicked */
	virtual TSharedRef<SWidget> MakeAddFilterMenu() override
	{
		const FName FilterMenuName = "RCFilterBar.RCFilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;

			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
				{
					if (URCFilterBarContext* Context = InMenu->FindContext<URCFilterBarContext>())
					{
						Context->PopulateFilterMenu.ExecuteIfBound(InMenu, Context->MenuExpansion, FOnFilterEntityType());
					}
				})
			);
		}

		URCFilterBarContext* FilterBarContext = NewObject<URCFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddRCFilterMenu::CreateSP(this, &SRCFilterBar<FilterType>::PopulateAddFilterMenu);
		
		// Auto expand the Structures Category if it is needed
		if (TSharedPtr<FFilterCategory>* PrimaryCategory = TypeFilterCategories.Find(EEntityTypeCategories::Primary))
		{
			FilterBarContext->MenuExpansion = *PrimaryCategory;
		}

		FToolMenuContext ToolMenuContext(FilterBarContext);

		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}

	/** Handler to Populate the Add Filter Menu. Use OnFilterEntityType in Subclasses to add classes to the exclusion list */
	virtual void PopulateAddFilterMenu(UToolMenu* Menu, TSharedPtr<FFilterCategory> MenuExpansion, FOnFilterEntityType OnFilterEntityType)
	{
		// A local struct to describe a category in the filter menu
		struct FCategoryMenu
		{
			/** The Property Type that belong to this category */
			TArray<TSharedPtr<FCustomTypeFilterData>> EntityTypes;

			//Menu section
			FName SectionExtensionHook;
			FText SectionHeading;

			FCategoryMenu(const FName& InSectionExtensionHook, const FText& InSectionHeading)
				: SectionExtensionHook(InSectionExtensionHook)
				, SectionHeading(InSectionHeading)
			{}
		};

		// Create a map of Categories to Menus
		TMap<TSharedPtr<FFilterCategory>, FCategoryMenu> CategoryToMenuMap;

		// For every asset type, move it into all the categories it should appear in
		for (const TSharedRef<FCustomTypeFilterData>& CustomTypeFilter : CustomTypeFilters)
		{
			bool bPassesExternalFilters = true;

			// Run any external class filters we have
			if (OnFilterEntityType.IsBound())
			{
				bPassesExternalFilters = OnFilterEntityType.Execute(CustomTypeFilter->GetEntityType());
			}

			if (bPassesExternalFilters)
			{
				/** Get all the categories this filter belongs to */
				TArray<TSharedPtr<FFilterCategory>> Categories = CustomTypeFilter->GetCategories();

				for (const TSharedPtr<FFilterCategory>& Category : Categories)
				{
					// If the category for this custom class already exists
					if (FCategoryMenu* CategoryMenu = CategoryToMenuMap.Find(Category))
					{
						CategoryMenu->EntityTypes.Add(CustomTypeFilter);
					}
					// Otherwise create a new FCategoryMenu for the category and add it to the map
					else
					{
						const FName ExtensionPoint = NAME_None;
						const FText SectionHeading = FText::Format(LOCTEXT("WildcardFilterHeadingHeadingTooltip", "{0} Filters"), Category->Title);

						FCategoryMenu NewCategoryMenu(ExtensionPoint, SectionHeading);
						NewCategoryMenu.EntityTypes.Add(CustomTypeFilter);

						CategoryToMenuMap.Add(Category, NewCategoryMenu);
					}
				}
			}
		}

		// Remove any empty categories
		for (auto MenuIt = CategoryToMenuMap.CreateIterator(); MenuIt; ++MenuIt)
		{
			if (MenuIt.Value().EntityTypes.Num() == 0)
			{
				CategoryToMenuMap.Remove(MenuIt.Key());
			}
		}

		// Set the extension hook for the primary category, if it exists and we have any entities for it
		if (TSharedPtr<FFilterCategory>* PrimaryCategory = TypeFilterCategories.Find(EEntityTypeCategories::Primary))
		{
			if (*PrimaryCategory == MenuExpansion)
			{
				if (FCategoryMenu* PrimaryMenu = CategoryToMenuMap.Find(MenuExpansion))
				{
					PrimaryMenu->SectionExtensionHook = "FilterBarFilterPrimaryEntity";
				}
			}
		}

		// Populate the common filter sections (Reset Filters etc)
		{
			FToolMenuSection& Section = Menu->AddSection("FilterBarResetFilters");
			Section.AddMenuEntry(
				"ResetFilters",
				LOCTEXT("FilterListResetFilters", "Reset Filters"),
				LOCTEXT("FilterListResetToolTip", "Resets current filter selection"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault"),
				FUIAction(
					FExecuteAction::CreateSP(this, &SRCFilterBar<FilterType>::OnResetFilters),
					FCanExecuteAction::CreateLambda([this]() { return HasAnyFilters(); }))
			);

			this->OnExtendAddFilterMenu.ExecuteIfBound(Menu);
		}

		// If we want to expand a category
		if (MenuExpansion)
		{
			// First add the expanded category, this appears as standard entries in the list (Note: intentionally not using FindChecked here as removing it from the map later would cause the ref to be garbage)
			FCategoryMenu* ExpandedCategory = CategoryToMenuMap.Find(MenuExpansion);
			if (ExpandedCategory)
			{
				FToolMenuSection& Section = Menu->AddSection(ExpandedCategory->SectionExtensionHook, LOCTEXT("QuickFiltersLabel", "Quick Filters"));
				
				// If we are doing a full menu (i.e expanding primary) we add a menu entry which toggles all other categories
				if (MenuExpansion == TypeFilterCategories.FindChecked(EEntityTypeCategories::Primary))
				{
					Section.AddMenuEntry(
						NAME_None,
						MenuExpansion->Title,
						MenuExpansion->Tooltip,
						FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Kismet.VariableList.TypeIcon"),
						FUIAction(
							FExecuteAction::CreateSP(this, &SRCFilterBar<FilterType>::FilterByTypeCategoryClicked, MenuExpansion, ExpandedCategory->EntityTypes),
							FCanExecuteAction(),
							FGetActionCheckState::CreateSP(this, &SRCFilterBar<FilterType>::IsEntityTypeCategoryChecked, MenuExpansion, ExpandedCategory->EntityTypes)),
						EUserInterfaceActionType::ToggleButton
					);
				}

				// Now populate with all the assets from the expanded category
				SRCFilterBar<FilterType>::CreateFiltersMenuCategory(Section, ExpandedCategory->EntityTypes);
				
				// Remove the Expanded from the map now, as this is treated differently and is no longer needed.
				CategoryToMenuMap.Remove(MenuExpansion);
			}
		}

		TSharedPtr<FFilterCategory>* PrimaryCategory = TypeFilterCategories.Find(EEntityTypeCategories::Primary);

		// We are in Full Menu Mode if there is no menu expansion, or the menu expansion is EEntityTypeCategories::Primary
		bool bInFullMenuMode = !MenuExpansion || (PrimaryCategory && MenuExpansion == *PrimaryCategory);

		// If we are in full menu mode, add all the other categories as submenus
		if (bInFullMenuMode)
		{
			FToolMenuSection& Section = Menu->AddSection("TypeFilterBarFilterAdvancedEntity", LOCTEXT("AdvancedTypesMenuHeading", "Advanced Filters"));

			// Sort by category name so that we add the submenus in alphabetical order
			CategoryToMenuMap.KeySort([](const TSharedPtr<FFilterCategory>& A, const TSharedPtr<FFilterCategory>& B) {
				return A->Title.CompareTo(B->Title) < 0;
				});

			// For all the remaining categories, add them as submenus
			for (const TPair<TSharedPtr<FFilterCategory>, FCategoryMenu>& CategoryMenuPair : CategoryToMenuMap)
			{
				Section.AddSubMenu(
					NAME_None,
					CategoryMenuPair.Key->Title,
					CategoryMenuPair.Key->Tooltip,
					FNewToolMenuDelegate::CreateSP(this, &SRCFilterBar<FilterType>::CreateFiltersMenuCategory, CategoryMenuPair.Value.EntityTypes),
					FUIAction(
						FExecuteAction::CreateSP(this, &SRCFilterBar<FilterType>::FilterByTypeCategoryClicked, CategoryMenuPair.Key, CategoryMenuPair.Value.EntityTypes),
						FCanExecuteAction(),
						FGetActionCheckState::CreateSP(this, &SRCFilterBar<FilterType>::IsEntityTypeCategoryChecked, CategoryMenuPair.Key, CategoryMenuPair.Value.EntityTypes)),
					EUserInterfaceActionType::ToggleButton
				);
			}
		}

		// Now add all non-asset filters
		this->PopulateCustomFilters(Menu);
	}

	/* Asset Type filter related functionality */

	/** Handler for when filter by type is selected */
	void FilterByTypeClicked(TSharedPtr<FCustomTypeFilterData> CustomTypeFilterData)
	{
		if (CustomTypeFilterData.IsValid())
		{
			if (IsEntityTypeInUse(CustomTypeFilterData))
			{
				RemoveEntityFilter(CustomTypeFilterData);
			}
			else
			{
				TSharedRef<FEntityFilter> NewFilter = AddEntityFilter(CustomTypeFilterData);
				NewFilter->SetEnabled(true);
			}
		}
	}

	/** Handler to determine the "checked" state of class filter in the filter dropdown */
	bool IsEntityTypeInUse(TSharedPtr<FCustomTypeFilterData> EntityType) const
	{
		for (const TSharedPtr<FEntityFilter> EntityFilter : this->EntityFilters)
		{
			if (EntityFilter.IsValid() && EntityFilter->GetCustomTypeFilterData() == EntityType)
			{
				return true;
			}
		}

		return false;
	}

	/** Handler for when filter by type category is selected */
	void FilterByTypeCategoryClicked(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomTypeFilterData>> EntityTypes)
	{
		bool bFullCategoryInUse = IsEntityTypeCategoryInUse(TypeCategory, EntityTypes);
		bool ExecuteOnFilterChanged = false;

		for (const TSharedPtr<FCustomTypeFilterData>& CustomEntityType : EntityTypes)
		{
			if (bFullCategoryInUse)
			{
				RemoveEntityFilter(CustomEntityType);
				ExecuteOnFilterChanged = true;
			}
			else if (!IsEntityTypeInUse(CustomEntityType))
			{
				TSharedRef<FEntityFilter> NewFilter = AddEntityFilter(CustomEntityType);
				NewFilter->SetEnabled(true, false);
				ExecuteOnFilterChanged = true;
			}
		}

		if (ExecuteOnFilterChanged)
		{
			this->OnFilterChanged.ExecuteIfBound();
		}
	}

	/** Handler to determine the "checked" state of an type category in the filter dropdown */
	ECheckBoxState IsEntityTypeCategoryChecked(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomTypeFilterData>> EntityTypes) const
	{
		bool bIsAnyActionInUse = false;
		bool bIsAnyActionNotInUse = false;

		for (const TSharedPtr<FCustomTypeFilterData>& CustomEntityTypeFilter : EntityTypes)
		{
			if (IsEntityTypeInUse(CustomEntityTypeFilter))
			{
				bIsAnyActionInUse = true;
			}
			else
			{
				bIsAnyActionNotInUse = true;
			}

			if (bIsAnyActionInUse && bIsAnyActionNotInUse)
			{
				return ECheckBoxState::Undetermined;
			}

		}

		if (bIsAnyActionInUse)
		{
			return ECheckBoxState::Checked;
		}
		else
		{
			return ECheckBoxState::Unchecked;
		}
	}

	/** Function to check if a given type category is in use */
	bool IsEntityTypeCategoryInUse(TSharedPtr<FFilterCategory> TypeCategory, TArray<TSharedPtr<FCustomTypeFilterData>> EntityTypes) const
	{
		ECheckBoxState AssetTypeCategoryCheckState = IsEntityTypeCategoryChecked(TypeCategory, EntityTypes);

		if (AssetTypeCategoryCheckState == ECheckBoxState::Unchecked)
		{
			return false;
		}

		// An entity type category is in use if any of its type actions are in use (ECheckBoxState::Checked or ECheckBoxState::Undetermined)
		return true;
	}

private:

	/** Populates CustomTypeFilters from Entity Type and its Category. */
	void PopulateFilterData(EEntityTypeCategories::Type InEntityTypeCategory, FFieldClass* InEntityType, FText InFilterDisplayName, const FName& InCustomTypeName = NAME_None)
	{
		TSharedRef<FCustomTypeFilterData> CustomTypeFilterData = MakeShared<FCustomTypeFilterData>(InEntityTypeCategory, InEntityType, InFilterDisplayName, InCustomTypeName);
		CustomTypeFilters.Add(CustomTypeFilterData);
	}
	
	/** Populates CustomTypeFilters from Asset Type Action. */
	void PopulateFilterData(TWeakPtr<IAssetTypeActions> InAssetTypeAction)
	{
		TSharedRef<FCustomTypeFilterData> CustomTypeFilterData = MakeShared<FCustomTypeFilterData>(InAssetTypeAction);
		CustomTypeFilters.Add(CustomTypeFilterData);
	}

protected:

	/** A copy of all active Entity Filters in this->EntityFilters for convenient access. */
	TArray<TSharedRef<FEntityFilter>> EntityFilters;

	/** List of all custom entity filters. */
	TArray<TSharedRef<FCustomTypeFilterData>> CustomTypeFilters;

	/** A map of property type with its corresponding filter category. */
	TMap<EEntityTypeCategories::Type, TSharedPtr<FFilterCategory>> TypeFilterCategories;
	
	/** A map of asset type with its corresponding filter category. */
	TMap<EAssetTypeCategories::Type, TSharedPtr<FFilterCategory>> AssetFilterCategories;

	/** A list of Asset Types to ignore. */
	TSet<FString> AssetTypesToIgnore = { "C++ Class" };

	/** Whether the filter bar provides the default RC Filters */
	bool bUseDefaultEntityFilters;
};

#undef LOCTEXT_NAMESPACE
