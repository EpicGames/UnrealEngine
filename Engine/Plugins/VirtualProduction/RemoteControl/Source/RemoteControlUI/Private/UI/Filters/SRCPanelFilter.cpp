// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Filters/SRCPanelFilter.h"

#include "Misc/ConfigCacheIni.h"

#include "SlateOptMacros.h"

#define LOCTEXT_NAMESPACE "RCPanelFilter"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCPanelFilter::Construct(const FArguments& InArgs)
{
	this->OnFilterChanged = InArgs._OnFilterChanged;

	SRCFilterBar<FEntityFilterType>::FArguments Args;

	/** Explicitly setting this to true as it should ALWAYS be true for SRCPanelFilter */
	Args._UseDefaultEntityFilters = true;
	Args._OnFilterChanged = this->OnFilterChanged;

	SRCFilterBar<FEntityFilterType>::Construct(Args);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCPanelFilter::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
{
	// Holds the list of filters that are active but not enabled.
	FString ActiveTypeFilterString;

	// Holds the list of enabled filters in this session.
	FString EnabledTypeFilterString;

	for (const TSharedRef<FEntityFilter>& Filter : this->EntityFilters)
	{
		const FString FilterName = Filter->GetFilterName();

		if (ActiveTypeFilterString.Len() > 0)
		{
			ActiveTypeFilterString += TEXT(",");
		}

		ActiveTypeFilterString += FilterName;

		if (Filter->IsEnabled())
		{
			if (EnabledTypeFilterString.Len() > 0)
			{
				EnabledTypeFilterString += TEXT(",");
			}

			EnabledTypeFilterString += FilterName;
		}
	}

	GConfig->SetString(*IniSection, *(SettingsString + TEXT(".ActiveTypeFilters")), *ActiveTypeFilterString, IniFilename);
	GConfig->SetString(*IniSection, *(SettingsString + TEXT(".EnabledTypeFilters")), *EnabledTypeFilterString, IniFilename);
}

void SRCPanelFilter::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
{
	// Holds all the type filters that were found in the ActiveTypeFilters
	FString ActiveTypeFilterString;

	// Holds all the type filters that were found in the EnabledTypeFilters
	FString EnabledTypeFilterString;
	
	GConfig->GetString(*IniSection, *(SettingsString + TEXT(".ActiveTypeFilters")), ActiveTypeFilterString, IniFilename);
	GConfig->GetString(*IniSection, *(SettingsString + TEXT(".EnabledTypeFilters")), EnabledTypeFilterString, IniFilename);

	// Parse comma delimited strings into arrays
	TArray<FString> TypeFilterNames;
	TArray<FString> EnabledTypeFilterNames;
	constexpr bool bCullEmpty = true;
	ActiveTypeFilterString.ParseIntoArray(TypeFilterNames, TEXT(","), bCullEmpty);
	EnabledTypeFilterString.ParseIntoArray(EnabledTypeFilterNames, TEXT(","), bCullEmpty);

	for (const TSharedRef<FCustomTypeFilterData>& CustomTypeFilter : CustomTypeFilters)
	{
		if (!this->IsEntityTypeInUse(CustomTypeFilter))
		{
			const FString FilterName = CustomTypeFilter->GetFilterName();
			
			if (TypeFilterNames.Contains(FilterName))
			{
				TSharedRef<FEntityFilter> NewFilter = AddEntityFilter(CustomTypeFilter);

				if (EnabledTypeFilterNames.Contains(FilterName))
				{
					// NOTE: Do not trigger execution OnFilterChanged at this moment as it will cause panel refresh altogether.
					NewFilter->SetEnabled(true, false);
				}
			}
		}
	}

	OnFilterChanged.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
