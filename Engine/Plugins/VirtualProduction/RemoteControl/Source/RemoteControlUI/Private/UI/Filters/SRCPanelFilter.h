// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SRCFilterBar.h"

struct SRCPanelTreeNode;

using FEntityFilterType = TSharedRef<const SRCPanelTreeNode>;

/**
 * A custom list of filters currently applied to an Remote Control Panel.
 */
class SRCPanelFilter : public SRCFilterBar<FEntityFilterType>
{
public:

	using FOnFilterChanged = typename SRCFilterBar<FEntityFilterType>::FOnFilterChanged;

	SLATE_BEGIN_ARGS(SRCPanelFilter)
	{}

		/** Delegate for when filters have changed */
		SLATE_EVENT(FOnFilterChanged, OnFilterChanged)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
	
	/** Saves any settings to config that should be persistent between editor sessions */
	void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const;

	/** Loads any settings to config that should be persistent between editor sessions */
	void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString);

private:

	/** Delegate for when filters have changed */
	FOnFilterChanged OnFilterChanged;
};
