// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "RemoteControlPanelMenuContext.generated.h"

class SRemoteControlPanel;

UCLASS()
class URemoteControlPanelMenuContext : public UObject
{
	GENERATED_BODY()

public:
	/** RemoteControlPanel of this context */
	TWeakPtr<SRemoteControlPanel> RemoteControlPanel;
};