// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/Panels/SRCDockPanel.h"

#include "SlateOptMacros.h"
#include "Styling/RemoteControlStyles.h"
#include "Styling/StyleColors.h"

#include "UI/RemoteControlPanelStyle.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"

#include "Widgets/SBoxPanel.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

SLATE_IMPLEMENT_WIDGET(SRCDockPanel)

void SRCDockPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

static const FRCPanelStyle* GetPanelStyle()
{
	return &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");
}

void SRCDockPanel::Construct(const FArguments& InArgs)
{
	RCPanelStyle = GetPanelStyle();

	bIsFooterEnabled = false;
	bIsHeaderEnabled = false;

	Clipping = EWidgetClipping::ClipToBounds;

	SBorder::FArguments SuperArgs = SBorder::FArguments();

	SuperArgs.BorderImage(&RCPanelStyle->ContentAreaBrush);
	SuperArgs.Padding(RCPanelStyle->PanelPadding);

	SBorder::Construct(SuperArgs
		[
			InArgs._Content.Widget
		]
	);
}

SLATE_IMPLEMENT_WIDGET(SRCMajorPanel)

void SRCMajorPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCMajorPanel::Construct(const SRCMajorPanel::FArguments& InArgs)
{
	RCPanelStyle = GetPanelStyle();
	const float SplitterHandleSize = RCPanelStyle->SplitterHandleSize;

	SRCDockPanel::Construct(SRCDockPanel::FArguments()
		[
			SAssignNew(ContentPanel, SSplitter)
			.Orientation(InArgs._Orientation)

			// Content Panel
			+ SSplitter::Slot()
			.Value(1.f)
			[
				SAssignNew(Children, SSplitter)
				.Orientation(InArgs._ChildOrientation)
				.HitDetectionSplitterHandleSize(SplitterHandleSize)
				.PhysicalSplitterHandleSize(SplitterHandleSize)
				.OnSplitterFinishedResizing_Lambda([this]()
				{
					OnSplitterFinishedResizing().ExecuteIfBound();
				})
			]
		]
	);

	bIsFooterEnabled = InArgs._EnableFooter;
	bIsHeaderEnabled = InArgs._EnableHeader;

	// Enable Header upon explicit request.
	if (bIsHeaderEnabled.Get() && ContentPanel.IsValid())
	{
		// Header Panel
		ContentPanel->AddSlot(0)
			.Resizable(false)
			.SizeRule(SSplitter::SizeToContent)
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.Padding(2.f, 4.f)
				[
					SNew(STextBlock)
					.TextStyle(&RCPanelStyle->HeaderTextStyle)
					.Text(InArgs._HeaderLabel)
				]
			];
	}

	// Enable Footer upon explicit request.
	if (bIsFooterEnabled.Get() && ContentPanel.IsValid())
	{
		// Footer Panel
		ContentPanel->AddSlot(bIsHeaderEnabled.Get() ? 2 : 1)
			.Resizable(false)
			.SizeRule(SSplitter::SizeToContent)
			[
				SNew(SHorizontalBox)

				// Left Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 4.f, 2.f, 4.f)
				[
					SAssignNew(LeftToolbar, SHorizontalBox)
				]

				// Spacer.
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
				]
				
				// Right Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 4.f, 4.f, 4.f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightToolbar, SHorizontalBox)
				]
			];
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

int32 SRCMajorPanel::AddPanel(TSharedRef<SWidget> InContent, const TAttribute<float> InDesiredSize, const bool bResizable/* = true */)
{
	const SSplitter::ESizeRule SizeRule = InDesiredSize.Get() == 0.f ? SSplitter::ESizeRule::SizeToContent : SSplitter::ESizeRule::FractionOfParent;

	if (Children.IsValid())
	{
		Children->AddSlot()
			.SizeRule(SizeRule)
			.Value(InDesiredSize)
			.Resizable(bResizable)
			[
				InContent
			];

		const int32 SlotsCount = Children->GetChildren()->Num();
		return SlotsCount - 1;
	}

	return -1;
}

const TSharedRef<SWidget>& SRCMajorPanel::GetContent() const
{
	return ChildSlot.GetWidget();
}

void SRCMajorPanel::ClearContent()
{
	ChildSlot.DetachWidget();
}

void SRCMajorPanel::AddFooterToolItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget)
{
	switch (InToolbar)
	{
		case Left:
			if (LeftToolbar.IsValid() && bIsFooterEnabled.Get())
			{
				LeftToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		case Right:
			if (RightToolbar.IsValid() && bIsFooterEnabled.Get())
			{
				RightToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		default:
			break;
	}
}

SSplitter::FSlot& SRCMajorPanel::GetSplitterSlotAt(const int32 InIndex) const
{
	return Children->SlotAt(InIndex);
}

SLATE_IMPLEMENT_WIDGET(SRCMinorPanel)

void SRCMinorPanel::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCMinorPanel::Construct(const SRCMinorPanel::FArguments& InArgs)
{
	SRCDockPanel::Construct(SRCDockPanel::FArguments()
		[
			SAssignNew(ContentPanel, SSplitter)
			.Orientation(InArgs._Orientation)

			// Content Panel
			+ SSplitter::Slot()
			.Value(1.f)
			[
				InArgs._Content.Widget
			]
		]
	);

	bIsFooterEnabled = InArgs._EnableFooter;
	bIsHeaderEnabled = InArgs._EnableHeader;

	// Enable Header upon explicit request.
	if (bIsHeaderEnabled.Get() && ContentPanel.IsValid())
	{
		ContentIndex = 1;

		// Header Panel
		ContentPanel->AddSlot(0)
			.SizeRule(SSplitter::SizeToContent)
			.Resizable(false)
			[
				SNew(SHorizontalBox)

				// Left Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 0.f, 2.f, 0.f)
				[
					SAssignNew(LeftHeaderToolbar, SHorizontalBox)
				]

				// Header Label.
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f)
				[
					SNew(STextBlock)
					.TextStyle(&RCPanelStyle->HeaderTextStyle)
					.Text(InArgs._HeaderLabel)
				]

				// Center Toolbar
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.f, 0.f, 5.f, 0.f)
				[
					SAssignNew(CenterHeaderToolbar, SHorizontalBox)
				]

				// Right Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 0.f, 4.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightHeaderToolbar, SHorizontalBox)
				]
			];
	}
	
	// Enable Footer upon explicit request.
	if (bIsFooterEnabled.Get() && ContentPanel.IsValid())
	{
		// Footer Panel
		ContentPanel->AddSlot(bIsHeaderEnabled.Get() ? 2 : 1)
			.SizeRule(SSplitter::SizeToContent)
			.Resizable(false)
			[
				SNew(SHorizontalBox)

				// Left Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.f, 4.f, 2.f, 4.f)
				[
					SAssignNew(LeftFooterToolbar, SHorizontalBox)
				]

				// Spacer.
				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(SSpacer)
				]
				
				// Right Toolbar
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.f, 4.f, 4.f, 4.f)
				.VAlign(VAlign_Center)
				[
					SAssignNew(RightFooterToolbar, SHorizontalBox)
				]
			];
	}
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SRCMinorPanel::SetContent(TSharedRef<SWidget> InContent)
{
	if (ContentPanel.IsValid())
	{
		if (const FChildren* Children = ContentPanel->GetChildren())
		{
			if (ContentIndex >= 0 && Children->NumSlot() > ContentIndex)
			{
				SSplitter::FSlot& Content = ContentPanel->SlotAt(ContentIndex);
				Content[InContent];
			}
		}
	}
}

const TSharedRef<SWidget>& SRCMinorPanel::GetContent() const
{
	return ChildSlot.GetWidget();
}

void SRCMinorPanel::ClearContent()
{
	ChildSlot.DetachWidget();
}

void SRCMinorPanel::AddFooterToolbarItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget)
{
	switch (InToolbar)
	{
		case Left:
			if (LeftFooterToolbar.IsValid() && bIsFooterEnabled.Get())
			{
				LeftFooterToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		case Right:
			if (RightFooterToolbar.IsValid() && bIsFooterEnabled.Get())
			{
				RightFooterToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		default:
			break;
	}
}

void SRCMinorPanel::AddHeaderToolbarItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget)
{
	switch (InToolbar)
	{
		case Left:
			if (LeftHeaderToolbar.IsValid() && bIsHeaderEnabled.Get())
			{
				LeftHeaderToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		case Right:
			if (RightHeaderToolbar.IsValid() && bIsHeaderEnabled.Get())
			{
				RightHeaderToolbar->AddSlot()
					.Padding(5.f, 0.f)
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		case Center:
			if (CenterHeaderToolbar.IsValid() && bIsHeaderEnabled.Get())
			{
				CenterHeaderToolbar->AddSlot()
					.FillWidth(1.0f)
					.Padding(5.f, 0.f)
					.VAlign(VAlign_Center)
					[
						InWidget
					];
			}
			break;
		default:
			break;
	}
}

void SRCMinorPanel::ClearHeaderToolbarItems() const
{
	// Clear left header
	if (LeftHeaderToolbar.IsValid())
	{
		LeftHeaderToolbar->ClearChildren();
	}
	
	// Clear center header
	if (CenterHeaderToolbar.IsValid())
	{
		CenterHeaderToolbar->ClearChildren();
	}
	
	// Clear right header
	if (RightHeaderToolbar.IsValid())
	{
		RightHeaderToolbar->ClearChildren();
	}
}
