// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"

struct FRCPanelStyle;
class SHorizontalBox;
class SWidget;

enum EToolbar
{
	/** Left toolbar. */
	Left,

	/** Right toolbar. */
	Right,

	/** Center toolbar. */
	Center
};

/**
 * A custom slate widget representing a dockable panel on RC Panel.
 */
class SRCDockPanel : public SBorder
{
	SLATE_DECLARE_WIDGET(SRCDockPanel, SCompoundWidget)

public:

	SLATE_BEGIN_ARGS(SRCDockPanel)
		: _Content()
	{}

		SLATE_DEFAULT_SLOT(FArguments, Content)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

protected:

	/** Holds whether the footer panel is enabled or not. */
	TAttribute<bool> bIsFooterEnabled;

	/** Holds whether the header panel is enabled or not. */
	TAttribute<bool> bIsHeaderEnabled;

	/** Holds the entire content panel. */
	TSharedPtr<SSplitter> ContentPanel;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
};

/**
 * A custom slate widget representing a major panel on RC Panel.
 */
class SRCMajorPanel : public SRCDockPanel
{
	SLATE_DECLARE_WIDGET(SRCMajorPanel, SRCDockPanel)

public:

	SLATE_BEGIN_ARGS(SRCMajorPanel)
		: _EnableFooter(false)
		, _EnableHeader(true)
		, _HeaderLabel(FText::GetEmpty())
		, _Orientation(Orient_Vertical)
		, _ChildOrientation(Orient_Horizontal)
	{}

		SLATE_ARGUMENT(bool, EnableFooter)

		SLATE_ARGUMENT(bool, EnableHeader)

		SLATE_ATTRIBUTE(FText, HeaderLabel)

		SLATE_ARGUMENT(EOrientation, Orientation)

		SLATE_ARGUMENT(EOrientation, ChildOrientation)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/**
	 * Adds the given panel to the children.
	 *
	 * @param InPanel The panel widget.
	 * @param InDesiredSize The desired size for the Panel widget
	 * @param bResizable Can the slot be resize by the user
	 * @return the index of the newly added slot. Slot can be accessed using this index by calling SRCMajorPanel::GetSplitterSlotAt(Index)
	 */
	virtual int32 AddPanel(TSharedRef<SWidget> InPanel, const TAttribute<float> InDesiredSize, const bool bResizable = true);

	/**
	 * Gets the content of this panel
	 *
	 * @return The widget used as content for the panel
	 */
	const TSharedRef<SWidget>& GetContent() const;

	/** Clears out the content for the panel */
	void ClearContent();

	/** Adds widgets to the specified footer toolbar (Left or Right). */
	void AddFooterToolItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget);

	/** Delegate called whenever the splitter is finished resizing */
	FSimpleDelegate& OnSplitterFinishedResizing() { return OnSplitterFinishedResizingDelegate; }

	/** Returns the slot at the specified index */
	SSplitter::FSlot& GetSplitterSlotAt(const int32 InIndex) const;
	
private:

	/** Actual toolbar widget located left to the footer. */
	TSharedPtr<SHorizontalBox> LeftToolbar;

	/** Actual toolbar widget located right to the footer. */
	TSharedPtr<SHorizontalBox> RightToolbar;

	/** Actual toolbar widget located Center to the footer. */
	TSharedPtr<SHorizontalBox> CenterToolbar;

	/** Holds the entire child panels. */
	TSharedPtr<SSplitter> Children;

	/** Delegate called whenever the splitter is finished resizing */
	FSimpleDelegate OnSplitterFinishedResizingDelegate;
};

/**
 * A custom slate widget representing a minor panel on RC Panel.
 */
class SRCMinorPanel : public SRCDockPanel
{
	SLATE_DECLARE_WIDGET(SRCMinorPanel, SRCDockPanel)

public:

	SLATE_BEGIN_ARGS(SRCMinorPanel)
		: _Content()
		, _EnableFooter(false)
		, _EnableHeader(true)
		, _HeaderLabel(FText::GetEmpty())
		, _Orientation(Orient_Vertical)
	{}
		SLATE_DEFAULT_SLOT(FArguments, Content)

		SLATE_ARGUMENT(bool, EnableFooter)

		SLATE_ARGUMENT(bool, EnableHeader)

		SLATE_ATTRIBUTE(FText, HeaderLabel)

		SLATE_ARGUMENT(EOrientation, Orientation)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/**
	 * Sets the content for this panel
	 *
	 * @param	InContent	The widget to use as content for the panel
	 */
	virtual void SetContent(TSharedRef<SWidget> InContent );

	/**
	 * Gets the content for this panel
	 *
	 * @return The widget used as content for the panel
	 */
	const TSharedRef<SWidget>& GetContent() const;

	/** Clears out the content for the panel */
	void ClearContent();

	/** Adds widgets to the specified footer toolbar (Left or Right). */
	void AddFooterToolbarItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget);
	
	/** Adds widgets to the specified header toolbar (Left or Right). */
	void AddHeaderToolbarItem(EToolbar InToolbar, TSharedRef<SWidget> InWidget);

	/** Clear toolbar items */
	void ClearHeaderToolbarItems() const;

private:

	/** Actual toolbar widget located left to the footer. */
	TSharedPtr<SHorizontalBox> LeftFooterToolbar;

	/** Actual toolbar widget located right to the footer. */
	TSharedPtr<SHorizontalBox> RightFooterToolbar;
	
	/** Actual toolbar widget located left to the header. */
	TSharedPtr<SHorizontalBox> LeftHeaderToolbar;

	/** Actual toolbar widget located right to the header. */
	TSharedPtr<SHorizontalBox> RightHeaderToolbar;

	/** Actual toolbar widget located Center to the header. */
    TSharedPtr<SHorizontalBox> CenterHeaderToolbar;

	/** Index of the Content slot */
	int32 ContentIndex = 0;
};
