// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCPanelWidgetRegistry.h"

#include "Engine/BlueprintGeneratedClass.h"
#include "GameFramework/Actor.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "UObject/StructOnScope.h"

namespace WidgetRegistryUtils
{
	bool FindPropertyHandleRecursive(const TSharedPtr<IPropertyHandle>& PropertyHandle, const FString& PropertyNameOrPath, ERCFindNodeMethod FindMethod)
	{
		if (PropertyHandle && PropertyHandle->IsValidHandle())
		{
			uint32 ChildrenCount = 0;
			PropertyHandle->GetNumChildren(ChildrenCount);
			for (uint32 Index = 0; Index < ChildrenCount; ++Index)
			{
				TSharedPtr<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(Index);
				if (FindPropertyHandleRecursive(ChildHandle, PropertyNameOrPath, FindMethod))
				{
					return true;
				}
			}

			if (PropertyHandle->GetProperty())
			{
				if (FindMethod == ERCFindNodeMethod::Path)
				{
					if (PropertyHandle->GeneratePathToProperty() == PropertyNameOrPath)
					{
						return true;
					}
				}
				else if (PropertyHandle->GetProperty()->GetName() == PropertyNameOrPath)
				{
					return true;
				}
			}
		}

		return false;
	}

	TSharedPtr<IDetailTreeNode> FindTreeNodeRecursive(const TSharedRef<IDetailTreeNode>& RootNode, const FString& PropertyNameOrPath, ERCFindNodeMethod FindMethod)
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		RootNode->GetChildren(Children);
		for (TSharedRef<IDetailTreeNode>& Child : Children)
		{
			TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(Child, PropertyNameOrPath, FindMethod);
			if (FoundNode.IsValid())
			{
				return FoundNode;
			}
		}

		TSharedPtr<IPropertyHandle> Handle = RootNode->CreatePropertyHandle();
		if (FindPropertyHandleRecursive(Handle, PropertyNameOrPath, FindMethod))
		{
			return RootNode;
		}

		return nullptr;
	}

	/** Find a node by its name in a detail tree node hierarchy. */
	TSharedPtr<IDetailTreeNode> FindNode(const TArray<TSharedRef<IDetailTreeNode>>& RootNodes, const FString& QualifiedPropertyName, ERCFindNodeMethod FindMethod)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FRCPanelWidgetRegistry::FindNode);

		for (const TSharedRef<IDetailTreeNode>& CategoryNode : RootNodes)
		{
			TSharedPtr<IDetailTreeNode> FoundNode = FindTreeNodeRecursive(CategoryNode, QualifiedPropertyName, FindMethod);
			if (FoundNode.IsValid())
			{
				return FoundNode;
			}
		}

		return nullptr;
	}
}

FRCPanelWidgetRegistry::FRCPanelWidgetRegistry()
{
	FRCTreeNodeFinderHandler NewHandler;
	NewHandler.IsExceptionFunc = [this](UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod) { return IsNDisplayObject(InObject, InField, InFindMethod); };
	NewHandler.FinderFunction = [this](UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod) { return FindNDisplayTreeNode(InObject, InField, InFindMethod); };
	SpecialTreeNodeHandlers.Add(MoveTemp(NewHandler));
}

FRCPanelWidgetRegistry::~FRCPanelWidgetRegistry()
{
	for (const TPair <TWeakObjectPtr<UObject>, TSharedPtr<IPropertyRowGenerator>>& Pair : ObjectToRowGenerator)
	{
		if (Pair.Value)
		{
			Pair.Value->OnRowsRefreshed().Clear();
		}
	}

	ObjectToRowGenerator.Reset();
}

TSharedPtr<IDetailTreeNode> FRCPanelWidgetRegistry::GetObjectTreeNode(UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCPanelWidgetRegistry::GetObjectTreeNode);

	const TPair<TWeakObjectPtr<UObject>, FString> CacheKey{InObject, InField};
	if (TWeakPtr<IDetailTreeNode>* Node = TreeNodeCache.Find(CacheKey))
	{
		if (Node->IsValid() && Node->Pin()->CreatePropertyHandle() && Node->Pin()->CreatePropertyHandle()->GetNumOuterObjects() != 0)
		{
			if (TSharedPtr<IPropertyRowGenerator>* FoundGenerator = ObjectToRowGenerator.Find({ InObject }))
			{
				// Since we are returning cached node from the second time refresh onwards
				// It is important that we refresh the corresponding row to keep the consistency.
				FoundGenerator->Get()->OnRowsRefreshed().Broadcast();
			}

			return Node->Pin();
		}
		else
		{
			TreeNodeCache.Remove(CacheKey);
		}
	}

	//Verify if desired object has special handling and cache result if it does
	for (const FRCTreeNodeFinderHandler& Handler : SpecialTreeNodeHandlers)
	{
		if (Handler.IsExceptionFunc(InObject, InField, InFindMethod))
		{
			TSharedPtr<IDetailTreeNode> Node = Handler.FinderFunction(InObject, InField, InFindMethod);
			TreeNodeCache.Add(CacheKey, Node);
			return Node;
		}
	}
		
	TSharedPtr<IPropertyRowGenerator> Generator;
	TWeakObjectPtr<UObject> WeakObject = InObject;
	
	if (TSharedPtr<IPropertyRowGenerator>* FoundGenerator = ObjectToRowGenerator.Find(WeakObject))
	{
		Generator = *FoundGenerator;
	}
	else
	{
		Generator = CreateGenerator(InObject);
		ObjectToRowGenerator.Add(WeakObject, Generator);
	}
	
	TSharedPtr<IDetailTreeNode> Node = WidgetRegistryUtils::FindNode(Generator->GetRootTreeNodes(), InField, InFindMethod);
	// Cache the node to avoid having to do the recursive find again.
	TreeNodeCache.Add(CacheKey, Node);
	
	return Node;
}

TSharedPtr<IDetailTreeNode> FRCPanelWidgetRegistry::GetStructTreeNode(const TSharedPtr<FStructOnScope>& InStruct, const FString& InField, ERCFindNodeMethod InFindMethod)
{
	TSharedPtr<IPropertyRowGenerator> Generator;
	
	if (TSharedPtr<IPropertyRowGenerator>* FoundGenerator = StructToRowGenerator.Find(InStruct))
	{
		Generator = *FoundGenerator;
	}
	else
	{
		// In RC, struct details node are only used for function arguments, so default to showing hidden properties. 
		FPropertyRowGeneratorArgs Args;
		Args.bShouldShowHiddenProperties = true;
		
		Generator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
		Generator->SetStructure(InStruct);
		StructToRowGenerator.Add(InStruct, Generator);
	}

	return WidgetRegistryUtils::FindNode(Generator->GetRootTreeNodes(), InField, InFindMethod);
}

void FRCPanelWidgetRegistry::Refresh(UObject* InObject, bool bForceRefresh)
{
	if (!InObject)
	{
		return;
	}

	if (TSharedPtr<IPropertyRowGenerator>* Generator = ObjectToRowGenerator.Find({InObject}))
	{
		const TArray<TWeakObjectPtr<UObject>>& SelectedObjects = (*Generator)->GetSelectedObjects();
		if (bForceRefresh || SelectedObjects.Num() != 1 || SelectedObjects[0] != InObject)
		{
			(*Generator)->SetObjects({InObject});
		}
	}
	else if (IsNDisplayObject(InObject))
	{
		if (TSharedPtr<IPropertyRowGenerator>* GeneratorFromParent = ObjectToRowGenerator.Find({ InObject->GetTypedOuter<AActor>() }))
		{
			(*GeneratorFromParent)->SetObjects({ InObject->GetTypedOuter<AActor>() });
		}
	}
}

void FRCPanelWidgetRegistry::Refresh(const TSharedPtr<FStructOnScope>& InStruct)
{
	if (TSharedPtr<IPropertyRowGenerator>* Generator = StructToRowGenerator.Find(InStruct))
	{
		(*Generator)->SetStructure(InStruct);
	}
}

void FRCPanelWidgetRegistry::UpdateGeneratorAndTreeCache(UObject* InOldObject, UObject* InNewObject, FString InPath)
{
	ReplaceGeneratorObject(InOldObject, InNewObject);
	ReplaceTreeCacheObject(InOldObject, InNewObject, InPath);
}

void FRCPanelWidgetRegistry::ReplaceGeneratorObject(UObject* InOldObject, UObject* InNewObject)
{
	if (!InOldObject || !InNewObject)
	{
		return;
	}

	for (TPair<TWeakObjectPtr<UObject>, TSharedPtr<IPropertyRowGenerator>>& Generator : ObjectToRowGenerator)
	{
		if (InOldObject && Generator.Value.IsValid() && Generator.Key != InNewObject)
		{
			if (Generator.Key == InOldObject)
			{
				Generator.Key = InNewObject;
				Generator.Value->SetObjects({InNewObject});
			}
		}
	}
}

void FRCPanelWidgetRegistry::ReplaceTreeCacheObject(UObject* InOldObject, UObject* InNewObject, FString InPath)
{
	if (!InOldObject || !InNewObject)
	{
		return;
	}

	const TPair<TWeakObjectPtr<UObject>, FString> OldCacheKey{InOldObject, InPath};
	if (TreeNodeCache.Contains(OldCacheKey))
	{
		if (TSharedPtr<IPropertyRowGenerator>* Generator = ObjectToRowGenerator.Find(InNewObject))
		{
			const TPair<TWeakObjectPtr<UObject>, FString> NewCacheKey{InNewObject, InPath};
			TreeNodeCache.Remove(OldCacheKey);
			const TSharedPtr<IDetailTreeNode> Node = WidgetRegistryUtils::FindNode((*Generator)->GetRootTreeNodes(), InPath, ERCFindNodeMethod::Path);
			// Cache the node to avoid having to do the recursive find again.
			TreeNodeCache.Add(NewCacheKey, Node);
		}
	}
}

void FRCPanelWidgetRegistry::Clear()
{
	for (const TPair <TWeakObjectPtr<UObject>, TSharedPtr<IPropertyRowGenerator>>& Pair : ObjectToRowGenerator)
	{
		if (Pair.Value)
		{
			Pair.Value->OnRowsRefreshed().Clear();
		}
	}

	ObjectToRowGenerator.Empty();
	StructToRowGenerator.Empty();
	TreeNodeCache.Empty();
}

bool FRCPanelWidgetRegistry::IsNDisplayObject(UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod)
{
	if (InObject)
	{
		AActor* Actor = InObject->GetTypedOuter<AActor>();
		if (Actor != nullptr)
		{
			UBlueprintGeneratedClass* BlueprintClass = Cast<UBlueprintGeneratedClass>(Actor->GetClass());
			if (BlueprintClass && BlueprintClass->GetDefaultObject(false))
			{
				static constexpr TCHAR ConfigurationDataName[] = TEXT("DisplayClusterConfigurationData");
				const FString ObjectClassName = InObject->GetClass()->GetName();
				if (ObjectClassName.Contains(ConfigurationDataName))
				{
					//Can't easily test for actor class without depending on ndisplay
					//If user have inheritance in the way naming is not what's expected.
					//If we know we are dealing with the config object and it's under a blueprint generated actor, good chance we can handle it
					return true;
				}
			}
		}
	}

	return false;
}

TSharedPtr<IDetailTreeNode> FRCPanelWidgetRegistry::FindNDisplayTreeNode(UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod)
{
	//To support ndisplay customization, we need to go back to the root actor to trigger all required customizations to see the treenode
	AActor* Actor = InObject->GetTypedOuter<AActor>();
	if (Actor == nullptr)
	{
		return nullptr;
	}

	//Run the generation using the owner actor instead of the subobject
	InObject = Actor;

	TSharedPtr<IPropertyRowGenerator> Generator;
	TWeakObjectPtr<UObject> WeakObject = InObject;

	if (TSharedPtr<IPropertyRowGenerator>* FoundGenerator = ObjectToRowGenerator.Find(WeakObject))
	{
		Generator = *FoundGenerator;
	}
	else
	{
		Generator = CreateGenerator(InObject);
		ObjectToRowGenerator.Add(WeakObject, Generator);
	}

	return WidgetRegistryUtils::FindNode(Generator->GetRootTreeNodes(), InField, InFindMethod);
}

void FRCPanelWidgetRegistry::OnRowsRefreshed(TSharedPtr<IPropertyRowGenerator> Generator)
{
	if (Generator)
	{
		const TArray<TWeakObjectPtr<UObject>> WeakSelectedObjects = Generator->GetSelectedObjects();
		TArray<UObject*> SelectedObjects;
		SelectedObjects.Reserve(WeakSelectedObjects.Num());
		for (const TWeakObjectPtr<UObject>& WeakObject : WeakSelectedObjects)
		{
			SelectedObjects.Add(WeakObject.Get());
		}
		OnObjectRefreshedDelegate.Broadcast(SelectedObjects);
	}
}

TSharedPtr<IPropertyRowGenerator> FRCPanelWidgetRegistry::CreateGenerator(UObject* InObject)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCPanelWidgetRegistry::CreateGenerator);

	// Since we must keep many PRG objects alive in order to access the handle data, validating the nodes each tick is very taxing.
	// We can override the validation with a lambda since the validation function in PRG is not necessary for our implementation
	auto ValidationLambda = ([](const FRootPropertyNodeList& PropertyNodeList) { return true; });
	FPropertyRowGeneratorArgs Args;
	TSharedPtr<IPropertyRowGenerator> Generator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);
	Generator->SetObjects({ InObject });
	Generator->SetCustomValidatePropertyNodesFunction(FOnValidatePropertyRowGeneratorNodes::CreateLambda(MoveTemp(ValidationLambda)));

	Generator->OnRowsRefreshed().AddLambda([WeakGenerator = TWeakPtr<IPropertyRowGenerator>(Generator), WeakThis = TWeakPtr<FRCPanelWidgetRegistry>(AsShared())]
		{
			// This was crashing on an invalid widget registry ptr, maybe because Clear() getting called and the generator being kept alive by the shared ptr 
			// passed to the AddRaw delegate was keeping the Genrator and its invocation list alive.
			// This should be revisited and retested thoroughly after returning to AddRaw or AddSP and passing in a weak generator ptr.
			if (TSharedPtr<FRCPanelWidgetRegistry> WidgetRegistry = WeakThis.Pin())
			{
				if (TSharedPtr<IPropertyRowGenerator> Generator = WeakGenerator.Pin())
				{
					WidgetRegistry->OnRowsRefreshed(Generator);
				}
			}
		});

	return Generator;
}
