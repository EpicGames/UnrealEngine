// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IDetailTreeNode;
class IPropertyRowGenerator;
class FStructOnScope;

enum class ERCFindNodeMethod
{
	Name, // Find a node by providing its property name.
	Path  // Find a node by providing a property path.
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnObjectsRefreshed, const TArray<UObject*>&);

/** Local special case handlers to generate widgets to be displayed */
using FRCTreeNodeFinderIsException = TFunction< bool(UObject* /*InObject*/, const FString& /*InField*/, ERCFindNodeMethod /*InFindMethod*/) >;
using FRCTreeNodeFinderFunction = TFunction< TSharedPtr<IDetailTreeNode>(UObject* /*InObject*/, const FString& /*InField*/, ERCFindNodeMethod /*InFindMethod*/) >;
struct FRCTreeNodeFinderHandler
{
	FRCTreeNodeFinderIsException IsExceptionFunc;
	FRCTreeNodeFinderFunction FinderFunction;
};

class FRCPanelWidgetRegistry : public TSharedFromThis<FRCPanelWidgetRegistry>
{
public:
	FRCPanelWidgetRegistry();
	~FRCPanelWidgetRegistry();

	/**
	 * Get a detail tree node for a given object and property.
	 * @param InObject the object used to generate the detail row.
	 * @param InField a string specifying which field to generate a row for.
	 * @param InFindMethod whether the field parameter is a path or a field name.
	 */
	TSharedPtr<IDetailTreeNode> GetObjectTreeNode(UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod);

	/**
	 * Get a detail tree node for a given struct and field.
	 * @param InStruct the struct used to generate the detail row.
	 * @param InField a string specifying which field to generate a row for.
	 * @param InFindMethod whether the field parameter is a path or a field name.
	 */
	TSharedPtr<IDetailTreeNode> GetStructTreeNode(const TSharedPtr<FStructOnScope>& InStruct, const FString& InField, ERCFindNodeMethod InFindMethod);

	/**
	 * Update the generator for a given object.
	 */
	void Refresh(UObject* InObject, bool bForceRefresh = false);

	/**
	 * Update the generator for a given struct.
	 */
	void Refresh(const TSharedPtr<FStructOnScope>& InStruct);

	/**
	 * Update the InOldOObject generator and the tree cache with the InNewObject.
	 */
	void UpdateGeneratorAndTreeCache(UObject* InOldObject, UObject* InNewObject, FString InPath);

	/**
	 * Clear the registry and its cache.
	 */
	void Clear();

	/**
	 * Get the on object refresh delegate.
	 */
	FOnObjectsRefreshed& OnObjectRefreshed() 
	{
		return OnObjectRefreshedDelegate; 
	}

private:

	/** Special handlers for ndisplay root actor and its details customizations */
	bool IsNDisplayObject(UObject* InObject, const FString& InField = FString(), ERCFindNodeMethod InFindMethod = ERCFindNodeMethod::Path);
	TSharedPtr<IDetailTreeNode> FindNDisplayTreeNode(UObject* InObject, const FString& InField, ERCFindNodeMethod InFindMethod);
	void OnRowsRefreshed(TSharedPtr<IPropertyRowGenerator> Generator);
	/** Create a property row generator */
	TSharedPtr<IPropertyRowGenerator> CreateGenerator(UObject* InObject);
	/** Replace old object if found with new object */
	void ReplaceGeneratorObject(UObject* InOldObject, UObject* InNewObject);
	/** Update Cached tree, main use when switching between different world needs to be called after ReplaceGeneratorObject */
	void ReplaceTreeCacheObject(UObject* InOldObject, UObject* InNewObject, FString InPath);
private:
	/** Map of objects to row generator, used to have one row generator per object. */
	TMap<TWeakObjectPtr<UObject>, TSharedPtr<IPropertyRowGenerator>> ObjectToRowGenerator;
	/** Map of struct on scope to row generator, used to have one row generator per struct ptr. */
	TMap<TSharedPtr<FStructOnScope>, TSharedPtr<IPropertyRowGenerator>> StructToRowGenerator;
	/** Cache of Object&Field to tree nodes. */
	TMap<TPair<TWeakObjectPtr<UObject>, FString>, TWeakPtr<IDetailTreeNode>> TreeNodeCache;
	/** List of tree node finder handlers for certain type */
	TArray<FRCTreeNodeFinderHandler> SpecialTreeNodeHandlers;
	/** Called when a generator gets refreshed. (ie. forcefully refreshed by a customization) */
	FOnObjectsRefreshed OnObjectRefreshedDelegate;
};
