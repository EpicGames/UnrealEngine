// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCUIHelpers.h"

#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Controller/RCController.h"
#include "Controller/RCCustomControllerUtilities.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraphSchema_K2.h"
#include "Editor.h"
#include "Engine/Texture2D.h"
#include "Framework/Application/SlateApplication.h"
#include "GraphEditorSettings.h"
#include "IDetailTreeNode.h"
#include "IPropertyRowGenerator.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "RCVirtualProperty.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"
#include "TimerManager.h"
#include "UI/Controller/SRCControllerPanel.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"

FLinearColor UE::RCUIHelpers::GetFieldClassTypeColor(const FProperty* InProperty)
{
	if (ensure(InProperty))
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		FEdGraphPinType PinType;
		if (Schema->ConvertPropertyToPinType(InProperty, PinType))
		{
			return Schema->GetPinTypeColor(PinType);
		}
	}

	return FLinearColor::White;
}

FName UE::RCUIHelpers::GetFieldClassDisplayName(const FProperty* InProperty)
{
	FName FieldClassDisplayName = NAME_None;

	if (ensure(InProperty))
	{
		const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();
		FEdGraphPinType PinType;

		if (Schema->ConvertPropertyToPinType(InProperty, PinType))
		{
			if(UObject* SubCategoryObject = PinType.PinSubCategoryObject.Get())
			{
				FieldClassDisplayName = *SubCategoryObject->GetName();
			}
			else
			{
				FieldClassDisplayName = PinType.PinCategory;
			}
		}
	}

	if (!FieldClassDisplayName.IsNone())
	{
		FString ResultString = FieldClassDisplayName.ToString();

		if (ResultString.StartsWith("Bool"))
		{
			ResultString = "Boolean";
		}

		if (ResultString.StartsWith("Int"))
		{
			ResultString = *ResultString.Replace(TEXT("Int"), TEXT("Integer"));
		}

		if (ResultString == "real")
		{
			ResultString = "Float";
		}

		ResultString[0] = FChar::ToUpper(ResultString[0]);

		return *ResultString;
	}

	return NAME_None;
}

TSharedPtr<IDetailTreeNode> UE::RCUIHelpers::GetDetailTreeNodeForVirtualProperty(TObjectPtr<URCVirtualPropertySelfContainer> InVirtualPropertySelfContainer, 
	TSharedPtr<IPropertyRowGenerator>& OutPropertyRowGenerator)
{
	TSharedPtr<IDetailTreeNode> DetailTreeNode;

	FPropertyRowGeneratorArgs Args;
	Args.bShouldShowHiddenProperties = true;
	OutPropertyRowGenerator = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreatePropertyRowGenerator(Args);

	OutPropertyRowGenerator->SetStructure(InVirtualPropertySelfContainer->CreateStructOnScope());

	for (const TSharedRef<IDetailTreeNode>& CategoryNode : OutPropertyRowGenerator->GetRootTreeNodes())
	{
		TArray<TSharedRef<IDetailTreeNode>> Children;
		CategoryNode->GetChildren(Children);

		// Use the first child as the detail tree node
		if (Children.Num())
		{
			DetailTreeNode = Children[0];
			break;
		}
	}

	return DetailTreeNode;
}

static void SetFocusToWidgetNextTick(const TSharedRef< SWidget> InWidget)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(
		FTimerDelegate::CreateLambda([InWidget]()
			{
				UE::RCUIHelpers::FindFocusableWidgetAndSetKeyboardFocus(InWidget);
			})
	);
}

TSharedRef<SWidget> UE::RCUIHelpers::GetGenericFieldWidget(const TSharedPtr<IDetailTreeNode> DetailTreeNode, 
	TSharedPtr<IPropertyHandle>* OutPropertyHandle /*=nullptr*/, bool bFocusInputWidget/*= false*/)
{
	if (!DetailTreeNode.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	if (OutPropertyHandle)
	{
		*OutPropertyHandle = DetailTreeNode->CreatePropertyHandle();
	}

	const FNodeWidgets NodeWidgets = DetailTreeNode->CreateNodeWidgets();

	const TSharedRef<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);

	if (NodeWidgets.ValueWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.VAlign(VAlign_Center)
			[
				NodeWidgets.ValueWidget.ToSharedRef()
			];

		if (bFocusInputWidget)
			SetFocusToWidgetNextTick(NodeWidgets.ValueWidget.ToSharedRef());

	}
	else if (NodeWidgets.WholeRowWidget)
	{
		FieldWidget->AddSlot()
			.Padding(FMargin(3.0f, 2.0f))
			.VAlign(VAlign_Center)
			[
				NodeWidgets.WholeRowWidget.ToSharedRef()
			];

		if (bFocusInputWidget)
			SetFocusToWidgetNextTick(NodeWidgets.WholeRowWidget.ToSharedRef());
	}

	return FieldWidget;
}

bool UE::RCUIHelpers::FindFocusableWidgetAndSetKeyboardFocus(const TSharedRef< SWidget> InWidget)
{
	if (InWidget->SupportsKeyboardFocus())
	{
		FSlateApplication::Get().SetKeyboardFocus(InWidget, EFocusCause::Navigation);

		return true; // Success!
	}

	// Check child widgets...
	if (FChildren* Children = InWidget->GetChildren())
	{
		if(Children->Num() > 0)
		{
			// Note: We're only interested in the first child for all current usecases.
			return FindFocusableWidgetAndSetKeyboardFocus(Children->GetChildAt(0)); // Recurse!
		}
	}

	return false; // No focusable widget found in the entire widget hierarchy
}

TSharedRef<SWidget> UE::RCUIHelpers::GetTypeColorWidget(const FProperty* InProperty)
{
	if (!ensure(InProperty))
	{
		return SNullWidget::NullWidget;
	}

	const FLinearColor TypeColor = UE::RCUIHelpers::GetFieldClassTypeColor(InProperty);
	const FText TooltipText = FText::FromName(UE::RCUIHelpers::GetFieldClassDisplayName(InProperty));

	return GetTypeColorWidget(TypeColor, TooltipText);
}

TSharedRef<SWidget> UE::RCUIHelpers::GetTypeColorWidget(const FLinearColor& InTypeColor, const FText& InTypeDescription)
{
	return SNew(SBox)
		.HeightOverride(5.f)
		.HAlign(HAlign_Left)
		.Padding(FMargin(5.f, 0.f))
		.ToolTipText(InTypeDescription)
		[
			SNew(SBorder)
				.Visibility(EVisibility::Visible)
				.BorderImage(FAppStyle::Get().GetBrush("NumericEntrySpinBox.Decorator"))
				.BorderBackgroundColor(InTypeColor)
				.Padding(FMargin(5.0f, 0.0f, 0.0f, 0.f))
		];
}

UObject* UE::RCUIHelpers::GetEntityOwner(const TSharedPtr<const FRemoteControlEntity>& InEntity)
{
	if (UObject* BoundObject = InEntity->GetBoundObject())
	{
		if (AActor* OwnerActor = BoundObject->GetTypedOuter<AActor>())
		{
			return OwnerActor;
		}
		return BoundObject;
	}
	return nullptr;
}

FText UE::RCUIHelpers::GenerateControllerDescriptionFromEntity(const TSharedPtr<const FRemoteControlEntity>& InEntity)
{
	if (!InEntity.IsValid())
	{
		return FText::GetEmpty();
	}

	return FText::Format(INVTEXT("{0} - {1}")
		, FText::FromString(UKismetSystemLibrary::GetDisplayName(GetEntityOwner(InEntity)))
		, FText::FromName(InEntity->GetLabel()));
}

URCController* UE::RCUIHelpers::CreateControllerFromEntity(URemoteControlPreset* InPreset, const TSharedPtr<const FRemoteControlProperty>& InRemoteControlProperty)
{
	if (!InPreset || !InRemoteControlProperty.IsValid())
	{
		return nullptr;
	}

	// Derive the input data needed for creating a new Controller
	FProperty* Property = InRemoteControlProperty->GetProperty();
	EPropertyBagPropertyType PropertyBagType = EPropertyBagPropertyType::None;
	UObject* StructObject = nullptr;

	// In the Logic realm we use a single type like (eg: String / Int) to represent various related types (String/Name/Text, Int32, Int64, etc)
	// For this reason explicit mapping conversion is required between a given FProperty type and the desired Controller type
	if (!URCBehaviourBind::GetPropertyBagTypeFromFieldProperty(Property, PropertyBagType, StructObject))
	{
		return nullptr;
	}

	// Preparation step, in case we are dealing with a custom controller
	FString CustomControllerName = TEXT("");
	if (StructObject == UTexture::StaticClass() || StructObject == UTexture2D::StaticClass())
	{
		if (PropertyBagType == EPropertyBagPropertyType::String)
		{
			StructObject = nullptr;
			CustomControllerName = UE::RCCustomControllers::CustomTextureControllerName;
		}
	}

	// Create a Controller of matching type
	URCController* const NewController = CastChecked<URCController>(InPreset->AddController(URCController::StaticClass(), PropertyBagType, StructObject));
	NewController->DisplayIndex = InPreset->GetNextControllerDisplayIndex();
	NewController->Description = GenerateControllerDescriptionFromEntity(InRemoteControlProperty);

	if (!InPreset->NewControllerCategory.IsEmpty())
	{
		using namespace UE::RemoteControl::UI::Private;
		NewController->SetMetadataValue(FRCControllerPropertyInfo::CategoryName, InPreset->NewControllerCategory);
	}

	// Add metadata to this controller, if this is a custom controller
	if (!CustomControllerName.IsEmpty())
	{
		const TMap<FName, FString>& CustomControllerMetaData = UE::RCCustomControllers::GetCustomControllerMetaData(CustomControllerName);
		for (const TPair<FName, FString>& Pair : CustomControllerMetaData)
		{
			NewController->SetMetadataValue(Pair.Key, Pair.Value);
		}
	}

	// Transfer property value from Exposed Property to the New Controller.
	// The goal is to keep values synced for a Controller newly created via "Auto Bind"
	URCBehaviourBind::CopyPropertyValueToController(NewController, InRemoteControlProperty.ToSharedRef());
	return NewController;
}
