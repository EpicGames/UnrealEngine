// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointerFwd.h"
#include "UObject/ObjectPtr.h"

class IDetailTreeNode;
class IPropertyHandle;
class IPropertyRowGenerator;
class SWidget;
class URCController;
class URCVirtualPropertySelfContainer;
class URemoteControlPreset;
struct FRemoteControlEntity;
struct FRemoteControlProperty;

namespace UE::RCUIHelpers
{
	/** GetFieldClassTypeColor
	 * Fetches the editor color associated with a given Unreal Type (FProperty)
	 * Used to provide color coding in the Remote Control Logic Actions panel 
	 */
	FLinearColor GetFieldClassTypeColor(const FProperty* InProperty);

	/** GetFieldClassDisplayName
	 * Fetches the display name associated with a given Unreal Type (FProperty)
	 */
	FName GetFieldClassDisplayName(const FProperty* InProperty);

	/**
	* GetDetailTreeNodeForVirtualProperty
	*
	* Given a Virtual Property this function generates the corresponding Detail Tree Node.
	* The Property Row Generator used to create this is also set as an output parameter.
	*/
	TSharedPtr<IDetailTreeNode> GetDetailTreeNodeForVirtualProperty(TObjectPtr<URCVirtualPropertySelfContainer> InVirtualPropertySelfContainer, TSharedPtr<IPropertyRowGenerator>& OutPropertyRowGenerator);

	/** 
	* GetGenericFieldWidget
	* 
	* Constructs a widget representing a generic property using an input detail tree node
    * Optionally returns the corresponding property handle as an output parameter.
	*/
	TSharedRef<SWidget> GetGenericFieldWidget(const TSharedPtr<IDetailTreeNode> DetailTreeNode,
		TSharedPtr<IPropertyHandle>* OutPropertyHandle = nullptr,
		bool bFocusInputWidget = false);

	/** 
	* FindFocusableWidgetAndSetKeyboardFocus
	* 
	* Searches the entire widget hierarchy of a given widget for a focusable child item and sets the focus on it 
	* If the input widget is itself focusable, then we simply set the focus on it and return
	*/
	bool FindFocusableWidgetAndSetKeyboardFocus(const TSharedRef< SWidget> InWidget);

	/**
	* GetTypeColorWidget
	*
	* Generates a Type Color widget which provides a color coding guide for the user given a desired color
	* Used to provide a consistent look for the type color widget across various Remote Control Logic panels
	*/
	TSharedRef<SWidget> GetTypeColorWidget(const FProperty* InProperty);

	/**
	* GetTypeColorWidget
	*
	* Generates a Type Color widget which provides a color coding guide for the user given a desired color
	* Used to provide a consistent look for the type color widget across various Remote Control Logic panels
	* This is the direct version which takes a color and description, instead of extracting them from a property.
	*/
	TSharedRef<SWidget> GetTypeColorWidget(const FLinearColor& InTypeColor, const FText& InTypeDescription);

	/** Retrieves the entity owner for a given entity. If the entity is bound to an actor's subobject, the actor will be returned. */
	UObject* GetEntityOwner(const TSharedPtr<const FRemoteControlEntity>& InEntity);

	/** Generates the description to use for a new or existing controller, from the given property */
	FText GenerateControllerDescriptionFromEntity(const TSharedPtr<const FRemoteControlEntity>& InEntity);

	/** Automatically creates a controller for a given remote control property */
	URCController* CreateControllerFromEntity(URemoteControlPreset* InPreset, const TSharedPtr<const FRemoteControlProperty>& InRemoteControlProperty);
}