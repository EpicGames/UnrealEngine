// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlExposeMenuStyle.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush(FRemoteControlExposeMenuStyle::InContent(RelativePath, ".png" ), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FRemoteControlExposeMenuStyle::InContent(RelativePath, ".svg"), __VA_ARGS__)
#define BOX_PLUGIN_BRUSH( RelativePath, ... ) FSlateBoxBrush(FRemoteControlExposeMenuStyle::InContent( RelativePath, ".png" ), __VA_ARGS__)
#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FRemoteControlExposeMenuStyle::StyleSet;

void FRemoteControlExposeMenuStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}

	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon3x10(3.0f, 10.0f);
	const FVector2D Icon12x12(12.0f, 12.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon32x32(32.0f);
	const FVector2D Icon48x48(48.0f, 48.0f);
	const FVector2D Icon28x14(28.0f, 14.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	
	// Remote Control Expose Menu
	StyleSet->Set("RemoteControlExposeMenu.WidgetBorder", new FSlateRoundedBoxBrush(FStyleColors::Input, 5.0f));

	StyleSet->Set("RemoteControlExposeMenu.SpinBox", FSpinBoxStyle()
	.SetBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Secondary, CoreStyleConstants::InputFocusThickness))
	.SetHoveredBackgroundBrush(FSlateRoundedBoxBrush(FStyleColors::Input, CoreStyleConstants::InputFocusRadius, FStyleColors::Hover, CoreStyleConstants::InputFocusThickness))
	.SetActiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Hover, CoreStyleConstants::InputFocusRadius, FLinearColor::Transparent, CoreStyleConstants::InputFocusThickness))
	.SetInactiveFillBrush(FSlateRoundedBoxBrush(FStyleColors::Secondary, CoreStyleConstants::InputFocusRadius, FLinearColor::Transparent, CoreStyleConstants::InputFocusThickness))
	.SetArrowsImage(FSlateNoResource())
	.SetForegroundColor(FStyleColors::ForegroundHover)
	.SetTextPadding(FMargin(10.f, 3.5f, 10.f, 4.f)));

	StyleSet->Set("RemoteControlExposeMenu.Background", new FSlateColorBrush(FStyleColors::Dropdown));
	StyleSet->Set("RemoteControlExposeMenu.Outline", new FSlateRoundedBoxBrush(FStyleColors::Transparent, 0.0f, FStyleColors::DropdownOutline, 1.f));

	const FButtonStyle MenuButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button");
	StyleSet->Set("RemoteControlExposeMenu.Button", MenuButtonStyle);

	const FTextBlockStyle MenuTextBlockStyle = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("NormalText");
	StyleSet->Set("RemoteControlExposeMenu.Label", MenuTextBlockStyle);

	StyleSet->Set("RemoteControlExposeMenu.MenuBar.Padding", FMargin(12, 4));
	StyleSet->Set("RemoteControlExposeMenu.MenuIconSize", 16.f);

	// DetailsView icons
	StyleSet->Set("RemoteControlExposeMenu.NoBrush", new FSlateNoResource());
	StyleSet->Set("RemoteControlExposeMenu.VisibleAndVisibleChildren", new IMAGE_PLUGIN_BRUSH("Icons/Visible_Children_Exposed", Icon16x16));
	StyleSet->Set("RemoteControlExposeMenu.Visible", new IMAGE_PLUGIN_BRUSH("Icons/Visible", Icon16x16));
	StyleSet->Set("RemoteControlExposeMenu.HiddenAndVisibleChildren", new IMAGE_PLUGIN_BRUSH("Icons/Hidden_Children_Exposed", Icon16x16));
	StyleSet->Set("RemoteControlExposeMenu.Hidden", new IMAGE_PLUGIN_BRUSH("Icons/Hidden", Icon16x16));
	// StyleSet->Set("RemoteControlExposeMenu.Expand", new IMAGE_PLUGIN_BRUSH("Icons/Expand", Icon3x10));
	StyleSet->Set("RemoteControlExposeMenu.Expand", new IMAGE_PLUGIN_BRUSH("Icons/ellipsis_12x", Icon12x12));

	FCheckBoxStyle MenuCheckBoxStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Menu.CheckBox");
	StyleSet->Set("RemoteControlExposeMenu.CheckBox", MenuCheckBoxStyle);

	FCheckBoxStyle MenuCheckStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	MenuCheckStyle.SetPadding(FMargin(10.f, 0.f));
	MenuCheckStyle.SetCheckedImage(*FAppStyle::GetBrush("Level.VisibleIcon16x"));
	MenuCheckStyle.SetUncheckedImage(*FAppStyle::GetBrush("Level.NotVisibleIcon16x"));
	MenuCheckStyle.SetCheckedHoveredImage(*FAppStyle::GetBrush("Level.VisibleIcon16x"));
	MenuCheckStyle.SetUncheckedHoveredImage(*FAppStyle::GetBrush("Level.NotVisibleIcon16x"));
	MenuCheckStyle.SetCheckedPressedImage(*FAppStyle::GetBrush("Level.VisibleIcon16x"));
	MenuCheckStyle.SetUncheckedPressedImage(*FAppStyle::GetBrush("Level.NotVisibleIcon16x"));
	MenuCheckStyle.SetHoveredForegroundColor(FStyleColors::ForegroundHover);
	MenuCheckStyle.SetCheckedHoveredForegroundColor(FStyleColors::ForegroundHover);
	MenuCheckStyle.SetForegroundColor(FStyleColors::ForegroundHover);
	StyleSet->Set("RemoteControlExposeMenu.Check", MenuCheckStyle);

	const FCheckBoxStyle MenuRadioButtonStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Menu.RadioButton");
	StyleSet->Set("RemoteControlExposeMenu.RadioButton", MenuRadioButtonStyle);

	const FCheckBoxStyle MenuToggleButtonStyle = FAppStyle::Get().GetWidgetStyle<FCheckBoxStyle>("Menu.ToggleButton");
	StyleSet->Set("RemoteControlExposeMenu.ToggleButton", MenuToggleButtonStyle);

	StyleSet->Set("RemoteControlExposeMenu.Keybinding", FTextBlockStyle(MenuTextBlockStyle).SetFont(FStyleFonts::Get().Small));

	const FEditableTextBoxStyle NormalEditableTextBoxStyle = FAppStyle::GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");
	StyleSet->Set("RemoteControlExposeMenu.EditableText", FEditableTextBoxStyle(NormalEditableTextBoxStyle).SetFont(FStyleFonts::Get().Normal));
	StyleSet->Set("MultiboxHookColor", FLinearColor(0.f, 1.f, 0.f, 1.f));

	const FMargin MenuBlockPadding(12.0f, 1.0f, 5.0f, 1.0f);
	StyleSet->Set("RemoteControlExposeMenu.Block.IndentedPadding", MenuBlockPadding + FMargin(18.0f, 0, 0, 0));
	StyleSet->Set("RemoteControlExposeMenu.Block.Padding", MenuBlockPadding);

	StyleSet->Set("RemoteControlExposeMenu.SubMenuIndicator", new IMAGE_BRUSH_SVG("Starship/Common/chevron-right", Icon16x16, FStyleColors::Foreground));
	StyleSet->Set("RemoteControlExposeMenu.Separator", new FSlateColorBrush(FStyleColors::White25));
	StyleSet->Set("RemoteControlExposeMenu.Separator.Padding", FMargin(12.0f, 6.f, 12.0f, 6.f));

	FSlateFontInfo XSFont(DEFAULT_FONT("Bold", 7));
	XSFont.LetterSpacing =  250;
	const FTextBlockStyle SmallButtonText = FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("SmallButtonText");
	StyleSet->Set("RemoteControlExposeMenu.Heading", FTextBlockStyle(SmallButtonText)
	.SetFont(XSFont)
	.SetColorAndOpacity(FStyleColors::White25));
	StyleSet->Set("RemoteControlExposeMenu.Heading.Padding", FMargin(12.0f, 6.f, 12.f, 6.f));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG
#undef BOX_PLUGIN_BRUSH
#undef RootToContentDir

void FRemoteControlExposeMenuStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

TSharedPtr<ISlateStyle> FRemoteControlExposeMenuStyle::Get()
{
	return StyleSet;
}

FName FRemoteControlExposeMenuStyle::GetStyleSetName()
{
	static const FName RemoteControlPanelStyleName(TEXT("RemoteControlExposeMenu"));
	return RemoteControlPanelStyleName;
}

FString FRemoteControlExposeMenuStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("RemoteControl"))->GetBaseDir() + TEXT("/Resources");
	return (ContentDir / RelativePath) + Extension;
}
