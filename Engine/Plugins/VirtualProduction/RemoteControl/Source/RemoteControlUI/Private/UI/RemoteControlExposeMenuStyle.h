// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointerFwd.h"

class ISlateStyle;
class FSlateStyleSet;
class FName;
class FString;

class FRemoteControlExposeMenuStyle
{
public:
	/**
	 * Initialize the style set.
	 */
	static void Initialize();

	/**
	 * Shutdown the style set.
	 */
	static void Shutdown();

	/**
	 * Static getter for the style set.
	 */
	static TSharedPtr<ISlateStyle> Get();

	/**
	 * Static getter for the style set name.
	 */
	static FName GetStyleSetName();
private:
	/** Get a path to the module's resources folder. */
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	/** Holds the style set. */
	static TSharedPtr<FSlateStyleSet> StyleSet;
};
