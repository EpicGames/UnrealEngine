// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/RemoteControlPanelStyle.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/RemoteControlStyles.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/StarshipCoreStyle.h"
#include "Styling/StyleColors.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Styling/SlateStyleMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush(FRemoteControlPanelStyle::InContent(RelativePath, ".png" ), __VA_ARGS__)
#define IMAGE_PLUGIN_BRUSH_SVG( RelativePath, ... ) FSlateVectorImageBrush(FRemoteControlPanelStyle::InContent(RelativePath, ".svg" ), __VA_ARGS__)
#define BOX_PLUGIN_BRUSH( RelativePath, ... ) FSlateBoxBrush(FRemoteControlPanelStyle::InContent( RelativePath, ".png" ), __VA_ARGS__)
#define RootToContentDir StyleSet->RootToContentDir

TSharedPtr<FSlateStyleSet> FRemoteControlPanelStyle::StyleSet;

void FRemoteControlPanelStyle::Initialize()
{
	if (StyleSet.IsValid())
	{
		return;
	}
	
	const FVector2D Icon8x8(8.0f, 8.0f);
	const FVector2D Icon16x16(16.0f, 16.0f);
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon32x32(32.f);
	const FVector2D Icon28x14(28.0f, 14.0f);
	const FVector2D Icon64x64(64.0f, 64.0f);

	StyleSet = MakeShared<FSlateStyleSet>(GetStyleSetName());
	StyleSet->SetContentRoot(FPaths::EngineContentDir() / TEXT("Editor/Slate"));
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	StyleSet->Set("ClassThumbnail.RemoteControlPreset", new IMAGE_PLUGIN_BRUSH("Icons/RemoteControlAPI_64x", Icon64x64));
	StyleSet->Set("Icons.RemoteControlPanel", new IMAGE_PLUGIN_BRUSH_SVG("Icons/RemoteControlPanel_16", Icon16x16));

	const ISlateStyle& AppStyle = FAppStyle::Get();

	FButtonStyle ExposeFunctionButtonStyle = AppStyle.GetWidgetStyle<FButtonStyle>("PropertyEditor.AssetComboStyle");
	ExposeFunctionButtonStyle.Normal = BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f));
	ExposeFunctionButtonStyle.Normal.TintColor = FLinearColor(0, 0, 0, 0.1f);
	StyleSet->Set("RemoteControlPanel.ExposeFunctionButton", ExposeFunctionButtonStyle);

	FHyperlinkStyle ObjectSectionNameStyle = AppStyle.GetWidgetStyle<FHyperlinkStyle>("Hyperlink");
	FButtonStyle SectionNameButtonStyle = ObjectSectionNameStyle.UnderlineStyle;
	SectionNameButtonStyle.Normal = ObjectSectionNameStyle.UnderlineStyle.Hovered;
	StyleSet->Set("RemoteControlPanel.SectionNameButton", SectionNameButtonStyle);

	FTextBlockStyle SectionNameTextStyle = ObjectSectionNameStyle.TextStyle;
	SectionNameTextStyle.Font = AppStyle.GetFontStyle("DetailsView.CategoryFontStyle");
	StyleSet->Set("RemoteControlPanel.SectionName", SectionNameTextStyle);

	FButtonStyle UnexposeButtonStyle = AppStyle.GetWidgetStyle<FButtonStyle>("FlatButton");
	UnexposeButtonStyle.Normal = FSlateNoResource();
	UnexposeButtonStyle.NormalPadding = FMargin(0, 1.5f);
	UnexposeButtonStyle.PressedPadding = FMargin(0, 1.5f);
	StyleSet->Set("RemoteControlPanel.UnexposeButton", UnexposeButtonStyle);

	FTextBlockStyle ButtonTextStyle = AppStyle.GetWidgetStyle<FTextBlockStyle>("ContentBrowser.TopBar.Font");
	FLinearColor ButtonTextColor = ButtonTextStyle.ColorAndOpacity.GetSpecifiedColor();
	ButtonTextColor.A /= 2;
	ButtonTextStyle.ColorAndOpacity = ButtonTextColor;
	ButtonTextStyle.ShadowColorAndOpacity.A /= 2;
	StyleSet->Set("RemoteControlPanel.Button.TextStyle", ButtonTextStyle);

	// Default to transparent
	StyleSet->Set("RemoteControlPanel.ExposedFieldBorder", new FSlateNoResource());

	FEditableTextBoxStyle SectionNameTextBoxStyle = AppStyle.GetWidgetStyle< FEditableTextBoxStyle >("NormalEditableTextBox");
	SectionNameTextBoxStyle.BackgroundImageNormal = BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f));
	StyleSet->Set("RemoteControlPanel.SectionNameTextBox", SectionNameTextBoxStyle);

	StyleSet->Set("RemoteControlPanel.Settings", new IMAGE_BRUSH("Icons/GeneralTools/Settings_40x", Icon20x20));
	
	StyleSet->Set("RemoteControlPanel.GroupBorder", new BOX_BRUSH("Common/GroupBorder", FMargin(4.0f / 16.0f), FLinearColor(.5,.5,.5, 1.0f)));
	StyleSet->Set("RemoteControlPanel.HorizontalDash", new IMAGE_BRUSH("Common/HorizontalDottedLine_16x1px", FVector2D(16.0f, 1.0f), FLinearColor::White, ESlateBrushTileType::Horizontal));
	StyleSet->Set("RemoteControlPanel.VerticalDash", new IMAGE_BRUSH("Common/VerticalDottedLine_1x16px", FVector2D(1.0f, 16.0f), FLinearColor::White, ESlateBrushTileType::Vertical));

	FLinearColor NewSelectionColor = AppStyle.GetWidgetStyle<FTableRowStyle>("TableView.Row").ActiveBrush.TintColor.GetSpecifiedColor();
	NewSelectionColor.R *= 1.8f;
	NewSelectionColor.G *= 1.8f;
	NewSelectionColor.B *= 1.8f;
	NewSelectionColor.A = 0.4f;


	StyleSet->Set("RemoteControlPanel.GroupRowSelected", new BOX_BRUSH("Common/GroupBorderLight", FMargin(4.0f / 16.0f), NewSelectionColor));
	StyleSet->Set("RemoteControlPanel.TransparentBorder", new FSlateColorBrush(FColor::Transparent));

	StyleSet->Set("RemoteControlPanel.EntityInUseBorder", new FSlateColorBrush(FStyleColors::SelectInactive.GetSpecifiedColor()));
	
	// Toggle Button Style
	StyleSet->Set("Switch.ToggleOff", new IMAGE_PLUGIN_BRUSH("Icons/Switch_OFF", Icon28x14));
	StyleSet->Set("Switch.ToggleOn", new IMAGE_PLUGIN_BRUSH("Icons/Switch_ON", Icon28x14));

	// Remote Control Logic UI
	StyleSet->Set("RemoteControlPanel.Behaviours.BehaviourDescription", FCoreStyle::GetDefaultFontStyle("Regular", 10));
	StyleSet->Set("RemoteControlPanel.Behaviours.CustomBlueprint", new IMAGE_BRUSH_SVG("Starship/MainToolbar/blueprints", Icon20x20));
	StyleSet->Set("RemoteControlPanel.Actions.ValuePanelHeader", FCoreStyle::GetDefaultFontStyle("Bold", 12));

	// Remote Control Panel UI
	SetupPanelStyles(StyleSet.ToSharedRef());

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
}

void FRemoteControlPanelStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

TSharedPtr<ISlateStyle> FRemoteControlPanelStyle::Get()
{
	return StyleSet;
}

FName FRemoteControlPanelStyle::GetStyleSetName()
{
	static const FName RemoteControlPanelStyleName(TEXT("RemoteControlPanelStyle"));
	return RemoteControlPanelStyleName;
}

FString FRemoteControlPanelStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	static const FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("RemoteControl"))->GetBaseDir() + TEXT("/Resources");
	return (ContentDir / RelativePath) + Extension;
}

void FRemoteControlPanelStyle::SetupPanelStyles(TSharedRef<FSlateStyleSet> InStyle)
{
	const ISlateStyle& AppStyle = FAppStyle::Get();

	// Flat Button Style
	FButtonStyle FlatButtonStyle = AppStyle.GetWidgetStyle<FButtonStyle>("DetailsView.NameAreaButton");
	FlatButtonStyle.Normal = *AppStyle.GetBrush("NoBorder");
	FlatButtonStyle.Disabled = *AppStyle.GetBrush("NoBorder");

	// Toggle Button Style
	FCheckBoxStyle ToggleButtonStyle = FCheckBoxStyle()
		.SetForegroundColor(FLinearColor::White)
		.SetUncheckedImage(*StyleSet->GetBrush("Switch.ToggleOff"))
		.SetUncheckedHoveredImage(*StyleSet->GetBrush("Switch.ToggleOff"))
		.SetUncheckedPressedImage(*StyleSet->GetBrush("Switch.ToggleOff"))
		.SetCheckedImage(*StyleSet->GetBrush("Switch.ToggleOn"))
		.SetCheckedHoveredImage(*StyleSet->GetBrush("Switch.ToggleOn"))
		.SetCheckedPressedImage(*StyleSet->GetBrush("Switch.ToggleOn"))
		.SetPadding(FMargin(1, 1, 1, 1));

	// Table View Style
	FTableViewStyle TableViewStyle = AppStyle.GetWidgetStyle<FTableViewStyle>("SimpleListView");
	TableViewStyle.BackgroundBrush = FSlateColorBrush(FStyleColors::Background);

	// Header Row Style
	FHeaderRowStyle HeaderRowStyle = AppStyle.GetWidgetStyle<FHeaderRowStyle>("PropertyTable.HeaderRow");
	HeaderRowStyle.BackgroundBrush = FSlateColorBrush(FStyleColors::Background);
	HeaderRowStyle.SplitterHandleSize = 1.f;
	HeaderRowStyle.ColumnSplitterStyle = FSplitterStyle()
	.SetHandleNormalBrush(FSlateColorBrush(FStyleColors::AccentGray))
	.SetHandleHighlightBrush(FSlateColorBrush(FStyleColors::AccentGray));
	HeaderRowStyle.HorizontalSeparatorBrush = FSlateColorBrush(FStyleColors::Background);

	// Table Row Style
	FTableRowStyle TableRowStyle = AppStyle.GetWidgetStyle<FTableRowStyle>("DetailsView.TreeView.TableRow");
	TableRowStyle.SetSelectorFocusedBrush(FSlateColorBrush(FStyleColors::Highlight));

	TableRowStyle.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel));
	TableRowStyle.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::Panel));
	TableRowStyle.SetEvenRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Background));
	TableRowStyle.SetOddRowBackgroundHoveredBrush(FSlateColorBrush(FStyleColors::Background));
	TableRowStyle.SetActiveHoveredBrush(FSlateColorBrush(FStyleColors::Select));
	TableRowStyle.SetInactiveHoveredBrush(FSlateColorBrush(FStyleColors::SelectInactive));
	TableRowStyle.SetActiveBrush(FSlateColorBrush(FStyleColors::Select));
	TableRowStyle.SetInactiveBrush(FSlateColorBrush(FStyleColors::SelectInactive));

	StyleSet->Set("RemoteControlPanel.GroupRow", TableRowStyle);

	// Mode Switcher Styles
	FSlateBrush ContentAreaBrushDark = BOX_BRUSH("Common/DarkGroupBorder", FMargin(4.f / 16.f), FLinearColor(0.5f, 0.5f, 0.5f, 1.f));
	FSlateBrush ContentAreaBrushLight = BOX_BRUSH("Common/LightGroupBorder", FMargin(4.f / 16.f));

	FCheckBoxStyle SwitchButtonStyle = AppStyle.GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	SwitchButtonStyle.SetCheckBoxType(ESlateCheckBoxType::ToggleButton);
	SwitchButtonStyle.SetPadding(FMargin(4.f, 2.f));

	FCheckBoxStyle AssetPathToggleButtonStyle = AppStyle.GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckbox");
	AssetPathToggleButtonStyle.SetCheckBoxType(ESlateCheckBoxType::ToggleButton);
	AssetPathToggleButtonStyle.SetUncheckedImage(BOX_BRUSH("Common/RoundedSelection_16x", 4.0f / 16.0f, FStyleColors::Background));

	StyleSet->Set("RemoteControlPathBehaviour.AssetCheckBox", AssetPathToggleButtonStyle);
	// Text Styles
	const FStyleFonts& StyleFonts = FStyleFonts::Get();

	const FTextBlockStyle& NormalText = AppStyle.GetWidgetStyle<FTextBlockStyle>("NormalText");

	const FTextBlockStyle HeaderText = FTextBlockStyle(NormalText)
		.SetFont(StyleFonts.Large);

	FRCPanelStyle RCMinorPanelStyle = FRCPanelStyle()
		.SetFlatButtonStyle(FlatButtonStyle)
		.SetSwitchButtonStyle(SwitchButtonStyle)
		.SetToggleButtonStyle(ToggleButtonStyle)

		.SetContentAreaBrush(FSlateColorBrush(FStyleColors::Panel))
		.SetContentAreaBrushDark(ContentAreaBrushDark)
		.SetContentAreaBrushLight(ContentAreaBrushLight)
		.SetSectionHeaderBrush(FSlateColorBrush(FStyleColors::Foldout))

		.SetHeaderRowStyle(HeaderRowStyle)
		.SetTableRowStyle(TableRowStyle)
		.SetTableViewStyle(TableViewStyle)

		.SetHeaderTextStyle(HeaderText)
		.SetPanelTextStyle(NormalText)
		.SetSectionHeaderTextStyle(HeaderText);

	StyleSet->Set("RemoteControlPanel.MinorPanel", RCMinorPanelStyle);

	FRCPanelStyle RCLogicControllersPanelStyle = RCMinorPanelStyle;
	RCLogicControllersPanelStyle.SetHeaderRowPadding(FMargin(15.f, 2.f));

	// Table Row Style
	FTableRowStyle ControllersTableRowStyle = RCLogicControllersPanelStyle.TableRowStyle;
	ControllersTableRowStyle.SetEvenRowBackgroundBrush(FSlateColorBrush(FStyleColors::WindowBorder));
	ControllersTableRowStyle.SetOddRowBackgroundBrush(FSlateColorBrush(FStyleColors::WindowBorder));

	RCLogicControllersPanelStyle.SetTableRowStyle(ControllersTableRowStyle);

	StyleSet->Set("RemoteControlPanel.LogicControllersPanel", RCLogicControllersPanelStyle);

	// Behaviour Panel style
	FRCPanelStyle RCBehaviourPanelStyle = RCMinorPanelStyle;

	const FTextBlockStyle BehaviourPanelHeaderText = FTextBlockStyle(NormalText)
		.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 12));
	RCBehaviourPanelStyle.SetHeaderTextStyle(BehaviourPanelHeaderText);

	StyleSet->Set("RemoteControlPanel.BehaviourPanel", RCBehaviourPanelStyle);
}

#undef IMAGE_PLUGIN_BRUSH
#undef IMAGE_PLUGIN_BRUSH_SVG
#undef BOX_PLUGIN_BRUSH
#undef RootToContentDir
