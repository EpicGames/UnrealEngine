// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCHeaderRow.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Layout/WidgetPath.h"
#include "RemoteControlSettings.h"

void SRCHeaderRow::Construct(const FArguments& InArgs)
{
	bOverrideCanSelectGeneratedColumn = InArgs._CanSelectGeneratedColumn;

	TArray<SHeaderRow::FColumn*> Slots;
	for (FColumn* const Column : InArgs.Slots)
	{
		Column->bIsVisible = !InArgs._HiddenColumnsList.Contains(Column->ColumnId);
		Columns.Add(Column);
		SHeaderRow::FColumn* HeaderRowSlot = CreateHeaderRowColumn(*Column);
		Slots.Add(HeaderRowSlot);
	}

	SHeaderRow::FArguments HeaderRowArgs;
	HeaderRowArgs._Style = InArgs._Style;
	HeaderRowArgs.Slots = Slots;
	HeaderRowArgs._CanSelectGeneratedColumn = InArgs._CanSelectGeneratedColumn;
	HeaderRowArgs._HiddenColumnsList = InArgs._HiddenColumnsList;
	SHeaderRow::Construct(HeaderRowArgs);
}

FReply SRCHeaderRow::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bOverrideCanSelectGeneratedColumn && MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		FVector2f SummonLocation = MouseEvent.GetScreenSpacePosition();
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		const bool CloseAfterSelection = true;
		FMenuBuilder MenuBuilder(CloseAfterSelection, nullptr);
		OnOverrideGenerateSelectColumnsSubMenu(MenuBuilder);

		FSlateApplication::Get().CloseToolTip();
		FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

SHeaderRow::FColumn* SRCHeaderRow::CreateHeaderRowColumn(const FColumn InColumn)
{
	SHeaderRow::FColumn::FArguments SlotsArgument;
	SlotsArgument._ColumnId = InColumn.ColumnId;
	SlotsArgument._DefaultLabel = InColumn.DefaultText;
	SlotsArgument._HAlignHeader = InColumn.HeaderHAlignment;
	SlotsArgument._VAlignHeader = InColumn.HeaderVAlignment;
	SlotsArgument._HAlignCell = InColumn.CellHAlignment;
	SlotsArgument._VAlignCell = InColumn.CellVAlignment;
	SlotsArgument._HeaderContentPadding = InColumn.HeaderContentPadding;
	SlotsArgument._ShouldGenerateWidget = InColumn.ShouldGenerateWidget;

	SHeaderRow::FColumn* HeaderRowSlot = new SHeaderRow::FColumn(SlotsArgument);

	HeaderRowSlot->Width = InColumn.DefaultWidth;
	HeaderRowSlot->DefaultWidth = InColumn.DefaultWidth;
	HeaderRowSlot->SizeRule = InColumn.SizeRule;
	HeaderRowSlot->bIsVisible = InColumn.bIsVisible;
	return HeaderRowSlot;
}

void SRCHeaderRow::RegenerateOverriddenWidgets()
{
	ClearColumns();

	for (const FColumn& Column : Columns)
	{
		SHeaderRow::FColumn* HeaderRowSlot = CreateHeaderRowColumn(Column);
		AddColumn(*HeaderRowSlot);
	}

	RefreshColumns();
}

void SRCHeaderRow::OnOverrideGenerateSelectColumnsSubMenu(FMenuBuilder& InSubMenuBuilder)
{
	for (const SRCHeaderRow::FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ShouldGenerateSubMenuEntry.IsSet() && !SomeColumn.ShouldGenerateSubMenuEntry.Get())
		{
			continue;
		}

		const bool bCanExecuteAction = !SomeColumn.ShouldGenerateWidget.IsSet();
		const FName ColumnId = SomeColumn.ColumnId;

		InSubMenuBuilder.AddMenuEntry(
			SomeColumn.DefaultText,
			FText::GetEmpty(),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(this, &SRCHeaderRow::ToggleOverriddenGeneratedColumn, ColumnId),
				FCanExecuteAction::CreateLambda([bCanExecuteAction]() { return bCanExecuteAction; }),
				FGetActionCheckState::CreateSP(this, &SRCHeaderRow::GetOverriddenGeneratedColumnCheckedState, ColumnId)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton);
	}
}

void SRCHeaderRow::ToggleOverriddenGeneratedColumn(FName ColumnId)
{
	for (FColumn& SomeColumn : Columns)
	{
		if (SomeColumn.ColumnId == ColumnId)
		{
			if (!SomeColumn.ShouldGenerateWidget.IsSet())
			{
				SomeColumn.bIsVisible = !SomeColumn.bIsVisible;
				RegenerateOverriddenWidgets();
			}

			break;
		}
	}

	StoreHiddenColumnsSettings();
}

ECheckBoxState SRCHeaderRow::GetOverriddenGeneratedColumnCheckedState(FName ColumnId) const
{
	return IsColumnGenerated(ColumnId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SRCHeaderRow::StoreHiddenColumnsSettings()
{
	URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
	
	if (!Settings)
	{
		return;
	}

	TSet<FName> HiddenColumns;
	
	for (const FColumn& SomeColumn : Columns)
	{
		if (!SomeColumn.bIsVisible)
		{
			const FName& ColumnName = SomeColumn.ColumnId;
			if (URemoteControlSettings::GetExposedEntitiesColumnNames().Contains(ColumnName))
			{
				HiddenColumns.Add(ColumnName);
			}
		}
	}
	
	Settings->EntitiesListHiddenColumns = HiddenColumns;
	Settings->PostEditChange();
	Settings->SaveConfig();
}
