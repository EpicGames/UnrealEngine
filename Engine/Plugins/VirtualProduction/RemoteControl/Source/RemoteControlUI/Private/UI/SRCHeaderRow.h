// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/Views/SHeaderRow.h"

class FMenuBuilder;
enum class ECheckBoxState : uint8;

/**
 * A custom Header Row to hide entry in the SubMenu to Hide/Show the Header.
 */
class SRCHeaderRow : public SHeaderRow
{
public:

	class FColumn
	{
	public:

		SLATE_BEGIN_ARGS(FColumn)
			: _ColumnId()
			, _DefaultLabel()
			, _FillWidth(1.f)
			, _FixedWidth()
			, _HAlignHeader(HAlign_Fill)
			, _VAlignHeader(VAlign_Fill)
			, _HeaderContentPadding()
			, _HAlignCell(HAlign_Fill)
			, _VAlignCell(VAlign_Fill)
		{}

			SLATE_ARGUMENT(FName, ColumnId)
			SLATE_ATTRIBUTE(FText, DefaultLabel)
			SLATE_ATTRIBUTE(float, FillWidth)
			SLATE_ARGUMENT(TOptional<float>, FixedWidth)
			SLATE_ARGUMENT(EHorizontalAlignment, HAlignHeader)
			SLATE_ARGUMENT(EVerticalAlignment, VAlignHeader)
			SLATE_ARGUMENT(TOptional<FMargin>, HeaderContentPadding)
			SLATE_ARGUMENT(EHorizontalAlignment, HAlignCell)
			SLATE_ARGUMENT(EVerticalAlignment, VAlignCell)
			SLATE_ATTRIBUTE(bool, ShouldGenerateWidget)

			/* If true the Entry is showed in the SubMenu to make the Column Visible or Hide it	*/
			SLATE_ATTRIBUTE(bool, ShouldGenerateSubMenuEntry)

		SLATE_END_ARGS()

		FColumn(const FArguments& InArgs)
			: ColumnId(InArgs._ColumnId)
			, DefaultText(InArgs._DefaultLabel)
			, Width(1.0f)
			, DefaultWidth(1.0f)
			, SizeRule(EColumnSizeMode::Fill)
			, HeaderHAlignment(InArgs._HAlignHeader)
			, HeaderVAlignment(InArgs._VAlignHeader)
			, HeaderContentPadding(InArgs._HeaderContentPadding)
			, CellHAlignment(InArgs._HAlignCell)
			, CellVAlignment(InArgs._VAlignCell)
			, ShouldGenerateWidget(InArgs._ShouldGenerateWidget)
			, ShouldGenerateSubMenuEntry(InArgs._ShouldGenerateSubMenuEntry)
			, bIsVisible(true)
		{
			if (InArgs._FixedWidth.IsSet())
			{
				Width = InArgs._FixedWidth.GetValue();
				SizeRule = EColumnSizeMode::Fixed;
			}
			else
			{
				Width = InArgs._FillWidth;
				SizeRule = EColumnSizeMode::Fill;
			}

			DefaultWidth = Width.Get();
		}

	public:

		FName ColumnId;
		TAttribute<FText> DefaultText;
		TAttribute<float> Width;
		float DefaultWidth;
		EColumnSizeMode::Type SizeRule;
		EHorizontalAlignment HeaderHAlignment;
		EVerticalAlignment HeaderVAlignment;
		TOptional<FMargin> HeaderContentPadding;
		EHorizontalAlignment CellHAlignment;
		EVerticalAlignment CellVAlignment;
		TAttribute<bool> ShouldGenerateWidget;
		TAttribute<bool> ShouldGenerateSubMenuEntry;
		bool bIsVisible;
	};

	static FColumn::FArguments Column(const FName& InColumnId)
	{
		FColumn::FArguments NewArgs;
		NewArgs.ColumnId(InColumnId);
		return NewArgs;
	}

	SLATE_BEGIN_ARGS(SRCHeaderRow)
		: _CanSelectGeneratedColumn(false)
		, _HiddenColumnsList({})
	{}

		SLATE_STYLE_ARGUMENT(FHeaderRowStyle, Style)
		SLATE_SUPPORTS_SLOT_WITH_ARGS(FColumn)
		SLATE_ARGUMENT(bool, CanSelectGeneratedColumn)
	
		/** Add here the columns to be hidden by default. They can still be re-enabled via right click context-menu in the header row. */
		SLATE_ARGUMENT(TArray<FName>, HiddenColumnsList)

	SLATE_END_ARGS()

	/**
	 * Construct the widget with given InArgs.
	 */
	void Construct(const FArguments& InArgs);

	//~ Begin SWidget interface
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	//~ End SWidget interface

private:

	/**
	 * Creates and returns a customized column object.
	 */
	SHeaderRow::FColumn* CreateHeaderRowColumn(const FColumn InColumn);
	
	/**
	 * Overridden method that is used to regenerate the custom column(s).
	 */
	void RegenerateOverriddenWidgets();

	/**
	 * Overridden method that is called when the user perform right click on custom column(s).
	 */
	void OnOverrideGenerateSelectColumnsSubMenu(FMenuBuilder& InSubMenuBuilder);

	/**
	 * Toggles the visibility of the overridden custom column(s).
	 */
	void ToggleOverriddenGeneratedColumn(FName ColumnId);

	/**
	 * Rretrieves the check box state of the overridden custom column(s).
	 */
	ECheckBoxState GetOverriddenGeneratedColumnCheckedState(FName ColumnId) const;

	/** Save the current settings for shown/hidden columns to Remote Control Config */
	void StoreHiddenColumnsSettings();

private:

	/**
	 * Holds the array of overridden custom column(s).
	 */
	TIndirectArray<FColumn> Columns;
	
	/**
	 * Determines whether the column can be shown in the context menu or not.
	 */
	bool bOverrideCanSelectGeneratedColumn;
};
