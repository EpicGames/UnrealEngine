// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCLogger.h"

#include "MessageLogModule.h"
#include "RemoteControlLogger.h"
#include "Modules/ModuleManager.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"

void SRCLogger::Construct(const FArguments& InArgs)
{
	FRemoteControlLogger& RemoteControlLogger = FRemoteControlLogger::Get();

	// Create widget from MessageLog module
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	LogListingWidget = MessageLogModule.CreateLogListingWidget(RemoteControlLogger.GetMessageLogListing().ToSharedRef());

	SetVisibility
	(
		TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateLambda([]()
		{
			return FRemoteControlLogger::Get().IsEnabled() ? EVisibility::Visible : EVisibility::Collapsed;
		}))
	);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Menu.Background"))
		.Padding(2.f)
		[
			LogListingWidget.ToSharedRef()
		]
	];
}
