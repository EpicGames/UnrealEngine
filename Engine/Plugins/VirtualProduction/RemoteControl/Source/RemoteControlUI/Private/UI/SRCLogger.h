// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;

/**
 * Lists RemoteControlLogging entries
 */
class REMOTECONTROLUI_API SRCLogger : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCLogger)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

private:
	/** Pointer to log widget */
	TSharedPtr<SWidget> LogListingWidget;
};
