// Copyright Epic Games, Inc. All Rights Reserved.

#include "UI/SRCModeSwitcher.h"

#include "RemoteControlPanelStyle.h"
#include "Styling/RemoteControlStyles.h"

#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SRCModeSwitcher"

void SRCModeSwitcher::Construct(const FArguments& InArgs)
{
	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	ActiveMode = InArgs._DefaultMode;

	ModeSwitched = InArgs._OnModeSwitched;

	if (InArgs._OnModesChanged.IsBound())
	{
		ModesChanged.Add(InArgs._OnModesChanged);
	}

	SBorder::Construct(SBorder::FArguments()
		.Padding(RCPanelStyle->PanelPadding)
		.BorderImage(&RCPanelStyle->ContentAreaBrushDark)
	);

	FRCMode* ActiveModeElem = nullptr;
	
	// Copy all the mode info from the declaration
	for (FRCMode* const Mode : InArgs.Slots)
	{
		RCModes.Add(*Mode);

		Mode->bIsVisible = true;

		if (Mode->ModeId == ActiveMode)
		{
			ActiveModeElem = Mode;
		}
	}

	// Generate widgets for all modes
	RegenerateWidgets();

	// Force-enable the default mode
	if (ActiveModeElem)
	{
		constexpr bool bForceModeEnable = true;
		SetModeEnabled_Internal(ECheckBoxState::Checked, ActiveModeElem, bForceModeEnable);
	}
}

ECheckBoxState SRCModeSwitcher::IsModeEnabled(FRCMode* InMode) const
{
	check(InMode);

	return ActiveMode == InMode->ModeId ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SRCModeSwitcher::RegenerateWidgets()
{
	TSharedRef<SHorizontalBox> ModeSwitcher = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::ClipToBounds);

	// Construct widgets for all modes
	{
		for (int32 SlotIndex = 0; SlotIndex < RCModes.Num(); ++SlotIndex)
		{
			FRCMode& SomeMode = RCModes[SlotIndex];

			if (SomeMode.ShouldGenerateWidget.Get(true) && SomeMode.bIsVisible)
			{
				const FText ModeLabel = FText::Format(LOCTEXT("ModeLabel", "{0}"), { SomeMode.DefaultText.IsSet() ? SomeMode.DefaultText.Get() : FText::FromString(SomeMode.ModeId.ToString() + TEXT("[LabelMissing]")) });

				TSharedRef<SWidget> ContentWidget = SNullWidget::NullWidget;

				if (SomeMode.bShowIcon)
				{
					// If we were supplied an image than go ahead and use that.
					TSharedRef<SLayeredImage> IconWidget = SNew(SLayeredImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Visibility(EVisibility::HitTestInvisible)
						.Image(SomeMode.OptionalIcon.Get().GetIcon());

					ContentWidget = SNew(SHorizontalBox)

					// Icon Widget
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Center)
					.Padding(2.f, 4.f)
					[
						IconWidget
					]
					
					// Label Text
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(RCPanelStyle->HeaderRowPadding)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(ModeLabel)
						.TextStyle(&RCPanelStyle->PanelTextStyle)
					];
				}
				else
				{
					ContentWidget = SNew(STextBlock)
						.Text(ModeLabel)
						.TextStyle(&RCPanelStyle->PanelTextStyle);
				}

				TSharedRef<SWidget> NewModeSwitcher = SNew(SCheckBox)
					.Style(&RCPanelStyle->SwitchButtonStyle)
					.IsChecked(this, &SRCModeSwitcher::IsModeEnabled, &SomeMode)
					.OnCheckStateChanged(this, &SRCModeSwitcher::SetModeEnabled, &SomeMode)
					.HAlign(HAlign_Center)
					.ToolTip(SomeMode.ToolTip)
					.ToolTipText(SomeMode.ToolTip.IsSet() ? TAttribute<FText>() :
						SomeMode.DefaultTooltip.IsSet() ? SomeMode.DefaultTooltip : ModeLabel)
					.Padding(FMargin(4.f, 0.f))
					[
						ContentWidget
					];

				switch (SomeMode.SizeRule)
				{
					case EModeWidgetSizing::Fill:
					default:
					{
						// Add resizable cell
						ModeSwitcher->AddSlot()
							.FillWidth(SomeMode.GetWidth())
							.HAlign(SomeMode.CellHAlignment)
							.VAlign(SomeMode.CellVAlignment)
							[
								NewModeSwitcher
							];
					}
					break;
					case EModeWidgetSizing::Fixed:
					{
						// Add fixed size cell
						ModeSwitcher->AddSlot()
							.AutoWidth()
							.HAlign(SomeMode.CellHAlignment)
							.VAlign(SomeMode.CellVAlignment)
							[
								SNew(SBox)
								.WidthOverride(SomeMode.GetWidth())
								[
									NewModeSwitcher
								]
							];
					}
					break;
					case EModeWidgetSizing::Auto:
					{
						// Add size to content cell
						ModeSwitcher->AddSlot()
							.AutoWidth()
							.HAlign(SomeMode.CellHAlignment)
							.VAlign(SomeMode.CellVAlignment)
							[
								NewModeSwitcher
							];
					}
					break;
				}
			}
		}
	}

	SetContent(ModeSwitcher);
}

void SRCModeSwitcher::SetModeEnabled(ECheckBoxState NewState, FRCMode* NewMode)
{
	SetModeEnabled_Internal(NewState, NewMode);
}

void SRCModeSwitcher::SetModeEnabled_Internal(ECheckBoxState NewState, FRCMode* NewMode, bool bInForceModeEnabled /* false */)
{
	check(NewMode);

	if (ActiveMode != NewMode->ModeId || bInForceModeEnabled)
	{
		if (NewState == ECheckBoxState::Checked)
		{
			ActiveMode = NewMode->ModeId;

			OnModeSwitched().ExecuteIfBound(*NewMode);
		}
	}
}

const TArray<SRCModeSwitcher::FRCMode>& SRCModeSwitcher::GetModes() const
{
	return RCModes;
}

void SRCModeSwitcher::AddMode(const FRCMode::FArguments& NewModeArgs)
{
	SRCModeSwitcher::FRCMode NewMode = SRCModeSwitcher::FRCMode(NewModeArgs);

	AddMode(NewMode);
}

void SRCModeSwitcher::AddMode(FRCMode& NewMode)
{
	int32 InsertIdx = RCModes.Num();

	InsertMode(NewMode, InsertIdx);
}

void SRCModeSwitcher::InsertMode(const FRCMode::FArguments& NewModeArgs, int32 InsertIdx)
{
	SRCModeSwitcher::FRCMode NewMode = SRCModeSwitcher::FRCMode(NewModeArgs);

	InsertMode(NewMode, InsertIdx);
}

void SRCModeSwitcher::InsertMode(FRCMode& NewMode, int32 InsertIdx)
{
	check(NewMode.ModeId != NAME_None);

	if (NewMode.bIsDefault)
	{
		ActiveMode = NewMode.ModeId;
	}

	RCModes.Insert(NewMode, InsertIdx);

	ModesChanged.Broadcast(SharedThis(this));

	RegenerateWidgets();
}

void SRCModeSwitcher::RemoveMode(const FName& InModeId)
{
	check(InModeId != NAME_None);

	for (int32 SlotIndex = RCModes.Num() - 1; SlotIndex >= 0; --SlotIndex)
	{
		FRCMode& RCMode = RCModes[SlotIndex];

		if (RCMode.ModeId == InModeId)
		{
			RCModes.RemoveAt(SlotIndex);
		}
	}

	ModesChanged.Broadcast(SharedThis(this));

	RegenerateWidgets();
}

void SRCModeSwitcher::RefreshModes()
{
	RegenerateWidgets();
}

void SRCModeSwitcher::ClearModes()
{
	const bool bHadModeSwitches = RCModes.Num() > 0;
	
	RCModes.Empty();
	
	if (bHadModeSwitches)
	{
		ModesChanged.Broadcast(SharedThis(this));
	}

	RegenerateWidgets();
}

#undef LOCTEXT_NAMESPACE
