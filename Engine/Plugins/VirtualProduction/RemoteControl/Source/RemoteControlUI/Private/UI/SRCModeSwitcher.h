// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Layout/SBorder.h"

struct FRCPanelStyle;
class SWidget;

namespace EModeWidgetSizing
{
	enum Rule
	{
		/** Mode widget stretches to a fraction of the mode switcher. */
		Fill = 0,

		/**	Mode widget is fixed width and cannot be resized. */
		Fixed = 1,

		/** Mode widget will be sized to fit the content. */
		Auto = 2,
	};
};

/**
 * A custom mode switcher for Remote Control Panel.
 */
class SRCModeSwitcher : public SBorder
{
public:
	/** Describes a single mode switch. */
	class FRCMode
	{
	public:

		SLATE_BEGIN_ARGS(FRCMode)
			: _ModeId()
			, _DefaultLabel()
			, _DefaultTooltip()
			, _ToolTip()
			, _FillWidth(1.0f)
			, _FixedWidth()
			, _HAlignCell(HAlign_Fill)
			, _VAlignCell(VAlign_Fill)
			, _IsDefault(false)
		{}
		
			SLATE_ARGUMENT(FName, ModeId)

			SLATE_ATTRIBUTE(FText, DefaultLabel)
			SLATE_ATTRIBUTE(FText, DefaultTooltip)
			SLATE_ATTRIBUTE(TSharedPtr< IToolTip >, ToolTip)

			/** Set the Mode Size Mode to Fill. It's a fraction between 0 and 1 */
			SLATE_ATTRIBUTE(float, FillWidth)
			/** Set the Mode Size Mode to Fixed. */
			SLATE_ARGUMENT(TOptional<float>, FixedWidth)

			SLATE_ARGUMENT(EHorizontalAlignment, HAlignCell)
			SLATE_ARGUMENT(EVerticalAlignment, VAlignCell)

			SLATE_ATTRIBUTE(FSlateIcon, OptionalIcon)

			SLATE_ARGUMENT(bool, IsDefault)
			SLATE_ATTRIBUTE(bool, ShouldGenerateWidget)

		SLATE_END_ARGS()

		FRCMode(const FArguments& InArgs)
			: bIsDefault(InArgs._IsDefault)
			, bIsVisible(true)
			, bShowIcon(false)
			, CellHAlignment(InArgs._HAlignCell)
			, CellVAlignment(InArgs._VAlignCell)
			, DefaultText(InArgs._DefaultLabel)
			, DefaultTooltip(InArgs._DefaultTooltip)
			, ModeId(InArgs._ModeId)
			, OptionalIcon(InArgs._OptionalIcon)
			, ToolTip(InArgs._ToolTip)
			, ShouldGenerateWidget(InArgs._ShouldGenerateWidget)
			, SizeRule(EModeWidgetSizing::Fill)
			, Width(1.f)
		{
			if ( InArgs._FixedWidth.IsSet() )
			{
				Width = InArgs._FixedWidth.GetValue();
				SizeRule = EModeWidgetSizing::Fixed;
			}
			else if ( InArgs._FillWidth.IsSet() )
			{
				Width = InArgs._FillWidth;
				SizeRule = EModeWidgetSizing::Fill;
			}
			else
			{
				SizeRule = EModeWidgetSizing::Auto;
			}

			if (OptionalIcon.IsSet())
			{
				bShowIcon = true;
			}
		}

		bool operator==(const FRCMode& OtherMode)
		{
			return ModeId == OtherMode.ModeId;
		}
		
		bool operator==(const FRCMode& OtherMode) const
		{
			return ModeId == OtherMode.ModeId;
		}

	public:

		void SetWidth(float NewWidth)
		{
			Width = NewWidth;
		}

		float GetWidth() const
		{
			return Width.Get();
		}

		/** Whether this mode is default or not. */
		bool bIsDefault;
		
		/** Determines the visibility of the mode widget for this mode. */
		bool bIsVisible;

		/** Determines whether we need to generate icon widget for this mode or not. */
		bool bShowIcon;

		/** Holds the mode widget alignment by horizontal. */
		EHorizontalAlignment CellHAlignment;

		/** Holds the mode widget alignment by vertical. */
		EVerticalAlignment CellVAlignment;

		/** Default text to use if no widget is passed in. */
		TAttribute<FText> DefaultText;

		/** Default tooltip to use if no widget is passed in */
		TAttribute<FText> DefaultTooltip;

		/** A unique ID for this mode, so that it can be saved and restored. */
		FName ModeId;

		/** Optional icon to be used. */
		TAttribute<FSlateIcon> OptionalIcon;

		/** Custom tooltip to use */
		TAttribute<TSharedPtr<IToolTip>> ToolTip;

		/** Determines whether we need to generate mode widget for this mode or not. */
		TAttribute<bool> ShouldGenerateWidget;

		/** Determines the sizing of the mode widget for this mode. */
		EModeWidgetSizing::Rule SizeRule;

		/** Width of the mode widget in Slate Units */
		TAttribute<float> Width;
	};

	/** Create a mode with the specified ModeId */
	static FRCMode::FArguments Mode(const FName& InModeId)
	{
		FRCMode::FArguments NewArgs;
		NewArgs.ModeId(InModeId);
		return NewArgs;
	}

	/** Delegate to be invoked when the active mode is switched/changed. */
	DECLARE_DELEGATE_OneParam(FOnModeSwitched, const FRCMode& /* NewMode */);
	FOnModeSwitched& OnModeSwitched() { return ModeSwitched; }

	/** Delegate to be invoked when the number of modes changed. */
	DECLARE_EVENT_OneParam(SRCModeSwitcher, FModesChanged, const TSharedRef<SRCModeSwitcher>& /* ModeSwitcher */);
	FModesChanged* OnModesChanged() { return &ModesChanged; }
	
	SLATE_BEGIN_ARGS(SRCModeSwitcher)
	{}

		SLATE_SUPPORTS_SLOT_WITH_ARGS(FRCMode)

		SLATE_ARGUMENT(FName, DefaultMode)

		SLATE_EVENT(FOnModeSwitched, OnModeSwitched)
		SLATE_EVENT(FModesChanged::FDelegate, OnModesChanged)

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);
		
	/** @return the Modes driven by the mode switcher. */
	const TArray<FRCMode>& GetModes() const;

	/** Adds a mode to the mode switcher. */
	void AddMode(const FRCMode::FArguments& NewModeArgs);
	void AddMode(FRCMode& NewMode);

	/** Inserts a mode at the specified index in the mode switcher. */
	void InsertMode(const FRCMode::FArguments& NewModeArgs, int32 InsertIdx);
	void InsertMode(FRCMode& NewMode, int32 InsertIdx);

	/** Removes a mode from the mode switcher. */
	void RemoveMode(const FName& InModeId);

	/** Force refreshing of the mode widgets*/
	void RefreshModes();

	/** Removes all modes from the mode switcher. */
	void ClearModes();

private:

	/** Retrieves whether the given mode is enabled or not. */
	ECheckBoxState IsModeEnabled(FRCMode* InMode) const;
	
	/** Regenerates all widgets in the mode switcher. */
	void RegenerateWidgets();

	/** Sets the given mode as active. */
	void SetModeEnabled(ECheckBoxState NewState, FRCMode* NewMode);

private:

	/**
	 * Sets the given mode as active. Also allows to force the operation.
	 * Useful e.g. when Widget is being created and the current mode already matches the required one.
	 */
	void SetModeEnabled_Internal(ECheckBoxState NewState, FRCMode* NewMode, bool bInForceModeEnabled = false);

	/** Holdes the active mode. */
	FName ActiveMode;
	
	/** Information about the various RC modes. */
	TArray<FRCMode> RCModes;

	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;

	/** Holds the delegate to be invoked when the number of modes changed. */
	FModesChanged ModesChanged;
	
	/** Holds the delegate to be invoked when the active mode is switched/changed. */
	FOnModeSwitched ModeSwitched;
};
