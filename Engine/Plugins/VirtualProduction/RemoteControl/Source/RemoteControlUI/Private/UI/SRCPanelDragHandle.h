// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Styling/CoreStyle.h"
#include "Input/DragAndDrop.h"
#include "DragAndDrop/DecoratedDragDropOp.h"

class SRCPanelGroup;
class FExposedEntityDragDrop;
class FFieldGroupDragDropOp;

template <typename DragDropType>
class SRCPanelDragHandle : public SCompoundWidget
{
public:
	using DragDropWidgetType = typename DragDropType::WidgetType;
	SLATE_BEGIN_ARGS(SRCPanelDragHandle)
	{}
		SLATE_ARGUMENT(TSharedPtr<DragDropWidgetType>, Widget)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, FGuid InId)
	{
		Widget = InArgs._Widget;
		Id = MoveTemp(InId);

		SetCursor(EMouseCursor::GrabHand);

		ChildSlot
		[
			SNew(SBox)
			.WidthOverride(25.0f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.Padding(5.f, 0.f)
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(SImage)
					.Image(FCoreStyle::Get().GetBrush("VerticalBoxDragIndicatorShort"))
				]
			]
		];
	}

	FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
	};

	FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			TSharedPtr<FDragDropOperation> DragDropOp = CreateDragDropOperation();
			if (DragDropOp.IsValid())
			{
				return FReply::Handled().BeginDragDrop(DragDropOp.ToSharedRef());
			}
		}

		return FReply::Unhandled();
	}

	TSharedPtr<FDragDropOperation> CreateDragDropOperation()
	{
		if (TSharedPtr<DragDropWidgetType> WidgetPtr = Widget.Pin())
		{
			TSharedPtr<FDecoratedDragDropOp> DragDropOperation = MakeShared<DragDropType>(Widget.Pin(), Id);
			DragDropOperation->Construct();
			return DragDropOperation;
		}
		return nullptr;
	}

private:
	/** Holds the widget to display when dragging. */
	TWeakPtr<DragDropWidgetType> Widget;
	/** Holds the ID of the item being dragged. */
	FGuid Id;
};

