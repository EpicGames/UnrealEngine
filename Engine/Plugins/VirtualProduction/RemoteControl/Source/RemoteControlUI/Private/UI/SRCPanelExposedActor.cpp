// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelExposedActor.h"

#include "AssetRegistry/AssetData.h"
#include "PropertyCustomizationHelpers.h"
#include "RemoteControlActor.h"
#include "RemoteControlPreset.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "ScopedTransaction.h"
#include "SRCPanelDragHandle.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanelActor"

TSharedPtr<SRCPanelTreeNode> SRCPanelExposedActor::MakeInstance(const FGenerateWidgetArgs& Args)
{
	return SNew(SRCPanelExposedActor, StaticCastSharedPtr<FRemoteControlActor>(Args.Entity), Args.Preset, Args.ColumnSizeData)
		.LiveMode(Args.bIsInLiveMode)
		.CommandList(Args.CommandList);
}

void SRCPanelExposedActor::Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlActor> InWeakActor, URemoteControlPreset* InPreset, FRCColumnSizeData InColumnSizeData)
{
	FString ActorPath;

	const TSharedPtr<FRemoteControlActor> Actor = InWeakActor.Pin();
	if (ensure(Actor))
	{
		FInitParams InitParams;
		InitParams.Preset = InPreset;
		InitParams.EntityId = Actor->GetId();
		InitParams.bLiveMode = InArgs._LiveMode;
		InitParams.CommandList = InArgs._CommandList;
		Initialize(InitParams);

		ColumnSizeData = MoveTemp(InColumnSizeData);
		WeakPreset = InPreset;
		bLiveMode = InArgs._LiveMode;
		WeakActor = MoveTemp(InWeakActor);
		
		if (AActor* ResolvedActor = Actor->GetActor())
		{
			ActorPath = ResolvedActor->GetPathName();
		}
	}
	
	ChildSlot
	[
		RecreateWidget(MoveTemp(ActorPath))
	];
}

SRCPanelTreeNode::ENodeType SRCPanelExposedActor::GetRCType() const
{
	return ENodeType::Actor;
}

void SRCPanelExposedActor::Refresh()
{
	SRCPanelExposedEntity::Refresh();
	
	if (TSharedPtr<FRemoteControlActor> RCActor = WeakActor.Pin())
	{
		CachedLabel = RCActor->GetLabel();

		if (AActor* Actor = RCActor->GetActor())
		{
			ChildSlot.AttachWidget(RecreateWidget(Actor->GetPathName()));
			return;
		}
	}

	ChildSlot.AttachWidget(RecreateWidget(FString()));
}

TSharedRef<SWidget> SRCPanelExposedActor::RecreateWidget(const FString& Path)
{
	// Create combo button to open actor picker with correct custom world
	if (Preset.IsValid() && Preset->IsEmbeddedPreset())
	{
		FText ActorName = LOCTEXT("MissingReference", "Unable to find reference");

		if (TSharedPtr<FRemoteControlActor> RCActor = WeakActor.Pin())
		{
			if (RCActor->GetActor())
			{
				ActorName = FText::AsCultureInvariant(RCActor->GetActor()->GetActorLabel());
			}
		}

		TSharedRef<STextBlock> ActorNameText = SNew(STextBlock)
			.Text(ActorName)
			.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8));

		TSharedRef<SComboButton> ActorButton = SNew(SComboButton)
			.ButtonStyle(FAppStyle::Get(), TEXT("SimpleButton"))
			.ButtonContent()
			[
				ActorNameText
			]
			.OnGetMenuContent(this, &SRCPanelExposedActor::CreateEmbeddedPresetActorPicker);

		return CreateEntityWidget(MoveTemp(ActorButton));
	}

	// Otherwise use SObjectPropertyEntryBox which does not support non-level editor worlds.
	TSharedRef<SObjectPropertyEntryBox> EntryBox = SNew(SObjectPropertyEntryBox)
		.ObjectPath(Path)
		.AllowedClass(AActor::StaticClass())
		.OnObjectChanged(this, &SRCPanelExposedActor::OnChangeActor)
		.AllowClear(false)
		.DisplayUseSelected(true)
		.DisplayBrowse(true)
		.NewAssetFactories(TArray<UFactory*>());

	float MinWidth = 0.f;
	float MaxWidth = 0.f;
	EntryBox->GetDesiredWidth(MinWidth, MaxWidth);

	TSharedRef<SWidget> ValueWidget =
		SNew(SBox)
		.MinDesiredWidth(MinWidth)
		.MaxDesiredWidth(MaxWidth)
		[
			MoveTemp(EntryBox)
		];

	return CreateEntityWidget(MoveTemp(ValueWidget));
}

void SRCPanelExposedActor::OnChangeActor(const FAssetData& AssetData)
{
	if (AActor* Actor = Cast<AActor>(AssetData.GetAsset()))
	{
		OnChangeActor(Actor);
	}
}

void SRCPanelExposedActor::OnChangeActor(AActor* Actor)
{
	if (TSharedPtr<FRemoteControlActor> RCActor = WeakActor.Pin())
	{
		FScopedTransaction Transaction(LOCTEXT("ChangeExposedActor", "Change Exposed Actor"));
		RCActor->SetActor(Actor);
		ChildSlot.AttachWidget(RecreateWidget(Actor->GetPathName()));
	}
}

TSharedRef<SWidget> SRCPanelExposedActor::CreateEmbeddedPresetActorPicker()
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	FSceneOutlinerInitializationOptions InitOptions;
	constexpr bool bAllowPIE = false;
	UWorld* PresetWorld = URemoteControlPreset::GetWorld(Preset.Get(), bAllowPIE);

	TSharedRef<SWidget> ActorPicker =
		SceneOutlinerModule.CreateActorPicker(
			InitOptions,
			FOnActorPicked::CreateSP(this, &SRCPanelExposedActor::OnChangeActor),
			PresetWorld
		);

	return ActorPicker;
}

#undef LOCTEXT_NAMESPACE /*RemoteControlPanelActor*/
