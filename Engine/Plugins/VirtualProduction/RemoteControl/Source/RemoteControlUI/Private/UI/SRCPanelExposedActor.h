// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IRemoteControlUIModule.h"
#include "RemoteControlEntity.h"
#include "SRCPanelExposedEntity.h"
#include "Misc/Guid.h"
#include "UObject/WeakObjectPtrTemplates.h"

struct FRemoteControlActor;
struct FGenerateWidgetArgs;
class SInlineEditableTextBlock;
class SObjectPropertyEntryBox;
class URemoteControlPreset;
struct FAssetData;

DECLARE_DELEGATE_OneParam(FOnUnexposeActor, const FGuid& /**ActorId*/);

/** Represents an actor exposed to remote control. */
struct SRCPanelExposedActor : public SRCPanelExposedEntity
{
	using SCompoundWidget::AsShared;
	
	SLATE_BEGIN_ARGS(SRCPanelExposedActor)
		: _LiveMode(false)
	{}
		SLATE_ATTRIBUTE(bool, LiveMode)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	static TSharedPtr<SRCPanelTreeNode> MakeInstance(const FGenerateWidgetArgs& Args);
	
	void Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlActor> InWeakActor, URemoteControlPreset* InPreset, FRCColumnSizeData InColumnSizeData);

	//~ Begin SRCPanelTreeNode interface
	virtual ENodeType GetRCType() const override;
	virtual void Refresh() override;
	//~ End SRCPanelTreeNode interface

private:
	/** Regenerate this row's content. */
	TSharedRef<SWidget> RecreateWidget(const FString& Path);
	/** Handle the user selecting a different actor to expose. */
	void OnChangeActor(const FAssetData& AssetData);
	/** Handle direct actor value event instead of by asset. */
	void OnChangeActor(AActor* Actor);
	/** Creates an actor picker for hosted presets because the engine default one cannot use custom worlds. */
	TSharedRef<SWidget> CreateEmbeddedPresetActorPicker();
private:
	/** Weak reference to the preset that exposes the actor. */
	TWeakObjectPtr<URemoteControlPreset> WeakPreset;
	/** Weak ptr to the remote control actor structure. */
	TWeakPtr<FRemoteControlActor> WeakActor;
	/** Holds this row's panel live mode. */
	TAttribute<bool> bLiveMode;
};
