// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelExposedEntitiesGroup.h"

#include "RemoteControlPreset.h"
#include "RemoteControlUIModule.h"
#include "SRCPanelExposedField.h"
#include "UI/IRCPanelExposedEntitiesGroupWidgetFactory.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBox.h"

#define LOCTEXT_NAMESPACE "SRCPanelExposedEntitiesGroup"

void SRCPanelExposedEntitiesGroup::Construct(const FArguments& InArgs, ERCFieldGroupType InFieldGroupType, URemoteControlPreset* Preset)
{
	FieldKey = InArgs._FieldKey;
	GroupType = InFieldGroupType;
	PresetWeak = Preset;
	OnGroupPropertyIdChangedDelegate = InArgs._OnGroupPropertyIdChanged;

	const FMakeNodeWidgetArgs Args = CreateNodeWidgetArgs();
	MakeNodeWidget(Args);
}

TSharedRef<SWidget> SRCPanelExposedEntitiesGroup::GetWidget(const FName ForColumnName, const FName InActiveProtocol)
{
	const FRemoteControlUIModule& RemoteControlUIModule = FModuleManager::GetModuleChecked<FRemoteControlUIModule>("RemoteControlUI");
	const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>* CustomFactoryPtr = RemoteControlUIModule.GetExposedEntitiesGroupWidgetFactory(ForColumnName, InActiveProtocol);
	if (CustomFactoryPtr)
	{
		const FRCPanelExposedEntitiesGroupWidgetFactoryArgs Args(
			PresetWeak,
			GetChildProperties()
		);

		return (*CustomFactoryPtr)->MakeWidget(Args);
	}
	else
	{
		return SRCPanelTreeNode::GetWidget(ForColumnName, InActiveProtocol);
	}
}

SRCPanelTreeNode::FMakeNodeWidgetArgs SRCPanelExposedEntitiesGroup::CreateNodeWidgetArgs()
{
	FMakeNodeWidgetArgs Args;

	Args.PropertyIdWidget = SNew(SBox)
		[
			SNew(SEditableTextBox)
			.Justification(ETextJustify::Left)
			.MinDesiredWidth(50.f)
			.SelectAllTextWhenFocused(true)
			.RevertTextOnEscape(true)
			.ClearKeyboardFocusOnCommit(true)
			.Text_Lambda([this]{ return FText::FromName(PropertyIdName); })
			.OnTextCommitted(this, &SRCPanelExposedEntitiesGroup::OnPropertyIdTextCommitted)
		];

	Args.OwnerNameWidget = SNew(SBox)
		.HeightOverride(25)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text_Lambda([this]{ return OwnerDisplayName; })
		];

	FText GroupText;

	switch (GroupType)
	{
	case ERCFieldGroupType::PropertyId:
		GroupText = LOCTEXT("GroupPropertyId", "Group by Id");
		break;

	case ERCFieldGroupType::Owner:
		GroupText = LOCTEXT("GroupOwner", "Group by Owner");
		break;
	}

	Args.NameWidget = SNew(SBox)
		.HeightOverride(25)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(GroupText)
		];

	Args.SubObjectPathWidget = SNullWidget::NullWidget;
	Args.ValueWidget = SNullWidget::NullWidget;
	Args.ResetButton = SNullWidget::NullWidget;

	return Args;
}

void SRCPanelExposedEntitiesGroup::OnPropertyIdTextCommitted(const FText& InText, ETextCommit::Type InTextCommitType)
{
	if (!PresetWeak.IsValid() || PresetWeak.IsStale())
	{
		return;
	}

	const FName NewId = FName(InText.ToString());

	if (FieldKey.Compare(NewId) == 0)
	{
		return;
	}

	URemoteControlPreset* Preset = PresetWeak.Get();
	FieldKey = NewId;

	for (const TSharedPtr<SRCPanelTreeNode>& Child : ChildWidgets)
	{
		TWeakPtr<FRemoteControlField> ExposedField = Preset->GetExposedEntity<FRemoteControlField>(Child->GetRCId());
		if (ExposedField.IsValid())
		{
			ExposedField.Pin()->PropertyId = NewId;
			Child->SetPropertyId(NewId);
			Preset->UpdateIdentifiedField(ExposedField.Pin().ToSharedRef());
		}
	}

	OnGroupPropertyIdChangedDelegate.ExecuteIfBound();
}

void SRCPanelExposedEntitiesGroup::AssignChildren(const TArray<TSharedPtr<SRCPanelTreeNode>>& InFieldEntities)
{
	ChildWidgets.Empty();

	OwnerDisplayName = FText::GetEmpty();
	PropertyIdName = NAME_None;

	for (const TSharedPtr<SRCPanelTreeNode>& Entity : InFieldEntities)
	{
		if (const TSharedPtr<SRCPanelExposedField> ExposedField = StaticCastSharedPtr<SRCPanelExposedField>(Entity))
		{
			const FName FieldPropertyId = ExposedField->GetPropertyId();

			const bool bOwnerMatch = GroupType == ERCFieldGroupType::Owner && ExposedField->GetOwnerPathName() == FieldKey;
			const bool bPropertyIdMatch = GroupType == ERCFieldGroupType::PropertyId && FieldPropertyId == FieldKey;

			if (bOwnerMatch || bPropertyIdMatch)
			{
				ChildWidgets.Add(Entity);

				const FText FieldOwnerDisplayName = ExposedField->GetOwnerDisplayName();
				if (OwnerDisplayName.IsEmpty())
				{
					OwnerDisplayName = FieldOwnerDisplayName;
				}
				else if (!OwnerDisplayName.EqualTo(FieldOwnerDisplayName))
				{
					OwnerDisplayName = LOCTEXT("MultipleValues", "Multiple Values");
				}

				if (PropertyIdName == NAME_None)
				{
					PropertyIdName = FieldPropertyId;
				}
				else if (PropertyIdName != FieldPropertyId)
				{
					PropertyIdName = TEXT("Multiple Values");
				}
			}
		}
	}
}

void SRCPanelExposedEntitiesGroup::GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const
{
	OutChildren.Append(ChildWidgets);
	SRCPanelTreeNode::GetNodeChildren(OutChildren);
}

TArray<TSharedRef<FRemoteControlProperty>> SRCPanelExposedEntitiesGroup::GetChildProperties() const
{
	URemoteControlPreset* Preset = PresetWeak.Get();
	if (!Preset)
	{
		return {};
	}

	TArray<TSharedPtr<SRCPanelTreeNode>> Children;
	GetNodeChildren(Children);

	TArray<TSharedRef<FRemoteControlProperty>> ChildProperties;
	for (const TSharedPtr<SRCPanelTreeNode>& Child : Children)
	{
		const TWeakPtr<FRemoteControlProperty> WeakProperty = Child.IsValid() ? Preset->GetExposedEntity<FRemoteControlProperty>(Child->GetRCId()) : nullptr;
		if (WeakProperty.IsValid())
		{
			ChildProperties.Add(WeakProperty.Pin().ToSharedRef());
		}
	}

	return ChildProperties;
}

#undef LOCTEXT_NAMESPACE
