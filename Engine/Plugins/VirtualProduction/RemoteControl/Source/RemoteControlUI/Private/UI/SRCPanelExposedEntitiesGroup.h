// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRCPanelExposedEntity.h"
#include "SRCPanelTreeNode.h"
#include "UI/RCFieldGroupType.h"
#include "Widgets/SCompoundWidget.h"

struct FRemoteControlProperty;

/** Delegate called when the PropertyId of a group change */
DECLARE_DELEGATE(FOnGroupPropertyIdChanged)

/** Widget that is used as a parent for field widget */
class SRCPanelExposedEntitiesGroup : public SRCPanelTreeNode
{
public:
	SLATE_BEGIN_ARGS(SRCPanelExposedEntitiesGroup)
	{}

	SLATE_EVENT(FOnGroupPropertyIdChanged, OnGroupPropertyIdChanged)
	SLATE_ARGUMENT(FName, FieldKey)

	SLATE_END_ARGS()

	//~ Begin SRCPanelTreeNode interface

	/** Get get this node's type. */
	virtual ENodeType GetRCType() const override { return ENodeType::FieldGroup; }

	//~ End SRCPanelTreeNode interface

	void Construct(const FArguments& InArgs, ERCFieldGroupType InFieldGroupType, URemoteControlPreset* Preset);

	//~ Begin SRCPanelTreeNode interface
	virtual TSharedRef<SWidget> GetWidget(const FName ForColumnName, const FName InActiveProtocol) override;
	//~ End SRCPanelTreeNode interface

	/**
	 * Take the current fields in RC and assign them to its child widgets based on the group key
	 * @param InFieldEntities Current fields exposed in RC
	 */
	void AssignChildren(const TArray<TSharedPtr<SRCPanelTreeNode>>& InFieldEntities);

	/** Get this tree node's children. */
	virtual void GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const override;

	/** Returns true if this tree node has children. */
	virtual bool HasChildren() const override { return true; }

	/** Get the group type of this group */
	ERCFieldGroupType GetGroupType() const { return GroupType; }

	/** Get the field key of this group */
	FName GetFieldKey() const { return FieldKey; }

private:
	/** Create the node Args for this class that will be used to create the widget */
	FMakeNodeWidgetArgs CreateNodeWidgetArgs();

	/**
	 * Called when the text label is renamed
	 * @param InText New Text
	 * @param InTextCommitType Type of commit, look at ETextCommit for more information
	 */
	void OnPropertyIdTextCommitted(const FText& InText, ETextCommit::Type InTextCommitType);

	/** Returns the properties that are children of this group */
	TArray<TSharedRef<FRemoteControlProperty>> GetChildProperties() const;

	/** Child widgets of this group */
	TArray<TSharedPtr<SRCPanelTreeNode>> ChildWidgets;

	/** Field key value of this group */
	FName FieldKey;

	/** Shared owner of the entity group */
	FText OwnerDisplayName;

	/** Shared property id of the entity group */
	FName PropertyIdName;

	/** Field group type of this group */
	ERCFieldGroupType GroupType = ERCFieldGroupType::None;

	/** Weak ptr to the preset where this group reside */
	TWeakObjectPtr<URemoteControlPreset> PresetWeak;

	/** Delegate called when the group type is PropertyId and the value change */
	FOnGroupPropertyIdChanged OnGroupPropertyIdChangedDelegate;
};
