// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelExposedEntitiesList.h"

#include "Action/RCAction.h"
#include "Behaviour/RCBehaviour.h"
#include "Commands/RemoteControlCommands.h"
#include "Controller/RCController.h"
#include "Editor.h"
#include "Engine/Selection.h"
#include "Filters/SRCPanelFilter.h"
#include "GameFramework/Actor.h"
#include "Input/DragAndDrop.h"
#include "IRemoteControlProtocolModule.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "Misc/Attribute.h"
#include "Misc/Guid.h"
#include "Misc/MessageDialog.h"
#include "PropertyPath.h"
#include "RCPanelWidgetRegistry.h"
#include "RCUIHelpers.h"
#include "RemoteControlPanelStyle.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "RemoteControlUIModule.h"
#include "ScopedTransaction.h"
#include "SRCHeaderRow.h"
#include "SRCModeSwitcher.h"
#include "SRCPanelExposedEntitiesGroup.h"
#include "SRCPanelExposedField.h"
#include "SRCPanelFieldGroup.h"
#include "SRCPanelPerPresetProtocolSettings.h"
#include "SSearchToggleButton.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/IRCExposedEntitiesPanelExtender.h"
#include "UI/IRCPanelExposedEntitiesListSettingsForProtocol.h"
#include "UI/Panels/SRCDockPanel.h"
#include "UObject/Object.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanelEntitiesList"

TSet<FName> SRCPanelExposedEntitiesList::DefaultProtocolColumns = {
	RemoteControlPresetColumns::Status,
	RemoteControlPresetColumns::BindingStatus,
};

/**
 * A custom row to describe each entity in the list.
 */
class SEntityRow : public SMultiColumnTableRow<TSharedPtr<SRCPanelTreeNode>>
{
public:

	SLATE_BEGIN_ARGS(SEntityRow)
	{}
		
		SLATE_ATTRIBUTE(FName, ActiveProtocol)
		SLATE_ARGUMENT(TSharedPtr<SRCPanelTreeNode>, Entity)

		SLATE_ATTRIBUTE(FMargin, Padding)
		SLATE_STYLE_ARGUMENT(FTableRowStyle, Style)

		// Low level DragAndDrop
		SLATE_EVENT(FOnAcceptDrop, OnAcceptDrop)
		SLATE_EVENT(FOnDragDetected, OnDragDetected)
		SLATE_EVENT(FOnTableRowDragEnter, OnDragEnter)
		SLATE_EVENT(FOnTableRowDragLeave, OnDragLeave)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView)
	{
		ActiveProtocol = InArgs._ActiveProtocol;
		Entity = InArgs._Entity;

		FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments()
			.OnCanAcceptDrop_Lambda(
				[](const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<SRCPanelTreeNode> Node)
				{
					return InDropZone;
				})
			.OnAcceptDrop(InArgs._OnAcceptDrop)
			.OnDragDetected(InArgs._OnDragDetected)
			.OnDragEnter(InArgs._OnDragEnter)
			.OnDragLeave(InArgs._OnDragLeave)
			.ExpanderStyleSet(&FCoreStyle::Get())
			.Padding(InArgs._Padding)
			.ShowWires(false)
			.Style(InArgs._Style);

		FSuperRowType::Construct(SuperArgs, OwnerTableView);
	}

	TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override
	{
		if (Entity.IsValid())
		{
			TSharedRef<SHorizontalBox> HorizontalBox = SNew(SHorizontalBox)
				.Clipping(EWidgetClipping::OnDemand);

			if (InColumnName == RemoteControlPresetColumns::OwnerName)
			{
				HorizontalBox->AddSlot()
					.AutoWidth()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(ExpanderArrowWidget, SExpanderArrow, SharedThis(this))
						.StyleSet(ExpanderStyleSet)
						.ShouldDrawWires(false)
					];
			}

			// todo: this alignment should be handled by the generated widgets for each column themselves
			EHorizontalAlignment HorizontalAlignment = HAlign_Fill;
			if (InColumnName == RemoteControlPresetColumns::Status || InColumnName == RemoteControlPresetColumns::BindingStatus)
			{
				HorizontalAlignment = HAlign_Center;
			}

			HorizontalBox->AddSlot()
				.FillWidth(1.f)
				.Padding(2.f, 0.f)
				.HAlign(HorizontalAlignment)
				.VAlign(VAlign_Center)
				[
					Entity->GetWidget(InColumnName, ActiveProtocol.Get(NAME_None))
				];

			return HorizontalBox;
		}

		return SNullWidget::NullWidget;
	}

private:
	//~ SWidget Interface
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override
	{
		if (Entity.IsValid() && Entity->GetRCType() != SRCPanelTreeNode::FieldGroup)
		{
			if (const TSharedPtr<SRCPanelExposedEntity> ExposedEntityWidget = StaticCastSharedPtr<SRCPanelExposedEntity>(Entity))
			{
				if (const TSharedPtr<FRemoteControlEntity> ExposedEntity = ExposedEntityWidget->GetEntity())
				{
					FRemoteControlUIModule::Get().SelectSource(ExposedEntity);
				}
			}
		}

		return FSuperRowType::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}

private:

	/** Holds the active protocol. */
	TAttribute<FName> ActiveProtocol;

	/** Cached reference of Entity */
	TSharedPtr<SRCPanelTreeNode> Entity;
};

void SRCPanelExposedEntitiesList::Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, TWeakPtr<FRCPanelWidgetRegistry> InWidgetRegistry)
{
	CommandList = MakeShared<FUICommandList>();

	if (InArgs._CommandList.IsValid())
	{
		CommandList->Append(InArgs._CommandList.ToSharedRef());
	}

	const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

	CommandList->MapAction(
		Commands.SelectSource,
		FUIAction(FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::SelectSource_Execute))
	);

	bIsInLiveMode = InArgs._LiveMode;
	bIsInProtocolsMode = InArgs._ProtocolsMode;
	PresetWeak = InPreset;
	OnEntityListUpdatedDelegate = InArgs._OnEntityListUpdated;
	WidgetRegistry = MoveTemp(InWidgetRegistry);

	OnActorLabelChangedHandle = FCoreDelegates::OnActorLabelChanged.AddSP(this, &SRCPanelExposedEntitiesList::OnActorLabelChanged);

	WidgetRegistry.Pin()->OnObjectRefreshed().AddSP(this, &SRCPanelExposedEntitiesList::OnWidgetRegistryRefreshed);

	ColumnSizeData.LeftColumnWidth = TAttribute<float>(this, &SRCPanelExposedEntitiesList::OnGetLeftColumnWidth);
	ColumnSizeData.RightColumnWidth = TAttribute<float>(this, &SRCPanelExposedEntitiesList::OnGetRightColumnWidth);
	ColumnSizeData.OnWidthChanged = SSplitter::FOnSlotResized::CreateSP(this, &SRCPanelExposedEntitiesList::OnSetColumnWidth);

	bFilterApplicationRequested = false;
	bSearchRequested = false;
	bUpdateEntityUsage = false; // Perform this on creation
	
	RecallListSettings();

	// Setup search filter.
	SearchTextFilter = MakeShared<TTextFilter<const SRCPanelTreeNode&>>(TTextFilter<const SRCPanelTreeNode&>::FItemToStringArray::CreateSP(this, &SRCPanelExposedEntitiesList::PopulateSearchStrings));
	SearchedText = MakeShared<FText>(FText::GetEmpty());

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");
	ActiveListMode = EEntitiesListMode::Default;

	// Retrieve from settings which columns should be hidden
	const URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
	const TArray<FName>& HiddenColumns = Settings->EntitiesListHiddenColumns.Array();

	// The Default group will be selected by default.
	if (InPreset)
	{
		const FRemoteControlPresetGroup& DefaultGroup = InPreset->Layout.GetDefaultGroup();

		CurrentlySelectedGroup = DefaultGroup.Id;
	}

	// Major Panel
	TSharedPtr<SRCMajorPanel> ExposePanel = SNew(SRCMajorPanel)
		.EnableHeader(false)
		.EnableFooter(false);

	// Groups List
	SAssignNew(GroupsListView, SListView<TSharedPtr<SRCPanelTreeNode>>)
		.SelectionMode(ESelectionMode::Single)
		.ListItemsSource(reinterpret_cast<TArray<TSharedPtr<SRCPanelTreeNode>>*>(&FieldGroups))
		.ClearSelectionOnClick(true)
		.OnGenerateRow(this, &SRCPanelExposedEntitiesList::OnGenerateRow)
		.OnSelectionChanged(this, &SRCPanelExposedEntitiesList::OnSelectionChanged)
		.OnContextMenuOpening(this, &SRCPanelExposedEntitiesList::OnContextMenuOpening, SRCPanelTreeNode::Group);

	// Group Dock Panel
	TSharedPtr<SRCMinorPanel> GroupDockPanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("GroupLabel", "Group"))
		.EnableFooter(false)
		[
			GroupsListView.ToSharedRef()
		];

	// Add New Group Button
	TSharedPtr<SWidget> NewGroupButton = SNew(SButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Group")))
		.IsEnabled_Lambda([this]() { return !bIsInLiveMode.Get(); })
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ForegroundColor(FSlateColor::UseForeground())
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ToolTipText(LOCTEXT("NewGroupToolTip", "Create new group."))
		.OnClicked(this, &SRCPanelExposedEntitiesList::OnCreateGroup)
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle->IconSize.X)
			.HeightOverride(RCPanelStyle->IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
		];

	GroupDockPanel->AddHeaderToolbarItem(EToolbar::Left, NewGroupButton.ToSharedRef());

	ExposePanel->AddPanel(GroupDockPanel.ToSharedRef(), 0.25f);

	// Fields List
	SAssignNew(FieldsListView, STreeView<TSharedPtr<SRCPanelTreeNode>>)
		.OnGenerateRow(this, &SRCPanelExposedEntitiesList::OnGenerateRow)
		.OnSelectionChanged(this, &SRCPanelExposedEntitiesList::OnSelectionChanged)
		.SelectionMode(ESelectionMode::Multi)
		.TreeItemsSource(&FieldEntities)
		.OnContextMenuOpening(this, &SRCPanelExposedEntitiesList::OnContextMenuOpening, SRCPanelTreeNode::Field)
		.ClearSelectionOnClick(true)
		.TreeViewStyle(&RCPanelStyle->TableViewStyle)
		.OnGetChildren(this, &SRCPanelExposedEntitiesList::OnGetNodeChildren)
		.HeaderRow(
			SAssignNew(FieldsHeaderRow, SRCHeaderRow)
			.Style(&RCPanelStyle->HeaderRowStyle)
			.CanSelectGeneratedColumn(true) //To show/hide columns
			.HiddenColumnsList(HiddenColumns) // List of columns to hide by default. User can un-hide via context menu list

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::PropertyIdentifier)
			.DefaultLabel(LOCTEXT("RCPresetPropertyIdColumnHeader", "Property ID"))
			.HAlignHeader(HAlign_Center)
			.FillWidth(0.15f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::OwnerName)
			.DefaultLabel(LOCTEXT("RCPresetOwnerNameColumnHeader", "Owner"))
			.HAlignHeader(HAlign_Center)
			.FillWidth(0.1f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::SubobjectPath)
			.DefaultLabel(LOCTEXT("RCPresetSubobjectPathColumnHeader", "Subobject Path"))
			.HAlignHeader(HAlign_Center)
			.FillWidth(0.15f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::Description)
			.DefaultLabel(LOCTEXT("RCPresetDescColumnHeader", "Description"))
			.HAlignHeader(HAlign_Center)
			.FillWidth(0.25f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::Value)
			.DefaultLabel(LOCTEXT("RCPresetValueColumnHeader", "Value"))
			.HAlignHeader(HAlign_Center)
			.FillWidth(0.35f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)

			+ SRCHeaderRow::Column(RemoteControlPresetColumns::Reset)
			.DefaultLabel(FText())
			.FixedWidth(48.f)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding)
			.ShouldGenerateWidget(true)
			.ShouldGenerateSubMenuEntry(false)
		);

	// Exposed Entities Dock Panel
	const TSharedRef<SVerticalBox> ExposedEntitiesPanelContent = SNew(SVerticalBox);

	const FRemoteControlUIModule& RemoteControlUIModule = FModuleManager::Get().GetModuleChecked<FRemoteControlUIModule>("RemoteControlUI");
	for (const TSharedRef<IRCExposedEntitiesPanelExtender>& Extender : RemoteControlUIModule.GetExposedEntitiesPanelExtenders())
	{
		// Add extensions
		IRCExposedEntitiesPanelExtender::FArgs Args;
		Args.ActiveProtocolAttribute = TAttribute<FName>::CreateLambda([this]()
			{
				return bIsInProtocolsMode.Get() ? ActiveProtocol : NAME_None;
			});
		const TSharedRef<SWidget> ExtenderWidget = Extender->MakeWidget(InPreset, Args);

		ExposedEntitiesPanelContent->AddSlot()
			.AutoHeight()
			[
				ExtenderWidget
			];
	}


	ExposedEntitiesPanelContent->AddSlot()
		.FillHeight(1.f)
		[
			FieldsListView.ToSharedRef()
		];

	const TSharedPtr<SRCMinorPanel> ExposeDockPanel = SNew(SRCMinorPanel)
		.HeaderLabel(this, &SRCPanelExposedEntitiesList::HandleEntityListHeaderLabel)
		.Visibility_Lambda([this]() {return bIsInLiveMode.Get() ? EVisibility::Collapsed : EVisibility::Visible; })
		.EnableFooter(false)
		[
			ExposedEntitiesPanelContent
		];

	// Mode Switcher
	TSharedPtr<SRCModeSwitcher> ModeSwitcher = SNew(SRCModeSwitcher)
		.OnModeSwitched_Lambda([this](const SRCModeSwitcher::FRCMode& NewMode)
			{
				if (ActiveProtocol != NewMode.ModeId)
				{
					StoreListSettings();

					ActiveListMode = EEntitiesListMode::Default; // Hack to trigger mode change via mode switcher.

					ActiveProtocol = NewMode.ModeId;

					IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();

					RCProtocolWidgetsModule.OnActiveProtocolChanged().Broadcast(ActiveProtocol);

					if (bIsInProtocolsMode.Get())
					{
						RebuildListWithColumns(EEntitiesListMode::Protocols);
					}
				}
			}
		)
		.Visibility_Lambda([this]()
			{
				return (bIsInProtocolsMode.Get() && !ActiveProtocol.IsNone()) ? EVisibility::Visible : EVisibility::Collapsed;
			}
		);

	{ // Add dynamic entries based on the available protocols.
		IRemoteControlProtocolModule& RCProtocolModule = IRemoteControlProtocolModule::Get();

		TArray<FName> Protocols = RCProtocolModule.GetProtocolNames();

		// To avoid random orientations, sort them in alphabetical order.
		Algo::Sort(Protocols, [&](const FName& ThisProtocol, const FName& OtherProtocol) { return ThisProtocol.LexicalLess(OtherProtocol); });

		for (int32 ProtocolIndex = 0; ProtocolIndex < Protocols.Num(); ProtocolIndex++)
		{
			const FName& ProtocolName = Protocols[ProtocolIndex];

			const bool bIsDefault = ProtocolIndex == 0;

			if (bIsDefault)
			{
				ActiveProtocol = ProtocolName;
	
				IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();

				RCProtocolWidgetsModule.OnActiveProtocolChanged().Broadcast(ActiveProtocol);
			}

			SRCModeSwitcher::FRCMode::FArguments NewMode = SRCModeSwitcher::Mode(ProtocolName)
				.DefaultLabel(FText::FromName(ProtocolName))
				.DefaultTooltip(FText::FromName(ProtocolName))
				.HAlignCell(HAlign_Fill)
				.IsDefault(bIsDefault)
				.VAlignCell(VAlign_Fill)
				.FixedWidth(48.f);

			ModeSwitcher->AddMode(NewMode);
		}
	}

	// Create the Filter Widget
	FilterPtr = SNew(SRCPanelFilter)
		.OnFilterChanged(this, &SRCPanelExposedEntitiesList::OnFilterChanged)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("RemoteControlFilters")));

	// Create the Filter Combo Button
	const TSharedPtr<SWidget> FilterComboButton = SRCPanelFilter::MakeAddFilterButton(FilterPtr.ToSharedRef());

	const TSharedPtr<ISlateMetaData> FilterComboButtonMetaData = MakeShared<FTagMetaData>(TEXT("ContentBrowserFiltersCombo"));
	FilterComboButton->AddMetadata(FilterComboButtonMetaData.ToSharedRef());

	//Create the SearchBox for the EntitiesList
	SAssignNew(SearchBoxPtr, SSearchBox)
		.HintText(LOCTEXT("SearchHint", "Search"))
		.OnTextChanged(this, &SRCPanelExposedEntitiesList::OnSearchTextChanged)
		.OnTextCommitted(this, &SRCPanelExposedEntitiesList::OnSearchTextCommitted)
		.DelayChangeNotificationsWhileTyping(true);

	// Create grouping button
	ComboButtonGroupButton = SNew(SComboButton)
		.ToolTipText(LOCTEXT("RCFieldsGroupingTooltip", "Select grouping type for the fields"))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.CollapseMenuOnParentFocus(true)
		.HasDownArrow(false)
		.ContentPadding(FMargin(4.f, 2.f))
		.ButtonContent()
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle->IconSize.X)
			.HeightOverride(RCPanelStyle->IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("InputBindingEditor.LevelViewport"))
			]
		]
		.MenuContent()
		[
			GetGroupMenuContentWidget()
		];

	// Per Preset Protocol Settings
	const TSharedRef<SRCPanelPerPresetProtocolSettings> PerPresetProtocolSettings =
		SNew(SRCPanelPerPresetProtocolSettings, InPreset)
		.Visibility_Lambda([this]()
			{
				return bIsInProtocolsMode.Get() ? EVisibility::Visible : EVisibility::Collapsed;
			});

	// Expose Button
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Center, SearchBoxPtr.ToSharedRef());
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Right, SNew(SSearchToggleButton, SearchBoxPtr.ToSharedRef()));
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Right, ComboButtonGroupButton.ToSharedRef());
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Right, FilterComboButton.ToSharedRef());
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Right, ModeSwitcher.ToSharedRef());
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Right, PerPresetProtocolSettings);
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Left, InArgs._ExposeActorsComboButton.Get().ToSharedRef());
	ExposeDockPanel->AddHeaderToolbarItem(EToolbar::Left, InArgs._ExposeFunctionsComboButton.Get().ToSharedRef());

	ExposePanel->AddPanel(ExposeDockPanel.ToSharedRef(), 0.75f);

	ChildSlot
		.Padding(RCPanelStyle->PanelPadding)
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)

			+ SSplitter::Slot()
			.Value(1.f)
			[
				ExposePanel.ToSharedRef()
			]
		];

	RegisterEvents();
	RegisterPresetDelegates();
	Refresh();
}

SRCPanelExposedEntitiesList::~SRCPanelExposedEntitiesList()
{
	if (TSharedPtr<FRCPanelWidgetRegistry> Registry = WidgetRegistry.Pin())
	{
		Registry->OnObjectRefreshed().RemoveAll(this);
	}

	UnregisterPresetDelegates();
	UnregisterEvents();

	StoreListSettings();

	FCoreDelegates::OnActorLabelChanged.Remove(OnActorLabelChangedHandle);
	OnActorLabelChangedHandle.Reset();
}

void SRCPanelExposedEntitiesList::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bSearchRequested)
	{
		TryRefreshingSearch(*SearchedText);

		bSearchRequested = false;
	}
	
	if (bFilterApplicationRequested)
	{
		ApplyFilters();

		bFilterApplicationRequested = false;
	}

	if (bRefreshRequested)
	{
		ProcessRefresh();
		bRefreshRequested = false;
	}

	if (bRefreshEntitiesGroups)
	{
		RefreshGroupsAndRestoreExpansions();
		bRefreshEntitiesGroups = false;
	}

	ExposedEntitiesNodesRefresh();

	if (bUpdateEntityUsage)
	{
		if (URemoteControlPreset* Preset = GetPreset())
		{
			Preset->UpdateEntityUsage();
		}

		bUpdateEntityUsage = false;
	}
}

TSharedPtr<SRCPanelTreeNode> SRCPanelExposedEntitiesList::GetSelectedGroup() const
{
	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedNodes;

	GroupsListView->GetSelectedItems(SelectedNodes);

	if (SelectedNodes.Num())
	{
		return SelectedNodes[0];
	}

	return nullptr;
}

TSharedPtr<SRCPanelTreeNode> SRCPanelExposedEntitiesList::GetSelectedEntity() const
{
	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedNodes;

	FieldsListView->GetSelectedItems(SelectedNodes);

	if (SelectedNodes.Num())
	{
		return SelectedNodes[0];
	}

	return nullptr;
}

TArray<TSharedPtr<SRCPanelTreeNode>> SRCPanelExposedEntitiesList::GetSelectedEntities() const
{
	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedNodes;
	if (FieldsListView.IsValid())
	{
		FieldsListView->GetSelectedItems(SelectedNodes);
	}
	return SelectedNodes;
}

int32 SRCPanelExposedEntitiesList::GetSelectedEntitiesNum() const
{
	if (FieldsListView.IsValid())
	{
		return FieldsListView->GetNumItemsSelected();
	}
	return -1;
}

bool SRCPanelExposedEntitiesList::IsEntitySelected(const TSharedPtr<SRCPanelTreeNode>& InNode) const
{
	if (FieldsListView.IsValid())
	{
		return FieldsListView->IsItemSelected(InNode);
	}
	return false;
}

void SRCPanelExposedEntitiesList::SetSelection(const TSharedPtr<SRCPanelTreeNode>& Node, const bool bForceMouseClick)
{
	if (Node)
	{
		const ESelectInfo::Type SelectInfo = bForceMouseClick ? ESelectInfo::OnMouseClick : ESelectInfo::Direct;

		if (TSharedPtr<SRCPanelTreeNode>* FoundTreeNode = FieldWidgetMap.Find(Node->GetRCId()))
		{
			FieldsListView->SetItemSelection(*FoundTreeNode, true, SelectInfo);
			return;
		}

		if (TSharedPtr<SRCPanelGroup> SRCGroup = FindGroupById(Node->GetRCId()))
		{
			GroupsListView->SetSelection(SRCGroup, SelectInfo);
		}	
	}
}

void SRCPanelExposedEntitiesList::SetBackendFilter(const FRCFilter& InBackendFilter)
{
	BackendFilter = InBackendFilter;

	bFilterApplicationRequested = true;
}

void SRCPanelExposedEntitiesList::RebuildListWithColumns(EEntitiesListMode InListMode)
{
	RecallListSettings();

	// Only activate when the active mode changes to different one.
	// Drop multiple calls to same mode change except for Protocols Mode.
	if (ActiveListMode != InListMode && !ActiveProtocol.IsNone())
	{
		ActiveListMode = InListMode;

		TSet<FName> ExisitingColumns;

		if (FieldsListView.IsValid() && FieldsHeaderRow.IsValid())
		{
			const TIndirectArray<SHeaderRow::FColumn>& Columns = FieldsHeaderRow->GetColumns();

			if (!Columns.IsEmpty())
			{
				Algo::TransformIf(Columns, ExisitingColumns
					, [](const SHeaderRow::FColumn& InColumn) { return !InColumn.ColumnId.IsNone() && !SRCPanelTreeNode::DefaultColumns.Contains(InColumn.ColumnId); }
					, [](const SHeaderRow::FColumn& InColumn) { return InColumn.ColumnId; }
				);
			}
		}

		const bool bShouldRemoveColumns = !ExisitingColumns.IsEmpty();

		TSet<FName> ColumnsToBeRemoved = ExisitingColumns;

		const bool bShouldAddColumns = InListMode == EEntitiesListMode::Protocols;

		TSet<FName> ColumnsToBeAdded;

		IRemoteControlProtocolModule& RCProtocolModule = IRemoteControlProtocolModule::Get();

		TSet<FName> ActiveProtocolColumns;

		TSet<FName> OtherProtocolColumns;

		for (const FName& ProtocolName : RCProtocolModule.GetProtocolNames())
		{
			if (TSharedPtr<IRemoteControlProtocol> RCProtocol = RCProtocolModule.GetProtocolByName(ProtocolName))
			{
				if (ActiveProtocol == ProtocolName)
				{
					RCProtocol->GetRegisteredColumns(ActiveProtocolColumns);
				}
				else
				{
					RCProtocol->GetRegisteredColumns(OtherProtocolColumns);
				}
			}
		}

		if (bShouldRemoveColumns)
		{
			ColumnsToBeRemoved.Append(ActiveProtocolColumns.Intersect(OtherProtocolColumns));
		
			for (const FName& ColumnToBeRemoved : ColumnsToBeRemoved)
			{
				RemoveColumn(ColumnToBeRemoved);
			}
		}

		if (bShouldAddColumns)
		{
			ColumnsToBeAdded = DefaultProtocolColumns.Union(ActiveProtocolColumns);

			for (const FName& ColumnToBeAdded : ColumnsToBeAdded)
			{
				InsertColumn(ColumnToBeAdded);
			}
		}
	}
}

void SRCPanelExposedEntitiesList::UpdateSearch()
{
	if (SearchedText.IsValid() && !SearchedText->IsEmptyOrWhitespace() && SearchedText->ToString().Len() > 3)
	{
		OnSearchTextChanged(*SearchedText);
	}
}

FText SRCPanelExposedEntitiesList::HandleEntityListHeaderLabel() const
{
	const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

	if (bIsInProtocolsMode.Get() && Commands.ActivateProtocolsMode.IsValid())
	{
		return Commands.ActivateProtocolsMode->GetLabel();
	}

	return LOCTEXT("PropertiesLabel", "Properties");
}

void SRCPanelExposedEntitiesList::ExposedEntitiesNodesRefresh()
{
	if (bNodesRefreshRequested)
	{
		for (const TPair <FGuid, TSharedPtr<SRCPanelTreeNode>>& Node : FieldWidgetMap)
		{
			Node.Value->Refresh();
		}

		bNodesRefreshRequested = false;
	}
}

void SRCPanelExposedEntitiesList::OnPropertyIdRenamed(const FName InNewId, TSharedPtr<SRCPanelTreeNode> InNode)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (Preset && IsEntitySelected(InNode))
	{
		TArray<TSharedPtr<SRCPanelTreeNode>> SelectedEntities = GetSelectedEntities();
		for (const TSharedPtr<SRCPanelTreeNode>& Entity : SelectedEntities)
		{
			TWeakPtr<FRemoteControlField> ExposedField = Preset->GetExposedEntity<FRemoteControlField>(Entity->GetRCId());
			if (ExposedField.IsValid())
			{
				ExposedField.Pin()->PropertyId = InNewId;
				Entity->SetPropertyId(InNewId);
				Preset->UpdateIdentifiedField(ExposedField.Pin().ToSharedRef());
			}
		}
	}

	if (CurrentGroupType == ERCFieldGroupType::PropertyId)
	{
		CreateFieldGroup();
		OrderGroups();
	}
}

void SRCPanelExposedEntitiesList::OnLabelModified(const FName InOldName, const FName InNewName)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedEntities = GetSelectedEntities();
	for (const TSharedPtr<SRCPanelTreeNode>& EntityNode : SelectedEntities)
	{
		TWeakPtr<FRemoteControlField> ExposedField = Preset->GetExposedEntity<FRemoteControlField>(EntityNode->GetRCId());
		if (ExposedField.IsValid())
		{
			const bool bIsLabelAlreadyNew = ExposedField.Pin()->GetLabel() == InNewName;
			FName OldLabel = InOldName;
			if (!bIsLabelAlreadyNew)
			{
				OldLabel = ExposedField.Pin()->GetLabel();
			}

			ExposedField.Pin()->Rename(InNewName);
			const FName NewName = ExposedField.Pin()->GetLabel();
			EntityNode->SetName(NewName);
			Preset->OnFieldRenamed().Broadcast(Preset, OldLabel, NewName);
		}
	}
}

FReply SRCPanelExposedEntitiesList::OnNodeDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TSharedPtr<SRCPanelTreeNode> InNode)
{
	if (InNode.IsValid())
	{
		const SRCPanelTreeNode::ENodeType FirstNodeType = InNode->GetRCType();
		if (FirstNodeType == SRCPanelTreeNode::Field || FirstNodeType == SRCPanelTreeNode::FieldGroup)
		{
			TArray<TSharedPtr<SRCPanelTreeNode>> SelectedEntities;
			Algo::TransformIf(GetSelectedEntities(), SelectedEntities
				, [] (const TSharedPtr<SRCPanelTreeNode>& TreeNode)
					{
						const SRCPanelTreeNode::ENodeType NodeType = TreeNode->GetRCType();
						return NodeType == SRCPanelTreeNode::Field || NodeType == SRCPanelTreeNode::FieldGroup;
					}
				, [] (const TSharedPtr<SRCPanelTreeNode>& TreeNode)
					{
						return TreeNode;
					});

			const TSharedRef<FExposedEntityDragDrop> DragDropOp = MakeShared<FExposedEntityDragDrop>(InNode->GetDragAndDropWidget(SelectedEntities.Num()), InNode->GetRCId(), SelectedEntities);
			DragDropOp->Construct();
			return FReply::Handled().BeginDragDrop(DragDropOp);
		}
	}
	return FReply::Unhandled();
}

void SRCPanelExposedEntitiesList::StoreListSettings()
{
	const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>* SettingsPtr = bIsInProtocolsMode.Get() ?
		FRemoteControlUIModule::Get().GetExposedEntitiesListSettingsForProtocol(ActiveProtocol) :
		nullptr;


	const TOptional<FScopedTransaction> ChangeListSettingsTransaction = !GIsTransacting ?
		FScopedTransaction(LOCTEXT("ChangeListSettingsTransaction", "Change Remote Control List Order")) :
		TOptional<FScopedTransaction>{};

	if (SettingsPtr)
	{
		FRCPanelExposedEntitiesListSettingsData Settings;
		Settings.FieldGroupType = CurrentGroupType;
		Settings.FieldGroupOrder = CurrentGroupSortType;

		(*SettingsPtr)->OnSettingsChanged(GetPreset(), Settings);
	}
	else
	{
		DefaultSettings.FieldGroupType = CurrentGroupType;
		DefaultSettings.FieldGroupOrder = CurrentGroupSortType;
	}
}

void SRCPanelExposedEntitiesList::RecallListSettings()
{
	const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>* SettingsPtr = bIsInProtocolsMode.Get() ? 
		FRemoteControlUIModule::Get().GetExposedEntitiesListSettingsForProtocol(ActiveProtocol) : 
		nullptr;

	if (SettingsPtr)
	{
		const FRCPanelExposedEntitiesListSettingsData Settings = (*SettingsPtr)->GetListSettings(GetPreset());
		CurrentGroupType = Settings.FieldGroupType;
		CurrentGroupSortType = Settings.FieldGroupOrder;
	}
	else
	{
		CurrentGroupType = DefaultSettings.FieldGroupType;
		CurrentGroupSortType = DefaultSettings.FieldGroupOrder;
	}

	CreateFieldGroup();
	OrderGroups();
}

void SRCPanelExposedEntitiesList::SelectSource_Execute()
{
	TSharedPtr<SRCPanelTreeNode> SelectedEntity = GetSelectedEntity();

	if (!SelectedEntity.IsValid() || SelectedEntity->GetRCType() == SRCPanelTreeNode::FieldGroup)
	{
		return;
	}

	TSharedPtr<FRemoteControlEntity> Entity = StaticCastSharedPtr<SRCPanelExposedEntity>(SelectedEntity)->GetEntity();

	if (!Entity.IsValid())
	{
		return;
	}

	FRemoteControlUIModule::Get().SelectSource(Entity);
}

void SRCPanelExposedEntitiesList::UpdateEntityUsage_Execute()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->UpdateEntityUsage();
	}
}

void SRCPanelExposedEntitiesList::UpdateEntityUsage()
{
	bUpdateEntityUsage = true;
}

void SRCPanelExposedEntitiesList::OnObjectPropertyChange(UObject* InObject, FPropertyChangedEvent& InChangeEvent)
{
	if (!InObject)
	{
		return;
	}

	static constexpr EPropertyChangeType::Type TypesNeedingRefresh = EPropertyChangeType::ArrayAdd | EPropertyChangeType::ArrayClear | EPropertyChangeType::ArrayRemove | EPropertyChangeType::ValueSet | EPropertyChangeType::Duplicate;
	auto IsRelevantProperty = [](FFieldClass* PropertyClass)
	{
		return PropertyClass && (PropertyClass == FArrayProperty::StaticClass() || PropertyClass == FSetProperty::StaticClass() || PropertyClass == FMapProperty::StaticClass());
	};

	if ((InChangeEvent.ChangeType & TypesNeedingRefresh) != 0 
		&& ((InChangeEvent.MemberProperty && IsRelevantProperty(InChangeEvent.MemberProperty->GetClass()))
			|| (InChangeEvent.Property && IsRelevantProperty(InChangeEvent.Property->GetClass()))))
	{
		if (TSharedPtr<FRCPanelWidgetRegistry> Registry = WidgetRegistry.Pin())
		{
			if (InChangeEvent.Property)
			{
				// Force the refresh only if the property itself pass IsRelevantProperty check
				Registry->Refresh(InObject, IsRelevantProperty(InChangeEvent.Property->GetClass()));
			}
			else
			{
				Registry->Refresh(InObject);
			}
		}

		if (URemoteControlPreset* const Preset = GetPreset())
		{
			// If the modified property is a parent of an exposed property, re-enable the edit condition.
			// This is useful in case we re-add an array element which contains a nested property that is exposed.
			for (TWeakPtr<FRemoteControlProperty> WeakProp : Preset->GetExposedEntities<FRemoteControlProperty>())
			{
				if (TSharedPtr<FRemoteControlProperty> RCProp = WeakProp.Pin())
				{
					if (RCProp->FieldPathInfo.IsResolved() && InObject && InObject->GetClass()->IsChildOf(RCProp->GetSupportedBindingClass()))
					{
						for (int32 SegmentIndex = 0; SegmentIndex < RCProp->FieldPathInfo.GetSegmentCount(); SegmentIndex++)
						{
							FProperty* ResolvedField = RCProp->FieldPathInfo.GetFieldSegment(SegmentIndex).ResolvedData.Field;
							if (ResolvedField == InChangeEvent.MemberProperty || ResolvedField == InChangeEvent.Property)
							{
								RCProp->EnableEditCondition();
								break;
							}
						}
					}
				}
			}
		}
		
		bool bShouldRefreshNodes = true;
		if (const URemoteControlSettings* Settings = GetDefault<URemoteControlSettings>())
		{
			bShouldRefreshNodes = Settings->bRefreshExposedEntitiesOnObjectPropertyUpdate;
		}

		if (bShouldRefreshNodes)
		{
			bNodesRefreshRequested = true;
		}
	}

	GroupsListView->RequestListRefresh();
	FieldsListView->RequestListRefresh();
}

void SRCPanelExposedEntitiesList::OnActorLabelChanged(AActor* InActor)
{
	if (CachedEntityOwners.Contains(InActor))
	{
		Refresh();
	}
}

void SRCPanelExposedEntitiesList::Refresh()
{
	bRefreshRequested = true;
}

void SRCPanelExposedEntitiesList::TryRefreshingSearch(const FText& InSearchText, bool bApplyFilter)
{
	FieldEntities.Reset();

	URemoteControlPreset* const Preset = GetPreset();
	if (FieldWidgetMap.IsEmpty() || !Preset)
	{
		FieldsListView->RequestListRefresh();

		*SearchedText = FText::GetEmpty();

		return;
	}

	*SearchedText = InSearchText;

	for (TWeakPtr<FRemoteControlEntity> WeakEntity : Preset->GetExposedEntities())
	{
		if (const TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
		{
			if (TSharedPtr<SRCPanelTreeNode> SelectedGroup = GetSelectedGroup())
			{
				if (FRemoteControlPresetGroup* EntityGroup = Preset->Layout.FindGroupFromField(Entity->GetId()))
				{
					const FString& EntityLabel = Entity->GetLabel().ToString();

					if (EntityLabel.Contains(*InSearchText.ToString()) &&
						(SelectedGroup->GetRCId() == EntityGroup->Id ||
							Preset->Layout.IsDefaultGroup(SelectedGroup->GetRCId())))
					{
						if (TSharedPtr<SRCPanelTreeNode>* FoundNode = FieldWidgetMap.Find(Entity->GetId()))
						{
							(*FoundNode)->SetHighlightText(InSearchText);

							FieldEntities.Add(*FoundNode);
						}
					}
				}
			}
		}
	}

	// Only apply filter if it is requested otherwise skip this.
	if (BackendFilter.HasAnyActiveFilters() && bApplyFilter)
	{
		bFilterApplicationRequested = true;
	}

	FieldsListView->RequestListRefresh();
}

void SRCPanelExposedEntitiesList::GenerateListWidgets()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedEntitiesList::GenerateListWidgets);

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	TArray<FGuid> OrderMap = Preset->Layout.GetDefaultGroupOrder();
	FieldWidgetMap.Reset();
	CachedEntityOwners.Reset();

	TArray<TWeakPtr<FRemoteControlEntity>> ExposedEntities = Preset->GetExposedEntities();
	CachedEntityOwners.Reserve(ExposedEntities.Num());

	for (TWeakPtr<FRemoteControlEntity> WeakEntity : ExposedEntities)
	{
		if (const TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
		{
			if (const UObject* EntityOwner = UE::RCUIHelpers::GetEntityOwner(Entity))
			{
				CachedEntityOwners.Add(EntityOwner);
			}

			FGenerateWidgetArgs Args;
			Args.Entity = Entity;
			Args.Preset = Preset;
			Args.WidgetRegistry = WidgetRegistry;
			Args.ColumnSizeData = ColumnSizeData;
			Args.bIsInLiveMode = bIsInLiveMode;
			Args.CommandList = CommandList;

			FieldWidgetMap.Add(Entity->GetId(), FRemoteControlUIModule::Get().GenerateEntityWidget(Args));
		}
	}

	// We order the ALL group here otherwise the order would be the same order of the ExposedEntities
	if (OrderMap.Num() && FieldWidgetMap.Num())
	{
		FieldWidgetMap.KeySort(
			[&OrderMap]
			(const FGuid& A, const FGuid& B)
			{
				if (!OrderMap.Contains(A))
				{
					return false;
				}
				if (!OrderMap.Contains(B))
				{
					return true;
				}
				return OrderMap.IndexOfByKey(A) < OrderMap.IndexOfByKey(B);
			});
	}
}

void SRCPanelExposedEntitiesList::GenerateListWidgets(const FRemoteControlPresetGroup& FromGroup)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedEntitiesList::GenerateListWidgetsFromGroup);

	FieldEntities.Reset();
	CachedFieldEntities.Reset();

	if (FieldWidgetMap.IsEmpty())
	{
		FieldsListView->RequestListRefresh();

		return;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (Preset && Preset->Layout.IsDefaultGroup(FromGroup.Id))
	{
		FieldWidgetMap.GenerateValueArray(CachedFieldEntities);
		CreateGroupsAndSort();
	}
	else if (TSharedPtr<SRCPanelGroup> SRCGroup = FindGroupById(FromGroup.Id))
	{
		SRCGroup->GetNodeChildren(CachedFieldEntities);
		CreateGroupsAndSort();
	}

	FieldsListView->RequestListRefresh();
}

void SRCPanelExposedEntitiesList::RefreshGroups()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	FieldGroups.Reset(Preset->Layout.GetGroups().Num());

	for (const FRemoteControlPresetGroup& RCGroup : Preset->Layout.GetGroups())
	{
		TSharedRef<SRCPanelGroup> FieldGroup = SNew(SRCPanelGroup, Preset, ColumnSizeData)
			.Id(RCGroup.Id)
			.Name(RCGroup.Name)
			.OnFieldDropEvent(this, &SRCPanelExposedEntitiesList::OnDropOnGroup)
			.OnGetGroupId(this, &SRCPanelExposedEntitiesList::GetGroupId)
			.OnDeleteGroup(this, &SRCPanelExposedEntitiesList::OnDeleteGroup)
			.LiveMode(bIsInLiveMode)
			.CommandList(CommandList);
		
		FieldGroups.Add(FieldGroup);
		FieldGroup->GetNodes().Reserve(RCGroup.GetFields().Num());

		for (const FGuid& FieldId : RCGroup.GetFields())
		{
			if (TSharedPtr<SRCPanelTreeNode>* Widget = FieldWidgetMap.Find(FieldId))
			{
				FieldGroup->GetNodes().Add(*Widget);
			}
		}
	}

	GroupsListView->RequestListRefresh();
}

TSharedRef<ITableRow> SRCPanelExposedEntitiesList::OnGenerateRow(TSharedPtr<SRCPanelTreeNode> Node, const TSharedRef<STableViewBase>& OwnerTable)
{
	const TSharedRef<SWidget> NodeWidget = Node->AsShared();
	
	auto OnAcceptDropLambda = [this]
	(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, const TSharedPtr<SRCPanelTreeNode>& InNode)
	{
		if (const TSharedPtr<FExposedEntityDragDrop> DragDropOp = InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
		{
			if (const TSharedPtr<SRCPanelGroup> Group = FindGroupById(GetGroupId(InNode->GetRCId())))
			{
				if (DragDropOp->IsOfType<FExposedEntityDragDrop>())
				{
					return OnDropOnGroup(DragDropOp, InNode, Group);
				}
				else if (DragDropOp->IsOfType<FFieldGroupDragDropOp>())
				{
					return OnDropOnGroup(DragDropOp, nullptr, Group);
				}
			}
		}

		return FReply::Unhandled();
	};

	if (Node->GetRCType() == SRCPanelTreeNode::Group)
	{
		return SNew(STableRow<TSharedPtr<SWidget>>, OwnerTable)
			.Padding(1.f)
			.Style(&RCPanelStyle->TableRowStyle)
			[
				NodeWidget
			];
	}
	else
	{
		constexpr float LeftPadding = 3.f;
		const FMargin Margin = Node->GetRCType() == SRCPanelTreeNode::FieldChild ? FMargin(LeftPadding + 10.f, 1.f, 1.f, 1.f) : FMargin(LeftPadding, 1.f, 1.f, 1.f);
		Node->OnPropertyIdRenamed().BindSP(this, &SRCPanelExposedEntitiesList::OnPropertyIdRenamed, Node);
		Node->OnLabelModified().BindSP(this, &SRCPanelExposedEntitiesList::OnLabelModified);

		return SNew(SEntityRow, OwnerTable)
			.OnDragDetected(FOnDragDetected::CreateSP(this, &SRCPanelExposedEntitiesList::OnNodeDragDetected, Node))
			.OnDragEnter_Lambda([Node](const FDragDropEvent& Event) { if (Node && Node->GetRCType() == SRCPanelTreeNode::Field) StaticCastSharedPtr<SRCPanelExposedField>(Node)->SetIsHovered(true); })
			.OnDragLeave_Lambda([Node](const FDragDropEvent& Event) { if (Node && Node->GetRCType() == SRCPanelTreeNode::Field) StaticCastSharedPtr<SRCPanelExposedField>(Node)->SetIsHovered(false); })
			.OnAcceptDrop_Lambda(OnAcceptDropLambda)
			.Padding(Margin)
			.Style(&RCPanelStyle->TableRowStyle)
			.ActiveProtocol_Lambda([this]() { return ActiveProtocol; })
			.Entity(Node);
	}
}

void SRCPanelExposedEntitiesList::OnGetNodeChildren(TSharedPtr<SRCPanelTreeNode> Node, TArray<TSharedPtr<SRCPanelTreeNode>>& OutNodes)
{
	if (Node.IsValid())
	{
		Node->GetNodeChildren(OutNodes);
	}
}

void SRCPanelExposedEntitiesList::OnSelectionChanged(TSharedPtr<SRCPanelTreeNode> Node, ESelectInfo::Type SelectInfo)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Node || SelectInfo != ESelectInfo::OnMouseClick || !Preset)
	{
		FieldsListView->ClearSelection();

		OnSelectionChangeDelegate.Broadcast(nullptr);

		return;
	}

	if (Node->GetRCType() == SRCPanelTreeNode::Group)
	{
		if (FRemoteControlPresetGroup* RCGroup = Preset->Layout.GetGroup(Node->GetRCId()))
		{
			CurrentlySelectedGroup = RCGroup->Id;

			GenerateListWidgets(*RCGroup);

			if (BackendFilter.HasAnyActiveFilters())
			{
				bFilterApplicationRequested = true;
			}
		}

		FieldsListView->ClearSelection();
	}

	OnSelectionChangeDelegate.Broadcast(Node);
}

FReply SRCPanelExposedEntitiesList::OnDropOnGroup(const TSharedPtr<FDragDropOperation>& DragDropOperation, const TSharedPtr<SRCPanelTreeNode>& TargetEntity, const TSharedPtr<SRCPanelTreeNode>& DragTargetGroup)
{
	checkSlow(DragTargetGroup);

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return FReply::Unhandled();
	}

	// will be changed, for now as long as you have a grouping type active you will not be able to drag fields to re-order them
	if (DragDropOperation->IsOfType<FExposedEntityDragDrop>() && CurrentGroupType == ERCFieldGroupType::None)
	{
		if (TSharedPtr<FExposedEntityDragDrop> DragDropOp = StaticCastSharedPtr<FExposedEntityDragDrop>(DragDropOperation))
		{
			if (Preset->Layout.IsDefaultGroup(GetSelectedGroup()->GetRCId()))
			{
				FRemoteControlPresetLayout::FFieldSwapArgs Args;
				Args.OriginGroupId = GetSelectedGroup()->GetRCId();
				Args.TargetGroupId = DragTargetGroup->GetRCId();
				Args.DraggedFieldsIds = DragDropOp->GetSelectedFieldsId();

				if (TargetEntity)
				{
					Args.TargetFieldId = TargetEntity->GetRCId();
				}

				FScopedTransaction Transaction(LOCTEXT("MoveFieldDefaultGroup", "Move exposed field"));
				Preset->Modify();
				TArray<FGuid> OrderEntities;

				for (const TSharedPtr<SRCPanelTreeNode>& Entity : FieldEntities)
				{
					OrderEntities.Add(Entity->GetRCId());
				}

				Preset->Layout.SwapFieldsDefaultGroup(Args, GetGroupId(DragDropOp->GetNodeId()), OrderEntities);
				constexpr bool bForceMouseClick = true;
				SetSelection(FindGroupById(Args.OriginGroupId), bForceMouseClick);
				return FReply::Handled();
			}

			FGuid DragOriginGroupId = GetGroupId(DragDropOp->GetNodeId());
			if (!DragOriginGroupId.IsValid())
			{
				return FReply::Unhandled();
			}

			FRemoteControlPresetLayout::FFieldSwapArgs Args;
			Args.OriginGroupId = DragOriginGroupId;
			Args.TargetGroupId = DragTargetGroup->GetRCId();
			Args.DraggedFieldsIds = DragDropOp->GetSelectedFieldsId();

			if (TargetEntity)
			{
				Args.TargetFieldId = TargetEntity->GetRCId();
			}
			else
			{
				if (Args.OriginGroupId == Args.TargetGroupId)
				{
					// No-op if dragged from the same group.
					return FReply::Unhandled();
				}
			}

			FScopedTransaction Transaction(LOCTEXT("MoveField", "Move exposed field"));
			Preset->Modify();
			Preset->Layout.SwapFields(Args);
			constexpr bool bForceMouseClick = true;
			SetSelection(FindGroupById(Args.OriginGroupId), bForceMouseClick);
			return FReply::Handled();
		}
	}
	else if (DragDropOperation->IsOfType<FFieldGroupDragDropOp>())
	{
		if (TSharedPtr<FFieldGroupDragDropOp> DragDropOp = StaticCastSharedPtr<FFieldGroupDragDropOp>(DragDropOperation))
		{
			FGuid DragOriginGroupId = DragDropOp->GetGroupId();
			FGuid DragTargetGroupId = DragTargetGroup->GetRCId();

			if (DragOriginGroupId == DragTargetGroupId)
			{
				// No-op if dragged from the same group.
				return FReply::Unhandled();
			}

			FScopedTransaction Transaction(LOCTEXT("MoveGroup", "Move Group"));
			Preset->Modify();
			Preset->Layout.SwapGroups(DragOriginGroupId, DragTargetGroupId);
			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FGuid SRCPanelExposedEntitiesList::GetGroupId(const FGuid& EntityId)
{
	FGuid GroupId;
	if (URemoteControlPreset* Preset = GetPreset())
	{
		if (FRemoteControlPresetGroup* Group = Preset->Layout.FindGroupFromField(EntityId))
		{
			GroupId = Group->Id;
		}
	}
	return GroupId;
}

FReply SRCPanelExposedEntitiesList::OnCreateGroup()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		FScopedTransaction Transaction(LOCTEXT("CreateGroup", "Create Group"));
		Preset->Modify();
		Preset->Layout.CreateGroup();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

void SRCPanelExposedEntitiesList::OnDeleteGroup(const FGuid& GroupId)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteGroup", "Delete Group"));
		Preset->Modify();
		Preset->Layout.DeleteGroup(GroupId);
	}
}

void SRCPanelExposedEntitiesList::OnEntityRebind(const FGuid& InEntityGuid)
{
	const TSharedPtr<SRCPanelTreeNode>* FoundNode = FieldEntities.FindByPredicate([InEntityGuid] (const TSharedPtr<SRCPanelTreeNode>& InNode) { return InNode->GetRCId() == InEntityGuid; } );
	if (FoundNode && FoundNode->IsValid())
	{
		if (const TSharedPtr<SRCPanelExposedEntity>& Entity = StaticCastSharedPtr<SRCPanelExposedEntity>(*FoundNode))
		{
			Entity->Refresh();
		}
	}
}

void SRCPanelExposedEntitiesList::SelectActorsInlevel(const TArray<UObject*>& Objects)
{
	if (GEditor)
	{
		// Don't change selection if the target's component is already selected
		USelection* Selection = GEditor->GetSelectedComponents();
		
		if (Selection->Num() == 1
			&& Objects.Num() == 1
			&& Selection->GetSelectedObject(0) != nullptr
			&& Selection->GetSelectedObject(0)->GetTypedOuter<AActor>() == Objects[0])
		{
			return;
		}

		GEditor->SelectNone(false, true, false);

		for (UObject* Object : Objects)
		{
			if (AActor* Actor = Cast<AActor>(Object))
			{
				GEditor->SelectActor(Actor, true, true, true);
			}
		}
	}
}

void SRCPanelExposedEntitiesList::RegisterEvents()
{
	OnPropertyChangedHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddSP(this, &SRCPanelExposedEntitiesList::OnObjectPropertyChange);

	OnProtocolBindingAddedOrRemovedHandle = IRemoteControlProtocolWidgetsModule::Get().OnProtocolBindingAddedOrRemoved().AddSP(this, &SRCPanelExposedEntitiesList::OnProtocolBindingAddedOrRemoved);
}

void SRCPanelExposedEntitiesList::UnregisterEvents()
{
	FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(OnPropertyChangedHandle);

	IRemoteControlProtocolWidgetsModule::Get().OnProtocolBindingAddedOrRemoved().Remove(OnProtocolBindingAddedOrRemovedHandle);
}

void SRCPanelExposedEntitiesList::OnFilterChanged()
{
	check(FilterPtr.IsValid());

	const FRCFilter Filter = FilterPtr->GetCombinedBackendFilter();

	SetBackendFilter(Filter);
}

void SRCPanelExposedEntitiesList::OnSearchTextChanged(const FText& InFilterText)
{
	SearchTextFilter->SetRawFilterText(InFilterText);
	SearchBoxPtr->SetError(SearchTextFilter->GetFilterErrorText());
	*SearchedText = InFilterText;

	const int32 Length = InFilterText.ToString().Len();

	if (Length > 3)
	{
		TryRefreshingSearch(InFilterText);
	}
	else if (Length == 3 || Length == 0) // Avoid unnecessary refresh if search text is below the threshold.
		{
			ResetSearch();

			Refresh();
		}
}

void SRCPanelExposedEntitiesList::OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	if (InCommitType == ETextCommit::OnCleared || InFilterText.IsEmpty())
	{
		ResetSearch();

		Refresh();

		return;
	}

	OnSearchTextChanged(InFilterText);
}

void SRCPanelExposedEntitiesList::PopulateSearchStrings(const SRCPanelTreeNode& Item, TArray<FString>& OutSearchStrings) const
{
	if (const URemoteControlPreset* Preset = GetPreset())
	{
		if (const TSharedPtr<const FRemoteControlEntity> Entity = Preset->GetExposedEntity<FRemoteControlEntity>(Item.GetRCId()).Pin())
		{
			OutSearchStrings.Add(Entity->GetLabel().ToString());
		}
	}
}

TSharedPtr<SRCPanelGroup> SRCPanelExposedEntitiesList::FindGroupById(const FGuid& Id)
{
	TSharedPtr<SRCPanelGroup> TargetGroup;
	if (TSharedPtr<SRCPanelGroup>* FoundGroup = FieldGroups.FindByPredicate([Id](const TSharedPtr<SRCPanelGroup>& InGroup) {return InGroup->GetRCId() == Id; }))
	{
		TargetGroup = *FoundGroup;
	}
	return TargetGroup;
}

TSharedPtr<SWidget> SRCPanelExposedEntitiesList::OnContextMenuOpening(SRCPanelTreeNode::ENodeType InType)
{
	if (TSharedPtr<SRCPanelTreeNode> SelectedNode = InType == SRCPanelTreeNode::Group ? SRCPanelExposedEntitiesList::GetSelectedGroup() : SRCPanelExposedEntitiesList::GetSelectedEntity())
	{
		return SelectedNode->GetContextMenu();
	}
	
	return nullptr;
}

TSharedRef<SWidget> SRCPanelExposedEntitiesList::GetGroupMenuContentWidget()
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = false;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, nullptr);

	// Grouping type section
	MenuBuilder.BeginSection(FName(TEXT("RCGroupingType")), LOCTEXT("RCGroupingTypeHeader", "Group by"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCNoGroupingLabel", "No Grouping"),
		LOCTEXT("RCNoGroupingTooltip", "No grouping set. Show items as a flat list"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::OnCreateFieldGroup, ERCFieldGroupType::None),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]{ return CurrentGroupType == ERCFieldGroupType::None; })),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCGroupPropertyIdLabel", "Property Id"),
		LOCTEXT("RCGroupPropertyIdTooltip", "Group by Property Id"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::OnCreateFieldGroup, ERCFieldGroupType::PropertyId),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]{ return CurrentGroupType == ERCFieldGroupType::PropertyId; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCGroupOwnerLabel", "Owner"),
		LOCTEXT("RCGroupOwnerTooltip", "Group by Owner"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::OnCreateFieldGroup, ERCFieldGroupType::Owner),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]{ return CurrentGroupType == ERCFieldGroupType::Owner; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.EndSection();

	// Ordering type section
	MenuBuilder.BeginSection(FName(TEXT("RCSortingType")), LOCTEXT("RCSortingTypeHeader", "Sort by"));

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCSortAscendingGroupLabel", "Ascending"),
		LOCTEXT("RCSortAscendingGroupTooltip", "Sort groups ascending"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::OnGroupOrderChanged, ERCFieldGroupOrder::Ascending),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]{ return CurrentGroupSortType == ERCFieldGroupOrder::Ascending; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("RCSortDescendingGroupLabel", "Descending"),
		LOCTEXT("RCSortDescendingGroupTooltip", "Sort groups descending"),
		FSlateIcon(),
		FUIAction(
		FExecuteAction::CreateSP(this, &SRCPanelExposedEntitiesList::OnGroupOrderChanged, ERCFieldGroupOrder::Descending),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]{ return CurrentGroupSortType == ERCFieldGroupOrder::Descending; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton);

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRCPanelExposedEntitiesList::OnCreateFieldGroup(ERCFieldGroupType InFieldGroupType)
{
	CurrentGroupType = InFieldGroupType;
	CreateFieldGroup();
	OrderGroups();

	StoreListSettings();
}

void SRCPanelExposedEntitiesList::CreateFieldGroup()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	FieldEntitiesGroups.Reset();
	TSet<FName> GroupKeys;
	if (CurrentGroupType == ERCFieldGroupType::PropertyId)
	{
		for (TSharedPtr<SRCPanelTreeNode> Entity : CachedFieldEntities)
		{
			if (TSharedPtr<SRCPanelExposedField> ExposedField = StaticCastSharedPtr<SRCPanelExposedField>(Entity))
			{
				if (!GroupKeys.Contains(ExposedField->GetPropertyId()))
				{
					GroupKeys.Add(ExposedField->GetPropertyId());
					TSharedRef<SRCPanelExposedEntitiesGroup> GroupWidget = SNew(SRCPanelExposedEntitiesGroup, CurrentGroupType, Preset)
						.FieldKey(ExposedField->GetPropertyId())
						.OnGroupPropertyIdChanged(this, &SRCPanelExposedEntitiesList::CreateGroupsAndSort);

					GroupWidget->AssignChildren(CachedFieldEntities);
					FieldEntitiesGroups.Add(GroupWidget);
				}
			}
		}
	}
	else if (CurrentGroupType == ERCFieldGroupType::Owner)
	{
		for (TSharedPtr<SRCPanelTreeNode> Entity : CachedFieldEntities)
		{
			if (TSharedPtr<SRCPanelExposedField> ExposedField = StaticCastSharedPtr<SRCPanelExposedField>(Entity))
			{
				const FName OwnerPathName = ExposedField->GetOwnerPathName();

				if (!GroupKeys.Contains(OwnerPathName))
				{
					GroupKeys.Add(OwnerPathName);

					TSharedRef<SRCPanelExposedEntitiesGroup> GroupWidget = SNew(SRCPanelExposedEntitiesGroup, CurrentGroupType, Preset)
						.FieldKey(OwnerPathName);

					GroupWidget->AssignChildren(CachedFieldEntities);
					FieldEntitiesGroups.Add(GroupWidget);
				}
			}
		}
	}

	if (CurrentGroupType == ERCFieldGroupType::None &&
		FieldsListView.IsValid())
	{
		FieldEntities = CachedFieldEntities;
		FieldsListView->RequestTreeRefresh();
	}
	else if (FieldEntitiesGroups.Num())
	{
		FieldEntities.Empty();
		for (const TSharedPtr<SRCPanelExposedEntitiesGroup>& Group : FieldEntitiesGroups)
		{
			FieldEntities.Add(Group);
		}

		bRefreshEntitiesGroups = true;
	}
}

void SRCPanelExposedEntitiesList::OnGroupOrderChanged(ERCFieldGroupOrder InGroupOrder)
{
	CurrentGroupSortType = CurrentGroupSortType == InGroupOrder? ERCFieldGroupOrder::None : InGroupOrder;
	OrderGroups();
}

void SRCPanelExposedEntitiesList::OrderGroups()
{
	if (CurrentGroupType == ERCFieldGroupType::None)
	{
		return;
	}

	if (CurrentGroupSortType == ERCFieldGroupOrder::Ascending)
	{
		FieldEntities.Sort([](const TSharedPtr<SRCPanelTreeNode>& InFirst, const TSharedPtr<SRCPanelTreeNode>& InSecond)
		{
			if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroupFirst = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(InFirst))
			{
				if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroupSecond = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(InSecond))
				{
					return FieldGroupFirst->GetFieldKey().Compare(FieldGroupSecond->GetFieldKey()) <= 0;
				}
			}
			return false;
		});
	}
	else if (CurrentGroupSortType == ERCFieldGroupOrder::Descending)
	{
		FieldEntities.Sort([](const TSharedPtr<SRCPanelTreeNode>& InFirst, const TSharedPtr<SRCPanelTreeNode>& InSecond)
		{
			if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroupFirst = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(InFirst))
			{
				if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroupSecond = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(InSecond))
				{
					return FieldGroupFirst->GetFieldKey().Compare(FieldGroupSecond->GetFieldKey()) > 0;
				}
			}
			return false;
		});
	}

	bRefreshEntitiesGroups = true;
}

void SRCPanelExposedEntitiesList::RefreshGroupsAndRestoreExpansions()
{
	TSet<TSharedPtr<SRCPanelTreeNode>> ExpandedItems;
	FieldsListView->GetExpandedItems(ExpandedItems);
	FieldsListView->RequestTreeRefresh();
	for (const TSharedPtr<SRCPanelTreeNode>& ExpandedItem : ExpandedItems)
	{
		if (const TSharedPtr<SRCPanelExposedEntitiesGroup>& FieldGroup = StaticCastSharedPtr<SRCPanelExposedEntitiesGroup>(ExpandedItem))
		{
			if (FieldGroup->GetGroupType() == CurrentGroupType)
			{
				for (const TSharedPtr<SRCPanelExposedEntitiesGroup>& Group : FieldEntitiesGroups)
				{
					if (Group->GetFieldKey() == FieldGroup->GetFieldKey())
					{
						FieldsListView->SetItemExpansion(Group, true);
						break;
					}
				}
			}
		}
		else
		{
			FieldsListView->SetItemExpansion(ExpandedItem, true);
		}
	}
}

void SRCPanelExposedEntitiesList::CreateGroupsAndSort()
{
	CreateFieldGroup();
	OrderGroups();

	StoreListSettings();
}

void SRCPanelExposedEntitiesList::RegisterPresetDelegates()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	FRemoteControlPresetLayout& Layout = Preset->Layout;
	Layout.OnGroupAdded().AddSP(this, &SRCPanelExposedEntitiesList::OnGroupAdded);
	Layout.OnGroupDeleted().AddSP(this, &SRCPanelExposedEntitiesList::OnGroupDeleted);
	Layout.OnGroupOrderChanged().AddSP(this, &SRCPanelExposedEntitiesList::OnGroupOrderChanged);
	Layout.OnGroupRenamed().AddSP(this, &SRCPanelExposedEntitiesList::OnGroupRenamed);
	Layout.OnFieldAdded().AddSP(this, &SRCPanelExposedEntitiesList::OnFieldAdded);
	Layout.OnFieldDeleted().AddSP(this, &SRCPanelExposedEntitiesList::OnFieldDeleted);
	Layout.OnFieldOrderChanged().AddSP(this, &SRCPanelExposedEntitiesList::OnFieldOrderChanged);
	Preset->OnEntitiesUpdated().AddSP(this, &SRCPanelExposedEntitiesList::OnEntitiesUpdated);
	Preset->OnEntityRebind().AddSP(this, &SRCPanelExposedEntitiesList::OnEntityRebind);
}

void SRCPanelExposedEntitiesList::UnregisterPresetDelegates()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		FRemoteControlPresetLayout& Layout = Preset->Layout;
		Preset->OnEntitiesUpdated().RemoveAll(this);
		Preset->OnEntityRebind().RemoveAll(this);
		Layout.OnFieldOrderChanged().RemoveAll(this);
		Layout.OnFieldDeleted().RemoveAll(this);
		Layout.OnFieldAdded().RemoveAll(this);
		Layout.OnGroupRenamed().RemoveAll(this);
		Layout.OnGroupOrderChanged().RemoveAll(this);
		Layout.OnGroupDeleted().RemoveAll(this);
		Layout.OnGroupAdded().RemoveAll(this);
	}
}

void SRCPanelExposedEntitiesList::OnEntityAdded(const FGuid& InEntityId)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	auto ExposeEntity = [This=this, Preset, InEntityId](TSharedPtr<SRCPanelTreeNode>&& Node)
	{
		if (Node)
		{
			This->FieldWidgetMap.Add(InEntityId, Node);

			if (TSharedPtr<SRCPanelGroup> SRCGroup = This->FindGroupById(This->GetGroupId(InEntityId)))
			{
				SRCGroup->GetNodes().Add(MoveTemp(Node));
				This->GenerateListWidgets(*Preset->Layout.GetGroup(This->GetGroupId(InEntityId)));
			}
		}
	};
	
	FGenerateWidgetArgs Args;
	Args.Preset = Preset;
	Args.WidgetRegistry = WidgetRegistry;
	Args.ColumnSizeData = ColumnSizeData;
	Args.bIsInLiveMode = bIsInLiveMode;
	Args.Entity = Preset->GetExposedEntity(InEntityId).Pin();
	Args.CommandList = CommandList;

	ExposeEntity(FRemoteControlUIModule::Get().GenerateEntityWidget(Args));
}

void SRCPanelExposedEntitiesList::OnEntityRemoved(const FGuid& InGroupId, const FGuid& InEntityId)
{
	if (TSharedPtr<SRCPanelGroup> PanelGroup = FindGroupById(InGroupId))
	{
		const int32 EntityIndex = PanelGroup->GetNodes().IndexOfByPredicate([InEntityId](const TSharedPtr<SRCPanelTreeNode>& Node) { return Node->GetRCId() == InEntityId; });
		if (EntityIndex != INDEX_NONE)
		{
			PanelGroup->GetNodes().RemoveAt(EntityIndex);
		}
	}

	if (const TSharedPtr<SRCPanelTreeNode> Node = GetSelectedEntity())
	{
		if (Node->GetRCId() == InEntityId)
		{
			OnSelectionChangeDelegate.Broadcast(nullptr);
		}
	}

	FieldWidgetMap.Remove(InEntityId);

	if (URemoteControlPreset* Preset = GetPreset())
	{
		GenerateListWidgets(*Preset->Layout.GetGroup(InGroupId));
	}
}

void SRCPanelExposedEntitiesList::OnGroupAdded(const FRemoteControlPresetGroup& Group)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	TSharedRef<SRCPanelGroup> FieldGroup = SNew(SRCPanelGroup, Preset, ColumnSizeData)
		.Id(Group.Id)
		.Name(Group.Name)
		.OnFieldDropEvent(this, &SRCPanelExposedEntitiesList::OnDropOnGroup)
		.OnGetGroupId(this, &SRCPanelExposedEntitiesList::GetGroupId)
		.OnDeleteGroup(this, &SRCPanelExposedEntitiesList::OnDeleteGroup)
		.LiveMode(bIsInLiveMode)
		.CommandList(CommandList);
	
	FieldGroups.Add(FieldGroup);
	
	FieldGroup->GetNodes().Reserve(Group.GetFields().Num());

	for (FGuid FieldId : Group.GetFields())
	{
		if (TSharedPtr<SRCPanelTreeNode>* Widget = FieldWidgetMap.Find(FieldId))
		{
			FieldGroup->GetNodes().Add(*Widget);
		}
	}

	FieldGroup->EnterRenameMode();
	GroupsListView->SetSelection(FieldGroup, ESelectInfo::OnMouseClick);
	GroupsListView->ScrollToBottom();
	GroupsListView->RequestListRefresh();

	RequestSearchOrFilter();
}

void SRCPanelExposedEntitiesList::OnGroupDeleted(FRemoteControlPresetGroup DeletedGroup)
{
	int32 Index = FieldGroups.IndexOfByPredicate([&DeletedGroup](const TSharedPtr<SRCPanelGroup>& Group) { return Group->GetRCId() == DeletedGroup.Id; });

	if (TSharedPtr<SRCPanelTreeNode> Node = GetSelectedEntity())
	{
		if (DeletedGroup.GetFields().Contains(Node->GetRCId()))
		{
			OnSelectionChangeDelegate.Broadcast(nullptr);
		}
	}
	
	if (Index != INDEX_NONE)
	{
		FieldGroups.RemoveAt(Index);
		GroupsListView->RequestListRefresh();
	}

	RequestSearchOrFilter();
}

void SRCPanelExposedEntitiesList::OnGroupOrderChanged(const TArray<FGuid>& GroupIds)
{
	TMap<FGuid, int32> IndicesMap;
	IndicesMap.Reserve(GroupIds.Num());
	for (auto It = GroupIds.CreateConstIterator(); It; ++It)
	{
		IndicesMap.Add(*It, It.GetIndex());
	}

	auto SortFunc = [&IndicesMap]
	(const TSharedPtr<SRCPanelGroup>& A, const TSharedPtr<SRCPanelGroup>& B)
	{
		return IndicesMap.FindChecked(A->GetRCId()) < IndicesMap.FindChecked(B->GetRCId());
	};

	FieldGroups.Sort(SortFunc);
	GroupsListView->RequestListRefresh();
}

void SRCPanelExposedEntitiesList::OnGroupRenamed(const FGuid& GroupId, FName NewName)
{
	if (TSharedPtr<SRCPanelGroup> Group = FindGroupById(GroupId))
	{
		Group->SetName(NewName);
	}
}

void SRCPanelExposedEntitiesList::OnFieldAdded(const FGuid& GroupId, const FGuid& FieldId, int32 FieldPosition)
{
	OnEntityAdded(FieldId);

	RequestSearchOrFilter();
}

void SRCPanelExposedEntitiesList::OnFieldDeleted(const FGuid& GroupId, const FGuid& FieldId, int32 FieldPosition)
{
	OnEntityRemoved(GroupId, FieldId);

	RequestSearchOrFilter();
}

void SRCPanelExposedEntitiesList::OnFieldOrderChanged(const FGuid& GroupId, const TArray<FGuid>& Fields)
{
	if (TSharedPtr<SRCPanelGroup>* Group = FieldGroups.FindByPredicate([GroupId](const TSharedPtr<SRCPanelGroup>& InGroup) {return InGroup->GetRCId() == GroupId; }))
	{
		// Sort the group's fields according to the fields array.
		TMap<FGuid, int32> OrderMap;
		OrderMap.Reserve(Fields.Num());
		for (auto It = Fields.CreateConstIterator(); It; ++It)
		{
			OrderMap.Add(*It, It.GetIndex());
		}

		if (*Group)
		{
			URemoteControlPreset* const Preset = GetPreset();
			if (Preset && Preset->Layout.IsDefaultGroup((*Group)->GetRCId()))
			{
				FieldWidgetMap.ValueSort(
				[&OrderMap]
					(const TSharedPtr<SRCPanelTreeNode>& A, const TSharedPtr<SRCPanelTreeNode>& B)
					{
						return OrderMap.FindChecked(A->GetRCId()) < OrderMap.FindChecked(B->GetRCId());
					});
				TArray<FGuid> OrderedGuid;
				FieldWidgetMap.GenerateKeyArray(OrderedGuid);
				// We save the ALL group order here because it is not saved in the normal workflow
				Preset->Layout.SetDefaultGroupOrder(OrderedGuid);
			}
			else
			{
				(*Group)->GetNodes().Sort(
				[&OrderMap]
				(const TSharedPtr<SRCPanelTreeNode>& A, const TSharedPtr<SRCPanelTreeNode>& B)
					{
						return OrderMap.FindChecked(A->GetRCId()) < OrderMap.FindChecked(B->GetRCId());
					});
			}
		}
	}

	GroupsListView->RequestListRefresh();
	FieldsListView->RequestListRefresh();
}

void SRCPanelExposedEntitiesList::OnEntitiesUpdated(URemoteControlPreset*, const TSet<FGuid>& UpdatedEntities)
{
	GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([WeakListPtr = TWeakPtr<SRCPanelExposedEntitiesList>(StaticCastSharedRef<SRCPanelExposedEntitiesList>(AsShared()))]()
	{
		if (TSharedPtr<SRCPanelExposedEntitiesList> ListPtr = WeakListPtr.Pin())
		{
			ListPtr->GroupsListView->RebuildList();
			ListPtr->FieldsListView->RebuildList();
		}
	}));

	OnEntityListUpdatedDelegate.ExecuteIfBound();

	RequestSearchOrFilter();
}

void SRCPanelExposedEntitiesList::ApplyFilters()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	if (BackendFilter.HasAnyActiveFilters())
	{
		// If we are not actively searching anything then include all the entities.
		if (SearchedText.IsValid() && SearchedText->IsEmpty())
		{
			for (TWeakPtr<FRemoteControlEntity> WeakEntity : Preset->GetExposedEntities())
			{
				if (const TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
				{
					if (TSharedPtr<SRCPanelTreeNode> SelectedGroup = GetSelectedGroup())
					{
						if (FRemoteControlPresetGroup* EntityGroup = Preset->Layout.FindGroupFromField(Entity->GetId()))
						{
							if (SelectedGroup->GetRCId() == EntityGroup->Id ||
								Preset->Layout.IsDefaultGroup(SelectedGroup->GetRCId()))
							{
								if (TSharedPtr<SRCPanelTreeNode>* FoundNode = FieldWidgetMap.Find(Entity->GetId()))
								{
									if (BackendFilter.DoesPassFilters(FoundNode->ToSharedRef()))
									{
										FieldEntities.AddUnique(*FoundNode);
									}
								}
							}
						}
					}
				}
			}
		}
		else
		{
			// Caution : Avoid filter during this phase of search as it would cause an endless loop between search and filters.
			constexpr bool bApplyFilter = false;

			TryRefreshingSearch(*SearchedText, bApplyFilter);
		}
	
		// Apply the filter (always operate on the active list of entities).
		FieldEntities.RemoveAll([&](TSharedPtr<const SRCPanelTreeNode> InEntity) { return !BackendFilter.DoesPassFilters(InEntity.ToSharedRef()); });

		FieldsListView->RequestListRefresh();
	}
	else // Do a one time refresh in case all filters are cleared.
	{
		Refresh();
	}
}

void SRCPanelExposedEntitiesList::RequestSearchOrFilter()
{
	bSearchRequested = SearchedText.IsValid() && !SearchedText->IsEmpty();

	bFilterApplicationRequested = BackendFilter.HasAnyActiveFilters();
}

FReply SRCPanelExposedEntitiesList::RequestDeleteAllEntities()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!FieldsListView.IsValid() || !Preset)
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = FText::Format(LOCTEXT("DeleteAllEntitiesWarning", "You are about to delete '{0}' entities. This action might not be undone.\nAre you sure you want to proceed?"), FieldWidgetMap.Num());

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		FScopedTransaction Transaction(LOCTEXT("UnexposeAll", "Unexpose all remote control entities"));
		Preset->Modify();

		for (TMap<FGuid, TSharedPtr<SRCPanelTreeNode>>::TConstIterator EntityItr = FieldWidgetMap.CreateConstIterator(); EntityItr; ++EntityItr)
		{
			Preset->Unexpose(EntityItr->Key);
		}
	}

	return FReply::Handled();
}

FReply SRCPanelExposedEntitiesList::RequestDeleteAllGroups()
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!GroupsListView.IsValid() || !Preset)
	{
		return FReply::Unhandled();
	}

	const FText WarningMessage = FText::Format(LOCTEXT("DeleteAllGroupsWarning", "You are about to delete '{0}' groups. This action might not be undone.\nAre you sure you want to proceed?"), FieldGroups.Num() - 1);

	EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo, WarningMessage);

	if (UserResponse == EAppReturnType::Yes)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteAllGroups", "Delete all remote control groups"));
		Preset->Modify();

		const TArray<FRemoteControlPresetGroup>& RCGroups = Preset->Layout.GetGroups();

		// Perform deletion in reverse to avoid "Ensure condition failed: Lhs.CurrentNum == Lhs.InitialNum"
		for (int32 GroupIndex = RCGroups.Num() - 1; GroupIndex >= 0; GroupIndex--)
		{
			const FRemoteControlPresetGroup& RCGroup = RCGroups[GroupIndex];

			if (Preset->Layout.IsDefaultGroup(RCGroup.Id))
			{
				continue;
			}
		
			Preset->Layout.DeleteGroup(RCGroup.Id);
		}

		Refresh();
	}

	return FReply::Handled();
}

SHeaderRow::FColumn::FArguments SRCPanelExposedEntitiesList::CreateColumn(const FName ForColumnName)
{
	const FText DefaultColumnLabel = GetColumnLabel(ForColumnName);

	if (ForColumnName == RemoteControlPresetColumns::Status || ForColumnName == RemoteControlPresetColumns::BindingStatus) // Exception for Status & BindingStatus columns as they are fixed ones.
	{
		const SHeaderRow::FColumn::FArguments ColumnArgs = SHeaderRow::Column(ForColumnName)
			.DefaultLabel(DefaultColumnLabel)
			.FixedWidth(33.f)
			.HAlignHeader(HAlign_Center)
			.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);

		return ColumnArgs;
	}

	const SHeaderRow::FColumn::FArguments ColumnArgs = SHeaderRow::Column(ForColumnName)
		.DefaultLabel(DefaultColumnLabel)
		.FillWidth(this, &SRCPanelExposedEntitiesList::GetColumnSize, ForColumnName)
		.HAlignHeader(HAlign_Center)
		.HeaderContentPadding(RCPanelStyle->HeaderRowPadding);

	return ColumnArgs;
}

int32 SRCPanelExposedEntitiesList::GetColumnIndex(const FName& ForColumn) const
{
	return GetColumnIndex_Internal(ForColumn, RemoteControlPresetColumns::Value, ERCColumn::ERC_After);
}

int32 SRCPanelExposedEntitiesList::GetColumnIndex_Internal(const FName& ForColumn, const FName& ExistingColumnName, ERCColumn::Position InPosition) const
{
	int32 InsertIndex = INDEX_NONE;

	if (FieldsListView.IsValid() && FieldsHeaderRow.IsValid())
	{
		const TIndirectArray<SHeaderRow::FColumn>& ExistingColumns = FieldsHeaderRow->GetColumns();
		
		if (ExistingColumns.IsEmpty())
		{
			return InsertIndex;
		}

		auto ColumnItr = ExistingColumns.CreateConstIterator();
		
		while (ColumnItr)
		{
			// If the given column already exists then we need not to insert it again.
			if (ColumnItr->ColumnId == ForColumn)
			{
				InsertIndex = INDEX_NONE;

				break;
			}

			switch (InPosition)
			{
				case ERCColumn::ERC_After:
					if (ColumnItr->ColumnId == ExistingColumnName)
					{
						InsertIndex = ColumnItr.GetIndex() + 1;

						break;
					}
					break;
				case ERCColumn::ERC_Before:
					if (ColumnItr->ColumnId == ExistingColumnName)
					{
						InsertIndex = ColumnItr.GetIndex() - 1;

						break;
					}
					break;
			}

			++ColumnItr;
		}
	}

	return InsertIndex;
}

FText SRCPanelExposedEntitiesList::GetColumnLabel(const FName& ForColumn) const
{
	if (ForColumn == RemoteControlPresetColumns::BindingStatus)
	{
		return LOCTEXT("RCPresetBindingStatusColumnHeader", "REC");
	}
	else if (ForColumn == RemoteControlPresetColumns::PropertyIdentifier)
	{
		return LOCTEXT("RCPresetPropertyIdColumnHeader_Label", "Property ID");
	}
	else if (ForColumn == RemoteControlPresetColumns::Status)
	{
		return FText();
	}
	else
	{
		IRemoteControlProtocolModule& RCProtocolModule = IRemoteControlProtocolModule::Get();

		if (TSharedPtr<IRemoteControlProtocol> RCProtocol = RCProtocolModule.GetProtocolByName(ActiveProtocol))
		{
			if (const FProtocolColumnPtr& ProtocolColumn = RCProtocol->GetRegisteredColumn(ForColumn))
			{
				return ProtocolColumn->DisplayText;
			}
		}
	}

	return FText::GetEmpty();
}

float SRCPanelExposedEntitiesList::GetColumnSize(const FName ForColumn) const
{
	float ColumnSize = ProtocolColumnConstants::ColumnSizeNormal;

	if (ForColumn == RemoteControlPresetColumns::PropertyIdentifier)
	{
		ColumnSize = ProtocolColumnConstants::ColumnSizeMini;
	}
	else
	{
		IRemoteControlProtocolModule& RCProtocolModule = IRemoteControlProtocolModule::Get();

		if (TSharedPtr<IRemoteControlProtocol> RCProtocol = RCProtocolModule.GetProtocolByName(ActiveProtocol))
		{
			if (const FProtocolColumnPtr& ProtocolColumn = RCProtocol->GetRegisteredColumn(ForColumn))
			{
				ColumnSize = ProtocolColumn->ColumnSize;
			}
		}
	}

	return ColumnSize;
}

void SRCPanelExposedEntitiesList::InsertColumn(const FName& InColumnName)
{
	if (FieldsListView.IsValid() && FieldsHeaderRow.IsValid())
	{
		int32 InsertIndex = GetColumnIndex(InColumnName);

		if (InsertIndex != INDEX_NONE)
		{
			const SHeaderRow::FColumn::FArguments NewColumn = CreateColumn(InColumnName);

			FieldsHeaderRow->InsertColumn(NewColumn, InsertIndex);
		}
	}
}

void SRCPanelExposedEntitiesList::RemoveColumn(const FName& InColumnName)
{
	if (FieldsListView.IsValid() && FieldsHeaderRow.IsValid())
	{
		FieldsHeaderRow->RemoveColumn(InColumnName);
	}
}

void SRCPanelExposedEntitiesList::OnProtocolBindingAddedOrRemoved(ERCProtocolBinding::Op BindingOperation)
{
	ActiveListMode = EEntitiesListMode::Default; // Hack to trigger mode change via mode switcher.

	if (bIsInProtocolsMode.Get())
	{
		RebuildListWithColumns(EEntitiesListMode::Protocols);
	}
}

void SRCPanelExposedEntitiesList::OnWidgetRegistryRefreshed(const TArray<UObject*>& Objects)
{
	Refresh();
}

void SRCPanelExposedEntitiesList::ProcessRefresh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedEntitiesList::Refresh);

	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedItems;
	FieldsListView->GetSelectedItems(SelectedItems);
	GenerateListWidgets();

	RefreshGroups();

	if (URemoteControlPreset* const Preset = GetPreset())
	{
		constexpr bool bForceMouseClick = true;

		if (const FRemoteControlPresetGroup* SelectedGroup = Preset->Layout.GetGroup(CurrentlySelectedGroup))
		{
			GenerateListWidgets(*SelectedGroup);
			SetSelection(FindGroupById(SelectedGroup->Id), bForceMouseClick);
		}
		else
		{
			const FRemoteControlPresetGroup& DefaultGroup = Preset->Layout.GetDefaultGroup();
			GenerateListWidgets(DefaultGroup);
			SetSelection(FindGroupById(DefaultGroup.Id), bForceMouseClick);
		}

		for (TSharedPtr<SRCPanelTreeNode> SelectedItem : SelectedItems)
		{
			SetSelection(SelectedItem, bForceMouseClick);
		}
	}
}

bool FGroupDragEvent::IsDraggedFromSameGroup() const
{
	return DragOriginGroup->GetId() == DragTargetGroup->GetId();
}


#undef LOCTEXT_NAMESPACE /* RemoteControlPanelFieldList */