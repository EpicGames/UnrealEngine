// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "IRemoteControlUIModule.h"
#include "Misc/TextFilter.h"
#include "RemoteControlPreset.h"
#include "SRCPanelExposedEntitiesGroup.h"
#include "SRCPanelTreeNode.h"
#include "UI/Filters/RCFilter.h"
#include "UI/RCFieldGroupOrder.h"
#include "UI/RCFieldGroupType.h"
#include "UI/RCPanelExposedEntitiesListSettingsData.h"
#include "UObject/ObjectKey.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STreeView.h"

class AActor;
class FDragDropOperation;
class FRCPanelWidgetRegistry;
class FReply;
class ITableRow;
class SComboButton;
class SRCHeaderRow;
class SRCPanelFilter;
class SRCPanelGroup;
class SRemoteControlPanel;
class SRemoteControlTarget;
class SSearchBox;
class STableViewBase;
class URemoteControlPreset;
struct FGuid;
struct FRCPanelGroup;
struct FRCPanelStyle;
struct FRemoteControlFunction;
struct FRemoteControlPresetGroup;
struct FRemoteControlProperty;
struct SRCPanelExposedField;
struct SRCPanelTreeNode;

enum class EEntitiesListMode : uint8
{
	Default,
	Protocols
};

/** Holds information about a group drag and drop event  */
struct FGroupDragEvent
{
	FGroupDragEvent(TSharedPtr<SRCPanelGroup> InDragOriginGroup, TSharedPtr<SRCPanelGroup> InDragTargetGroup)
		: DragOriginGroup(MoveTemp(InDragOriginGroup))
		, DragTargetGroup(MoveTemp(InDragTargetGroup))
	{
	}

	bool IsDraggedFromSameGroup() const;

	/** Group the drag originated in. */
	TSharedPtr<SRCPanelGroup> DragOriginGroup;
	/** Group where the element was dropped. */
	TSharedPtr<SRCPanelGroup> DragTargetGroup;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnSelectionChange, const TSharedPtr<SRCPanelTreeNode>&/*SelectedNode*/);

class SRCPanelExposedEntitiesList : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCPanelExposedEntitiesList)
		: _LiveMode(false)
		, _ProtocolsMode(false)
	{}
		SLATE_ATTRIBUTE(bool, LiveMode)
		SLATE_ATTRIBUTE(bool, ProtocolsMode)
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, ExposeFunctionsComboButton)
		SLATE_ATTRIBUTE(TSharedPtr<SWidget>, ExposeActorsComboButton)
		SLATE_EVENT(FSimpleDelegate, OnEntityListUpdated)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, TWeakPtr<FRCPanelWidgetRegistry> InWidgetRegistry);

	~SRCPanelExposedEntitiesList();

	//~ Begin SWidget Interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	//~ End SWidget Interface

	URemoteControlPreset* GetPreset() const
	{
		return PresetWeak.Get();
	}

	/** Get the currently selected group. */
	TSharedPtr<SRCPanelTreeNode> GetSelectedGroup() const;
	
	/** Get the currently selected exposed entity. */
	TSharedPtr<SRCPanelTreeNode> GetSelectedEntity() const;

	/** Get the currently selected exposed entities. */
	TArray<TSharedPtr<SRCPanelTreeNode>> GetSelectedEntities() const;

	/** Get the currently selected exposed entities. */
	int32 GetSelectedEntitiesNum() const;

	/** Return true if the given node is currently selected. */
	bool IsEntitySelected(const TSharedPtr<SRCPanelTreeNode>& InNode) const;

	/** Set the currently selected group or exposed entity. */
	void SetSelection(const TSharedPtr<SRCPanelTreeNode>& Node, const bool bForceMouseClick = false);

	/** Notifies the entities list view that the filter-list filter has changed */
	void SetBackendFilter(const FRCFilter& InBackendFilter);

	/** Recreate the list with dynamic columns. */
	void RebuildListWithColumns(EEntitiesListMode InListMode);
	
	/** Recreate everything in the panel. */
	void Refresh();
	
	/** Tries to refresh the list according to the provided search text. */
	void TryRefreshingSearch(const FText& InSearchText, bool bApplyFilter = true);

	/** Notifies us that the search has ended. */
	void ResetSearch() { *SearchedText = FText::GetEmpty(); RequestSearchOrFilter(); };

	/** Returns delegate called on selection change. */
	FOnSelectionChange& OnSelectionChange() { return OnSelectionChangeDelegate; }

	/** Returns delegate triggered upon a modification to an exposed entity. */
	FSimpleDelegate OnEntityListUpdated() { return OnEntityListUpdatedDelegate; }

	/** Return the FilterPtr of this RCPanelEntitiesList*/
	TSharedPtr<SRCPanelFilter> GetFilterPtr() const { return FilterPtr; }

	/** Update the Search */
	void UpdateSearch();

	/** Update the In Use flag on entities. */
	void UpdateEntityUsage();

private:
	/** Handles label to be shown in the entity list header.  */
	FText HandleEntityListHeaderLabel() const;
	/** Handles object property changes, used to update arrays correctly.  */
	void OnObjectPropertyChange(UObject* InObject, FPropertyChangedEvent& InChangeEvent);
	/** Called when the label of an actor has changed */
	void OnActorLabelChanged(AActor* InActor);
	/** Create exposed entity widgets. */
	void GenerateListWidgets();
	/** Create exposed entity widgets. */
	void GenerateListWidgets(const FRemoteControlPresetGroup& FromGroup);
	/** Generate the groups using the preset's layout data. */
	void RefreshGroups();
	/** Generate row widgets for groups and exposed entities. */
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<SRCPanelTreeNode> Node, const TSharedRef<STableViewBase>& OwnerTable);
	/** Handle getting a node's children. */
	void OnGetNodeChildren(TSharedPtr<SRCPanelTreeNode> Node, TArray<TSharedPtr<SRCPanelTreeNode>>& OutNodes);
	/** Handle selection changes. */
	void OnSelectionChanged(TSharedPtr<SRCPanelTreeNode> Node, ESelectInfo::Type SelectInfo);
	/** Handlers for drag/drop events. */
	FReply OnDropOnGroup(const TSharedPtr<FDragDropOperation>& DragDropOperation, const TSharedPtr<SRCPanelTreeNode>& TargetEntity, const TSharedPtr<SRCPanelTreeNode>& DragTargetGroup);
	/** Handler for when a filter in the filter list has changed */
	void OnFilterChanged();
	/** Get the id of the group that holds a particular widget. */
	FGuid GetGroupId(const FGuid& EntityId);
	/** Handles creating a new group. */
	FReply OnCreateGroup();
	/** Handles group deletion. */
	void OnDeleteGroup(const FGuid& GroupId);
	/** Called when an Entity is rebound */
	void OnEntityRebind(const FGuid& InEntityGuid);
	/** Select actors in the current level. */
	void SelectActorsInlevel(const TArray<UObject*>& Objects);
	//~ Register to engine/editor events in order to correctly update widgets.
	void RegisterEvents();
	void UnregisterEvents();

	//~ Handlers for getting/setting the entity list's column width.
	float OnGetLeftColumnWidth() const { return 1.0f - ColumnWidth; }
	float OnGetRightColumnWidth() const { return ColumnWidth; }
	void OnSetColumnWidth(float InWidth) { ColumnWidth = InWidth; }

	// Exposed Entities filtering. (Filters the Exposed Entities view)
	void OnSearchTextChanged(const FText& InFilterText);
	void OnSearchTextCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);
	void PopulateSearchStrings(const SRCPanelTreeNode& Item, TArray<FString>& OutSearchStrings) const;

	/** Find a group using its id. */
	TSharedPtr<SRCPanelGroup> FindGroupById(const FGuid& Id);

	/** Handle context menu opening on a row. */
	TSharedPtr<SWidget> OnContextMenuOpening(SRCPanelTreeNode::ENodeType InType);

	TSharedRef<SWidget> GetGroupMenuContentWidget();

	/**
	 * Called when the group type changed, if the new group type is the same as the current one it will be set to none
	 * @param InFieldGroupType New grouping type
	 */
	void OnCreateFieldGroup(ERCFieldGroupType InFieldGroupType);

	/** Group fields based on the current group type (PropertyId/Owner) */
	void CreateFieldGroup();

	/**
	 * Called when the order type changed, if the new order type is the same as the current one it will be set to none
	 * @param InGroupOrder New group order
	 */
	void OnGroupOrderChanged(ERCFieldGroupOrder InGroupOrder);

	/** Order the groups based on the current order assigned (Ascending/Descending) */
	void OrderGroups();

	/** Refresh the fields groups and restore the expansion */
	void RefreshGroupsAndRestoreExpansions();

	/** Calls both the CreateFieldGroup and the OrderGroups */
	void CreateGroupsAndSort();

	//~ Register and handle preset delegates.
	void RegisterPresetDelegates();
	void UnregisterPresetDelegates();
	void OnEntityAdded(const FGuid& EntityId);
	void OnEntityRemoved(const FGuid& InGroupId, const FGuid& EntityId);
	void OnGroupAdded(const FRemoteControlPresetGroup& Group);
	void OnGroupDeleted(FRemoteControlPresetGroup DeletedGroup);
	void OnGroupOrderChanged(const TArray<FGuid>& GroupIds);
	void OnGroupRenamed(const FGuid& GroupId, FName NewName);
	void OnFieldAdded(const FGuid& GroupId, const FGuid& FieldId, int32 FieldPosition);
	void OnFieldDeleted(const FGuid& GroupId, const FGuid& FieldId, int32 FieldPosition);
	void OnFieldOrderChanged(const FGuid& GroupId, const TArray<FGuid>& Fields);
	void OnEntitiesUpdated(URemoteControlPreset*, const TSet<FGuid>& UpdatedEntities);

	/** Requests that the entities view refreshes all it's items. */
	void ApplyFilters();

	/** Notifies us that we need to do a search or filter in the next tick. */
	void RequestSearchOrFilter();

	/** Delete all operation for entities list. */
	FReply RequestDeleteAllEntities();
	/** Delete all operation for groups list. */
	FReply RequestDeleteAllGroups();

	SHeaderRow::FColumn::FArguments CreateColumn(const FName ForColumnName);
	int32 GetColumnIndex(const FName& ForColumn) const;
	int32 GetColumnIndex_Internal(const FName& ForColumn, const FName& ExistingColumnName, ERCColumn::Position InPosition) const;
	FText GetColumnLabel(const FName& ForColumn) const;
	float GetColumnSize(const FName ForColumn) const;
	void InsertColumn(const FName& InColumnName);
	void RemoveColumn(const FName& InColumnName);

	void OnProtocolBindingAddedOrRemoved(ERCProtocolBinding::Op BindingOperation);

	/** Called when the widget registry is refreshed by an underlying property generator. */
	void OnWidgetRegistryRefreshed(const TArray<UObject*>& Objects);

	void ProcessRefresh();

	/**
	 * If necessary, will perform a refresh of the exposed entities nodes widgets.
	 */
	void ExposedEntitiesNodesRefresh();

	/** Executed when a property Id is changed, will set all selected node(s) property id to the new one */
	void OnPropertyIdRenamed(const FName InNewId, TSharedPtr<SRCPanelTreeNode> InNode);

	/** Executed when a property Name is changed, will set all selected node(s) property Name to the new one */
	void OnLabelModified(const FName InOldName, const FName InNewName);

	/** Executed when a drag is detected, will create the Drag and Drop widget of the node(s) */
	FReply OnNodeDragDetected(const FGeometry& InGeometry, const FPointerEvent& InPointerEvent, TSharedPtr<SRCPanelTreeNode> InNode);

	/** Stores the list settings */
	void StoreListSettings();

	/** Recalls the list settings */
	void RecallListSettings();
	
	/** Selects the source of the currently selected entity. */
	void SelectSource_Execute();

	void UpdateEntityUsage_Execute();

private:
	/** Command list to use if specified */
	TSharedPtr<FUICommandList> CommandList;
	/** Holds the Groups list view. */
	TSharedPtr<SListView<TSharedPtr<SRCPanelTreeNode>>> GroupsListView;
	/** Holds the Fields list view. */
	TSharedPtr<STreeView<TSharedPtr<SRCPanelTreeNode>>> FieldsListView;
	/** Holds all the field groups. */
	TArray<TSharedPtr<SRCPanelGroup>> FieldGroups;
	/** Holds all the field entities. */
	TArray<TSharedPtr<SRCPanelTreeNode>> FieldEntities;
	/** Cached list of entity owners used to track whether owner changes should refresh the list*/
	TSet<FObjectKey> CachedEntityOwners;
	/** Cached field entities used to store the original Entities when switching groups. */
	TArray<TSharedPtr<SRCPanelTreeNode>> CachedFieldEntities;
	/** Holds all the exposed entities groups. */
	TArray<TSharedPtr<SRCPanelExposedEntitiesGroup>> ExposedEntitiesGroups;
	/** Holds the current group type */
	ERCFieldGroupType CurrentGroupType = ERCFieldGroupType::None;
	/** Holds the current sorting type */
	ERCFieldGroupOrder CurrentGroupSortType = ERCFieldGroupOrder::None;
	/** Holds all the entities groups currently in the list */
	TArray<TSharedPtr<SRCPanelExposedEntitiesGroup>> FieldEntitiesGroups;
	/** Map of field ids to field widgets. */
	TMap<FGuid, TSharedPtr<SRCPanelTreeNode>> FieldWidgetMap;
	/** Whether the panel is in live mode. */
	TAttribute<bool> bIsInLiveMode;
	/** Whether the panel is in protocols mode. */
	TAttribute<bool> bIsInProtocolsMode;
	/** Holds a weak reference to the preset asset. */
	TWeakObjectPtr<URemoteControlPreset> PresetWeak;
	/** Handle to the delegate called when an object property change is detected. */
	FDelegateHandle OnPropertyChangedHandle;
	/** Handle to the delegate called when a binding is added or removed. */
	FDelegateHandle OnProtocolBindingAddedOrRemovedHandle;
	/** Handle to the delegate called when an actor's label is renamed */
	FDelegateHandle OnActorLabelChangedHandle;
	/** Delegate called on selected group change. */
	FOnSelectionChange OnSelectionChangeDelegate;
	/** The column data shared between all tree nodes in order to share a splitter amongst all rows. */
	FRCColumnSizeData ColumnSizeData;
	/** The actual width of the right column.  The left column is 1-ColumnWidth */
	float ColumnWidth = 0.49f;
	/** Event triggered when the entity list is updated. */
	FSimpleDelegate OnEntityListUpdatedDelegate;
	/** Holds the cache of widgets to be used by this list's entities. */
	TWeakPtr<FRCPanelWidgetRegistry> WidgetRegistry;
	/** Holds the cache of filters to be used by this list's entities. */
	FRCFilter BackendFilter;
	/** If true, the entity items will be refreshed next frame. */
	bool bFilterApplicationRequested;
	/** If true, the entity items will be refreshed next frame. */
	bool bSearchRequested;
	/** If true, the entity list will have its in use flag updated next frame. */
	bool bUpdateEntityUsage;
	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
	/** Holds the header row of entities list. */
	TSharedPtr<SRCHeaderRow> FieldsHeaderRow;
	/** The filter list */
	TSharedPtr<SRCPanelFilter> FilterPtr;
	/** The text box used to search for tags. */
	TSharedPtr<SSearchBox> SearchBoxPtr;
	/** Button giving you the possibilities for grouping. */
	TSharedPtr<SComboButton> ComboButtonGroupButton;
	/** Text filter for the search text. */
    TSharedPtr<TTextFilter<const SRCPanelTreeNode&>> SearchTextFilter;
    /** Actively searched term. */
    TSharedPtr<FText> SearchedText;
	/** Holds the active list mode. */
	EEntitiesListMode ActiveListMode;
	/** Holds the active protocol enabled by the mode switcher. */
	FName ActiveProtocol;
	/** Columns to be present by default in protocols mode. */
	static TSet<FName> DefaultProtocolColumns;
	/** Holds identifier of the selected group. */
	FGuid CurrentlySelectedGroup;
	/** Default settings for this entities list */
	FRCPanelExposedEntitiesListSettingsData DefaultSettings;
	
	bool bRefreshRequested = false;
	bool bRefreshEntitiesGroups = false;

	/** When true, widgets of the Exposed Entities List will be refreshed on Tick */
	bool bNodesRefreshRequested = false;
};