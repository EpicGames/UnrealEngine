// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelExposedEntity.h"

#include "ActorTreeItem.h"
#include "Commands/RemoteControlCommands.h"
#include "Components/ActorComponent.h"
#include "Components/BillboardComponent.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Modules/ModuleManager.h"
#include "RCUIHelpers.h"
#include "RemoteControlBinding.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "RemoteControlPanelStyle.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "SRCPanelDragHandle.h"
#include "SceneOutlinerFilters.h"
#include "SceneOutlinerModule.h"
#include "ScopedTransaction.h"
#include "Styling/CoreStyle.h"
#include "Styling/RemoteControlStyles.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Notifications/SPopUpErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanel"

namespace RebindingUtils
{
	TMap<AActor*, TArray<UObject*>> GetLevelSubObjectsOfClass(UClass* TargetClass , UWorld* PresetWorld)
	{
		TMap<AActor*, TArray<UObject*>> ObjectMap;

		for (TActorIterator<AActor> It(PresetWorld, AActor::StaticClass(), EActorIteratorFlags::SkipPendingKill); It; ++It)
		{
			if (UE::RemoteControlBinding::IsValidActorForRebinding(*It, PresetWorld))
			{
				TArray<UObject*> SubObjects;
				GetObjectsWithOuter(*It, SubObjects);

				for (UObject* SubObject : SubObjects)
				{
					if (SubObject->IsA(TargetClass) && UE::RemoteControlBinding::IsValidSubObjectForRebinding(SubObject, PresetWorld))
					{
						ObjectMap.FindOrAdd(*It).Add(SubObject);
					}
				}
			}
		}

		return ObjectMap;
	}
}

const FText SRCPanelExposedEntity::SelectInOutliner = LOCTEXT("RCSelectInOutliner", "\nDouble Click to Select in Outliner");

TSharedPtr<FRemoteControlEntity> SRCPanelExposedEntity::GetEntity() const
{
	if (Preset.IsValid())
	{
		return Preset->GetExposedEntity(EntityId).Pin();
	}
	return nullptr;
}

TSharedPtr<SWidget> SRCPanelExposedEntity::GetContextMenu()
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection*/true, CommandList);

	MenuBuilder.BeginSection("Common");

	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().RenameEntity, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Rename")));
	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().ChangePropId, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Edit")));
	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().DeleteEntity, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Delete")));
	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().SelectSource, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.SelectInViewport")));

	MenuBuilder.EndSection();

	MenuBuilder.AddSeparator();

	constexpr bool bNoIndent = true;
	MenuBuilder.AddWidget(CreateUseContextCheckbox(), LOCTEXT("UseContextLabel", "Use Context"), bNoIndent);

	MenuBuilder.AddSubMenu(
		LOCTEXT("EntityRebindSubmenuLabel", "Rebind"),
		LOCTEXT("EntityRebindSubmenuToolTip", "Pick an object to rebind this exposed entity."),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
		{
			constexpr bool bNoIndent = true;
			SubMenuBuilder.AddWidget(CreateRebindMenuContent(), FText::GetEmpty(), bNoIndent);
		}));

	MenuBuilder.AddSubMenu(
		LOCTEXT("EntityRebindComponentSubmenuLabel", "Rebind Component"),
		LOCTEXT("EntityRebindComponentSubmenuToolTip", "Pick a component to rebind this exposed entity."),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
			{
				CreateRebindComponentMenuContent(SubMenuBuilder);
			}));

	MenuBuilder.AddSubMenu(
		LOCTEXT("EntityRebindSubObjectSubmenuLabel", "Rebind SubObject"),
		LOCTEXT("EntityRebindSubObjectSubmenuToolTip", "Pick a subobject to rebind this exposed entity."),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
			{
				CreateRebindSubObjectMenuContent(SubMenuBuilder);
			}));

	MenuBuilder.AddSubMenu(
		LOCTEXT("EntityRebindAllUnderActorSubmenuLabel", "Rebind all properties for this actor"),
		LOCTEXT("EntityRebindAllUnderActorSubmenuToolTip", "Pick an actor to rebind."),
		FNewMenuDelegate::CreateLambda([this](FMenuBuilder& SubMenuBuilder)
			{
				constexpr bool bNoIndent = true;
				SubMenuBuilder.AddWidget(CreateRebindAllPropertiesForActorMenuContent(), FText::GetEmpty(), bNoIndent);
			}));

	return MenuBuilder.MakeWidget();
}

void SRCPanelExposedEntity::EnterRenameMode()
{
	if (NameTextBox.IsValid())
	{
		NameTextBox->EnterEditingMode();
	}
}

FName SRCPanelExposedEntity::GetPropertyId()
{
	return PropertyIdLabel;
}

void SRCPanelExposedEntity::Refresh()
{
	if (EntityId.IsValid() && Preset.IsValid())
	{
		FInitParams InitParams;
		InitParams.Preset = Preset.Get();
		InitParams.EntityId = EntityId;
		InitParams.bLiveMode = bLiveMode;
		InitParams.CommandList = CommandList;
		Initialize(InitParams);
	}
}

void SRCPanelExposedEntity::Initialize(const FInitParams& InParams)
{
	EntityId = InParams.EntityId;
	Preset = InParams.Preset;
	bLiveMode = InParams.bLiveMode;
	CommandList = InParams.CommandList;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	if (ensure(InParams.Preset))
	{
		if (const TSharedPtr<FRemoteControlEntity> RCEntity = InParams.Preset->GetExposedEntity(InParams.EntityId).Pin())
		{
			const FString BindingPath = RCEntity->GetLastBindingPath().ToString();
			CachedLabel = RCEntity->GetLabel();
			CachedBindingPath = *BindingPath;

			if (RCEntity->GetStruct() == FRemoteControlProperty::StaticStruct())
			{
				const TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(RCEntity);
				CachedFieldPath = RCProperty->FieldPathInfo.ToString();
			}

			CachedOwnerPathName = NAME_None;

			// If the binding is valid, display the actor label if possible
			if (UObject* Owner = UE::RCUIHelpers::GetEntityOwner(RCEntity))
			{
				bValidBinding = true;
				CachedOwnerPathName = *Owner->GetPathName();
				CachedOwnerDisplayName = FText::FromString(UKismetSystemLibrary::GetDisplayName(Owner)); 
			}
			else
			{
				// If not, default to the owner fname
				bValidBinding = false;

				static const FString PersistentLevelString = TEXT(":PersistentLevel.");
				int32 PersistentLevelIndex = BindingPath.Find(PersistentLevelString);
				if (PersistentLevelIndex != INDEX_NONE)
				{
					CachedOwnerPathName = *BindingPath.RightChop(PersistentLevelIndex + PersistentLevelString.Len());
					CachedOwnerDisplayName = FText::FromName(CachedOwnerPathName);
				}
			}

			const int32 OwnerNameIndex = BindingPath.Find(CachedOwnerPathName.ToString() + TEXT("."));
			if (OwnerNameIndex != INDEX_NONE)
			{
				CachedSubobjectPath = *BindingPath.RightChop(OwnerNameIndex + CachedOwnerPathName.GetStringLength() + 1);
			}

			if (CachedOwnerPathName.IsNone())
			{
				CachedOwnerPathName = *LOCTEXT("InvalidOwner", "Invalid Owner").ToString();
			}
		}
	}
}

void SRCPanelExposedEntity::CreateRebindComponentMenuContent(FMenuBuilder& SubMenuBuilder)
{
	TInlineComponentArray<UActorComponent*> ComponentArray;

	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		TArray<UObject*> BoundObjects = Entity->GetBoundObjects();
		if (BoundObjects.Num() && BoundObjects[0])
		{
			if (BoundObjects[0]->IsA<UActorComponent>())
			{
				if (AActor* OwnerActor = BoundObjects[0]->GetTypedOuter<AActor>())
				{
					OwnerActor->GetComponents(Entity->GetSupportedBindingClass(), ComponentArray);
				}
			}
		}

		for (UActorComponent* Component : ComponentArray)
		{
			SubMenuBuilder.AddMenuEntry(
				FText::FromString(Component->GetName()),
				FText::GetEmpty(),
				FSlateIconFinder::FindIconForClass(Component->GetClass(), TEXT("SCS.Component")),
				FUIAction(
					FExecuteAction::CreateLambda([Entity, Component]
						{
							if (Entity)
							{
								Entity->BindObject(Component);
							}
						}),
					FCanExecuteAction())
			);
		}
	}
}

void SRCPanelExposedEntity::CreateRebindSubObjectMenuContent(FMenuBuilder& SubMenuBuilder)
{
	TInlineComponentArray<UActorComponent*> ComponentArray;

	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		constexpr bool bAllowPie = false;
		TMap<AActor*, TArray<UObject*>> GroupedObjects = RebindingUtils::GetLevelSubObjectsOfClass(Entity->GetSupportedBindingClass(), Preset->GetWorld(bAllowPie));
		for (const TPair<AActor*, TArray<UObject*>>& ActorsAndSubobjects : GroupedObjects)
		{
			SubMenuBuilder.BeginSection(NAME_None, FText::FromString(ActorsAndSubobjects.Key->GetActorLabel()));
			for (UObject* SubObject : ActorsAndSubobjects.Value)
			{
				FString EntryLabel = SubObject->GetName();
				const FString ToolTip = SubObject->GetPathName();
					
				// Special case for NDisplay properties.
				static const FName NDisplayViewportClassName = "DisplayClusterConfigurationViewport";
				if (SubObject->GetClass()->GetFName() == NDisplayViewportClassName)
				{
					static const FTopLevelAssetPath DisplayClusterConfigurationPath = FTopLevelAssetPath(TEXT("/Script/DisplayClusterConfiguration"), TEXT("DisplayClusterConfigurationClusterNode"));
					if (UClass* ClusterNodeClass = FindObject<UClass>(DisplayClusterConfigurationPath);
						UObject * ClusterNodeOuter = SubObject->GetTypedOuter(ClusterNodeClass))
					{
						EntryLabel = SubObject->GetPathName(ClusterNodeOuter->GetOuter());
					}
				}

				SubMenuBuilder.AddMenuEntry(
					FText::FromString(EntryLabel),
					FText::FromString(ToolTip),
					FSlateIconFinder::FindIconForClass(SubObject->GetClass(), TEXT("SCS.Component")),
					FUIAction(
						FExecuteAction::CreateLambda([Entity, SubObject]
						{
							if (Entity)
							{
								Entity->BindObject(SubObject);
							}
						}),
						FCanExecuteAction())
				);
			}
			SubMenuBuilder.EndSection();
		}
	}
}

TSharedRef<SWidget> SRCPanelExposedEntity::CreateRebindAllPropertiesForActorMenuContent()
{
	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	FSceneOutlinerInitializationOptions Options;
	Options.bFocusSearchBoxWhenOpened = true;
	Options.Filters = MakeShared<FSceneOutlinerFilters>();

	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		Options.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateSP(this, &SRCPanelExposedEntity::IsActorSelectable));
	}

	constexpr bool bAllowPIE = false;

	return SNew(SBox)
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(Options, FOnActorPicked::CreateSP(this, &SRCPanelExposedEntity::OnActorSelectedForRebindAllProperties), URemoteControlPreset::GetWorld(Preset.Get(), bAllowPIE))
		];
}

TSharedRef<SWidget> SRCPanelExposedEntity::CreateInvalidWidget(const FText& InErrorText)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedEntity::CreateInvalidWidget);

	TSharedRef<SPopupErrorText> ErrorText = SNew(SPopupErrorText)
		.Visibility(InErrorText.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible);

	ErrorText->SetError(InErrorText);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(20.0)
			[
				ErrorText
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SComboButton)
			.ButtonContent()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("RebindLabel", "Rebind"))
			]
			.OnGetMenuContent_Lambda([this]() 
			{ 
				return SNew(SBox)
				.MaxDesiredHeight(400.0f)
				.WidthOverride(300.0f)
				[
					CreateRebindMenuContent()
				];
			})
		];
}

EVisibility SRCPanelExposedEntity::GetVisibilityAccordingToLiveMode(EVisibility NonEditModeVisibility) const
{
	return !bLiveMode.Get() ? EVisibility::Visible : NonEditModeVisibility;
}

TSharedRef<SWidget> SRCPanelExposedEntity::CreateRebindMenuContent()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedEntity::CreateRebindMenuContent);

	FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::Get().LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	FSceneOutlinerInitializationOptions Options;
	Options.Filters = MakeShared<FSceneOutlinerFilters>();
	Options.Filters->AddFilterPredicate<FActorTreeItem>(FActorTreeItem::FFilterPredicate::CreateSP(this, &SRCPanelExposedEntity::IsActorSelectable));
	constexpr bool bAllowPIE = false;
	UWorld* PresetWorld = URemoteControlPreset::GetWorld(Preset.Get(), bAllowPIE);
	
	return SNew(SBox)
		.MaxDesiredHeight(400.0f)
		.WidthOverride(300.0f)
		[
			SceneOutlinerModule.CreateActorPicker(Options, FOnActorPicked::CreateSP(this, &SRCPanelExposedEntity::OnActorSelected), PresetWorld)
		];
}

bool SRCPanelExposedEntity::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	if (URemoteControlPreset* RCPreset = Preset.Get())
	{
		if (InLabel.ToString() != CachedLabel.ToString() && RCPreset->GetExposedEntityId(*InLabel.ToString()).IsValid())
		{
			OutErrorMessage = LOCTEXT("NameAlreadyExists", "This name already exists.");
			IRemoteControlModule::BroadcastError(OutErrorMessage.ToString());
			return false;
		}
	}

	return true;
}

void SRCPanelExposedEntity::OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	if (URemoteControlPreset* RCPreset = Preset.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("ModifyEntityLabel", "Modify exposed entity's label."));
		RCPreset->Modify();
		FName OldName = CachedLabel;
		CachedLabel = RCPreset->RenameExposedEntity(EntityId, *InLabel.ToString());
		OnLabelModified().ExecuteIfBound(OldName, CachedLabel);
	}
}

void SRCPanelExposedEntity::OnActorSelected(AActor* InActor) const
{
	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		Entity->BindObject(InActor);
	}

	FSlateApplication::Get().DismissAllMenus();
}

const FSlateBrush* SRCPanelExposedEntity::GetBorderImage() const
{
	return FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.ExposedFieldBorder");
}

bool SRCPanelExposedEntity::IsActorSelectable(const AActor* Actor) const
{
	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		if (Entity->GetBindings().Num() && Entity->GetBindings()[0].IsValid())
		{
			// Don't show what it's already bound to.
			if (UObject* Component = Entity->GetBindings()[0]->Resolve())
			{
				if (Component == Actor || Component->GetTypedOuter<AActor>() == Actor)
				{
					return false;
				}
			}

			if (ShouldUseRebindingContext())
			{
				if (URemoteControlLevelDependantBinding* Binding = Cast<URemoteControlLevelDependantBinding>(Entity->GetBindings()[0].Get()))
				{
					if (UClass* SupportedClass = Binding->GetSupportedOwnerClass())
					{
						return Actor->GetClass()->IsChildOf(SupportedClass);
					}
				}
			}

			return Entity->CanBindObject(Actor);
		}
	}
	return false;
}

TSharedRef<SWidget> SRCPanelExposedEntity::CreateEntityWidget(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget, const FText& OptionalWarningMessage, TSharedRef<SWidget> EditConditionWidget)
{
	const FMakeNodeWidgetArgs Args = CreateEntityWidgetInternal(ValueWidget, ResetWidget, OptionalWarningMessage, EditConditionWidget);

	TSharedRef<SBorder> Widget = SNew(SBorder)
		.Padding(0.0f)
		.BorderImage(this, &SRCPanelExposedEntity::GetBorderImage);

	Widget->SetContent(MakeNodeWidget(Args));
	return Widget;
}

SRCPanelTreeNode::FMakeNodeWidgetArgs SRCPanelExposedEntity::CreateEntityWidgetInternal(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget, const FText& OptionalWarningMessage, TSharedRef<SWidget> EditConditionWidget)
{
	FMakeNodeWidgetArgs Args;

	const FSlateBrush* TrashBrush = FAppStyle::Get().GetBrush("Icons.Delete");

	Args.OwnerNameWidget = SNew(SBox)
		.Visibility(this, &SRCPanelExposedEntity::GetVisibilityAccordingToLiveMode, EVisibility::Collapsed)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.Padding(FMargin(0.0, 0.0, 4.0, 0.0))
			.AutoWidth()
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Visibility_Lambda([this]() { return bValidBinding ? EVisibility::Collapsed : EVisibility::Visible; })
				.Image(TrashBrush)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
				.ColorAndOpacity_Lambda([this]() { return bValidBinding ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground(); })
				.Text(CachedOwnerDisplayName)
				.ToolTipText(FText::FromString(CachedBindingPath.ToString() + SelectInOutliner.ToString()))
			]
		];

	Args.SubObjectPathWidget = SNew(SBox)
		.Visibility(this, &SRCPanelExposedEntity::GetVisibilityAccordingToLiveMode, EVisibility::Collapsed)
		[
			SNew(STextBlock)
			.ColorAndOpacity_Lambda([this]() { return bValidBinding ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground(); })
			.Text(CachedSubobjectPath.IsNone() ? FText::GetEmpty() : FText::FromName(CachedSubobjectPath))
			.ToolTipText(FText::Format(LOCTEXT("SubobjectPathToolTip", "The path from the owner actor to the uobject holding the exposed property.{0}"), SelectInOutliner))
		];

	Args.NameWidget = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(0.f, 0.f, 2.0f, 0.f)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Visibility(!OptionalWarningMessage.IsEmpty() ? EVisibility::Visible : EVisibility::Collapsed)
            .TextStyle(FRemoteControlPanelStyle::Get(), "RemoteControlPanel.Button.TextStyle")
            .Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
            .ColorAndOpacity(FSlateColor(FLinearColor::Yellow))
            .ToolTipText(OptionalWarningMessage)
            .Text(FEditorFontGlyphs::Exclamation_Triangle)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			EditConditionWidget
		]
		+ SHorizontalBox::Slot()
		.Padding(4.f, 0.f, 2.0f, 0.f)
		.AutoWidth()
		[
			SAssignNew(NameTextBox, SInlineEditableTextBlock)
			.Text_Lambda([this] () { return FText::FromName(CachedLabel); })
			.ToolTipText(FText::FromString(CachedFieldPath + SelectInOutliner.ToString()))
			.OnTextCommitted(this, &SRCPanelExposedEntity::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SRCPanelExposedEntity::OnVerifyItemLabelChanged)
			.IsReadOnly_Lambda([this]() { return bLiveMode.Get(); })
			.HighlightText_Lambda([this]() { return HighlightText.Get().ToString().Len() > 3 ? HighlightText.Get() : FText::GetEmpty(); })
		];

	Args.PropertyIdWidget = SNullWidget::NullWidget;

	Args.ValueWidget = ValueWidget;

	Args.ResetButton = ResetWidget;

	return Args;
}

void SRCPanelExposedEntity::OnActorSelectedForRebindAllProperties(AActor* InActor) const
{
	if (TSharedPtr<FRemoteControlEntity> Entity = GetEntity())
	{
		const bool bShouldUseRebindingContext = ShouldUseRebindingContext();

		Preset->RebindAllEntitiesUnderSameActor(Entity->GetId(), InActor, bShouldUseRebindingContext);
		SelectActor(InActor);
	}

	FSlateApplication::Get().DismissAllMenus();
}

void SRCPanelExposedEntity::HandleUnexposeEntity()
{
	if (URemoteControlPreset* RCPreset = Preset.Get())
	{
		FScopedTransaction Transaction(LOCTEXT("UnexposeFunction", "Unexpose remote control entity"));
		RCPreset->Modify();
		RCPreset->Unexpose(EntityId);
	}
}

void SRCPanelExposedEntity::SelectActor(AActor* InActor) const
{
	if (GEditor)
	{
		// Don't change selection if the target's component is already selected
		USelection* Selection = GEditor->GetSelectedComponents();

		const bool bComponentSelected = Selection->Num() == 1
			&& Selection->GetSelectedObject(0) != nullptr
			&& Selection->GetSelectedObject(0)->GetTypedOuter<AActor>() == InActor;

		if (!bComponentSelected)
		{
			constexpr bool bNoteSelectionChange = false;
			constexpr bool bDeselectBSPSurfs = true;
			constexpr bool WarnAboutManyActors = false;
			GEditor->SelectNone(bNoteSelectionChange, bDeselectBSPSurfs, WarnAboutManyActors);

			constexpr bool bInSelected = true;
			constexpr bool bNotify = true;
			constexpr bool bSelectEvenIfHidden = true;
			GEditor->SelectActor(InActor, bInSelected, bNotify, bSelectEvenIfHidden);
		}
	}
}

TSharedRef<SWidget> SRCPanelExposedEntity::CreateUseContextCheckbox()
{
	return SNew(SCheckBox)
		.ToolTipText(LOCTEXT("UseRebindingContextTooltip", "Unchecking this will allow you to rebind this property to any object regardless of the underlying supported class."))

		// Bind the button's "on checked" event to our object's method for this
		.OnCheckStateChanged(this, &SRCPanelExposedEntity::OnUseContextChanged)

		// Bind the check box's "checked" state to our user interface action
		.IsChecked(this, &SRCPanelExposedEntity::IsUseContextEnabled);
}

void SRCPanelExposedEntity::OnUseContextChanged(ECheckBoxState State)
{
	if (URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>())
	{
		if (State == ECheckBoxState::Unchecked)
		{
			Settings->bUseRebindingContext = false;
		}
		else if (State == ECheckBoxState::Checked)
		{
			Settings->bUseRebindingContext = true;
		}
	}
}

ECheckBoxState SRCPanelExposedEntity::IsUseContextEnabled() const
{
	return ShouldUseRebindingContext() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

bool SRCPanelExposedEntity::ShouldUseRebindingContext() const
{
	if (const URemoteControlSettings* Settings = GetDefault<URemoteControlSettings>())
	{
		return Settings->bUseRebindingContext;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE
