// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SRCPanelTreeNode.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "RemoteControlPanelStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SNullWidget.h"

class AActor;
class FMenuBuilder;
struct FRCPanelStyle;
struct FRemoteControlEntity;
class SInlineEditableTextBlock;
class SWidget;
class URemoteControlPreset;

/**
 * @Note if you inherit from this struct, you must call SRCPanelExposedEntity::Initialize.
 */
struct SRCPanelExposedEntity : public SRCPanelTreeNode
{
	//~ SRCPanelTreeNode interface
	TSharedPtr<FRemoteControlEntity> GetEntity() const;
	virtual TSharedPtr<SWidget> GetContextMenu() override;
	virtual FGuid GetRCId() const override final { return EntityId; }
	/** Make the group name's text box editable. */
	virtual void EnterRenameMode() override;
	/** Get the PropertyId of this Node. */
	virtual FName GetPropertyId() override;
	/** Set the PropertyId of this Node. */
	virtual void SetPropertyId(FName InNewPropertyId) override { PropertyIdLabel = InNewPropertyId; };
	/** Set the Name of this Node. */
	virtual void SetName(FName InNewName) override { CachedLabel = InNewName; }
	/** Updates the highlight text to active search term. */
	virtual void SetHighlightText(const FText& InHightlightText = FText::GetEmpty()) override { HighlightText = InHightlightText; }

	virtual void Refresh() override;
	
protected:
	struct FInitParams
	{
		FGuid EntityId;
		URemoteControlPreset* Preset;
		TAttribute<bool> bLiveMode;
		TSharedPtr<FUICommandList> CommandList;
	};
	void Initialize(const FInitParams& InParams);

	/** Create a widget that displays the rebind button. */
	TSharedRef<SWidget> CreateInvalidWidget(const FText& InErrorText = FText::GetEmpty());

	/** Get the widget's visibility according to the panel's mode. */
	EVisibility GetVisibilityAccordingToLiveMode(EVisibility NonEditModeVisibility) const;

	/** Create an exposed entity widget with a drag handle and unexpose button. */
	TSharedRef<SWidget> CreateEntityWidget(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget = SNullWidget::NullWidget, const FText& OptionalWarningMessage = FText::GetEmpty(), TSharedRef<SWidget> EditConditionWidget = SNullWidget::NullWidget);

	/** Returns populated args to display this widget. */
	virtual FMakeNodeWidgetArgs CreateEntityWidgetInternal(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget = SNullWidget::NullWidget, const FText& OptionalWarningMessage = FText::GetEmpty(), TSharedRef<SWidget> EditConditionWidget = SNullWidget::NullWidget);

protected:
	/** Id of the entity. */
	FGuid EntityId;
	/** The underlying preset. */
	TWeakObjectPtr<URemoteControlPreset> Preset;
	/** Whether the panel is in live mode or not. */
	TAttribute<bool> bLiveMode;
	/** Command list to use for the context menu */
	TSharedPtr<FUICommandList> CommandList;
	/** Display name of the entity's owner (What actor it's bound to). */
	FText CachedOwnerDisplayName;
	/** Name of the entity's owner (What actor it's bound to). */
	FName CachedOwnerPathName;
	/** Path from the entity owner to the object holding the exposed property. */
	FName CachedSubobjectPath;
	/** Path of the entity's binding. */
	FName CachedBindingPath;
	/** Whether the binding is valid at the moment. */
	bool bValidBinding = false;
	/** Display name of the entity. */
	FName CachedLabel;
	/** Display PropertyId of the entity. */
	FName PropertyIdLabel;
	/** Cached entity field path. */
	FString CachedFieldPath;
	/** Text to be highlighted while searching. */
	TAttribute<FText> HighlightText;
	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
	/** Extra information to add to the tooltips. */
	static const FText SelectInOutliner;
private:
	/** Handles changing the object this entity is bound to upon selecting an actor in the rebinding dropdown. */
	void OnActorSelected(AActor* InActor) const;
	
	/** Get the widget's border. */
	const FSlateBrush* GetBorderImage() const;
	/** Create the content of the rebind button.  */
	TSharedRef<SWidget> CreateRebindMenuContent();

	/** Create the content of the rebind component button. */
	void CreateRebindComponentMenuContent(FMenuBuilder& SubMenuBuilder);

	/** Create the content of the rebind subobject button. */
	void CreateRebindSubObjectMenuContent(FMenuBuilder& SubMenuBuilder);

	/** Create the content for the menu used to rebind all properties for the actor that owns this entity. */
	TSharedRef<SWidget> CreateRebindAllPropertiesForActorMenuContent();

	/** Handle selecting an actor for a rebind for all properties under an actor. */
	void OnActorSelectedForRebindAllProperties(AActor* InActor) const;

	/** Verifies that the entity's label doesn't already exist. */
	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);
	/** Handles committing a entity label. */
	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);
	/** Returns whether or not the actor is selectable for a binding replacement. */
	bool IsActorSelectable(const AActor* Parent) const;
	/** Handle clicking on the unexpose button. */
	void HandleUnexposeEntity();

	void SelectActor(AActor* InActor) const;

	//~ Use context widget functions.
	TSharedRef<SWidget> CreateUseContextCheckbox();
	void OnUseContextChanged(ECheckBoxState State);
	ECheckBoxState IsUseContextEnabled() const;
	bool ShouldUseRebindingContext() const;

	/** The textbox for the row's name. */
	TSharedPtr<SInlineEditableTextBlock> NameTextBox;
};

class FExposedEntityDragDrop final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FExposedFieldDragDropOp, FDecoratedDragDropOp)

	using WidgetType = SWidget;

	FExposedEntityDragDrop(const TSharedPtr<SWidget>& InWidget, const FGuid& InNodeId, const TArray<TSharedPtr<SRCPanelTreeNode>>& InSelectedEntities)
		: NodeId(InNodeId)
		, SelectedEntities(InSelectedEntities)
	{
		MouseCursor = EMouseCursor::GrabHandClosed;

		Algo::TransformIf(SelectedEntities, SelectedFieldsIds,
			[] (const TSharedPtr<SRCPanelTreeNode>& InNode)
			{
				return InNode->GetRCType() == SRCPanelTreeNode::Field;
			},
			[] (const TSharedPtr<SRCPanelTreeNode>& InNode)
			{
				return InNode->GetRCId();
			});
		
		DecoratorWidget = SNew(SBorder)
			.Padding(1.0f)
			.BorderImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.ExposedFieldBorder"))
			.Content()
			[
				InWidget.ToSharedRef()
			];
	}

	/** Get the ID of the represented entity or group. */
	FGuid GetNodeId() const
	{
		return NodeId;
	}

	/** Get the IDs that were selected at the time the drag started. */
	const TArray<TSharedPtr<SRCPanelTreeNode>>& GetSelectedEntities() const
	{
		return SelectedEntities;
	}

	/** Get the IDs that were selected at the time the drag started. */
	const TArray<FGuid>& GetSelectedFieldsId() const
	{
		return SelectedFieldsIds;
	}

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override
	{
		FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	}

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

private:
	/** ID of the represented entity or group. */
	FGuid NodeId;
	/** Field IDs that were selected at the time the drag started. */
    TArray<FGuid> SelectedFieldsIds;
	/** Entities that were selected at the time the drag started. */
	TArray<TSharedPtr<SRCPanelTreeNode>> SelectedEntities;
	/** Decorator Drag and Drop widget. */
	TSharedPtr<SWidget> DecoratorWidget;
};
