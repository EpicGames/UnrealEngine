// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelExposedField.h"

#include "Algo/Transform.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Components/MeshComponent.h"
#include "Components/SceneComponent.h"
#include "DetailTreeNode.h"
#include "DetailWidgetRow.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorFontGlyphs.h"
#include "Factories/IRCDefaultValueFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "IDetailTreeNode.h"
#include "IRCProtocolBindingList.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlProtocolModule.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "IRemoteControlUIModule.h"
#include "Layout/Visibility.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "RCPanelWidgetRegistry.h"
#include "RemoteControlField.h"
#include "RemoteControlPanelStyle.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "RemoteControlUIModule.h"
#include "SRCPanelTreeNode.h"
#include "SResetToDefaultPropertyEditor.h"
#include "StructSerializer.h"
#include "Styling/AppStyle.h"
#include "Styling/RemoteControlStyles.h"
#include "Styling/SlateBrush.h"
#include "UI/IRCPanelExposedEntityWidgetFactory.h"
#include "UnrealEdGlobals.h"
#include "UObject/Object.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Masks/SRCProtocolMask.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanel"

namespace ExposedFieldUtils
{
	void CreateWidgets(const TSharedPtr<IDetailTreeNode>& Node, TSharedPtr<SWidget>& OutValueWidget, TSharedPtr<SWidget>& OutEditConditionWidget)
	{
		FNodeWidgets NodeWidgets = Node->CreateNodeWidgets();

		TSharedRef<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);

		if (NodeWidgets.ValueWidget)
		{
			FieldWidget->AddSlot()
				.Padding(FMargin(3.0f, 2.0f))
				.HAlign(HAlign_Left)
				[
					NodeWidgets.ValueWidget.ToSharedRef()
				];
		}
		else if (NodeWidgets.WholeRowWidget)
		{
			FieldWidget->AddSlot()
				.Padding(FMargin(3.0f, 2.0f))
				[
					NodeWidgets.WholeRowWidget.ToSharedRef()
				];
		}

		OutValueWidget = FieldWidget;
		OutEditConditionWidget = NodeWidgets.EditConditionWidget;
	}

	bool IsFieldPathMatchingWithNodePath(const TSharedPtr<IDetailTreeNode>& InNode, const TSharedPtr<FRemoteControlField>& InField)
	{
		if (!InField.IsValid())
		{
			// if not able to check it by default is true
			return true;
		}
		TSharedPtr<IPropertyHandle> ParentHandle = InNode->CreatePropertyHandle();
		const FStringView ParentPath = ParentHandle->GetPropertyPath();
		FString PropPath = InField->FieldPathInfo.ToPathPropertyString();
		PropPath = PropPath.Replace(TEXT("."), TEXT("->"));
		const FName ParentPathName = FName(ParentPath);
		return ParentPathName == FName(PropPath);
	}

	bool TryCreateWidgetForChildProperty(const TSharedPtr<IDetailTreeNode>& InNode, const TSharedPtr<FRemoteControlField>& InField, TSharedPtr<IPropertyHandle>& OutChildHandle, TSharedPtr<SWidget>& OutValueWidget, TSharedPtr<SWidget>& OutEditConditionWidget)
	{
		const TSharedPtr<IPropertyHandle> ParentHandle = InNode->CreatePropertyHandle();
		uint32 ChildNum;
		ParentHandle->GetNumChildren(ChildNum);
		const FStringView ParentPath = ParentHandle->GetPropertyPath();
		FString FieldPath = InField->FieldPathInfo.ToPathPropertyString();
		FieldPath = FieldPath.Replace(TEXT("."), TEXT("->"));
		const FName FieldPathName = FName(FieldPath);
		const FName ParentPathName = FName(ParentPath);
		// Need to check if it is a child of the location to update the arrows widget position in the scene when moving
		const bool bIsLocation = ParentPathName == USceneComponent::GetRelativeLocationPropertyName() || ParentPathName == USceneComponent::GetAbsoluteLocationPropertyName();

		for (uint32 ChildHandleIndex = 0; ChildHandleIndex < ChildNum; ++ChildHandleIndex)
		{
			const TSharedPtr<IPropertyHandle> ChildHandle = ParentHandle->GetChildHandle(ChildHandleIndex);
			const FName ChildPathName = FName(ChildHandle->GetPropertyPath());
			if (ChildPathName == FieldPathName)
			{
				ExposedFieldUtils::CreateWidgets(InNode, OutValueWidget, OutEditConditionWidget);
				OutChildHandle = ChildHandle;
				if (bIsLocation)
				{
					OutChildHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([]()
					{
						GUnrealEd->UpdatePivotLocationForSelection();
					}));
				}
				OutValueWidget = OutChildHandle->CreatePropertyValueWidget();
				return true;
			}
		}
		return false;
	}
}

TSharedPtr<SRCPanelTreeNode> SRCPanelExposedField::MakeInstance(const FGenerateWidgetArgs& Args)
{
	return SNew(SRCPanelExposedField, StaticCastSharedPtr<FRemoteControlField>(Args.Entity), Args.ColumnSizeData, Args.WidgetRegistry)
		.Preset(Args.Preset)
		.LiveMode(Args.bIsInLiveMode)
		.HighlightText(Args.HighlightText)
		.CommandList(Args.CommandList);
}

void SRCPanelExposedField::Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> InField, FRCColumnSizeData InColumnSizeData, TWeakPtr<FRCPanelWidgetRegistry> InWidgetRegistry)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedField::Construct);

	WeakField = MoveTemp(InField);

	ColumnSizeData = MoveTemp(InColumnSizeData);
	WidgetRegistry = MoveTemp(InWidgetRegistry);

	HighlightText = InArgs._HighlightText;

	ResetButtonWidget = SNullWidget::NullWidget;

	if (TSharedPtr<FRemoteControlField> FieldPtr = WeakField.Pin())
	{
		FInitParams InitParams;
		InitParams.Preset = InArgs._Preset.Get();
		InitParams.EntityId = FieldPtr->GetId();
		InitParams.bLiveMode = InArgs._LiveMode;
		InitParams.CommandList = InArgs._CommandList;
		Initialize(InitParams);

		CachedLabel = FieldPtr->GetLabel();
		PropertyIdLabel = FieldPtr->PropertyId;
		EntityId = FieldPtr->GetId();

		if (FieldPtr->FieldType == EExposedFieldType::Property)
		{
			ConstructPropertyWidget();
		}
		else
		{
			ConstructFunctionWidget();
		}
	}
}

void SRCPanelExposedField::GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const
{
	OutChildren.Append(ChildWidgets);
}

SRCPanelTreeNode::ENodeType SRCPanelExposedField::GetRCType() const
{
	return SRCPanelTreeNode::Field;
}

void SRCPanelExposedField::FocusPropertyIdWidget() const
{
	if (GetFieldType() == EExposedFieldType::Property && PropertyIdWidget.IsValid())
	{
		FSlateApplication::Get().SetUserFocus(0, PropertyIdWidget);
	}
}

bool SRCPanelExposedField::HasChildren() const
{
	return ChildWidgets.Num() > 0;
}

FName SRCPanelExposedField::GetFieldLabel() const
{
	return CachedLabel;
}

FText SRCPanelExposedField::GetOwnerDisplayName() const
{
	return CachedOwnerDisplayName;
}

FName SRCPanelExposedField::GetOwnerPathName() const
{
	return CachedOwnerPathName;
}

SRCPanelTreeNode::FMakeNodeWidgetArgs SRCPanelExposedField::CreateEntityWidgetInternal(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget, const FText& OptionalWarningMessage, TSharedRef<SWidget> EditConditionWidget)
{
	FMakeNodeWidgetArgs Args = SRCPanelExposedEntity::CreateEntityWidgetInternal(ValueWidget, ResetWidget, OptionalWarningMessage, EditConditionWidget);

	// TODO: Add support for function PropertyId
	if (GetFieldType() == EExposedFieldType::Property)
	{
		Args.PropertyIdWidget = SNew(SBox)
			[
				SAssignNew(PropertyIdWidget, SEditableTextBox)
				.MinDesiredWidth(50.f)
				.SelectAllTextWhenFocused(true)
				.RevertTextOnEscape(true)
				.ClearKeyboardFocusOnCommit(true)
				.Text_Lambda([this] () { return GetPropertyIdText(); })
				.OnTextCommitted(this, &SRCPanelExposedField::OnPropertyIdTextCommitted)
			];
	}

	return Args;
}

EExposedFieldType SRCPanelExposedField::GetFieldType() const
{
	if (TSharedPtr<FRemoteControlField> Field = WeakField.Pin())
	{
		return Field->FieldType;
	}

	return EExposedFieldType::Invalid;
}

void SRCPanelExposedField::SetIsHovered(bool bInIsHovered)
{
	bIsHovered = bInIsHovered;
}

TSharedRef<SWidget> SRCPanelExposedField::GetProtocolWidget(const FName ForColumnName, const FName InProtocolName)
{
	const TSharedPtr<FRemoteControlField> RCField = WeakField.Pin();
	const TSharedPtr<FRemoteControlProperty> RCProperty = RCField.IsValid() ? StaticCastSharedPtr<FRemoteControlProperty>(RCField) : nullptr;
	if (!RCProperty.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	// If a custom widget factory for this protocol and column exists, let it create the widget.
	const FRemoteControlUIModule& RemoteControlUIModule = FModuleManager::GetModuleChecked<FRemoteControlUIModule>("RemoteControlUI");
	const TSharedRef<IRCPanelExposedEntityWidgetFactory>* CustomFactoryPtr = RemoteControlUIModule.GetExposedEntityWidgetFactory(ForColumnName, InProtocolName);
	if (CustomFactoryPtr)
	{
		const FRCPanelExposedPropertyWidgetArgs Args(
			Preset,
			RCProperty.ToSharedRef()
		);

		return (*CustomFactoryPtr)->MakePropertyWidget(Args);
	}
	
	for (const FRemoteControlProtocolBinding& RCProtocolIter : RCProperty->ProtocolBindings)
	{
		if (RCProtocolIter.GetProtocolName() == InProtocolName)
		{
			if (TSharedPtr<TStructOnScope<FRemoteControlProtocolEntity>> RCProtocolEntityPtr = RCProtocolIter.GetRemoteControlProtocolEntityPtr())
			{
				// Create a binding status widget
				if (ForColumnName == RemoteControlPresetColumns::BindingStatus)
				{
					TSharedPtr<SWidget> BindingStatusWidget = SNew(SButton)
						.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
						.ToolTipText(LOCTEXT("RecordingButtonToolTip", "Status of the protocol entity binding"))
						.ForegroundColor(FSlateColor::UseForeground())
						.OnClicked_Lambda([RCProtocolEntityPtr]()
							{
								const ERCBindingStatus BindingStatus = (*RCProtocolEntityPtr)->ToggleBindingStatus();

								IRemoteControlProtocolWidgetsModule& RCProtocolWidgetsModule = IRemoteControlProtocolWidgetsModule::Get();

								if (TSharedPtr<IRCProtocolBindingList> RCProtocolBindingList = RCProtocolWidgetsModule.GetProtocolBindingList())
								{
									if (BindingStatus == ERCBindingStatus::Awaiting)
									{
										RCProtocolBindingList->OnStartRecording(RCProtocolEntityPtr);
									}
									else if (BindingStatus == ERCBindingStatus::Bound)
									{
										RCProtocolBindingList->OnStopRecording(RCProtocolEntityPtr);
									}
									else
									{
										checkNoEntry();
									}
								}

								return FReply::Handled();
							}
						)
						.Content()
						[
							SNew(SImage)
							.ColorAndOpacity_Lambda([RCProtocolEntityPtr]()
								{
									const ERCBindingStatus ActiveBindingStatus = (*RCProtocolEntityPtr)->GetBindingStatus();

									switch (ActiveBindingStatus)
									{
										case ERCBindingStatus::Awaiting:
											return FLinearColor::Red;
										case ERCBindingStatus::Bound:
											return FLinearColor::Green;
										case ERCBindingStatus::Unassigned:
											return FLinearColor::Gray;
										default:
											checkNoEntry();
									}

									return FLinearColor::Black;
								}
							)
							.Image(FAppStyle::Get().GetBrush(TEXT("Icons.FilledCircle")))
						];

					return BindingStatusWidget.ToSharedRef();
				}
				else if (TSharedPtr<FRCPanelWidgetRegistry> Registry = WidgetRegistry.Pin())
				{
					// Create a common property widget
					if (TSharedPtr<IDetailTreeNode> Node = Registry->GetStructTreeNode(RCProtocolEntityPtr, (*RCProtocolEntityPtr)->GetPropertyName(ForColumnName).ToString(), ERCFindNodeMethod::Name))
					{
						FNodeWidgets NodeWidgets = Node->CreateNodeWidgets();

						if (NodeWidgets.ValueWidget)
						{
							return NodeWidgets.ValueWidget.ToSharedRef();
						}
					}
				}
			}
		}
	}

	return SRCPanelTreeNode::GetProtocolWidget(ForColumnName, InProtocolName);
}

const bool SRCPanelExposedField::HasProtocolExtension() const
{
	return GetFieldType() == EExposedFieldType::Property;
}

const bool SRCPanelExposedField::GetProtocolBindingsNum() const
{
	IRemoteControlProtocolModule& RCProtocolModule = IRemoteControlProtocolModule::Get();

	const TArray<FName>& AvailableProtocols = RCProtocolModule.GetProtocolNames();

	if (AvailableProtocols.Num() < 2 || AvailableProtocols.IsEmpty())
	{
		return false;
	}

	const TArray<FName>& SupportedProtocols = AvailableProtocols.FilterByPredicate([&](const FName& ThisProtocol) { return SupportsProtocol(ThisProtocol); });

	return SupportedProtocols.Num() >= 2;
}

const bool SRCPanelExposedField::SupportsProtocol(const FName& InProtocolName) const
{
	if (TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		if (const TSharedPtr<FRemoteControlProperty> RCProperty = StaticCastSharedPtr<FRemoteControlProperty>(RCField))
		{
			if (RCProperty->ProtocolBindings.Num())
			{
				for (TSet<FRemoteControlProtocolBinding>::TConstIterator RCProtocolIter(RCProperty->ProtocolBindings); RCProtocolIter; ++RCProtocolIter)
				{
					if (RCProtocolIter->GetProtocolName() == InProtocolName)
					{
						return true;
					}
				}
			}
		}
	}

	return false;
}

void SRCPanelExposedField::Refresh()
{
	SRCPanelExposedEntity::Refresh();

	if (TSharedPtr<FRemoteControlField> Field = WeakField.Pin())
	{
		CachedLabel = Field->GetLabel();

		if (Field->FieldType == EExposedFieldType::Property)
		{
			ConstructPropertyWidget();
		}
		else if (Field->FieldType == EExposedFieldType::Function)
		{
			ConstructFunctionWidget();
		}
	}
}

TSharedRef<SWidget> SRCPanelExposedField::GetWidget(const FName ForColumnName, const FName InActiveProtocol)
{
	if (HasProtocolExtension())
	{
		if (ForColumnName == RemoteControlPresetColumns::Status)
		{
			return SNew(STextBlock)
				.Text(LOCTEXT("StatusText", "!"))
				.TextStyle(&RCPanelStyle->HeaderTextStyle)
				.ToolTipText_Lambda([this]() { return GetProtocolBindingsNum() ? LOCTEXT("StatusTooltip", "Entity is bound to two or more protocols.") : FText::GetEmpty(); })
				.Visibility_Lambda([this]() { return GetProtocolBindingsNum() ? EVisibility::Visible : EVisibility::Collapsed; });
		}
		else if (!DefaultColumns.Contains(ForColumnName))
		{
			return GetProtocolWidget(ForColumnName, InActiveProtocol);
		}
	}

	if (ForColumnName == RemoteControlPresetColumns::Reset)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SRCPanelTreeNode::GetWidget(ForColumnName, InActiveProtocol)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				CreateInUseWidget()
			];
	}

	return SRCPanelTreeNode::GetWidget(ForColumnName, InActiveProtocol);
}

void SRCPanelExposedField::GetBoundObjects(TSet<UObject*>& OutBoundObjects) const
{
	if (TSharedPtr<FRemoteControlField> Field = WeakField.Pin())
	{
		OutBoundObjects.Append(Field->GetBoundObjects());
	}
}

TSharedRef<SWidget> SRCPanelExposedField::ConstructWidget()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedField::ConstructWidget);

	if (TSharedPtr<FRemoteControlField> Field = WeakField.Pin())
	{
		// For the moment, just use the first object.
		if (const URemoteControlPreset* RCPreset = Preset.Get();
			RCPreset && RCPreset->SelectedWorld.IsValid())
		{
			UObject* Object = Field->GetBoundObjectForWorld(RCPreset->SelectedWorld.Get());
			if (Object && GetFieldType() == EExposedFieldType::Property)
			{
				if (TSharedPtr<FRCPanelWidgetRegistry> Registry = WidgetRegistry.Pin())
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedField::BeforeGetObjectTreeNode);

					if (TSharedPtr<IDetailTreeNode> Node = Registry->GetObjectTreeNode(Object, Field->FieldPathInfo.ToPathPropertyString(), ERCFindNodeMethod::Path))
					{
						TSharedPtr<SWidget> ValueWidget = SNullWidget::NullWidget;
						TSharedPtr<SWidget> EditConditionWidget = SNullWidget::NullWidget;
						TSharedPtr<IPropertyHandle> PropertyHandle;
						TOptional<FResetToDefaultOverride> ResetToDefaultOverride;
						bool bIsChildPropertyWidgetCreated = false;

						if (!ExposedFieldUtils::IsFieldPathMatchingWithNodePath(Node, Field))
						{
							bIsChildPropertyWidgetCreated = ExposedFieldUtils::TryCreateWidgetForChildProperty(Node, Field, PropertyHandle, ValueWidget, EditConditionWidget);
						}

						if (!bIsChildPropertyWidgetCreated)
						{
							TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
							Node->GetChildren(ChildNodes);
							ChildWidgets.Reset(ChildNodes.Num());

							for (const TSharedRef<IDetailTreeNode>& ChildNode : ChildNodes)
							{
								ChildWidgets.Add(SNew(SRCPanelFieldChildNode, ChildNode, ColumnSizeData));
							}

							ExposedFieldUtils::CreateWidgets(Node, ValueWidget, EditConditionWidget);

							PropertyHandle = Node->CreatePropertyHandle();

							// This is doing similar operations to what IDetailTreeNode::CreateNodeWidgets already does (via ExposedFieldUtils::CreateWidgets)
							// But this accesses the widget row that is not exposed by IDetailTreeNode::CreateNodeWidgets.
							// Widget Row is needed to access custom behavior like custom reset to default.
							FDetailWidgetRow WidgetRow;
							TSharedPtr<FDetailTreeNode> DetailTreeNode = StaticCastSharedPtr<FDetailTreeNode>(Node);
							DetailTreeNode->GenerateStandaloneWidget(WidgetRow);
							ResetToDefaultOverride = MoveTemp(WidgetRow.CustomResetToDefault);
						}

						TSharedPtr<SHorizontalBox> FieldWidget = SNew(SHorizontalBox);
						FieldWidget->AddSlot()
						[
							ValueWidget.ToSharedRef()
						];
						if (PropertyHandle && PropertyHandle->IsValidHandle() && (PropertyHandle->GetParentHandle()->GetPropertyDisplayName().ToString() == FString("Transform")
							|| PropertyHandle->GetOuterBaseClass() == UMaterialInstanceDynamic::StaticClass()))
						{
							// Set up a Zeroed DefaultValue, in case an ExposedEntity doesn't have a native Default. Needed for certain ResetToDefault cases.
							void* ValuePtr;
							if (PropertyHandle->GetValueData(ValuePtr) == FPropertyAccess::Result::Success)
							{
								DefaultValue.Reset(new uint8[PropertyHandle->GetProperty()->GetSize()]);
								PropertyHandle->GetProperty()->CopyCompleteValue(DefaultValue.Get(), ValuePtr);
								PropertyHandle->GetProperty()->ClearValue(DefaultValue.Get());
							}

							TWeakPtr<SRCPanelExposedField> WeakFieldPtr = SharedThis(this);
							auto IsVisible = [WeakFieldPtr, PropertyHandle]()
							{
								TSharedPtr<SRCPanelExposedField> FieldPtr = WeakFieldPtr.Pin();
								if (FieldPtr && PropertyHandle && PropertyHandle->IsValidHandle())
								{
									void* DataPtr;
									if (PropertyHandle->GetValueData(DataPtr) == FPropertyAccess::Result::Success)
									{
										FProperty* NodeProperty = PropertyHandle->GetProperty();
										bool bVisible = !NodeProperty->Identical(FieldPtr->DefaultValue.Get(), DataPtr);
										return bVisible ? EVisibility::Visible : EVisibility::Hidden;
									}
								}
								return EVisibility::Hidden;
							};

							ResetButtonWidget = SNew(SBox)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Padding(2.f, 4.f)
								[
									SNew(SButton)
									.IsFocusable(false)
									.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
									.ContentPadding(RCPanelStyle->PanelPadding)
									.Visibility_Lambda(IsVisible)
									.OnClicked_Lambda([PropertyHandle, ResetToDefaultOverride]()
									{
										if (ResetToDefaultOverride.IsSet())
										{
											ResetToDefaultOverride->OnResetToDefaultClicked(PropertyHandle);
										}
										else
										{
											PropertyHandle->ResetToDefault();
										}
										return FReply::Handled();
									})
									.Content()
									[
										SNew(SImage)
										.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
								];
						}
						else
						{
							ResetButtonWidget = SNew(SBox)
								.HAlign(HAlign_Center)
								.VAlign(VAlign_Center)
								.Padding(2.f, 4.f)
								[
									ConstructResetToDefaultWidget(Object, PropertyHandle)
								];
						}

						return MakeFieldWidget(FieldWidget.ToSharedRef(), EditConditionWidget.ToSharedRef());
					}
					else
					{
						FString PropertyName = Field->FieldPathInfo.ToPathPropertyString();
						FString TargetName = Object->GetName();

						FRCFieldPathInfo& FieldPath = Field->FieldPathInfo;

						if (FieldPath.Resolve(Object))
						{
							FRCFieldResolvedData ResolvedData = FieldPath.GetResolvedData();
							const FRCFieldPathSegment& LastSegment = FieldPath.GetFieldSegment(FieldPath.GetSegmentCount() - 1);
							if (LastSegment.ArrayIndex != INDEX_NONE)
							{
								if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(ResolvedData.Field))
								{
									FScriptArrayHelper_InContainer Helper(ArrayProperty, ResolvedData.ContainerAddress);
									int32 ArrayNum = Helper.Num();
									if (!Helper.IsValidIndex(LastSegment.ArrayIndex))
									{
										PropertyName = LastSegment.ToString();

										if (FieldPath.GetSegmentCount() > 1)
										{
											TargetName = Object->GetName() + TEXT(".") + FieldPath.ToString(FieldPath.GetSegmentCount() - 1);
										}
									}
								}
							}
						}

						FText ErrorText = FText::Format(LOCTEXT("ExposedPropertyInvalidErrorMessage", "Could not find property {0} on object {1}"), FText::FromString(PropertyName), FText::FromString(TargetName));
						IRemoteControlModule::BroadcastError(ErrorText.ToString());

						return MakeFieldWidget(CreateInvalidWidget(ErrorText));
					}
				}
			}
		}
		return MakeFieldWidget(CreateInvalidWidget());
	}

	return MakeFieldWidget(SNullWidget::NullWidget);
}

TSharedRef<SWidget> SRCPanelExposedField::MakeFieldWidget(const TSharedRef<SWidget>& InWidget, const TSharedRef<SWidget>& EditConditionWidget)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelExposedField::MakeFieldWidget);

	FText WarningMessage;

	if (const TSharedPtr<FRemoteControlField> RCField = GetRemoteControlField().Pin())
	{
		bool bIsEditorOnly = false,
            bIsEditableInPackaged = true,
            bIsCallableInPackaged = true,
			bIsEditableInEditor = true;

		FString ErrorTextInEditor;
		if (RCField->FieldType == EExposedFieldType::Property)
		{
			bIsEditableInEditor = StaticCastSharedPtr<FRemoteControlProperty>(RCField)->IsEditableInEditor(&ErrorTextInEditor);
		}

		FString ErrorTextInPackage;
		if (GetDefault<URemoteControlSettings>()->bDisplayInEditorOnlyWarnings)
		{
			bIsEditorOnly = RCField->IsEditorOnly();
			if (RCField->FieldType == EExposedFieldType::Property)
			{
				bIsEditableInPackaged = StaticCastSharedPtr<FRemoteControlProperty>(RCField)->IsEditableInPackaged(&ErrorTextInPackage);
			}
			else
			{
				bIsCallableInPackaged = StaticCastSharedPtr<FRemoteControlFunction>(RCField)->IsCallableInPackaged();
			}
		}

		FTextBuilder Builder;
		if (bIsEditorOnly)
		{
			Builder.AppendLine(LOCTEXT("EditorOnlyWarning", "This field will be unavailable in packaged projects."));
		}

		bool bEditorAndPackagedErrorEquals = false;
		if (!bIsEditableInPackaged)
		{
			if (!bIsEditableInEditor)
			{
				const FName PackagedWarning = FName(ErrorTextInPackage);
				const FName EditorWarning = FName(ErrorTextInEditor);
				if (PackagedWarning.IsEqual(EditorWarning))
				{
					Builder.AppendLine(LOCTEXT("NonEditableInPackagedAndEditorWarning", "Packaged and Editor:"));
					bEditorAndPackagedErrorEquals = true;
				}
			}

			if (!bEditorAndPackagedErrorEquals)
			{
				Builder.AppendLine(LOCTEXT("NonEditableInPackagedWarning", "Packaged:"));
			}

			Builder.AppendLine(ErrorTextInPackage);
		}

		if (!bIsEditableInEditor && !bEditorAndPackagedErrorEquals)
		{
			if (!bIsEditableInPackaged)
			{
				// put an empty line between the last error and this
				Builder.AppendLine();
			}

			Builder.AppendLine(LOCTEXT("NonEditableInEditorWarning", "Editor:"));
			Builder.AppendLine(ErrorTextInEditor);
		}

		if (!bIsCallableInPackaged)
		{
			Builder.AppendLine(LOCTEXT("NotCallableInPackagedWarning", "This function will not be callable in packaged projects."));
		}

		WarningMessage = Builder.ToText();
	}

	return CreateEntityWidget(InWidget, ResetButtonWidget.ToSharedRef(), WarningMessage, EditConditionWidget);
}

void SRCPanelExposedField::ConstructPropertyWidget()
{
	ChildSlot.AttachWidget(ConstructWidget());
}

void SRCPanelExposedField::ConstructFunctionWidget()
{
	TSharedPtr<SRCPanelExposedField> ExposedFieldWidget;

	URemoteControlPreset* RCPreset = Preset.Get();
	if (!RCPreset)
	{
		return;
	}

	if (const TSharedPtr<FRemoteControlFunction> RCFunction = RCPreset->GetExposedEntity<FRemoteControlFunction>(EntityId).Pin())
	{
		if (RCFunction->GetFunction() && RCFunction->GetBoundObjects().Num())
		{
			if (TSharedPtr<FRCPanelWidgetRegistry> Registry = WidgetRegistry.Pin())
			{
				Registry->Refresh(RCFunction->FunctionArguments);

				TArray<TSharedPtr<SRCPanelFieldChildNode>> ChildNodes;
				for (TFieldIterator<FProperty> It(RCFunction->GetFunction()); It; ++It)
				{
					const bool Param = It->HasAnyPropertyFlags(CPF_Parm);
					const bool OutParam = It->HasAnyPropertyFlags(CPF_OutParm) && !It->HasAnyPropertyFlags(CPF_ConstParm);
					const bool ReturnParam = It->HasAnyPropertyFlags(CPF_ReturnParm);

					if (!Param || OutParam || ReturnParam)
					{
						continue;
					}

					if (TSharedPtr<IDetailTreeNode> PropertyNode = Registry->GetStructTreeNode(RCFunction->FunctionArguments, It->GetFName().ToString(), ERCFindNodeMethod::Name))
					{
						ChildNodes.Add(SNew(SRCPanelFieldChildNode, PropertyNode.ToSharedRef(), ColumnSizeData));
					}
				}

				ChildSlot.AttachWidget(MakeFieldWidget(ConstructCallFunctionButton()));
				ChildWidgets = ChildNodes;
			}


			return;
		}
	}

	ChildSlot.AttachWidget(ConstructWidget());
}

TSharedRef<SWidget> SRCPanelExposedField::ConstructResetToDefaultWidget(UObject* InObject, TSharedPtr<IPropertyHandle> InPropertyHandle)
{
	TSharedPtr<SResetToDefaultPropertyEditor> ResetDefaultProperty = SNew(SResetToDefaultPropertyEditor, InPropertyHandle);

	const TAttribute<bool> CanResetToDefaultValue = TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateLambda([this, InObject, InPropertyHandle]()
		{
			IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();

			if (RemoteControlModule.HasDefaultValueCustomization(InObject, InPropertyHandle->GetProperty()))
			{
				FRCResetToDefaultArgs Args;
				Args.Property = InPropertyHandle->GetProperty();

				if (InPropertyHandle->IsValidHandle())
				{
					Args.Path = InPropertyHandle->GetPropertyPath();
					Args.ArrayIndex = InPropertyHandle->GetArrayIndex();
				}

				Args.bCreateTransaction = false;
				return RemoteControlModule.CanResetToDefaultValue(InObject, Args);
			}

			return InPropertyHandle->DiffersFromDefault();
		})
	);

	const FSimpleDelegate PerformValueReset = FSimpleDelegate::CreateLambda([this, InObject, InPropertyHandle]()
		{
			IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();

			if (RemoteControlModule.HasDefaultValueCustomization(InObject, InPropertyHandle->GetProperty()))
			{
				FRCResetToDefaultArgs Args;
				Args.Property = InPropertyHandle->GetProperty();

				if (InPropertyHandle->IsValidHandle())
				{
					Args.Path = InPropertyHandle->GetPropertyPath();
					Args.ArrayIndex = InPropertyHandle->GetArrayIndex();
				}

				Args.bCreateTransaction = true;
				RemoteControlModule.ResetToDefaultValue(InObject, Args);

				return;
			}

			InPropertyHandle->ResetToDefault();
		}
	);

	const FResetToDefaultOverride ResetValueOverride = FResetToDefaultOverride::Create(CanResetToDefaultValue, PerformValueReset);

	if (UMeshComponent* MeshComponent = Cast<UMeshComponent>(InObject))
	{
		if (InPropertyHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshComponent, OverrideMaterials) &&
			MeshComponent->OverrideMaterials[InPropertyHandle->GetArrayIndex()] == NULL)
		{
			IRemoteControlModule& RemoteControlModule = IRemoteControlModule::Get();

			FRCResetToDefaultArgs Args;
			Args.Property = InPropertyHandle->GetProperty();
			Args.Path = InPropertyHandle->GetPropertyPath();
			Args.ArrayIndex = InPropertyHandle->GetArrayIndex();

			Args.bCreateTransaction = false;
			RemoteControlModule.ResetToDefaultValue(InObject, Args);
		}

	}

	ResetDefaultProperty.Reset();

	ResetDefaultProperty = SNew(SResetToDefaultPropertyEditor, InPropertyHandle)
		.CustomResetToDefault(ResetValueOverride);

	return ResetDefaultProperty.ToSharedRef();
}

TSharedRef<SWidget> SRCPanelExposedField::ConstructCallFunctionButton(bool bIsEnabled)
{
	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.HAlign(HAlign_Left)
		.Padding(2.0f)
		.AutoHeight()
		[
			SNew(SButton)
			.IsEnabled(bIsEnabled)
			.VAlign(VAlign_Center)
			.OnClicked(this, &SRCPanelExposedField::OnClickFunctionButton)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CallFunctionLabel", "Call Function"))
			]
		];
}

FReply SRCPanelExposedField::OnClickFunctionButton()
{
	if (URemoteControlPreset* RCPreset = Preset.Get())
	{
		if (TSharedPtr<FRemoteControlFunction> Function = RCPreset->GetExposedEntity<FRemoteControlFunction>(EntityId).Pin())
		{
			for (UObject* Object : Function->GetBoundObjects())
			{
				if (Function->FunctionArguments && Function->FunctionArguments->IsValid())
				{
					FRCCallReference CallRef;
					CallRef.Object = Object;
					CallRef.Function = Function->GetFunction();

					FRCCall Call;
					Call.CallRef = MoveTemp(CallRef);
					Call.TransactionMode = ERCTransactionMode::AUTOMATIC;
					Call.ParamStruct = FStructOnScope(Function->FunctionArguments->GetStruct(), Function->FunctionArguments->GetStructMemory());

					TArray<uint8> FunctionPayload;
					FMemoryWriter Writer(FunctionPayload);
					FCborStructSerializerBackend SerializerBackend{ Writer, EStructSerializerBackendFlags::Default };
					FStructSerializer::Serialize(Function->FunctionArguments->GetStructMemory(), *const_cast<UStruct*>(Function->FunctionArguments->GetStruct()), SerializerBackend);
					
					IRemoteControlModule::Get().InvokeCall(Call, ERCPayloadType::Cbor, FunctionPayload);
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("Function default arguments could not be resolved."));
				}
			}
		}
	}

	return FReply::Handled();
}

FText SRCPanelExposedField::GetPropertyIdText()
{
	if (const TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		if (RCField->PropertyId != PropertyIdLabel)
		{
			PropertyIdLabel = RCField->PropertyId;
		}
	}
	return FText::FromName(PropertyIdLabel);
}

void SRCPanelExposedField::OnPropertyIdTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo)
{
	if (const URemoteControlPreset* RCPreset = Preset.Get())
	{
		if (const TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
		{
			const FName NewId = FName(InText.ToString());
			if (RCField->PropertyId.Compare(NewId) != 0)
			{
				RCField->PropertyId = NewId;
				PropertyIdLabel = NewId;
				RCPreset->UpdateIdentifiedField(RCField.ToSharedRef());
				OnPropertyIdRenamed().ExecuteIfBound(NewId);
			}
		}
	}
}

TSharedRef<SWidget> SRCPanelExposedField::CreateInUseWidget()
{
	if (const TSharedPtr<FRemoteControlField> RCField = WeakField.Pin())
	{
		return SNew(SImage)
			.DesiredSizeOverride(FVector2D(12.f))
			.Image_Lambda(
				[ThisWeak = AsWeak()]
				{
					if (TSharedPtr<SRCPanelExposedField> This = StaticCastSharedPtr<SRCPanelExposedField>(ThisWeak.Pin()))
					{
						if (This->GetBoundCount() > 1)
						{
							return FAppStyle::Get().GetBrush("Icons.WarningWithColor");
						}
					}

					return FAppStyle::Get().GetBrush("Icons.Link");
				}
			)
			.ToolTipText_Lambda(
				[ThisWeak = AsWeak()]
				{
					if (TSharedPtr<SRCPanelExposedField> This = StaticCastSharedPtr<SRCPanelExposedField>(ThisWeak.Pin()))
					{
						if (This->GetBoundCount() > 1)
						{
							return LOCTEXT("PropertyBoundMultiple", "WARNING: Property is bound to multiple behaviors!");
						}
					}

					return LOCTEXT("PropertyBound", "Property is bound to a controller.");
				}
			)
			.Visibility_Lambda(
				[ThisWeak = AsWeak()]
				{
					if (TSharedPtr<SRCPanelExposedField> This = StaticCastSharedPtr<SRCPanelExposedField>(ThisWeak.Pin()))
					{
						if (This->GetBoundCount() > 0)
						{
							return EVisibility::Visible;
						}
					}

					return EVisibility::Hidden;
				}
			);
	}

	return SNullWidget::NullWidget;
}

int32 SRCPanelExposedField::GetBoundCount() const
{
	if (TSharedPtr<FRemoteControlField> Field = WeakField.Pin())
	{
		if (URemoteControlPreset* NodePreset = Field->GetOwner())
		{
			const FGuid Id = Field->GetId();

			if (Id.IsValid())
			{
				return NodePreset->GetEntityBoundCount(Id);
			}
		}
	}

	return 0;
}

void SRCPanelFieldChildNode::Construct(const FArguments& InArgs, const TSharedRef<IDetailTreeNode>& InNode, FRCColumnSizeData InColumnSizeData)
{
	TArray<TSharedRef<IDetailTreeNode>> ChildNodes;
	InNode->GetChildren(ChildNodes);

	Algo::Transform(ChildNodes, ChildrenNodes, [InColumnSizeData](const TSharedRef<IDetailTreeNode>& ChildNode) { return SNew(SRCPanelFieldChildNode, ChildNode, InColumnSizeData); });

	ColumnSizeData = InColumnSizeData;

	FNodeWidgets Widgets = InNode->CreateNodeWidgets();
	FMakeNodeWidgetArgs Args;
	Args.NameWidget = Widgets.NameWidget;
	TSharedPtr<SWidget> EditConditionWidget;
	ExposedFieldUtils::CreateWidgets(InNode, Args.ValueWidget, EditConditionWidget);
	ChildSlot
	[
		MakeNodeWidget(Args)
	];
}

#undef LOCTEXT_NAMESPACE /*RemoteControlPanel*/
