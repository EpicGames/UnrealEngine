// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RemoteControlField.h"
#include "SRCPanelExposedEntity.h"
#include "SlateFwd.h"
#include "SResetToDefaultPropertyEditor.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

struct EVisibility;
enum class EExposedFieldType : uint8;
struct FGenerateWidgetArgs;
struct FSlateBrush;
class FRCPanelWidgetRegistry;
struct FRemoteControlField;
struct FGuid;
class IDetailTreeNode;
class SInlineEditableTextBlock;
struct SRCPanelFieldChildNode;
class URemoteControlPreset;
class SWidget;

/**
 * Widget that displays an exposed field.
 */
struct SRCPanelExposedField : public SRCPanelExposedEntity
{
	SLATE_BEGIN_ARGS(SRCPanelExposedField)
		: _LiveMode(false)
		, _HighlightText()
		, _Preset(nullptr)
	{}
		SLATE_ATTRIBUTE(bool, LiveMode)
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_ATTRIBUTE(URemoteControlPreset*, Preset)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	static TSharedPtr<SRCPanelTreeNode> MakeInstance(const FGenerateWidgetArgs& Args);

	void Construct(const FArguments& InArgs, TWeakPtr<FRemoteControlField> Field, FRCColumnSizeData ColumnSizeData, TWeakPtr<FRCPanelWidgetRegistry> InWidgetRegistry);

	//~ SRCPanelTreeNode Interface 
	virtual void GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const override;
	virtual ENodeType GetRCType() const override;
	virtual void FocusPropertyIdWidget() const override;
	virtual bool HasChildren() const override;
	virtual void Refresh() override;
	virtual TSharedRef<SWidget> GetWidget(const FName ForColumnName, const FName InActiveProtocol) override;
	//~ End SRCPanelTreeNode Interface

	//~ SRCPanelExposedEntity Interface
	virtual void SetIsHovered(bool bIsBeingHovered) override;
	//~ End SRCPanelExposedEntity Interface

	//~ BEGIN : IHasProtocolExtensibility Interface
	virtual TSharedRef<SWidget> GetProtocolWidget(const FName ForColumnName, const FName InProtocolName = NAME_None) override;
	virtual const bool HasProtocolExtension() const override;
	virtual const bool GetProtocolBindingsNum() const override;
	virtual const bool SupportsProtocol(const FName& InProtocolName) const override;
	//~ END : IHasProtocolExtensibility Interface

	/** Get a weak pointer to the underlying remote control field. */
	TWeakPtr<FRemoteControlField> GetRemoteControlField() const { return WeakField; }

	/** Get this field's label. */
	FName GetFieldLabel() const;

	/** Get this field's type. */
	EExposedFieldType GetFieldType() const;

	/** Returns this widget's underlying objects. */
	void GetBoundObjects(TSet<UObject*>& OutBoundObjects) const;

	/** Get the owner display name of this field */
	FText GetOwnerDisplayName() const;

	/** Get the owner path name of this field */
	FName GetOwnerPathName() const;

protected:
	/** Returns populated args to display this widget. */
	virtual FMakeNodeWidgetArgs CreateEntityWidgetInternal(TSharedPtr<SWidget> ValueWidget, TSharedPtr<SWidget> ResetWidget = SNullWidget::NullWidget, const FText& OptionalWarningMessage = FText::GetEmpty(), TSharedRef<SWidget> EditConditionWidget = SNullWidget::NullWidget) override;

private:
	/** Construct a property widget. */
	TSharedRef<SWidget> ConstructWidget();
	/** Create the wrapper around the field value widget. */
	TSharedRef<SWidget> MakeFieldWidget(const TSharedRef<SWidget>& InWidget, const TSharedRef<SWidget>& EditConditionWidget = SNullWidget::NullWidget);
	/** Construct this field widget as a property widget. */
	void ConstructPropertyWidget();
	/** Construct this field widget as a function widget. */
	void ConstructFunctionWidget();
	/** Construct the ResetToDefault Widget that can have CustomReset for certain Property (Right now only Light Intensity) */
	TSharedRef<SWidget> ConstructResetToDefaultWidget(UObject* InObject, TSharedPtr<IPropertyHandle> InPropertyHandle);
	/**
	 * Construct a call function button
	 * @param bIsEnabled Whether the button should be clickable or not.
	 * @return The constructed widget.
	 */
	TSharedRef<SWidget> ConstructCallFunctionButton(bool bIsEnabled = true);
	/** Handles calling an exposed function.*/
	FReply OnClickFunctionButton();

	/** Return the text to be displayed in the PropertyId column */
	FText GetPropertyIdText();
	/** Handles committing a property id label. */
	void OnPropertyIdTextCommitted(const FText& InText, ETextCommit::Type InCommitInfo);

	/** Creates a widget which says whether this field is bound to a controller. */
	TSharedRef<SWidget> CreateInUseWidget();

	/** Returns the number of behaviors bound to this field. */
	int32 GetBoundCount() const;

private:
	/** Weak pointer to the underlying RC Field. */
	TWeakPtr<FRemoteControlField> WeakField;
	/** Whether the widget is currently hovered by a drag and drop operation. */
	bool bIsHovered = false;
	/** This exposed field's child widgets (ie. An array's rows) */
	TArray<TSharedPtr<SRCPanelFieldChildNode>> ChildWidgets;
	/** Holds the panel's cached widgets. */
	TWeakPtr<FRCPanelWidgetRegistry> WidgetRegistry;
	/** Holds the zeroed Default Value of the ExposedField */
	TUniquePtr<uint8[]> DefaultValue;
	/** Holds the shared reference of reset button for this field. */
	TSharedPtr<SWidget> ResetButtonWidget;
	/** Holds the SharedPtr of the PropertyId widget of this field. */
	TSharedPtr<SWidget> PropertyIdWidget;
};


/** Represents a child of an exposed field widget. */
struct SRCPanelFieldChildNode : public SRCPanelTreeNode
{
	SLATE_BEGIN_ARGS(SRCPanelFieldChildNode)
	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<IDetailTreeNode>& InNode, FRCColumnSizeData InColumnSizeData);
	virtual void GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const override { return OutChildren.Append(ChildrenNodes); }
	virtual FGuid GetRCId() const override { return FGuid(); }
	virtual ENodeType GetRCType() const override { return SRCPanelTreeNode::FieldChild; }

	TArray<TSharedPtr<SRCPanelFieldChildNode>> ChildrenNodes;
};
