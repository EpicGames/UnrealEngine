// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelFieldGroup.h"

#include "Commands/RemoteControlCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "IRemoteControlModule.h"
#include "RemoteControlPanelStyle.h"
#include "RemoteControlPreset.h"
#include "SDropTarget.h"
#include "SRCPanelDragHandle.h"
#include "SRCPanelExposedField.h"
#include "SRCPanelTreeNode.h"
#include "SRemoteControlPanel.h"
#include "ScopedTransaction.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateColor.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanel"

void SRCPanelGroup::Tick(const FGeometry&, const double, const float)
{
	if (bNeedsRename)
	{
		if (NameTextBox)
		{
			NameTextBox->EnterEditingMode();
		}
		bNeedsRename = false;
	}
}

void SRCPanelGroup::Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, FRCColumnSizeData InColumnSizeData)
{
	CommandList = MakeShared<FUICommandList>();

	if (InArgs._CommandList.IsValid())
	{
		CommandList->Append(InArgs._CommandList.ToSharedRef());
	}

	Preset = InPreset;
	
	Id = InArgs._Id;
	Name = InArgs._Name;
	Nodes = InArgs._Children;
	ColumnSizeData = MoveTemp(InColumnSizeData);
	
	OnFieldDropEvent = InArgs._OnFieldDropEvent;
	OnGetGroupId = InArgs._OnGetGroupId;
	OnDeleteGroup = InArgs._OnDeleteGroup;
	bLiveMode = InArgs._LiveMode;

	TSharedRef<SWidget> LeftColumn = 
		SNew(SHorizontalBox)
		// Color band
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Center)
		.Padding(0.f, 0.f, 2.f, 0.f)
		.AutoWidth()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("WhiteBrush"))
			.BorderBackgroundColor(this, &SRCPanelGroup::GetGroupColor)
		]
		// Drag and drop handle
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Fill)
		.HAlign(HAlign_Center)
		.Padding(0.f, 4.f)
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(2.f, 0.f)
			.Visibility(this, &SRCPanelGroup::GetVisibilityAccordingToLiveMode, EVisibility::Collapsed)
			[
				SNew(SRCPanelDragHandle<FFieldGroupDragDropOp>, Id)
				.Widget(SharedThis(this))
			]
		]
		+ SHorizontalBox::Slot()
		// Group name
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		.Padding(6.f, 4.f)
		.AutoWidth()
		[
			SAssignNew(NameTextBox, SInlineEditableTextBlock)
			.ColorAndOpacity(this, &SRCPanelGroup::GetGroupNameTextColor)
			.Font(FAppStyle::Get().GetFontStyle("PropertyWindow.BoldFont"))
			.Text(FText::FromName(Name))
			.OnTextCommitted(this, &SRCPanelGroup::OnLabelCommitted)
			.OnVerifyTextChanged(this, &SRCPanelGroup::OnVerifyItemLabelChanged)
			.IsReadOnly_Lambda([this]() { return bLiveMode.Get(); })
		];

	ChildSlot
	[
		SNew(SBorder)
		.Padding(0.f)
		.BorderImage(this, &SRCPanelGroup::GetBorderImage)
		.VAlign(VAlign_Fill)
		[
			SNew(SBox)
			.HeightOverride(32.f)
			[
				SNew(SDropTarget)
				.VerticalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.VerticalDash"))
				.HorizontalImage(FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.HorizontalDash"))
				.OnDropped_Lambda([this](const FGeometry& InGeometry, const FDragDropEvent& InDragDropEvent) { return OnFieldDropGroup(InDragDropEvent.GetOperation(), nullptr); })
				.OnAllowDrop(this, &SRCPanelGroup::OnAllowDropFromOtherGroup)
				.OnIsRecognized(this, &SRCPanelGroup::OnAllowDropFromOtherGroup)
				[
					LeftColumn
				]
			]
		]
	];
}

FName SRCPanelGroup::GetGroupName() const
{
	return Name;
}

void SRCPanelGroup::SetName(FName InName)
{
	Name = InName;
	NameTextBox->SetText(FText::FromName(Name));
}

void SRCPanelGroup::EnterRenameMode()
{
	bNeedsRename = true;
}

void SRCPanelGroup::GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const
{
	OutChildren.Append(Nodes);
}

TSharedPtr<SWidget> SRCPanelGroup::GetContextMenu()
{
	FMenuBuilder MenuBuilder(/*bShouldCloseWindowAfterMenuSelection*/true, CommandList);

	MenuBuilder.BeginSection("Common");

	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().RenameEntity, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Rename")));
	MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().DeleteEntity, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GenericCommands.Delete")));

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FGuid SRCPanelGroup::GetRCId() const
{
	return Id;
}

SRCPanelTreeNode::ENodeType SRCPanelGroup::GetRCType() const
{
	return SRCPanelTreeNode::Group;
}

FReply SRCPanelGroup::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	check(Preset.IsValid());

	if (Preset->Layout.IsDefaultGroup(Id))
	{
		return SCompoundWidget::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
	}

	return SRCPanelTreeNode::OnMouseButtonDoubleClick(InMyGeometry, InMouseEvent);
}

FReply SRCPanelGroup::OnFieldDropGroup(const FDragDropEvent& Event, TSharedPtr<SRCPanelTreeNode> TargetField)
{
	if (const TSharedPtr<FExposedEntityDragDrop> DragDropOp = Event.GetOperationAs<FExposedEntityDragDrop>())
	{
		return OnFieldDropGroup(DragDropOp, TargetField);
	}
	return FReply::Unhandled();
}

FReply SRCPanelGroup::OnFieldDropGroup(TSharedPtr<FDragDropOperation> DragDropOperation, TSharedPtr<SRCPanelTreeNode> TargetField)
{
	if (DragDropOperation)
	{
		if (DragDropOperation->IsOfType<FExposedEntityDragDrop>() && OnFieldDropEvent.IsBound())
		{
			return OnFieldDropEvent.Execute(DragDropOperation, TargetField, StaticCastSharedRef<SRCPanelGroup>(AsShared()));
		}
		else if (DragDropOperation->IsOfType<FFieldGroupDragDropOp>() && OnFieldDropEvent.IsBound())
		{
			return OnFieldDropEvent.Execute(DragDropOperation, nullptr, StaticCastSharedRef<SRCPanelGroup>(AsShared()));
		}
	}

	return FReply::Unhandled();
}

bool SRCPanelGroup::OnAllowDropFromOtherGroup(TSharedPtr<FDragDropOperation> DragDropOperation)
{
	if (DragDropOperation->IsOfType<FExposedEntityDragDrop>())
	{
		if (TSharedPtr<FExposedEntityDragDrop> DragDropOp = StaticCastSharedPtr<FExposedEntityDragDrop>(DragDropOperation))
		{
			if (OnGetGroupId.IsBound())
			{
				for (FGuid SelectedId : DragDropOp->GetSelectedFieldsId())
				{
					const FGuid OriginGroupId = OnGetGroupId.Execute(SelectedId);
					if (OriginGroupId != Id)
					{
						return true;
					}
				}
			}
		}
	}
	else if (DragDropOperation->IsOfType<FFieldGroupDragDropOp>())
	{
		if (TSharedPtr<FFieldGroupDragDropOp> DragDropOp = StaticCastSharedPtr<FFieldGroupDragDropOp>(DragDropOperation))
		{
			return DragDropOp->GetGroupId() != Id;
		}
	}

	return false;
}

FReply SRCPanelGroup::HandleDeleteGroup()
{
	OnDeleteGroup.ExecuteIfBound(Id);
	return FReply::Handled();
}

FSlateColor SRCPanelGroup::GetGroupNameTextColor() const
{
	return FLinearColor(1, 1, 1, 0.7f);
}

const FSlateBrush* SRCPanelGroup::GetBorderImage() const
{
	return FRemoteControlPanelStyle::Get()->GetBrush("RemoteControlPanel.TransparentBorder");
}

EVisibility SRCPanelGroup::GetVisibilityAccordingToLiveMode(EVisibility NonEditModeVisibility) const
{
	return !bLiveMode.Get() ? EVisibility::Visible : NonEditModeVisibility;
}

bool SRCPanelGroup::OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage)
{
	check(Preset.IsValid());
	FName TentativeName = FName(*InLabel.ToString());
	if (TentativeName != Name && !!Preset->Layout.GetGroupByName(TentativeName))
	{
		OutErrorMessage = LOCTEXT("NameAlreadyExists", "This name already exists.");
		IRemoteControlModule::BroadcastError(OutErrorMessage.ToString());
		return false;
	}

	return true;
}

void SRCPanelGroup::OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo)
{
	check(Preset.IsValid());
	FScopedTransaction Transaction(LOCTEXT("RenameGroup", "Rename Group"));
	Preset->Modify();
	Preset->Layout.RenameGroup(Id, FName(*InLabel.ToString()));
}

FSlateColor SRCPanelGroup::GetGroupColor() const
{
	return Preset.IsValid()
		? Preset->Layout.GetTagColor(Id)
		: FLinearColor::White;
}

#undef LOCTEXT_NAMESPACE
