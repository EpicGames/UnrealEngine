// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Brushes/SlateColorBrush.h"
#include "CoreMinimal.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Input/DragAndDrop.h"
#include "Misc/Guid.h"
#include "SRCPanelTreeNode.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SRemoteControlPanel;
struct SRCPanelTreeNode;
class STableViewBase;
struct FSlateColor;
struct FSlateBrush;
class SInlineEditableTextBlock;
class URemoteControlPreset;

/** Widget representing a group. */
class SRCPanelGroup : public SRCPanelTreeNode
{
public:
	using SWidget::SharedThis;
	using SWidget::AsShared;

public:
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnFieldDropEvent, const TSharedPtr<FDragDropOperation>& /* Event */, const TSharedPtr<SRCPanelTreeNode>& /* TargetField */, const TSharedPtr<SRCPanelTreeNode>& /* DragTargetGroup */);
	DECLARE_DELEGATE_RetVal_OneParam(FGuid, FOnGetGroupId, const FGuid& /* EntityId */);
	DECLARE_DELEGATE_OneParam(FOnDeleteGroup, const FGuid& /*GroupId*/);

	SLATE_BEGIN_ARGS(SRCPanelGroup)
		: _LiveMode(false)
	{}
		SLATE_ARGUMENT(FGuid, Id)
		SLATE_ARGUMENT(FName, Name)
		SLATE_ARGUMENT(TArray<TSharedPtr<SRCPanelTreeNode>>, Children)
		SLATE_EVENT(FOnFieldDropEvent, OnFieldDropEvent)
		SLATE_EVENT(FOnGetGroupId, OnGetGroupId)
		SLATE_EVENT(FOnDeleteGroup, OnDeleteGroup)
		SLATE_ATTRIBUTE(bool, LiveMode)
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, CommandList)
	SLATE_END_ARGS()

	void Tick(const FGeometry&, const double, const float);
	void Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, FRCColumnSizeData IInColumnSizeData);

	/** Get this group's name. */
	FName GetGroupName() const;

	/** Set this widget's name. */
	void SetName(FName InName);

	/** Get raw access to this group's child nodes. */
	TArray<TSharedPtr<SRCPanelTreeNode>>& GetNodes() { return Nodes; }

	/** Make the group name's text box editable. */
	virtual void EnterRenameMode() override;

	//~ SRCPanelTreeNode Interface
	virtual void GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const override;
	virtual TSharedPtr<SWidget> GetContextMenu() override;
	virtual FGuid GetRCId() const override;
	virtual ENodeType GetRCType() const override;
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:
	//~ Handle drag/drop events
	FReply OnFieldDropGroup(const FDragDropEvent& Event, TSharedPtr<SRCPanelTreeNode> TargetField);
	FReply OnFieldDropGroup(TSharedPtr<FDragDropOperation> DragDropOperation, TSharedPtr<SRCPanelTreeNode> TargetField);
	bool OnAllowDropFromOtherGroup(TSharedPtr<FDragDropOperation> DragDropOperation);

	/** Handles group deletion */
	FReply HandleDeleteGroup();
	/** Returns group name's text color according to the current selection. */
	FSlateColor GetGroupNameTextColor() const;
	/** Get the border image according to the current selection. */
	const FSlateBrush* GetBorderImage() const;
	/** Get the visibility according to the panel's current mode. */
	EVisibility GetVisibilityAccordingToLiveMode(EVisibility DefaultHiddenVisibility) const;

	bool OnVerifyItemLabelChanged(const FText& InLabel, FText& OutErrorMessage);

	void OnLabelCommitted(const FText& InLabel, ETextCommit::Type InCommitInfo);

	FSlateColor GetGroupColor() const;

private:
	/** Command list to use for context menu actions */
	TSharedPtr<FUICommandList> CommandList;
	/** Event called when something is dropped on this group. */
	FOnFieldDropEvent OnFieldDropEvent;
	/** Getter for this group's name. */
	FOnGetGroupId OnGetGroupId;
	/** Event called then the user deletes the group. */
	FOnDeleteGroup OnDeleteGroup;
	/** Holds the text box for the group name. */
	TSharedPtr<SInlineEditableTextBlock> NameTextBox;
	/** Whether the panel is currently in live mode. */
	TAttribute<bool> bLiveMode;
	/** Whether the group needs to be renamed. (As requested by a click on the rename button) */
	bool bNeedsRename = false;
	/** Weak ptr to the preset that contains the field group. */
	TWeakObjectPtr<URemoteControlPreset> Preset;
	/** Name of the group. */
	FName Name;
	/** Id for this group. (Matches the one in the preset layout data. */
	FGuid Id;
	/** This group's child nodes */
	TArray<TSharedPtr<SRCPanelTreeNode>> Nodes;
};


class FFieldGroupDragDropOp final : public FDecoratedDragDropOp
{
public:
	DRAG_DROP_OPERATOR_TYPE(FFieldGroupDragDropOp, FDragDropOperation)

	using WidgetType = SRCPanelGroup;

	FFieldGroupDragDropOp(TSharedPtr<SRCPanelGroup> InWidget, FGuid InId)
		: Id(MoveTemp(InId))
	{
		MouseCursor = EMouseCursor::GrabHandClosed;

		DecoratorWidget = SNew(SBorder)
			.Padding(0.f)
			.BorderImage(FAppStyle::Get().GetBrush("Graph.ConnectorFeedback.Border"))
			.Content()
			[
				InWidget.ToSharedRef()
			];
	}

	FGuid GetGroupId() const
	{
		return Id;
	}

	virtual void OnDrop(bool bDropWasHandled, const FPointerEvent& MouseEvent) override
	{
		FDecoratedDragDropOp::OnDrop(bDropWasHandled, MouseEvent);
	}

	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return DecoratorWidget;
	}

private:
	/** Id of the held group. */
	FGuid Id;
	/** Holds the displayed widget. */
	TSharedPtr<SWidget> DecoratorWidget;
};

