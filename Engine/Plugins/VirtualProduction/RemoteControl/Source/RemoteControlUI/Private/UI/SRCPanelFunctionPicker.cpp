// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelFunctionPicker.h"

#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Engine/GameEngine.h"
#include "Framework/Application/SlateApplication.h"
#include "Toolkits/IToolkitHost.h"
#include "Kismet/GameplayStatics.h"
#include "GameFramework/Actor.h"
#include "Styling/SlateColor.h"
#include "RemoteControlPreset.h"
#include "SRemoteControlPanel.h"
#include "Subsystems/Subsystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanel"

/** A node in the blueprint picker tree view. */
struct FRCFunctionPickerTreeNode
{
	virtual ~FRCFunctionPickerTreeNode() = default;
	virtual FText GetName() const = 0;
	virtual bool IsFunctionNode() const = 0;
	virtual UObject* GetObject() const = 0;
	virtual UFunction* GetFunction() const = 0;
	virtual void GetChildNodes(TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& OutNodes) const = 0;
};

namespace FunctionPickerUtils
{	
	/** Handle creating the widget to select a function.  */
	TArray<UFunction*> GetExposableFunctions(UClass* Class)
	{
		auto FunctionFilter = [](const UFunction* TestFunction)
		{
			return !TestFunction->HasAnyFunctionFlags(FUNC_Delegate | FUNC_MulticastDelegate) 
				&& (TestFunction->HasAnyFunctionFlags(FUNC_Public) || TestFunction->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_Event | FUNC_BlueprintEvent));
		};
		
		auto CategoryFilter = [] (const UFunction* TestFunction)
		{
			static TSet<FString> CategoriesDenyList = {
				TEXT("Replication"),
                TEXT("Tick"),
                TEXT("Rendering"),
                TEXT("Cooking"),
                TEXT("Material Parameters"),
                TEXT("HLOD"),
                TEXT("Collision"),
                TEXT("Input"),
				TEXT("Utilities")};

			static const FName CategoryName("Category");
			const FString& Category = TestFunction->GetMetaData(CategoryName);
			const bool bCanHaveEmptyCategory = TestFunction->HasAnyFunctionFlags(FUNC_BlueprintEvent);
			const bool bIsValid = (bCanHaveEmptyCategory || Category.Len() != 0)
				&& !CategoriesDenyList.Contains(Category);
			
			return bIsValid;
		};

		auto NameFilter = [](const UFunction* TestFunction)
		{
			static TSet<FName> NamesDenyList = {
				"ExecuteUbergraph",
				"ReceiveTick",
				"Destroyed",
				"UserConstructionScript"};

			return !NamesDenyList.Contains(TestFunction->GetFName());
		};
		
		TArray<UFunction*> ExposableFunctions;

		for (TFieldIterator<UFunction> FunctionIter(Class, EFieldIteratorFlags::IncludeSuper); FunctionIter; ++FunctionIter)
		{
			UFunction* TestFunction = *FunctionIter;
			if (FunctionFilter(TestFunction) && CategoryFilter(TestFunction) && NameFilter(TestFunction))
			{
				if (!ExposableFunctions.FindByPredicate([FunctionName = TestFunction->GetFName()](const UFunction* Func) { return Func->GetFName() == FunctionName; }))
				{
					ExposableFunctions.Add(*FunctionIter);
				}
			}
		}

		return ExposableFunctions;
	}

	const FString DefaultPrefix = TEXT("Default__");
	const FString CPostfix = TEXT("_C");

	struct FRCFunctionNode : public FRCFunctionPickerTreeNode
	{
		FRCFunctionNode(UFunction* InFunction)
			: Function(InFunction)
		{
			if (InFunction)
			{
				Name = Function->GetDisplayNameText();
			}
		}

		//~ Begin FRCFunctionPickerTreeNode
		virtual FText GetName() const override { return Name; }
		virtual bool IsFunctionNode() const override { return true; }
		virtual UFunction* GetFunction() const override { return Function.Get(); }
		virtual UObject* GetObject() const override { return nullptr; }
		virtual void GetChildNodes(TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& OutNodes) const override {}
		//~ End FRCFunctionPickerTreeNode

	private:
		/** This node's function. */
		TWeakObjectPtr<UFunction> Function = nullptr;
		/** This node's function's name. */
		FText Name;
	};

	struct FRCFunctionNodeWrapper : public FRCFunctionPickerTreeNode
	{
		FRCFunctionNodeWrapper(TSharedPtr<FRCFunctionPickerTreeNode> InNode)
			: Node(InNode)
		{
		}

		//~ Begin FRCFunctionPickerTreeNode
		virtual FText GetName() const override { return Node ? Node->GetName() : FText::GetEmpty(); }
		virtual bool IsFunctionNode() const override { return true; }
		virtual UFunction* GetFunction() const override { return Node ? Node->GetFunction() : nullptr; }
		virtual UObject* GetObject() const override { return nullptr; }
		virtual void GetChildNodes(TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& OutNodes) const override {}
		//~ End FRCFunctionPickerTreeNode

		private:
		TSharedPtr<FRCFunctionPickerTreeNode> Node;
	};

	struct FRCObjectNode : public FRCFunctionPickerTreeNode
	{
		FRCObjectNode(UObject* InObject, const TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& InFunctionNodes)
			: WeakObject(InObject)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FRCObjectNode::FRCObjectNode());

			if (ensure(InObject && InObject->GetFName().IsValid() && InObject->GetFName() != NAME_None))
			{
				FString NameString;

				if (InObject->IsA<AActor>())
				{
					NameString = Cast<AActor>(InObject)->GetActorLabel();
				}
				else
				{
					NameString = InObject->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) || InObject->IsA<USubsystem>() ? InObject->GetClass()->GetName() : InObject->GetName();
					NameString.RemoveFromStart(DefaultPrefix, ESearchCase::CaseSensitive);
					NameString.RemoveFromEnd(CPostfix, ESearchCase::CaseSensitive);
				}

				Name = FText::FromString(NameString);

				// Children Items must be a different item to avoid issues when list is flattened in search view.
				ChildNodes.Reserve(InFunctionNodes.Num());
				for (const TSharedPtr<FRCFunctionPickerTreeNode>& FunctionNode : InFunctionNodes)
				{
					ChildNodes.Add(MakeShared<FRCFunctionNodeWrapper>(FunctionNode));
				}
			}
		}

		//~ Begin FRCFunctionPickerTreeNode
		virtual FText GetName() const override { return Name; }
		virtual bool IsFunctionNode() const override { return false; }
		virtual UObject* GetObject() const override { return WeakObject.Get(); }
		virtual UFunction* GetFunction() const override { return nullptr; }
		virtual void GetChildNodes(TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& OutNodes) const override
		{
			OutNodes.Append(ChildNodes);
		}
		//~ End FRCFunctionPickerTreeNode

	private:
		/** The object represented by this node. */
		TWeakObjectPtr<UObject> WeakObject;
		/** This node's child functions. */
		mutable TArray<TSharedPtr<FRCFunctionPickerTreeNode>> ChildNodes;
		/** This object's name. */
		FText Name;
	};
}

void SRCPanelFunctionPicker::Construct(const FArguments& InArgs)
{
	using namespace FunctionPickerUtils;

	RemoteControlPanel = InArgs._RemoteControlPanel;
	ObjectClass = InArgs._ObjectClass;
	Label = InArgs._Label;
	bAllowDefaultObjects = InArgs._AllowDefaultObjects;

	auto OnGetChildren = []
	(TSharedPtr<FRCFunctionPickerTreeNode> Node, TArray<TSharedPtr<FRCFunctionPickerTreeNode>>& OutChildren)
	{
		Node->GetChildNodes(OutChildren);
	};

	OnSelectFunction = InArgs._OnSelectFunction;
	check(OnSelectFunction.IsBound());

	static FText InvalidNameText = LOCTEXT("InvalidEntryText", "INVALID_ENTRY");

	ChildSlot
	[
		SNew(SBox)
		.WidthOverride(300.0f)
		.MaxDesiredHeight(200.0f)
		[
			SAssignNew(ObjectsTreeView, SSearchableTreeView<TSharedPtr<FRCFunctionPickerTreeNode>>)
			.Items(&ObjectNodes)
			.OnGetChildren_Lambda(OnGetChildren)
			.OnGetDisplayName_Lambda([](TSharedPtr<FRCFunctionPickerTreeNode> InEntry) { return InEntry ? InEntry->GetName() : InvalidNameText; })
			.IsSelectable_Lambda([](TSharedPtr<FRCFunctionPickerTreeNode> TreeNode) { return TreeNode ? TreeNode->IsFunctionNode() : false; })
			.OnItemSelected_Lambda(
				[this] (TSharedPtr<FRCFunctionPickerTreeNode> TreeNode, TSharedPtr<FRCFunctionPickerTreeNode> ParentNode)
				{
					UObject* Owner = ParentNode->GetObject();
					UFunction* Function = TreeNode->GetFunction();
					if (Owner && Function)
					{
						OnSelectFunction.Execute(Owner, Function);
						FSlateApplication::Get().SetUserFocus(0, SharedThis(this));
					}
				})
		]
	];
}

void SRCPanelFunctionPicker::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	constexpr double MinimumTimeToSwitchFocus = 0.2;
	if (LastTimeSinceTick != 0.0 && InCurrentTime - LastTimeSinceTick > MinimumTimeToSwitchFocus)
	{
		ObjectsTreeView->Focus();
	}

	LastTimeSinceTick = InCurrentTime;
}

FReply SRCPanelFunctionPicker::OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent)
{
	if (InFocusEvent.GetCause() == EFocusCause::Navigation)
	{
		ObjectsTreeView->Focus();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

void SRCPanelFunctionPicker::Refresh()
{
	using namespace FunctionPickerUtils;

	ObjectNodes.Reset();

	auto Filter = [this](UObject* Object)
	{
		UClass* TestObjectClass = Object->GetClass();
		FName ClassName = TestObjectClass->GetFName();

		return TestObjectClass != UObject::StaticClass()
			&& ClassName.IsValid()
			&& ClassName != NAME_None
			&& !TestObjectClass->HasAllClassFlags(CLASS_NewerVersionExists | CLASS_CompiledFromBlueprint)
			&& !TestObjectClass->GetName().Contains(TEXT("SKEL"), ESearchCase::CaseSensitive)
			&& !bAllowDefaultObjects ? Object != TestObjectClass->GetDefaultObject() : true;
	};

	UClass* Class = ObjectClass.Get();
	if (!Class)
	{
		Class = UObject::StaticClass();
	}

	TMap<UClass*, TArray<TSharedPtr<FRCFunctionPickerTreeNode>>> CachedNodesPerClass;

	auto MakeNode = [this, Filter, &CachedNodesPerClass](UObject* Object)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelFunctionPicker::MakeNode);
		if (Filter(Object))
		{
			if (TArray<TSharedPtr<FRCFunctionPickerTreeNode>>* FunctionNodes = CachedNodesPerClass.Find(Object->GetClass()))
			{
				if (FunctionNodes->Num())
				{
					ObjectNodes.Add(MakeShared<FRCObjectNode>(Object, *FunctionNodes));
				}
				return;
			}


			TArray<UFunction*> Functions = GetExposableFunctions(Object->GetClass());
			TArray<TSharedPtr<FRCFunctionPickerTreeNode>> FunctionNodes;
			FunctionNodes.Reserve(Functions.Num());
			for (UFunction* Function : Functions)
			{
				FunctionNodes.Add(MakeShared<FRCFunctionNode>(Function));
			}

			CachedNodesPerClass.Add(Object->GetClass(), FunctionNodes);

			if (FunctionNodes.Num())
			{
				ObjectNodes.Add(MakeShared<FRCObjectNode>(Object, FunctionNodes));
			}
		}
	};


	// When displaying functions for actors, only display actors in the current level.
	if (Class->IsChildOf(AActor::StaticClass()) && GEditor)
	{
		TArray<AActor*> ActorList;
		constexpr bool bIgnorePIE = false;

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SRCPanelFunctionPicker::GetAllActorsOfClass);
			UGameplayStatics::GetAllActorsOfClass(GetWorld(bIgnorePIE), Class, ActorList);
		}

		ObjectNodes.Reserve(ActorList.Num());

		for (AActor* Actor : ActorList)
		{
			MakeNode(Actor);
		}
	}
	else
	{
		for (auto It = FUnsafeObjectIterator(Class, EObjectFlags::RF_NoFlags); It; ++It)
		{
			MakeNode(*It);
		}
	}

	if (ObjectsTreeView)
	{
		ObjectsTreeView->Refresh();
	}
}

UWorld* SRCPanelFunctionPicker::GetWorld(bool bIgnorePIE) const
{
	TSharedPtr<SRemoteControlPanel> RCP = RemoteControlPanel.Pin();

	if (RCP.IsValid())
	{
		return URemoteControlPreset::GetWorld(RCP->GetPreset(), !bIgnorePIE);
	}

	return URemoteControlPreset::GetWorld(nullptr, !bIgnorePIE);
}

#undef LOCTEXT_NAMESPACE /* RemoteControlPanel */