// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "SSearchableTreeView.h"

DECLARE_DELEGATE_TwoParams(FOnSelectFunction, UObject*, UFunction*);

struct FRCFunctionPickerTreeNode;
class SRemoteControlPanel;

/**
 * Widget that displays a picker for blueprint functions.
 */
class SRCPanelFunctionPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCPanelFunctionPicker)
		: _AllowDefaultObjects(false)
	{}
		SLATE_EVENT(FOnSelectFunction, OnSelectFunction)
		SLATE_ARGUMENT(TSharedPtr<SRemoteControlPanel>, RemoteControlPanel)
		SLATE_ARGUMENT(UClass*, ObjectClass)
		SLATE_ARGUMENT(FText, Label)
		SLATE_ARGUMENT(bool, AllowDefaultObjects)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	//~ End SWidget interface

	virtual FReply OnFocusReceived(const FGeometry& MyGeometry, const FFocusEvent& InFocusEvent) override;
	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	/** Regenerate the list of functions. */
	void Refresh();

private:
	/** Delegate that handles selecting a function in the picker. */
	FOnSelectFunction OnSelectFunction;
	/** Holds the object nodes. */
	TArray<TSharedPtr<FRCFunctionPickerTreeNode>> ObjectNodes;
	/** Holds the object picker tree view. */
	TSharedPtr<SSearchableTreeView<TSharedPtr<FRCFunctionPickerTreeNode>>> ObjectsTreeView;
	/** The class used to filter objects available in the dropdown. */
	TWeakObjectPtr<UClass> ObjectClass;
	/** The label that is displayed on the button. */
	FText Label;
	/** Allow default objects when refreshing. */
	bool bAllowDefaultObjects = false;
	/** Keep track of the last time this widget tick in order to focus it if it's been more than a frame. */
	double LastTimeSinceTick = 0.0;
	/** Keep a weak reference to the RCPanel for access to toolkit host. */
	TWeakPtr<SRemoteControlPanel> RemoteControlPanel;
	/** Checks the remote control panel to get a reference to the preset world. */
	UWorld* GetWorld(bool bIgnorePIE = false) const;
};