// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelPerPresetProtocolSettings.h"

#include "Commands/RemoteControlCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "RCModifyOperationFlags.h"
#include "RemoteControlPreset.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "SRCPanelPerPresetProtocolSettings"

void SRCPanelPerPresetProtocolSettings::Construct(const FArguments& InArgs, TWeakObjectPtr<URemoteControlPreset> InPreset)
{
	WeakPreset = InPreset;

	SetupCommandList();

	ChildSlot
	[
		SNew(SComboButton)
		.ButtonStyle(FAppStyle::Get(), "SimpleButton")
		.HasDownArrow(false)
		.OnGetMenuContent(this, &SRCPanelPerPresetProtocolSettings::CreateSettingsMenu)
		.ButtonContent()
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]
	];
}

void SRCPanelPerPresetProtocolSettings::SetupCommandList()
{
	CommandList = MakeShared<FUICommandList>();

	CommandList->MapAction(FRemoteControlCommands::Get().ProtocolsGeneratePropertyChangeEvents,
		FExecuteAction::CreateSP(this, &SRCPanelPerPresetProtocolSettings::ToggleProtocolsGeneratePropertyChangeEvents),
		FCanExecuteAction(),
		FIsActionChecked::CreateSP(this, &SRCPanelPerPresetProtocolSettings::AreProtocolsGeneratingPropertyChangedEvents)
	);

	CommandList->MapAction(FRemoteControlCommands::Get().ProtocolsGenerateTransactions,
		FExecuteAction::CreateSP(this, &SRCPanelPerPresetProtocolSettings::ToggleProtocolsGenerateTransactions),
		FCanExecuteAction::CreateSP(this, &SRCPanelPerPresetProtocolSettings::AreProtocolsGeneratingPropertyChangedEvents),
		FIsActionChecked::CreateSP(this, &SRCPanelPerPresetProtocolSettings::AreProtocolsGeneratingTransactions)
	);
}

TSharedRef<SWidget> SRCPanelPerPresetProtocolSettings::CreateSettingsMenu() const
{
	constexpr bool bShouldCloseWindowAfterMenuSelection = true;
	const FName NoExtensionHook = NAME_None;

	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, CommandList);

	MenuBuilder.BeginSection("ProtocolSettingsSection", LOCTEXT("ProtocolsSectionLabel", "Protocol Performance"));
	{
		MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().ProtocolsGeneratePropertyChangeEvents);
		MenuBuilder.AddMenuEntry(FRemoteControlCommands::Get().ProtocolsGenerateTransactions);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SRCPanelPerPresetProtocolSettings::ToggleProtocolsGeneratePropertyChangeEvents()
{
	if (URemoteControlPreset* Preset = WeakPreset.Get())
	{
		ERCModifyOperationFlags Flags = Preset->GetModifyOperationFlagsForProtocols();

		if (EnumHasAnyFlags(Preset->GetModifyOperationFlagsForProtocols(), ERCModifyOperationFlags::SkipPropertyChangeEvents))
		{
			EnumRemoveFlags(Flags, ERCModifyOperationFlags::SkipPropertyChangeEvents);
			Preset->SetModifyOperationFlagsForProtocols(Flags);
		}
		else
		{
			EnumAddFlags(Flags, ERCModifyOperationFlags::SkipPropertyChangeEvents);
			Preset->SetModifyOperationFlagsForProtocols(Flags);
		}
	}
}

bool SRCPanelPerPresetProtocolSettings::AreProtocolsGeneratingPropertyChangedEvents()
{
	if (const URemoteControlPreset* Preset = WeakPreset.Get())
	{
		return !EnumHasAnyFlags(Preset->GetModifyOperationFlagsForProtocols(), ERCModifyOperationFlags::SkipPropertyChangeEvents);
	}

	return false;
}

void SRCPanelPerPresetProtocolSettings::ToggleProtocolsGenerateTransactions()
{
	if (URemoteControlPreset* Preset = WeakPreset.Get())
	{
		ERCModifyOperationFlags Flags = Preset->GetModifyOperationFlagsForProtocols();

		if (EnumHasAnyFlags(Preset->GetModifyOperationFlagsForProtocols(), ERCModifyOperationFlags::SkipTransactions))
		{
			EnumRemoveFlags(Flags, ERCModifyOperationFlags::SkipTransactions);
			Preset->SetModifyOperationFlagsForProtocols(Flags);
		}
		else
		{
			EnumAddFlags(Flags, ERCModifyOperationFlags::SkipTransactions);
			Preset->SetModifyOperationFlagsForProtocols(Flags);
		}
	}
}

bool SRCPanelPerPresetProtocolSettings::AreProtocolsGeneratingTransactions()
{
	if (!AreProtocolsGeneratingPropertyChangedEvents())
	{
		return false;
	}

	if (const URemoteControlPreset* Preset = WeakPreset.Get())
	{
		return !EnumHasAnyFlags(Preset->GetModifyOperationFlagsForProtocols(), ERCModifyOperationFlags::SkipTransactions);
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
