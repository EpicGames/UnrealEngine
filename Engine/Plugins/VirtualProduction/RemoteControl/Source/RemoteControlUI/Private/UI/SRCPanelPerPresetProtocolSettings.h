// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Widgets/SCompoundWidget.h"

class FUICommandList;
class URemoteControlPreset;

/**
 * Widget that displays a picker for blueprint functions.
 */
class SRCPanelPerPresetProtocolSettings : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCPanelPerPresetProtocolSettings)
	{}

	SLATE_END_ARGS()

	/** Constructs this widget */
	void Construct(const FArguments& InArgs, TWeakObjectPtr<URemoteControlPreset> InPreset);

private:
	/** Creates the command list to use for this widget */
	void SetupCommandList();

	/** Creates the settings menu */
	TSharedRef<SWidget> CreateSettingsMenu() const;

	/** Called when the Protocols Generate Property Changed Events setting changed */
	void ToggleProtocolsGeneratePropertyChangeEvents();

	/** Returns true if protocols are generating Property Changed Events */
	bool AreProtocolsGeneratingPropertyChangedEvents();

	/** Called when the Protocols Generate Transactions setting changed */
	void ToggleProtocolsGenerateTransactions();

	/** Returns true if protocols are generating transactions */
	bool AreProtocolsGeneratingTransactions();

	/** The command list for this widget */
	TSharedPtr<FUICommandList> CommandList;

	/** The preset for which the settings are made */
	TWeakObjectPtr<URemoteControlPreset> WeakPreset;
};
