// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCPanelTreeNode.h"

#include "IRemoteControlProtocolWidgetsModule.h"

#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanelNode"

TSet<FName> SRCPanelTreeNode::DefaultColumns = {
	RemoteControlPresetColumns::PropertyIdentifier,
	RemoteControlPresetColumns::OwnerName,
	RemoteControlPresetColumns::SubobjectPath,
	RemoteControlPresetColumns::Description,
	RemoteControlPresetColumns::Value,
	RemoteControlPresetColumns::Reset
};

TSharedRef<SWidget> SRCPanelTreeNode::GetProtocolWidget(const FName ForColumnName, const FName InProtocolName)
{
	if (ForColumnName == RemoteControlPresetColumns::BindingStatus)
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SEditableTextBox)
		.OnTextChanged(this, &SRCPanelTreeNode::OnProtocolTextChanged, InProtocolName);
}

const bool SRCPanelTreeNode::HasProtocolExtension() const
{
	return false;
}

const bool SRCPanelTreeNode::GetProtocolBindingsNum() const
{
	return false;
}

const bool SRCPanelTreeNode::SupportsProtocol(const FName& InProtocolName) const
{
	return false;
}

TSharedRef<SWidget> SRCPanelTreeNode::GetWidget(const FName ForColumnName, const FName InActiveProtocol)
{
	if (ForColumnName == RemoteControlPresetColumns::PropertyIdentifier)
	{
		return PropertyIdWidget.ToSharedRef();
	}
	else if (ForColumnName == RemoteControlPresetColumns::OwnerName)
	{
		return NodeOwnerNameWidget.ToSharedRef();
	}
    else if (ForColumnName == RemoteControlPresetColumns::SubobjectPath)
    {
    	return SubobjectPathWidget.ToSharedRef();
    }
	else if (ForColumnName == RemoteControlPresetColumns::Description)
	{
		return NodeNameWidget.ToSharedRef();
	}
	else if (ForColumnName == RemoteControlPresetColumns::Reset)
	{
		return ResetValueWidget.ToSharedRef();
	}
	else if (ForColumnName == RemoteControlPresetColumns::Value)
	{
		return NodeValueWidget.ToSharedRef();
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SRCPanelTreeNode::GetDragAndDropWidget(int32 InSelectedEntitiesNum)
{
	if (NodeNameWidget && NodeValueWidget)
	{
		const TSharedRef<SWidget> LeftColumn = SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
			// Field name
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				NodeNameWidget.ToSharedRef()
			];

		const TSharedRef<SHorizontalBox> RightColumn = SNew(SHorizontalBox)
			.Clipping(EWidgetClipping::OnDemand)
			// Node Value
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoWidth()
			[
				SNew(SBox)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					NodeValueWidget.ToSharedRef()
				]
			];

		if (InSelectedEntitiesNum > 1)
		{
			RightColumn->AddSlot()
				.HAlign(HAlign_Fill)
				.AutoWidth()
				.Padding(2, 2)
				[
					SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(FText::Format(FText::FromString("and {0} other item(s)"), InSelectedEntitiesNum - 1))
					]
				];
		}
		return SNew(SSplitter)
			.Style(FAppStyle::Get(), "DetailsView.Splitter")
			.PhysicalSplitterHandleSize(1.0f)
			.HitDetectionSplitterHandleSize(5.0f)
			+ SSplitter::Slot()
			[
				LeftColumn
			]
			+ SSplitter::Slot()
			.SizeRule(SSplitter::SizeToContent)
			[
				RightColumn
			];
	}

	return SNullWidget::NullWidget;
}

TSharedRef<SWidget> SRCPanelTreeNode::MakeSplitRow(TSharedRef<SWidget> LeftColumn, TSharedRef<SWidget> RightColumn)
{
	TAttribute<float> LeftColumnAttribute;
	LeftColumnAttribute.Bind(this, &SRCPanelTreeNode::GetLeftColumnWidth);
	TAttribute<float> RightColumnAttribute;
	RightColumnAttribute.Bind(this, &SRCPanelTreeNode::GetRightColumnWidth);
	
	return SNew(SSplitter)
		.Style(FAppStyle::Get(), "DetailsView.Splitter")
		.PhysicalSplitterHandleSize(1.0f)
		.HitDetectionSplitterHandleSize(5.0f)
		+ SSplitter::Slot()
		.Value(MoveTemp(LeftColumnAttribute))
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SRCPanelTreeNode::OnLeftColumnResized))
		[
			LeftColumn
		]
		+ SSplitter::Slot()
		.Value(MoveTemp(RightColumnAttribute))
		.OnSlotResized(SSplitter::FOnSlotResized::CreateSP(this, &SRCPanelTreeNode::SetColumnWidth))
		[
			RightColumn
		];
}

TSharedRef<SWidget> SRCPanelTreeNode::MakeNodeWidget(const FMakeNodeWidgetArgs& Args)
{
	MakeNodeWidgets(Args);

	TSharedRef<SWidget> LeftColumn = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		// Link identifier widget
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			PropertyIdWidget.ToSharedRef()
		]
		// Owner name
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			NodeOwnerNameWidget.ToSharedRef()
		]
		// Subobject Path
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SubobjectPathWidget.ToSharedRef()
		]
		// Field name
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			NodeNameWidget.ToSharedRef()
		];

	TSharedRef<SWidget> RightColumn = SNew(SHorizontalBox)
		.Clipping(EWidgetClipping::OnDemand)
		// Node Value
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Fill)
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				NodeValueWidget.ToSharedRef()
			]
		]
		// Reset button
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			ResetValueWidget.ToSharedRef()
		];

	return MakeSplitRow(LeftColumn, RightColumn);
}

void SRCPanelTreeNode::MakeNodeWidgets(const FMakeNodeWidgetArgs& Args)
{
	auto WidgetOrNull = [](const TSharedPtr<SWidget>& Widget) {return Widget ? Widget.ToSharedRef() : SNullWidget::NullWidget; };

	PropertyIdWidget = WidgetOrNull(Args.PropertyIdWidget);

	NodeOwnerNameWidget = WidgetOrNull(Args.OwnerNameWidget);
	
	SubobjectPathWidget = WidgetOrNull(Args.SubObjectPathWidget);

	NodeNameWidget = WidgetOrNull(Args.NameWidget);

	NodeValueWidget = WidgetOrNull(Args.ValueWidget);

	ResetValueWidget = WidgetOrNull(Args.ResetButton);
}

void SRCPanelTreeNode::OnLeftColumnResized(float) const
{
	// This has to be bound or the splitter will take it upon itself to determine the size
	// We do nothing here because it is handled by the column size data
}

void SRCPanelTreeNode::OnProtocolTextChanged(const FText& InText, const FName InProtocolName)
{
	IRemoteControlProtocolWidgetsModule& RCProtocolsWidgets = IRemoteControlProtocolWidgetsModule::Get();

	RCProtocolsWidgets.AddProtocolBinding(InProtocolName);
}

float SRCPanelTreeNode::GetLeftColumnWidth() const
{
	const float Offset = GetRCType() == ENodeType::Group ? SplitterOffset : 0;
	return FMath::Clamp(ColumnSizeData.LeftColumnWidth.Get() + Offset, 0.f, 1.f);
}

float SRCPanelTreeNode::GetRightColumnWidth() const
{
	const float Offset = GetRCType() == ENodeType::Group ? SplitterOffset : 0;
	return  FMath::Clamp(ColumnSizeData.RightColumnWidth.Get() - Offset, 0.f, 1.f);
}

void SRCPanelTreeNode::SetColumnWidth(float InWidth)
{
	const float Offset = GetRCType() == ENodeType::Group ? SplitterOffset : -SplitterOffset;
	ColumnSizeData.SetColumnWidth(FMath::Clamp(InWidth + SplitterOffset, 0.f, 1.f));
}

#undef LOCTEXT_NAMESPACE /*RemoteControlPanelNode*/
