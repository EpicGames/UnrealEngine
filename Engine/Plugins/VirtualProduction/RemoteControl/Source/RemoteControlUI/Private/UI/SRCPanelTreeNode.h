// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IRemoteControlUIModule.h"
#include "IHasProtocolExtensibility.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"
#include "Misc/Guid.h"
#include "Widgets/SCompoundWidget.h"

class SWidget;
class SRCPanelGroup;
struct SRCPanelExposedField;
struct SRCPanelExposedActor;
struct SRCPanelExposedMaterial;

namespace RemoteControlPresetColumns
{
	static FName PropertyIdentifier = TEXT("PropertyID");
	static FName OwnerName = TEXT("OwnerName");
	static FName SubobjectPath = TEXT("Subobject Path");
	static FName Description = TEXT("Description");
	static FName Value = TEXT("Value");
	static FName BindingStatus = TEXT("BindingStatus");
	static FName Status = TEXT("Status");
	static FName Reset = TEXT("Reset");
}

namespace ERCColumn
{
	enum Position
	{
		/** Places the given column after a specific column. */
		ERC_After,

		/** Places the given column before a specific column. */
		ERC_Before
	};
}

DECLARE_DELEGATE_TwoParams(FOnLabelModified, FName /* InOldName */, FName /* InNewName */)
DECLARE_DELEGATE_OneParam(FOnPropertyIdRenamed, FName /* InNewPropertyId */)

/** A node in the panel tree view. */
struct SRCPanelTreeNode : public SCompoundWidget, public IHasProtocolExtensibility
{
	enum ENodeType
	{
		Invalid,
		Group,
		Field,
		FieldChild,
		Actor,
		Material,
		FieldGroup
	};

	virtual ~SRCPanelTreeNode() {}

	//~ BEGIN : IHasProtocolExtensibility Interface
	virtual TSharedRef<SWidget> GetProtocolWidget(const FName ForColumnName, const FName InProtocolName = NAME_None) override;
	virtual const bool HasProtocolExtension() const override;
	virtual const bool GetProtocolBindingsNum() const override;
	virtual void OnProtocolTextChanged(const FText& InText, const FName InProtocolName);
	virtual const bool SupportsProtocol(const FName& InProtocolName) const override;
	//~ END : IHasProtocolExtensibility Interface

	/** Get this tree node's childen. */
	virtual void GetNodeChildren(TArray<TSharedPtr<SRCPanelTreeNode>>& OutChildren) const {}

	/** Get this node's ID if any. */
	virtual FGuid GetRCId() const { return FGuid(); }

	/** Get get this node's type. */
	virtual ENodeType GetRCType() const { return ENodeType::Invalid; }

	/** Returns true if this tree node has childen. */
	virtual bool HasChildren() const { return false; }

	/** Refresh the node. */
	virtual void Refresh() {}

	/** Get the context menu for this node. */
	virtual TSharedPtr<SWidget> GetContextMenu() { return nullptr; }

	/** Set whether this widget is currently hovered when drag and dropping. */
	virtual void SetIsHovered(bool bIsBeingHovered) {}

	/** Make the node name's text box editable. */
	virtual void EnterRenameMode() {}

	/** Get the PropertyId of this Node. */
	virtual FName GetPropertyId() { return FName(TEXT("")); }

	/** Set the PropertyId of this Node. */
	virtual void SetPropertyId(FName InNewPropertyId) {}

	/** Focus the PropertyId widget. */
	virtual void FocusPropertyIdWidget() const {}

	/** Set the Name of this Node. */
	virtual void SetName(FName InNewName) {}

	/** Updates the highlight text to active search term. */
	virtual void SetHighlightText(const FText& InHightlightText = FText::GetEmpty()) {}

	/** Retrieves the referenced widget corresponding to the given column name. */
	virtual TSharedRef<SWidget> GetWidget(const FName ForColumnName, const FName InActiveProtocol);

	/** Retrieves the DragAndDropWidget if possible otherwise returns a NullWidget. */
	TSharedRef<SWidget> GetDragAndDropWidget(int32 InSelectedEntitiesNum = 1);

	/** Executed when the Name of the node is changed. */
	FOnLabelModified& OnLabelModified() { return OnLabelModifiedDelegate; }

	/** Executed when the PropertyId of the node is changed. */
	FOnPropertyIdRenamed& OnPropertyIdRenamed() { return OnPropertyIdRenamedDelegate; }

protected:
	struct FMakeNodeWidgetArgs
	{
		TSharedPtr<SWidget> PropertyIdWidget;
		TSharedPtr<SWidget> OwnerNameWidget;
		TSharedPtr<SWidget> SubObjectPathWidget;
		TSharedPtr<SWidget> NameWidget;
		TSharedPtr<SWidget> ValueWidget;
		TSharedPtr<SWidget> ResetButton;
	};
	
	/** Create a widget that represents a row with a splitter. */
	TSharedRef<SWidget> MakeSplitRow(TSharedRef<SWidget> LeftColumn, TSharedRef<SWidget> RightColumn);

	/** Create a widget that represents a node in the panel tree hierarchy. */
	TSharedRef<SWidget> MakeNodeWidget(const FMakeNodeWidgetArgs& Args);

	/** Creates cached copies of underlying widgets. */
	void MakeNodeWidgets(const FMakeNodeWidgetArgs& Args);

private:
	/** Stub handler for column resize callback to prevent the splitter from handling it internally.  */
	void OnLeftColumnResized(float) const;
	
	//~ Wrappers around ColumnSizeData's delegate needed in order to offset the splitter for RC Groups. 
	float GetLeftColumnWidth() const;
	float GetRightColumnWidth() const;
	void SetColumnWidth(float InWidth);

public:

	/** Columns to be present by default in protocols mode. */
	static TSet<FName> DefaultColumns;

protected:
	/** Holds the row's columns' width. */
	FRCColumnSizeData ColumnSizeData;

private:

	/** Cached widget of PropertyId identifier. */
	TSharedPtr<SWidget> PropertyIdWidget;

	/** Cached widget of node owner name. */
	TSharedPtr<SWidget> NodeOwnerNameWidget;
	
	/** Cached widget of subobject path. */
	TSharedPtr<SWidget> SubobjectPathWidget;

	/** Cached widget of node name. */
	TSharedPtr<SWidget> NodeNameWidget;

	/** Cached widget of node value. */
	TSharedPtr<SWidget> NodeValueWidget;
	
	/** Cached widget of reset arrow. */
	TSharedPtr<SWidget> ResetValueWidget;

private:
	/** The splitter offset to align the group splitter with the other row's splitters. */
	static constexpr float SplitterOffset = 0.008f;

	/** Delegate called when the Node Name is renamed. */
	FOnLabelModified OnLabelModifiedDelegate;

	/** Delegate called when the Node Property Id change. */
	FOnPropertyIdRenamed OnPropertyIdRenamedDelegate;
};
