// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRemoteControlPanel.h"
#include "Action/SRCActionPanel.h"
#include "ActorEditorUtils.h"
#include "Behaviour/Builtin/Bind/RCBehaviourBind.h"
#include "Behaviour/SRCBehaviourPanel.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"
#include "Commands/RemoteControlCommands.h"
#include "Controller/SRCControllerPanel.h"
#include "Editor.h"
#include "Editor/EditorPerformanceSettings.h"
#include "EditorFontGlyphs.h"
#include "Engine/Selection.h"
#include "EngineUtils.h"
#include "FileHelpers.h"
#include "Framework/Application/IInputProcessor.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/Actor.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlProtocolModule.h"
#include "IRemoteControlProtocolWidgetsModule.h"
#include "IRemoteControlUIModule.h"
#include "ISettingsModule.h"
#include "IStructureDetailsView.h"
#include "Input/Reply.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Layout/Visibility.h"
#include "Materials/Material.h"
#include "MenuContext/RemoteControlPanelMenuContext.h"
#include "Modules/ModuleManager.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "RCPanelWidgetRegistry.h"
#include "RemoteControlActor.h"
#include "RemoteControlEntity.h"
#include "RemoteControlField.h"
#include "RemoteControlLogger.h"
#include "RemoteControlPanelStyle.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "RemoteControlUIModule.h"
#include "SClassViewer.h"
#include "SRCLogger.h"
#include "SRCModeSwitcher.h"
#include "SRCPanelExposedActor.h"
#include "SRCPanelExposedEntitiesList.h"
#include "SRCPanelExposedField.h"
#include "SRCPanelFunctionPicker.h"
#include "SRCPanelTreeNode.h"
#include "SWarningOrErrorBox.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "ScopedTransaction.h"
#include "Signature/SRCSignaturePanel.h"
#include "Styling/RemoteControlStyles.h"
#include "Styling/ToolBarStyle.h"
#include "Subsystems/Subsystem.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Templates/UnrealTypeTraits.h"
#include "ToolMenus.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Toolkits/IToolkitHost.h"
#include "UI/BaseLogicUI/RCLogicModeBase.h"
#include "UI/BaseLogicUI/SRCLogicPanelListBase.h"
#include "UI/Filters/SRCPanelFilter.h"
#include "UI/Panels/SRCDockPanel.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SLayeredImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "RemoteControlPanel"

const FName SRemoteControlPanel::DefaultRemoteControlPanelToolBarName("RemoteControlPanel.DefaultToolBar");
const FName SRemoteControlPanel::AuxiliaryRemoteControlPanelToolBarName("RemoteControlPanel.AuxiliaryToolBar");
const FName SRemoteControlPanel::TargetWorldRemoteControlPanelMenuName("RemoteControlPanel.TargetWorld");
const float SRemoteControlPanel::MinimumPanelWidth = 640.f;

TSharedRef<SBox> SRemoteControlPanel::CreateNoneSelectedWidget()
{
	return SNew(SBox)
		.Padding(0.f)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("NoneSelected", "Select an entity to view details."))
			.TextStyle(&FAppStyle::GetWidgetStyle<FTextBlockStyle>("NormalText"))
			.Justification(ETextJustify::Center)
		];
}

void SRemoteControlPanel::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(LogicClipboardItems);
}

void SRemoteControlPanel::ApplyProtocolBindings()
{
	IRemoteControlProtocolModule::Get().ApplyProtocolBindings(GetPreset());
}

void SRemoteControlPanel::UnapplyProtocolBindings()
{
	IRemoteControlProtocolModule::Get().UnapplyProtocolBindings(GetPreset());
}

namespace RemoteControlPanelUtils
{
	bool IsExposableActor(AActor* Actor)
	{
		return Actor->IsEditable()
            && Actor->IsListedInSceneOutliner()						// Only add actors that are allowed to be selected and drawn in editor
            && !Actor->IsTemplate()									// Should never happen, but we never want CDOs
            && !Actor->HasAnyFlags(RF_Transient)					// Don't add transient actors in non-play worlds
            && !FActorEditorUtils::IsABuilderBrush(Actor)			// Don't add the builder brush
            && !Actor->IsA(AWorldSettings::StaticClass());	// Don't add the WorldSettings actor, even though it is technically editable
	};

	TSharedPtr<FStructOnScope> GetEntityOnScope(const TSharedPtr<FRemoteControlEntity>& Entity, const UScriptStruct* EntityType)
	{
		if (ensure(Entity && EntityType))
		{
			check(EntityType->IsChildOf(FRemoteControlEntity::StaticStruct()));
			return MakeShared<FStructOnScope>(EntityType, reinterpret_cast<uint8*>(Entity.Get()));
		}

		return nullptr;
	}
}

void SRemoteControlPanel::Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, TSharedPtr<IToolkitHost> InToolkitHost)
{
	OnLiveModeChange = InArgs._OnLiveModeChange;
	PresetWeak = InPreset;
	WidgetRegistry = MakeShared<FRCPanelWidgetRegistry>();
	ToolkitHost = InToolkitHost;

	ActiveMode = ERCPanelMode::Controller;

	RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	TArray<TSharedRef<SWidget>> ExtensionWidgets;
	FRemoteControlUIModule::Get().GetExtensionGenerators().Broadcast(ExtensionWidgets);

	ApplyProtocolBindings();

	BindRemoteControlCommands();

	GenerateAuxiliaryToolbar();

	AddToolbarWidget(AuxiliaryToolbarWidgetContent.ToSharedRef());

	// Settings
	AddToolbarWidget(SNew(SButton)
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ContentPadding(2.0f)
		.TextStyle(FRemoteControlPanelStyle::Get(), "RemoteControlPanel.Button.TextStyle")
		.OnClicked(this, &SRemoteControlPanel::OnClickSettingsButton)
		.ToolTipText(LOCTEXT("OpenRemoteControlSettings", "Open Remote Control settings."))
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush("Icons.Toolbar.Settings"))
			.ColorAndOpacity(FSlateColor::UseForeground())
		]);

	// Show Log
	AddToolbarWidget(SNew(SCheckBox)
		.Style(&RCPanelStyle->ToggleButtonStyle)
		.ToolTipText(LOCTEXT("ShowLogTooltip", "Show/Hide remote control log."))
		.IsChecked_Lambda([]() { return FRemoteControlLogger::Get().IsEnabled() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
		.OnCheckStateChanged(this, &SRemoteControlPanel::OnLogCheckboxToggle)
		.Padding(4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ShowLogLabel", "Log"))
			.Justification(ETextJustify::Center)
			.TextStyle(&RCPanelStyle->PanelTextStyle)
		]
	);

	// Extension Generators
	for (const TSharedRef<SWidget>& Widget : ExtensionWidgets)
	{
		AddToolbarWidget(Widget);
	}

	GenerateToolbar();
	UpdateRebindButtonVisibility();

	EntityProtocolDetails = SNew(SBox);

	EntityList = SNew(SRCPanelExposedEntitiesList, InPreset, WidgetRegistry)
		.ExposeActorsComboButton(CreateExposeActorsButton())
		.ExposeFunctionsComboButton(CreateExposeFunctionsButton())
		.LiveMode(this, &SRemoteControlPanel::IsModeActive, ERCPanelMode::Live)
		.ProtocolsMode(this, &SRemoteControlPanel::IsModeActive, ERCPanelMode::Protocols)
		.OnEntityListUpdated_Lambda([this]()
			{
				UpdateEntityDetailsView(EntityList->GetSelectedEntity());
				UpdateRebindButtonVisibility();
				CachedExposedPropertyArgs.Reset();
			})
		.CommandList(CommandList);

	EntityList->OnSelectionChange().AddSP(this, &SRemoteControlPanel::UpdateEntityDetailsView);

	const TAttribute<float> TreeBindingSplitRatioTop = TAttribute<float>::Create(
		TAttribute<float>::FGetter::CreateLambda([]()
		{
			URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
			return Settings->TreeBindingSplitRatio;
		}));

	const TAttribute<float> TreeBindingSplitRatioBottom = TAttribute<float>::Create(
		TAttribute<float>::FGetter::CreateLambda([]()
		{
			URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
			return 1.0f - Settings->TreeBindingSplitRatio;
		}));

	TSharedRef<SWidget> HeaderPanel = SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.Padding(3.f, 0.f)
		.AutoHeight()
		[
			ToolbarWidgetContent.ToSharedRef()
		];

	TSharedRef<SWidget> FooterPanel = SNew(SVerticalBox)
		// Use less CPU Warning
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(5.f, 8.f, 5.f, 5.f))
		[
			CreateCPUThrottleWarning()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(5.f, 8.f, 5.f, 5.f))
		[
			CreateProtectedIgnoredWarning()
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(FMargin(5.f, 8.f, 5.f, 5.f))
		[
			CreateGetterSetterIgnoredWarning()
		];

	// Output Log Dock panel
	TSharedRef<SRCMinorPanel> OutputLogDockPanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("LogPanelLabel", "Log"))
		[
			SNew(SRCLogger)
		];

	TSharedRef<SWidget> OutputLogIcon = SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("MessageLog.TabIcon"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	OutputLogDockPanel->AddHeaderToolbarItem(EToolbar::Left, OutputLogIcon);

	// Make 3 Columns with Controllers + Behaviours + Actions
	TSharedRef<SRCMajorPanel> LogicPanel = SNew(SRCMajorPanel)
		.EnableHeader(false)
		.EnableFooter(false);

	// Controllers with Behaviours panel
	TSharedRef<SRCMajorPanel> ControllersAndBehavioursPanel = SNew(SRCMajorPanel)
		.EnableHeader(false)
		.EnableFooter(false);

	TSharedRef<SRemoteControlPanel> This = SharedThis(this);

	// Controller Mode
	ActionPanel = SNew(SRCActionPanel, This);
	BehaviorPanel = SNew(SRCBehaviourPanel, This);
	ControllerPanel = SNew(SRCControllerPanel, This)
		.LiveMode(this, &SRemoteControlPanel::IsModeActive, ERCPanelMode::Live);

	ControllersAndBehavioursPanel->AddPanel(ControllerPanel.ToSharedRef(), 0.8f);
	ControllersAndBehavioursPanel->AddPanel(BehaviorPanel.ToSharedRef(), 0.f, /*bResizable*/false);

	// Signature Mode
	SignaturePanel = SNew(SRCSignaturePanel, This)
		.LiveMode(this, &SRemoteControlPanel::IsModeActive, ERCPanelMode::Live);

	// Retrieve Action Panel Split ratio from RC Settings.
	// We can't just get the value, since it will update in real time as the user resizes the slot: this value needs to update in real time
	const TAttribute<float> ActionPanelSplitRatioAttribute = TAttribute<float>::Create(
		TAttribute<float>::FGetter::CreateLambda([]()
		{
			const URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
			return Settings->ActionPanelSplitRatio;
		}));

	int32 ActionPanelSlotIndex = LogicPanel->AddPanel(ControllersAndBehavioursPanel, ActionPanelSplitRatioAttribute, true);
	LogicPanel->AddPanel(ActionPanel.ToSharedRef(), 1.0 - ActionPanelSplitRatioAttribute.Get(), true);

	if (ActionPanelSlotIndex >= 0)
	{
		const TWeakPtr<SRCMajorPanel> LogicPanelWeak = LogicPanel.ToWeakPtr();

		// We setup a lambda to fire whenever the user is done resizing the splitter.
		// This lambda will record the current splitter position in Remote Control Settings
		LogicPanel->OnSplitterFinishedResizing().BindLambda([ActionPanelSlotIndex, LogicPanelWeak]()
		{
			if (LogicPanelWeak.IsValid())
			{
				if (const TSharedPtr<SRCMajorPanel> LogicPanelShared = LogicPanelWeak.Pin())
				{
					const SSplitter::FSlot& Slot = LogicPanelShared->GetSplitterSlotAt(ActionPanelSlotIndex);		
					URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
					Settings->ActionPanelSplitRatio = Slot.GetSizeValue();
					Settings->PostEditChange();
					Settings->SaveConfig();
				}
			}
		});
	}

	// Make 2 Columns with Panel Drawer + Main Panel
	TSharedRef<SWidgetSwitcher> ContentPanel = SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this](){ return static_cast<int32>(ActiveMode); })
		// 1. Logic Mode (Index: 0)
		+ SWidgetSwitcher::Slot()
		[
			BuildLogicModeContent(LogicPanel)
		]
		// 2. Entity Details Mode (Index: 1)
		+ SWidgetSwitcher::Slot()
		[
			BuildEntityDetailsModeContent(TreeBindingSplitRatioTop, TreeBindingSplitRatioBottom)
		]
		// 3. Protocols Mode (Index: 2)
		+ SWidgetSwitcher::Slot()
		[
			BuildProtocolsModeContent(TreeBindingSplitRatioTop, TreeBindingSplitRatioBottom)
		]
		// 4. Output Log Mode (Index : 3)
		+ SWidgetSwitcher::Slot()
		[
			OutputLogDockPanel
		]
		// 5. Live Mode (Index : 4)
		+ SWidgetSwitcher::Slot()
		[
			BuildLiveModeContent(LogicPanel)
		]
		// 7. Signatures Mode (Index : 5)
		+ SWidgetSwitcher::Slot()
		[
			BuildSignaturesModeContent()
		];

	ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				HeaderPanel
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				// Separator
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
				.Thickness(5.f)
				.Orientation(EOrientation::Orient_Horizontal)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.FillHeight(1.f)
			[
				ContentPanel
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				// Separator
				SNew(SSeparator)
				.SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
				.Thickness(5.f)
				.Orientation(EOrientation::Orient_Horizontal)
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Fill)
			.AutoHeight()
			[
				FooterPanel
			]
		];

	RegisterEvents();
	CacheLevelClasses();
	Refresh();
	LoadSettings(InPreset->GetPresetId());
}

SRemoteControlPanel::~SRemoteControlPanel()
{
	UnapplyProtocolBindings();

	SaveSettings();
	UnregisterEvents();

	// Clear the log
	FRemoteControlLogger::Get().ClearLog();

	// Remove protocol bindings
	IRemoteControlProtocolWidgetsModule& ProtocolWidgetsModule = FModuleManager::LoadModuleChecked<IRemoteControlProtocolWidgetsModule>("RemoteControlProtocolWidgets");
	ProtocolWidgetsModule.ResetProtocolBindingList();

	// Unregister with UI module
	if (FModuleManager::Get().IsModuleLoaded("RemoteControlUI"))
	{
		FRemoteControlUIModule::Get().UnregisterRemoteControlPanel(this);
	}
}

void SRemoteControlPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bMaterialsCompiledThisFrame)
	{
		TriggerMaterialCompiledRefresh();
		bMaterialsCompiledThisFrame = false;
	}
}

FReply SRemoteControlPanel::OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent)
{
	if (CommandList.IsValid() && CommandList->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

bool SRemoteControlPanel::IsExposed(const FRCExposesPropertyArgs& InPropertyArgs)
{
	if (!ensure(InPropertyArgs.IsValid()))
	{
		return false;
	}

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

	auto CheckCachedExposedArgs = [this, InPropertyArgs](const TArray<UObject*> InOwnerObjects, const FString& InPath, bool bIsCheckIsBoundByFullPath)
	{
		if (CachedExposedPropertyArgs.Contains(InPropertyArgs))
		{
			return true;
		}

		const bool bAllObjectsExposed = IsAllObjectsExposed(InOwnerObjects, InPath, bIsCheckIsBoundByFullPath);

		if (bAllObjectsExposed)
		{
			CachedExposedPropertyArgs.Emplace(InPropertyArgs);
		}

		return bAllObjectsExposed;
	};

	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		TArray<UObject*> OuterObjects;
		InPropertyArgs.PropertyHandle->GetOuterObjects(OuterObjects);
		const FString Path = InPropertyArgs.PropertyHandle->GeneratePathToProperty();

		constexpr bool bIsCheckIsBoundByFullPath = true;
		return CheckCachedExposedArgs({ OuterObjects }, Path, bIsCheckIsBoundByFullPath);
	}
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		constexpr bool bIsCheckIsBoundByFullPath = false;
		return CheckCachedExposedArgs({ InPropertyArgs.OwnerObject.Get()}, InPropertyArgs.PropertyPath, bIsCheckIsBoundByFullPath);
	}

	// It never should hit this point
	ensure(false);

	return false;
}

bool SRemoteControlPanel::IsAllObjectsExposed(TArray<UObject*> InOuterObjects, const FString& InPath, bool bUsingDuplicatesInPath)
{
	const URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return false;
	}

	TArray<TSharedPtr<const FRemoteControlProperty>, TInlineAllocator<1>> PotentialMatches;
	for (const TWeakPtr<const FRemoteControlProperty>& WeakProperty : Preset->GetExposedEntities<FRemoteControlProperty>())
	{
		if (TSharedPtr<const FRemoteControlProperty> Property = WeakProperty.Pin())
		{
			// If that was exposed by property path it should be checked by the full path with duplicated like propertypath.propertypath[0]
			// If that was exposed by the owner object it should be without duplicated in the path, just propertypath[0]
			const bool bIsBound = bUsingDuplicatesInPath ? Property->CheckIsBoundToPropertyPath(InPath) : Property->CheckIsBoundToString(InPath);
			if (bIsBound)
			{
				PotentialMatches.Add(Property);
			}
		}
	}

	bool bAllObjectsExposed = true;

	for (UObject* OuterObject : InOuterObjects)
	{
		bool bFoundPropForObject = false;

		for (const TSharedPtr<const FRemoteControlProperty>& Property : PotentialMatches)
		{
			if (Property->ContainsBoundObjects({ InOuterObjects } ))
			{
				bFoundPropForObject = true;
				break;
			}
		}

		bAllObjectsExposed &= bFoundPropForObject;
	}

	return bAllObjectsExposed;
}

void SRemoteControlPanel::ExecutePropertyAction(const FRCExposesPropertyArgs& InPropertyArgs, const FString& InDesiredName)
{
	if (!ensure(InPropertyArgs.IsValid()))
	{
		return;
	}

	if (IsModeActive(ERCPanelMode::Signature))
	{
		SignaturePanel->AddToSignature(InPropertyArgs);
		return;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	if (IsExposed(InPropertyArgs))
	{
		FScopedTransaction Transaction(LOCTEXT("UnexposeProperty", "Unexpose Property"));
		Preset->Modify();
		Unexpose(InPropertyArgs);
		return;
	}

	auto PostExpose = [this, InPropertyArgs]()
	{
		CachedExposedPropertyArgs.Emplace(InPropertyArgs);
	};

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();
	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		TSet<UObject*> UniqueOuterObjects;
		{
			// Make sure properties are only being exposed once per object.
			TArray<UObject*> OuterObjects;
			InPropertyArgs.PropertyHandle->GetOuterObjects(OuterObjects);
			UniqueOuterObjects.Append(MoveTemp(OuterObjects));
		}

		if (UniqueOuterObjects.Num())
		{
			FScopedTransaction Transaction(LOCTEXT("ExposeProperty", "Expose Property"));
			Preset->Modify();

			for (UObject* Object : UniqueOuterObjects)
			{
				if (Object)
				{
					constexpr bool bCleanDuplicates = true; // GeneratePathToProperty duplicates container name (Array.Array[1], Set.Set[1], etc...)
					ExposeProperty(Object, FRCFieldPathInfo{ InPropertyArgs.PropertyHandle->GeneratePathToProperty(), bCleanDuplicates }, InDesiredName);
				}
			}

			PostExpose();
		}
	}
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		FScopedTransaction Transaction(LOCTEXT("ExposeProperty", "Expose Property"));
		Preset->Modify();

		constexpr bool bCleanDuplicates = true; // GeneratePathToProperty duplicates container name (Array.Array[1], Set.Set[1], etc...)
		ExposeProperty(InPropertyArgs.OwnerObject.Get(), FRCFieldPathInfo{InPropertyArgs.PropertyPath, bCleanDuplicates});

		PostExpose();
	}
}

FGuid SRemoteControlPanel::GetSelectedGroup() const
{
	if (TSharedPtr<SRCPanelTreeNode> Node = EntityList->GetSelectedGroup())
	{
		return Node->GetRCId();
	}
	return FGuid();
}

bool SRemoteControlPanel::CanActivateMode(ERCPanelMode InPanelMode) const
{
	if (InPanelMode == ERCPanelMode::OutputLog)
	{
		return FRemoteControlLogger::Get().IsEnabled();
	}
	return true;
}

bool SRemoteControlPanel::IsModeActive(ERCPanelMode InPanelMode) const
{
	return ActiveMode == InPanelMode;
}

void SRemoteControlPanel::SetActiveMode(ERCPanelMode InPanelMode)
{
	if (InPanelMode == ActiveMode)
	{
		return;
	}

	ActiveMode = InPanelMode;

	if (EntityList.IsValid())
	{
		EEntitiesListMode EntitiesListMode = EEntitiesListMode::Default;
		if (IsModeActive(ERCPanelMode::Protocols))
		{
			EntitiesListMode = EEntitiesListMode::Protocols;
		}
		EntityList->RebuildListWithColumns(EntitiesListMode);
	}
}

FReply SRemoteControlPanel::OnClickDisableUseLessCPU() const
{
	UEditorPerformanceSettings* Settings = GetMutableDefault<UEditorPerformanceSettings>();
	Settings->bThrottleCPUWhenNotForeground = false;
	Settings->PostEditChange();
	Settings->SaveConfig();
	return FReply::Handled();
}

FReply SRemoteControlPanel::OnClickIgnoreWarnings() const
{
	URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
	Settings->bIgnoreWarnings = true;
	Settings->PostEditChange();
	Settings->SaveConfig();
	return FReply::Handled();
}

TSharedRef<SWidget> SRemoteControlPanel::CreateCPUThrottleWarning() const
{
	FProperty* PerformanceThrottlingProperty = FindFieldChecked<FProperty>(UEditorPerformanceSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(UEditorPerformanceSettings, bThrottleCPUWhenNotForeground));
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PropertyName"), PerformanceThrottlingProperty->GetDisplayNameText());
	FText PerformanceWarningText = FText::Format(LOCTEXT("RemoteControlPerformanceWarning", "Warning: The editor setting '{PropertyName}' is currently enabled\nThis will stop editor windows from updating in realtime while the editor is not in focus"), Arguments);

	return SNew(SWarningOrErrorBox)
		.Visibility_Lambda([]() { return GetDefault<UEditorPerformanceSettings>()->bThrottleCPUWhenNotForeground && !GetDefault<URemoteControlSettings>()->bIgnoreWarnings ? EVisibility::Visible : EVisibility::Collapsed; })
		.MessageStyle(EMessageStyle::Warning)
		.Message(PerformanceWarningText)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &SRemoteControlPanel::OnClickDisableUseLessCPU)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("RemoteControlPerformanceWarningDisable", "Disable"))
			]
			+ SHorizontalBox::Slot()
			[
				SNew(SButton)
				.OnClicked(this, &SRemoteControlPanel::OnClickIgnoreWarnings)
				.TextStyle(FAppStyle::Get(), "DialogButtonText")
				.Text(LOCTEXT("RemoteControlIgnoreWarningEnable", "Ignore"))
			]
		];
}

TSharedRef<SWidget> SRemoteControlPanel::CreateProtectedIgnoredWarning() const
{
	FProperty* IgnoreProtectedCheckProperty = FindFieldChecked<FProperty>(URemoteControlSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlSettings, bIgnoreProtectedCheck));
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PropertyName"), IgnoreProtectedCheckProperty->GetDisplayNameText());
	FText ProtectedIgnoredWarningText = FText::Format(LOCTEXT("RemoteControlProtectedIgnoredWarning", "Warning: The editor setting '{PropertyName}' is currently enabled\nThis will let properties with the protected flag be let through for certain checks"), Arguments);

	return SNew(SWarningOrErrorBox)
		.Visibility_Lambda([](){ return GetDefault<URemoteControlSettings>()->bIgnoreProtectedCheck && !GetDefault<URemoteControlSettings>()->bIgnoreWarnings ? EVisibility::Visible : EVisibility::Collapsed; })
		.MessageStyle(EMessageStyle::Warning)
		.Message(ProtectedIgnoredWarningText)
		[
			SNew(SButton)
			.OnClicked(this, &SRemoteControlPanel::OnClickIgnoreWarnings)
			.TextStyle(FAppStyle::Get(), "DialogButtonText")
			.Text(LOCTEXT("RemoteControlIgnoreWarningEnable", "Ignore"))
		];
}

TSharedRef<SWidget> SRemoteControlPanel::CreateGetterSetterIgnoredWarning() const
{
	FProperty* IgnoreGetterSetterCheckProperty = FindFieldChecked<FProperty>(URemoteControlSettings::StaticClass(), GET_MEMBER_NAME_CHECKED(URemoteControlSettings, bIgnoreGetterSetterCheck));
	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("PropertyName"), IgnoreGetterSetterCheckProperty->GetDisplayNameText());
	FText ProtectedIgnoredWarningText = FText::Format(LOCTEXT("RemoteControlGetterSetterIgnoredWarning", "Warning: The editor setting '{PropertyName}' is currently enabled\nThis will let properties which have Getter/Setter let through for certain checks"), Arguments);

	return SNew(SWarningOrErrorBox)
		.Visibility_Lambda([](){ return GetDefault<URemoteControlSettings>()->bIgnoreGetterSetterCheck && !GetDefault<URemoteControlSettings>()->bIgnoreWarnings ? EVisibility::Visible : EVisibility::Collapsed; })
		.MessageStyle(EMessageStyle::Warning)
		.Message(ProtectedIgnoredWarningText)
		[
			SNew(SButton)
			.OnClicked(this, &SRemoteControlPanel::OnClickIgnoreWarnings)
			.TextStyle(FAppStyle::Get(), "DialogButtonText")
			.Text(LOCTEXT("RemoteControlIgnoreWarningEnable", "Ignore"))
		];
}

TSharedRef<SWidget> SRemoteControlPanel::CreateExposeFunctionsButton()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	SAssignNew(BlueprintPicker, SRCPanelFunctionPicker)
		.RemoteControlPanel(SharedThis(this))
		.AllowDefaultObjects(true)
		.Label(LOCTEXT("FunctionLibrariesLabel", "Function Libraries"))
		.ObjectClass(UBlueprintFunctionLibrary::StaticClass())
		.OnSelectFunction(this, &SRemoteControlPanel::ExposeFunction);

	SAssignNew(SubsystemFunctionPicker, SRCPanelFunctionPicker)
		.RemoteControlPanel(SharedThis(this))
		.Label(LOCTEXT("SubsystemFunctionLabel", "Subsystem Functions"))
		.ObjectClass(USubsystem::StaticClass())
		.OnSelectFunction(this, &SRemoteControlPanel::ExposeFunction);

	SAssignNew(ActorFunctionPicker, SRCPanelFunctionPicker)
		.RemoteControlPanel(SharedThis(this))
		.Label(LOCTEXT("ActorFunctionsLabel", "Actor Functions"))
		.ObjectClass(AActor::StaticClass())
		.OnSelectFunction(this, &SRemoteControlPanel::ExposeFunction);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ExposeHeader", "Expose"));
	{
		constexpr bool bNoIndent = true;
		constexpr bool bSearchable = false;

		auto CreatePickerSubMenu = [this, bNoIndent, bSearchable, &MenuBuilder] (const FText& Label, const FText& ToolTip, const TSharedRef<SWidget>& Widget)
		{
			MenuBuilder.AddSubMenu(
				Label,
				ToolTip,
				FNewMenuDelegate::CreateLambda(
					[this, bNoIndent, bSearchable, Widget](FMenuBuilder& MenuBuilder)
					{
						MenuBuilder.AddWidget(Widget, FText::GetEmpty(), bNoIndent, bSearchable);
						FSlateApplication::Get().SetKeyboardFocus(Widget, EFocusCause::Navigation);
					}
				)
			);
		};

		CreatePickerSubMenu(
			LOCTEXT("BlueprintFunctionLibraryFunctionSubMenu", "Blueprint Function Library Function"),
			LOCTEXT("FunctionLibraryFunctionSubMenuToolTip", "Expose a function from a blueprint function library."),
			BlueprintPicker.ToSharedRef()
		);

		CreatePickerSubMenu(
			LOCTEXT("SubsystemFunctionSubMenu", "Subsystem Function"),
			LOCTEXT("SubsystemFunctionSubMenuToolTip", "Expose a function from a subsytem."),
			SubsystemFunctionPicker.ToSharedRef()
		);

		CreatePickerSubMenu(
			LOCTEXT("ActorFunctionSubMenu", "Actor Function"),
			LOCTEXT("ActorFunctionSubMenuToolTip", "Expose an actor's function."),
			ActorFunctionPicker.ToSharedRef()
		);
	}

	MenuBuilder.EndSection();

	return SAssignNew(ExposeFunctionsComboButton, SComboButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Expose Functions")))
		.IsEnabled_Lambda([this]() { return !IsModeActive(ERCPanelMode::Live); })
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.CollapseMenuOnParentFocus(true)
		.HasDownArrow(false)
		.ContentPadding(FMargin(4.f, 2.f))
		.ButtonContent()
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle->IconSize.X)
			.HeightOverride(RCPanelStyle->IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("GraphEditor.Function_16x"))
			]
		]
		.MenuContent()
		[
			MenuBuilder.MakeWidget()
		];
}

TSharedRef<SWidget> SRemoteControlPanel::CreateExposeActorsButton()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ExposeHeader", "Expose"));
	{
		constexpr bool bNoIndent = true;
		constexpr bool bSearchable = false;

		auto CreatePickerSubMenu = [this, bNoIndent, bSearchable, &MenuBuilder] (const FText& Label, const FText& ToolTip, const TSharedRef<SWidget>& Widget)
		{
			MenuBuilder.AddSubMenu(
				Label,
				ToolTip,
				FNewMenuDelegate::CreateLambda(
					[this, bNoIndent, bSearchable, Widget](FMenuBuilder& MenuBuilder)
					{
						MenuBuilder.AddWidget(Widget, FText::GetEmpty(), bNoIndent, bSearchable);
						FSlateApplication::Get().SetKeyboardFocus(Widget, EFocusCause::Navigation);
					}
				)
			);
		};

		URemoteControlPreset* const Preset = GetPreset();

		// SObjectPropertyEntryBox does not support non-level editor worlds.
		if (!Preset || !Preset->IsEmbeddedPreset())
		{
			MenuBuilder.AddWidget(
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(AActor::StaticClass())
				.OnObjectChanged(this, &SRemoteControlPanel::OnExposeActor)
				.AllowClear(false)
				.DisplayUseSelected(true)
				.DisplayBrowse(true)
				.NewAssetFactories(TArray<UFactory*>()),
				LOCTEXT("ActorEntry", "Actor"));
		}
		else
		{
			MenuBuilder.AddSubMenu(
				LOCTEXT("SelectActor", "Select Actor"),
				LOCTEXT("SelectActorTooltip", "Select an actor to Remote Control."),
				FNewMenuDelegate::CreateSPLambda(this,
					[This=this](FMenuBuilder& SubMenuBuilder)
					{
						FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
						FSceneOutlinerInitializationOptions InitOptions;
						constexpr bool bAllowPIE = false;
						UWorld* PresetWorld = URemoteControlPreset::GetWorld(This->GetPreset(), bAllowPIE);

						SubMenuBuilder.AddWidget(
							SceneOutlinerModule.CreateActorPicker(
								InitOptions,
								FOnActorPicked::CreateSP(This, &SRemoteControlPanel::ExposeActor),
								PresetWorld
							),
							FText::GetEmpty(), true, false
						);
					}
				)
			);
		}

		CreatePickerSubMenu(
			LOCTEXT("ClassPickerEntry", "Actors By Class"),
			LOCTEXT("ClassPickerEntrySubMenuToolTip", "Expose all actors of the chosen class."),
			CreateExposeByClassWidget()
		);
	}

	MenuBuilder.EndSection();

	return SAssignNew(ExposeActorsComboButton, SComboButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Expose Actors")))
		.IsEnabled_Lambda([this]() { return !IsModeActive(ERCPanelMode::Live); })
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
		.ForegroundColor(FSlateColor::UseForeground())
		.CollapseMenuOnParentFocus(true)
		.HasDownArrow(false)
		.ContentPadding(FMargin(4.f, 2.f))
		.ButtonContent()
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle->IconSize.X)
			.HeightOverride(RCPanelStyle->IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("ClassIcon.Actor"))
			]
		]
		.MenuContent()
		[
			MenuBuilder.MakeWidget()
		];
}

TSharedRef<SWidget> SRemoteControlPanel::CreateExposeByClassWidget()
{
	class FActorClassInLevelFilter : public IClassViewerFilter
	{
	public:
		FActorClassInLevelFilter(const TSet<TWeakObjectPtr<const UClass>>& InClasses)
			: Classes(InClasses)
		{
		}

		virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return Classes.Contains(TWeakObjectPtr<const UClass>{InClass});
		}

		virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const class IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<class FClassViewerFilterFuncs> InFilterFuncs) override
		{
			return false;
		}

	public:
		const TSet<TWeakObjectPtr<const UClass>>& Classes;
	};

	TSharedPtr<FActorClassInLevelFilter> Filter = MakeShared<FActorClassInLevelFilter>(CachedClassesInLevel);

	FClassViewerInitializationOptions Options;
	{
		Options.ClassFilters.Add(Filter.ToSharedRef());
		Options.bIsPlaceableOnly = true;
		Options.Mode = EClassViewerMode::ClassPicker;
		Options.DisplayMode = EClassViewerDisplayMode::ListView;
		Options.bShowObjectRootClass = true;
		Options.bShowNoneOption = false;
		Options.bShowUnloadedBlueprints = false;
	}

	TSharedRef<SWidget> Widget = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSPLambda(this,
		[This=this](UClass* ChosenClass)
		{
			constexpr bool bAllowPIE = false;

			if (UWorld* World = URemoteControlPreset::GetWorld(This->GetPreset(), bAllowPIE))
			{
				for (TActorIterator<AActor> It(World, ChosenClass, EActorIteratorFlags::SkipPendingKill); It; ++It)
				{
					if (RemoteControlPanelUtils::IsExposableActor(*It))
					{
						This->ExposeActor(*It);
					}
				}
			}

			if (This->ExposeActorsComboButton)
			{
				This->ExposeActorsComboButton->SetIsOpen(false);
			}
		}));

	ClassPicker = StaticCastSharedRef<SClassViewer>(Widget);

	return SNew(SBox)
		.MinDesiredWidth(200.f)
		[
			Widget
		];
}

void SRemoteControlPanel::CacheLevelClasses()
{
	CachedClassesInLevel.Empty();
	constexpr bool bAllowPIE = false;

	if (UWorld* World = URemoteControlPreset::GetWorld(GetPreset(), bAllowPIE))
	{
		for (TActorIterator<AActor> It(World, AActor::StaticClass(), EActorIteratorFlags::SkipPendingKill); It; ++It)
		{
			CacheActorClass(*It);
		}

		if (ClassPicker)
		{
			ClassPicker->Refresh();
		}
	}
}

void SRemoteControlPanel::OnActorAddedToLevel(AActor* Actor)
{
	if (Actor)
	{
		CacheActorClass(Actor);
		if (ClassPicker)
		{
			ClassPicker->Refresh();
		}

		UpdateActorFunctionPicker();
	}
}

void SRemoteControlPanel::OnLevelActorsRemoved(AActor* Actor)
{
	if (Actor)
	{
		if (ClassPicker)
		{
			ClassPicker->Refresh();
		}

		UpdateActorFunctionPicker();
	}
}

void SRemoteControlPanel::OnLevelActorListChanged()
{
	UpdateActorFunctionPicker();
}

void SRemoteControlPanel::CacheActorClass(AActor* Actor)
{
	if (RemoteControlPanelUtils::IsExposableActor(Actor))
	{
		UClass* Class = Actor->GetClass();
		do
		{
			CachedClassesInLevel.Emplace(Class);
			Class = Class->GetSuperClass();
		}
		while(Class != UObject::StaticClass() && Class != nullptr);
	}
}

void SRemoteControlPanel::OnMapChange(uint32)
{
	CacheLevelClasses();

	if (ClassPicker)
	{
		ClassPicker->Refresh();
	}

	UpdateRebindButtonVisibility();

	// Clear the widget cache on map change to make sure we don't keep widgets around pointing to potentially stale objects.
	WidgetRegistry->Clear();

	const URemoteControlPreset* const Preset = GetPreset();

	// Update the world to the newer level editor map change
	// Only do this if the current selected world is now invalid (stale due to map change).
	// GWorld is used here instead of GEditor->EditorWorld as it's the one set to the newer world prior to the delegate being called (GEditor->EditorWorld set at a later time)
	if (GWorld && (!Preset || !Preset->SelectedWorld.IsValid()))
	{
		UpdatePanelForWorld(GWorld);
	}

	Refresh();
}

TSharedRef<SWidget> SRemoteControlPanel::BuildLogicModeContent(const TSharedRef<SWidget>& InLogicPanel)
{
	return SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda([this]() { return IsModeActive(ERCPanelMode::Live) ? 0 : 1; })
		+ SWidgetSwitcher::Slot()
		[
			InLogicPanel
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+SSplitter::Slot()
			.Value(0.4)
			[
				EntityList.ToSharedRef()
			]
			+SSplitter::Slot()
			.Value(0.6)
			[
				InLogicPanel
			]
		];
}

TSharedRef<SWidget> SRemoteControlPanel::BuildEntityDetailsModeContent(const TAttribute<float>& InRatioTop, const TAttribute<float>& InRatioBottom)
{
	return SNew(SSplitter)
		.Orientation(EOrientation::Orient_Vertical)
		// Exposed entities List
		+ SSplitter::Slot()
		.Value(InRatioTop)
		.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([](float InNewSize)
		{
			URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
			Settings->TreeBindingSplitRatio = InNewSize;
			Settings->PostEditChange();
			Settings->SaveConfig();
		}))
		[
			EntityList.ToSharedRef()
		]
		// Entity Details
		+ SSplitter::Slot()
		.Value(InRatioBottom)
		[
			CreateEntityDetailsView()
		];
}

TSharedRef<SWidget> SRemoteControlPanel::BuildProtocolsModeContent(const TAttribute<float>& InRatioTop, const TAttribute<float>& InRatioBottom)
{
	return SNew(SSplitter)
		.Orientation(EOrientation::Orient_Vertical)
		// Exposed entities List
		+ SSplitter::Slot()
		.Value(InRatioTop)
		.OnSlotResized(SSplitter::FOnSlotResized::CreateLambda([](float InNewSize)
		{
			URemoteControlSettings* Settings = GetMutableDefault<URemoteControlSettings>();
			Settings->TreeBindingSplitRatio = InNewSize;
			Settings->PostEditChange();
			Settings->SaveConfig();
		}))
		[
			EntityList.ToSharedRef()
		]
		// Protocol Details
		+ SSplitter::Slot()
		.Value(InRatioBottom)
		[
			EntityProtocolDetails.ToSharedRef()
		];
}

TSharedRef<SWidget> SRemoteControlPanel::BuildLiveModeContent(const TSharedRef<SWidget>& InLogicPanel)
{
	return SNew(SSplitter)
		.Orientation(EOrientation::Orient_Horizontal)
		// Exposed entities List
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			EntityList.ToSharedRef()
		]
		// Logic Panel
		+ SSplitter::Slot()
		.Value(0.5f)
		[
			InLogicPanel
		];
}

TSharedRef<SWidget> SRemoteControlPanel::BuildSignaturesModeContent()
{
	return SignaturePanel.ToSharedRef();
}

void SRemoteControlPanel::BindRemoteControlCommands()
{
	CommandList = MakeShared<FUICommandList>();

	const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

	CommandList->MapAction(
		Commands.SavePreset,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::SaveAsset),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SRemoteControlPanel::CanSaveAsset));

	CommandList->MapAction(
		FGlobalEditorCommonCommands::Get().FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SRemoteControlPanel::CanFindInContentBrowser));

	auto MapModeAction = [&CommandList = CommandList, This=this](const TSharedPtr<FUICommandInfo>& InCommand, ERCPanelMode InMode)
		{
			CommandList->MapAction(
				InCommand,
				FExecuteAction::CreateSP(This, &SRemoteControlPanel::SetActiveMode, InMode),
				FCanExecuteAction::CreateSP(This, &SRemoteControlPanel::CanActivateMode, InMode),
				FIsActionChecked::CreateSP(This, &SRemoteControlPanel::IsModeActive, InMode),
				FIsActionButtonVisible::CreateSP(This, &SRemoteControlPanel::CanActivateMode, InMode));
		};

	MapModeAction(Commands.ActivateLogicMode    , ERCPanelMode::Controller);
	MapModeAction(Commands.ActivateDetailsMode  , ERCPanelMode::EntityDetails);
	MapModeAction(Commands.ActivateSignatureMode, ERCPanelMode::Signature);
	MapModeAction(Commands.ActivateProtocolsMode, ERCPanelMode::Protocols);
	MapModeAction(Commands.ActivateOutputLogMode, ERCPanelMode::OutputLog);

	CommandList->MapAction(
		Commands.DeleteEntity,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::DeleteEntity_Execute),
		FCanExecuteAction::CreateSP(this, &SRemoteControlPanel::CanDeleteEntity));

	CommandList->MapAction(
		Commands.RenameEntity,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::RenameEntity_Execute),
		FCanExecuteAction::CreateSP(this, &SRemoteControlPanel::CanRenameEntity));

	CommandList->MapAction(
		Commands.ChangePropId,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::ChangePropertyId_Execute),
		FCanExecuteAction::CreateSP(this, &SRemoteControlPanel::CanChangePropertyId));

	CommandList->MapAction(
		Commands.CopyItem,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::CopyItem_Execute),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SRemoteControlPanel::CanCopyItem));

	CommandList->MapAction(
		Commands.PasteItem,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::PasteItem_Execute),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SRemoteControlPanel::CanPasteItem));

	CommandList->MapAction(
		Commands.DuplicateItem,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::DuplicateItem_Execute),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SRemoteControlPanel::CanDuplicateItem));

	CommandList->MapAction(
		Commands.UpdateValue,
		FExecuteAction::CreateSP(this, &SRemoteControlPanel::UpdateValue_Execute),
		FCanExecuteAction(),
		FGetActionCheckState(),
		FIsActionButtonVisible::CreateSP(this, &SRemoteControlPanel::CanUpdateValue));
}

void SRemoteControlPanel::OnObjectReplaced(const TMap<UObject*, UObject*>& InObjectReplaced)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!WidgetRegistry.IsValid() || !Preset || !Preset->SelectedWorld.IsValid())
	{
		return;
	}

	for (TWeakPtr<FRemoteControlField> FieldWeak : Preset->GetExposedEntities<FRemoteControlField>())
	{
		if (const TSharedPtr<FRemoteControlField> Field = FieldWeak.Pin())
		{
			const UObject* BoundObject = Field->GetBoundObjectForWorld(Preset->SelectedWorld.Get());
			for (const TPair<UObject*, UObject*>& ObjectReplaced : InObjectReplaced)
			{
				if (ObjectReplaced.Value == BoundObject)
				{
					WidgetRegistry->UpdateGeneratorAndTreeCache(ObjectReplaced.Key, ObjectReplaced.Value, Field->FieldPathInfo.ToPathPropertyString());
				}
			}
		}
	}
}

void SRemoteControlPanel::PostPIEStarted(const bool bInIsSimulating)
{
	URemoteControlPreset* const Preset = GetPreset();

	// Don't do anything when it's Simulate In Editor. This panel will only react when it's a proper Play In Editor
	if (!Preset || bInIsSimulating)
	{
		return;
	}

	// If the currently selected preset is already a PIE world, no need to override what's there. Return early
	const UWorld* PresetWorld = Preset->SelectedWorld.Get();
	if (PresetWorld && PresetWorld->WorldType == EWorldType::PIE)
	{
		return;
	}

	// Default to the first PIE World
	if (FWorldContext* PIEWorldContext = GEditor->GetWorldContextFromPIEInstance(0))
	{
		UpdatePanelForWorld(PIEWorldContext->World());
	}
}

void SRemoteControlPanel::PostLoadMapWithWorld(UWorld* InWorld)
{
	if (!InWorld || InWorld->WorldType != EWorldType::PIE)
	{
		return;
	}

	URemoteControlPreset* const Preset = GetPreset();

	const UWorld* const SelectedWorld = Preset ? Preset->SelectedWorld.Get() : nullptr;
	if (SelectedWorld && SelectedWorld->WorldType == EWorldType::PIE)
	{
		UpdatePanelForWorld(InWorld);
	}
}

void SRemoteControlPanel::OnEndPIE(const bool bInIsSimulating)
{
	// Don't do anything when it's Simulate In Editor. This panel will only react when it's a proper Play In Editor
	if (bInIsSimulating)
	{
		return;
	}

	if (URemoteControlPreset* const Preset = GetPreset())
	{
		// If the currently selected preset is not a PIE world when ending, no need to override what's there. Return early
		const UWorld* PresetWorld = Preset->SelectedWorld.Get();
		if (PresetWorld && PresetWorld->WorldType != EWorldType::PIE)
		{
			return;
		}
	}

	if (GEditor)
	{
		UpdatePanelForWorld(GEditor->EditorWorld);
	}
}

void SRemoteControlPanel::RegisterEvents()
{
	FEditorDelegates::MapChange.AddSP(this, &SRemoteControlPanel::OnMapChange);

	if (GEditor)
	{
		GEditor->OnBlueprintReinstanced().AddSP(this, &SRemoteControlPanel::OnBlueprintReinstanced);
	}

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().AddSP(this, &SRemoteControlPanel::OnActorAddedToLevel);
		GEngine->OnLevelActorListChanged().AddSP(this, &SRemoteControlPanel::OnLevelActorListChanged);
		GEngine->OnLevelActorDeleted().AddSP(this, &SRemoteControlPanel::OnLevelActorsRemoved);
	}

	if (URemoteControlPreset* Preset = GetPreset())
	{	
		Preset->OnEntityExposed().AddSP(this, &SRemoteControlPanel::OnEntityExposed);
		Preset->OnEntityUnexposed().AddSP(this, &SRemoteControlPanel::OnEntityUnexposed);
	}

	UMaterial::OnMaterialCompilationFinished().AddSP(this, &SRemoteControlPanel::OnMaterialCompiled);
	FCoreUObjectDelegates::OnObjectsReplaced.AddSP(this, &SRemoteControlPanel::OnObjectReplaced);

	FCoreUObjectDelegates::PostLoadMapWithWorld.AddSP(this, &SRemoteControlPanel::PostLoadMapWithWorld);
	FEditorDelegates::PostPIEStarted.AddSP(this, &SRemoteControlPanel::PostPIEStarted);
	FEditorDelegates::EndPIE.AddSP(this, &SRemoteControlPanel::OnEndPIE);
}

void SRemoteControlPanel::UnregisterEvents()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->OnEntityExposed().RemoveAll(this);
		Preset->OnEntityUnexposed().RemoveAll(this);
	}

	if (GEngine)
	{
		GEngine->OnLevelActorDeleted().RemoveAll(this);
		GEngine->OnLevelActorListChanged().RemoveAll(this);
		GEngine->OnLevelActorAdded().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->OnBlueprintReinstanced().RemoveAll(this);
	}

	FEditorDelegates::MapChange.RemoveAll(this);
	UMaterial::OnMaterialCompilationFinished().RemoveAll(this);
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	FCoreUObjectDelegates::PostLoadMapWithWorld.RemoveAll(this);
	FEditorDelegates::PostPIEStarted.RemoveAll(this);
	FEditorDelegates::EndPIE.RemoveAll(this);
}

void SRemoteControlPanel::Refresh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(SRemoteControlPanel::Refresh);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SRemoteControlPanel::RefreshBlueprintPicker);
		BlueprintPicker->Refresh();
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SRemoteControlPanel::RefreshActorFunctionPicker);
		ActorFunctionPicker->Refresh();
	}
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SRemoteControlPanel::RefreshSubsystemFunctionPicker);
		SubsystemFunctionPicker->Refresh();
	}

	EntityList->Refresh();

	if (ControllerPanel.IsValid())
	{
		ControllerPanel->Refresh();
	}
}

void SRemoteControlPanel::AddToolbarWidget(TSharedRef<SWidget> Widget)
{
	ToolbarWidgets.AddUnique(Widget);
}

void SRemoteControlPanel::RemoveAllToolbarWidgets()
{
	ToolbarWidgets.Empty();
}

void SRemoteControlPanel::Unexpose(const FRCExposesPropertyArgs& InPropertyArgs)
{
	if (!InPropertyArgs.IsValid())
	{
		return;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	auto CheckAndUnexpose = [Preset](TArray<UObject*> InOuterObjects, const FString& InPath, bool bInUsingDuplicatesInPath)
	{
		// Find an exposed property with the same path.
		TArray<TSharedPtr<FRemoteControlProperty>, TInlineAllocator<1>> PotentialMatches;
		for (const TWeakPtr<FRemoteControlProperty>& WeakProperty : Preset->GetExposedEntities<FRemoteControlProperty>())
		{
			if (TSharedPtr<FRemoteControlProperty> Property = WeakProperty.Pin())
			{
				// If that was exposed by property path it should be checked by the full path with duplicated like propertypath.propertypath[0]
				// If that was exposed by the owner object it should be without duplicated in the path, just propertypath[0]
				const bool bIsBound = bInUsingDuplicatesInPath ? Property->CheckIsBoundToPropertyPath(InPath) : Property->CheckIsBoundToString(InPath);
				if (bIsBound)
				{
					PotentialMatches.Add(Property);
				}
			}
		}

		for (const TSharedPtr<FRemoteControlProperty>& Property : PotentialMatches)
		{
			if (Property->ContainsBoundObjects(InOuterObjects))
			{
				Preset->Unexpose(Property->GetId());
			}
		}
	};

	const FRCExposesPropertyArgs::EType ExtensionArgsType = InPropertyArgs.GetType();

	if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_Handle)
	{
		TArray<UObject*> OuterObjects;
		InPropertyArgs.PropertyHandle->GetOuterObjects(OuterObjects);

		constexpr bool bUsingDuplicatesInPath = true;
		CheckAndUnexpose(OuterObjects, InPropertyArgs.PropertyHandle->GeneratePathToProperty(), bUsingDuplicatesInPath);
	}
	else if (ExtensionArgsType == FRCExposesPropertyArgs::EType::E_OwnerObject)
	{
		constexpr bool bUsingDuplicatesInPath = false;
		CheckAndUnexpose({ InPropertyArgs.OwnerObject.Get()}, InPropertyArgs.PropertyPath, bUsingDuplicatesInPath);
	}
}

void SRemoteControlPanel::OnLogCheckboxToggle(ECheckBoxState InState)
{
	const bool bEnableLog = InState == ECheckBoxState::Checked;
	FRemoteControlLogger::Get().EnableLog(bEnableLog);
	SetActiveMode(bEnableLog ? ERCPanelMode::OutputLog : ERCPanelMode::Controller);
}

void SRemoteControlPanel::OnBlueprintReinstanced()
{
	Refresh();
}

void SRemoteControlPanel::ExposeProperty(UObject* Object, FRCFieldPathInfo Path, FString InDesiredName)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (Preset && Path.Resolve(Object))
	{
		FRemoteControlPresetExposeArgs Args;
		Args.Label = InDesiredName;
		Args.GroupId = GetSelectedGroup();
		Preset->ExposeProperty(Object, MoveTemp(Path), MoveTemp(Args));
	}
}

void SRemoteControlPanel::ExposeFunction(UObject* Object, UFunction* Function)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	if (ExposeFunctionsComboButton)
	{
		ExposeFunctionsComboButton->SetIsOpen(false);
	}

	FScopedTransaction Transaction(LOCTEXT("ExposeFunction", "ExposeFunction"));
	Preset->Modify();

	FRemoteControlPresetExposeArgs Args;
	Args.GroupId = GetSelectedGroup();
	Preset->ExposeFunction(Object, Function, MoveTemp(Args));
}

void SRemoteControlPanel::OnExposeActor(const FAssetData& AssetData)
{
	ExposeActor(Cast<AActor>(AssetData.GetAsset()));
}

void SRemoteControlPanel::ExposeActor(AActor* Actor)
{
	if (!Actor)
	{
		return;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ExposeActor", "Expose Actor"));
	Preset->Modify();

	FRemoteControlPresetExposeArgs Args;
	Args.GroupId = GetSelectedGroup();

	Preset->ExposeActor(Actor, Args);
}

TSharedRef<SWidget> SRemoteControlPanel::CreateEntityDetailsView()
{
	WrappedEntityDetailsView = SNew(SBorder)
		.BorderImage(&RCPanelStyle->ContentAreaBrush)
		.Padding(RCPanelStyle->PanelPadding);

	// Details Panel Dock Widget.
	TSharedRef<SRCMinorPanel> EntityDetailsDockPanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("EntityDetailsLabel", "Details"))
		[
			WrappedEntityDetailsView.ToSharedRef()
		];

	// Details Panel Icon
	const TSharedRef<SWidget> DetailsIcon = SNew(SImage)
		.Image(FAppStyle::Get().GetBrush("LevelEditor.Tabs.Details"))
		.ColorAndOpacity(FSlateColor::UseForeground());

	EntityDetailsDockPanel->AddHeaderToolbarItem(EToolbar::Left, DetailsIcon);

	FDetailsViewArgs Args;
	Args.bShowOptions = false;
	Args.bAllowFavoriteSystem = false;
	Args.bAllowSearch = false;
	Args.bShowScrollBar = false;

	EntityDetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateStructureDetailView(MoveTemp(Args), FStructureDetailsViewArgs(), nullptr);

	UpdateEntityDetailsView(EntityList->GetSelectedEntity());

	const bool bCanShowDetailsView = LastSelectedEntity.IsValid() && (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::Group) && (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::FieldChild);

	if (bCanShowDetailsView)
	{
		if (ensure(EntityDetailsView && EntityDetailsView->GetWidget()))
		{
			WrappedEntityDetailsView->SetContent(EntityDetailsView->GetWidget().ToSharedRef());
		}
	}
	else
	{
		WrappedEntityDetailsView->SetContent(CreateNoneSelectedWidget());
	}

	return EntityDetailsDockPanel;
}

void SRemoteControlPanel::UpdateEntityDetailsView(const TSharedPtr<SRCPanelTreeNode>& SelectedNode)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	TSharedPtr<FStructOnScope> SelectedEntityPtr;

	LastSelectedEntity = SelectedNode;

	if (LastSelectedEntity)
	{
		if (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::Group &&
			LastSelectedEntity->GetRCType() != SRCPanelTreeNode::FieldChild &&
			LastSelectedEntity->GetRCType() != SRCPanelTreeNode::FieldGroup) // Field Child does not contain entity ID, that is why it should not be processed
		{
			const TSharedPtr<FRemoteControlEntity> Entity = Preset->GetExposedEntity<FRemoteControlEntity>(LastSelectedEntity->GetRCId()).Pin();
			SelectedEntityPtr = RemoteControlPanelUtils::GetEntityOnScope(Entity, Preset->GetExposedEntityType(LastSelectedEntity->GetRCId()));
		}
	}

	const bool bCanShowDetailsView = LastSelectedEntity.IsValid() && (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::Group) && (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::FieldChild);

	if (bCanShowDetailsView)
	{
		if (EntityDetailsView)
		{
			EntityDetailsView->SetStructureData(SelectedEntityPtr);

			WrappedEntityDetailsView->SetContent(EntityDetailsView->GetWidget().ToSharedRef());
		}
	}
	else
	{
		if (EntityDetailsView)
		{
			EntityDetailsView->SetStructureData(nullptr);
		}

		WrappedEntityDetailsView->SetContent(CreateNoneSelectedWidget());
	}

	static const FName ProtocolWidgetsModuleName = "RemoteControlProtocolWidgets";
	if(LastSelectedEntity.IsValid() && FModuleManager::Get().IsModuleLoaded(ProtocolWidgetsModuleName))
	{
		if (const TSharedPtr<FRemoteControlEntity> RCEntity = Preset->GetExposedEntity(LastSelectedEntity->GetRCId()).Pin())
		{
			if(RCEntity->IsBound())
			{
				IRemoteControlProtocolWidgetsModule& ProtocolWidgetsModule = FModuleManager::LoadModuleChecked<IRemoteControlProtocolWidgetsModule>(ProtocolWidgetsModuleName);
				EntityProtocolDetails->SetContent(ProtocolWidgetsModule.GenerateDetailsForEntity(Preset, RCEntity->GetId()));
			}
			else
			{
				EntityProtocolDetails->SetContent(CreateNoneSelectedWidget());
			}
		}
	}
	else
	{
		EntityProtocolDetails->SetContent(CreateNoneSelectedWidget());
	}

	// Trigger search to list the search results specific to selected group.
	if (EntityList.IsValid())
	{
		EntityList->UpdateSearch();
	}
}

void SRemoteControlPanel::UpdateRebindButtonVisibility()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		for (TWeakPtr<FRemoteControlEntity> WeakEntity : Preset->GetExposedEntities<FRemoteControlEntity>())
		{
			if (TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
			{
				if (!Entity->IsBound())
				{
					bShowRebindButton = true;
					return;
				}
			}
		}
	}

	bShowRebindButton = false;
}

FReply SRemoteControlPanel::OnClickRebindAllButton()
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		Preset->RebindUnboundEntities();

		UpdateRebindButtonVisibility();
	}
	return FReply::Handled();
}

void SRemoteControlPanel::UpdateActorFunctionPicker()
{
	if (GEditor && ActorFunctionPicker && !NextTickTimerHandle.IsValid())
	{
		NextTickTimerHandle = GEditor->GetTimerManager()->SetTimerForNextTick(FTimerDelegate::CreateLambda([WeakPanelPtr = TWeakPtr<SRemoteControlPanel>(StaticCastSharedRef<SRemoteControlPanel>(AsShared()))]()
		{
			if (TSharedPtr<SRemoteControlPanel> PanelPtr = WeakPanelPtr.Pin())
			{
				PanelPtr->ActorFunctionPicker->Refresh();
				PanelPtr->NextTickTimerHandle.Invalidate();
			}
		}));
	}
}

void SRemoteControlPanel::OnEntityExposed(URemoteControlPreset* InPreset, const FGuid& InEntityId)
{
	CachedExposedPropertyArgs.Empty();
}

void SRemoteControlPanel::OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InEntityId)
{
	CachedExposedPropertyArgs.Empty();
}

FReply SRemoteControlPanel::OnClickSettingsButton()
{
	FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "Remote Control");
	return FReply::Handled();
}

void SRemoteControlPanel::OnMaterialCompiled(UMaterialInterface* MaterialInterface)
{
	bMaterialsCompiledThisFrame = true;
}

void SRemoteControlPanel::TriggerMaterialCompiledRefresh()
{
	bool bTriggerRefresh = true;

	// Clear the widget cache on material compiled to make sure we have valid property nodes for IPropertyRowGenerator
	if (TSharedPtr<SWindow> Window = FSlateApplication::Get().FindWidgetWindow(SharedThis(this)))
	{
		const TArray<TSharedRef<SWindow>>& ChildWindows = Window->GetChildWindows();
		// Selecting a material from the RC window can trigger material recompiles,
		// If we refresh the widgets right away we might end up closing the material selection window while the user is
		// still selecting a material. Therefore we must wait until that window is closed before refreshing the panel.
		if (ChildWindows.Num())
		{
			bTriggerRefresh = false;

			if (!ChildWindows[0]->GetOnWindowClosedEvent().IsBound())
			{
				ChildWindows[0]->GetOnWindowClosedEvent().AddLambda([WeakThis = TWeakPtr<SRemoteControlPanel>(SharedThis(this))](const TSharedRef<SWindow>&)
					{
						if (TSharedPtr<SRemoteControlPanel> Panel = WeakThis.Pin())
						{
							Panel->WidgetRegistry->Clear();
							Panel->Refresh();
						}
					});
			}
		}
	}

	if (bTriggerRefresh)
	{
		WidgetRegistry->Clear();
		Refresh();
	}
}


void SRemoteControlPanel::RegisterDefaultToolBar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus->IsMenuRegistered(DefaultRemoteControlPanelToolBarName))
	{
		UToolMenu* ToolbarMenu = ToolMenus->RegisterMenu(DefaultRemoteControlPanelToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		check(ToolbarMenu);
		ToolbarMenu->StyleName = "AssetEditorToolbar";

		FToolMenuSection& AssetSection = ToolbarMenu->AddSection("Asset");
		AssetSection.AddEntry(FToolMenuEntry::InitToolBarButton(FRemoteControlCommands::Get().SavePreset, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("Icons.Save"))));
		AssetSection.AddEntry(FToolMenuEntry::InitToolBarButton(FGlobalEditorCommonCommands::Get().FindInContentBrowser));
		AssetSection.AddSeparator("Common");
	}
}

void SRemoteControlPanel::GenerateToolbar()
{
	RegisterDefaultToolBar();

	ToolbarWidgetContent = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(0)
		[
			SNullWidget::NullWidget
		];

	UToolMenus* ToolMenus = UToolMenus::Get();

	UToolMenu* GeneratedToolbar = ToolMenus->FindMenu(DefaultRemoteControlPanelToolBarName);

	GeneratedToolbar->Context = FToolMenuContext(CommandList);

	TSharedRef<class SWidget> ToolBarWidget = ToolMenus->GenerateWidget(GeneratedToolbar);

	TSharedRef<SWidget> MiscWidgets = SNullWidget::NullWidget;

	if (ToolbarWidgets.Num() > 0)
	{
		TSharedRef<SHorizontalBox> MiscWidgetsHBox = SNew(SHorizontalBox);

		for (int32 WidgetIdx = 0; WidgetIdx < ToolbarWidgets.Num(); ++WidgetIdx)
		{
			MiscWidgetsHBox->AddSlot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Fill)
				.AutoWidth()
				[
					SNew(SSeparator)
					.SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
					.Thickness(1.5f)
					.Orientation(EOrientation::Orient_Vertical)
				];

			MiscWidgetsHBox->AddSlot()
				.Padding(5.f, 0.f)
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					ToolbarWidgets[WidgetIdx]
				];
		}

		MiscWidgets = MiscWidgetsHBox;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (Preset && !Preset->SelectedWorld.IsValid())
	{
		Preset->SelectedWorld = Preset->GetWorld(true);
	}

	if (Preset && Preset->SelectedWorld.IsValid())
	{
		SelectedWorldName = Preset->SelectedWorld->StreamingLevelsPrefix + Preset->SelectedWorld->GetName();
	}
	else
	{
		SelectedWorldName = TEXT("WORLD NOT FOUND");
	}

	Toolbar =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ToolBarWidget
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(this, &SRemoteControlPanel::HandlePresetName)
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.AutoWidth()
		[
			SNew(SComboButton)
			.ToolTipText(LOCTEXT("RSWorldSelectorTooltip", "Select World to Control"))
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.CollapseMenuOnParentFocus(true)
			.HasDownArrow(false)
			.ContentPadding(FMargin(4.f, 1.f))
			.OnGetMenuContent(this, &SRemoteControlPanel::OnGetSelectedWorldButtonContent)
			.ButtonContent()
			[
				SNew(STextBlock) 
				.Text_Lambda([this] () { return FText::FromString(SelectedWorldName); })
			]
		]
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Center)
		.FillWidth(1.f)
		[
			SNew(SSpacer)
		]
		// Companion Separator (Rebind All)
		+SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			SNew(SSeparator)
			.SeparatorImage(FAppStyle::Get().GetBrush("Separator"))
			.Thickness(1.5f)
			.Orientation(EOrientation::Orient_Vertical)
			.Visibility_Lambda([this]() { return bShowRebindButton ? EVisibility::Visible : EVisibility::Collapsed; })
		]
		// Rebind All (Statically added here as we cannot probagagte visibility to its companion separator when dynamically added.)
		+ SHorizontalBox::Slot()
		.Padding(5.f, 0.f)
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(&RCPanelStyle->FlatButtonStyle)
			.Visibility_Lambda([this]() { return bShowRebindButton ? EVisibility::Visible : EVisibility::Collapsed; })
			.OnClicked(this, &SRemoteControlPanel::OnClickRebindAllButton)
			.ContentPadding(2.f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2.f)
				[
					SNew(SImage)
					.Image(FAppStyle::Get().GetBrush("Icons.Link"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.Padding(4.f, 2.f)
				.FillWidth(1.f)
				[
					SNew(STextBlock)
					.ToolTipText(LOCTEXT("RebindButtonToolTip", "Attempt to rebind all unbound entites of the preset."))
					.Text(LOCTEXT("RebindButtonText", "Rebind All"))
					.TextStyle(&RCPanelStyle->PanelTextStyle)
				]
			]
		]
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		.VAlign(VAlign_Fill)
		.AutoWidth()
		[
			MiscWidgets
		];

	if (ToolbarWidgetContent.IsValid())
	{
		ToolbarWidgetContent->SetContent(Toolbar.ToSharedRef());
	}
}

void SRemoteControlPanel::RegisterAuxiliaryToolBar()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(AuxiliaryRemoteControlPanelToolBarName))
	{
		return;
	}

	UToolMenu* ToolbarBuilder = ToolMenus->RegisterMenu(AuxiliaryRemoteControlPanelToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	if (!ensure(ToolbarBuilder))
	{
		return;
	}

	ToolbarBuilder->StyleName = "ContentBrowser.ToolBar";

	FToolMenuSection& ToolsSection = ToolbarBuilder->AddSection("Tools");

	auto AddModeEntry = [&ToolsSection](const TSharedPtr<FUICommandInfo>& InCommand, FSlateIcon InIcon)
		{
			ToolsSection.AddEntry(FToolMenuEntry::InitToolBarButton(InCommand, TAttribute<FText>(), TAttribute<FText>(), InIcon));
		};

	const FRemoteControlCommands& Commands = FRemoteControlCommands::Get();

	AddModeEntry(Commands.ActivateLogicMode    , FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GraphEditor.StateMachine_16x")));
	AddModeEntry(Commands.ActivateDetailsMode  , FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.Tabs.Details")));
	AddModeEntry(Commands.ActivateSignatureMode, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("GraphEditor.Function_16x")));
	AddModeEntry(Commands.ActivateProtocolsMode, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("LevelEditor.Tabs.StatsViewer")));
	AddModeEntry(Commands.ActivateOutputLogMode, FSlateIcon(FAppStyle::GetAppStyleSetName(), TEXT("MessageLog.TabIcon")));
}

void SRemoteControlPanel::GenerateAuxiliaryToolbar()
{
	RegisterAuxiliaryToolBar();

	AuxiliaryToolbarWidgetContent = SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		.Padding(0)
		[
			SNullWidget::NullWidget
		];

	UToolMenus* ToolMenus = UToolMenus::Get();

	UToolMenu* GeneratedToolbar = ToolMenus->FindMenu(AuxiliaryRemoteControlPanelToolBarName);

	GeneratedToolbar->Context = FToolMenuContext(CommandList);

	TSharedRef<class SWidget> ToolBarWidget = ToolMenus->GenerateWidget(GeneratedToolbar);

	AuxiliaryToolbar =
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			ToolBarWidget
		];

	if (AuxiliaryToolbarWidgetContent.IsValid())
	{
		AuxiliaryToolbarWidgetContent->SetContent(AuxiliaryToolbar.ToSharedRef());
	}
}

FText SRemoteControlPanel::HandlePresetName() const
{
	if (const URemoteControlPreset* const Preset = GetPreset())
	{
		return FText::FromString(Preset->GetName());
	}

	return FText::GetEmpty();
}

bool SRemoteControlPanel::CanSaveAsset() const
{
	const URemoteControlPreset* const Preset = GetPreset();
	return Preset && Preset->IsAsset();
}

void SRemoteControlPanel::SaveAsset() const
{
	if (const URemoteControlPreset* const Preset = GetPreset())
	{
		TArray<UPackage*> PackagesToSave;

		if (!Preset->IsAsset())
		{
			// Log an invalid object but don't try to save it
			UE_LOGF(LogRemoteControl, Log, "Invalid object to save: %ls. Preset is not an asset", *Preset->GetFullName());
		}
		else
		{
			PackagesToSave.Add(Preset->GetOutermost());
		}

		constexpr bool bCheckDirtyOnAssetSave = false;
		constexpr bool bPromptToSave = false;

		FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirtyOnAssetSave, bPromptToSave);
	}
}

bool SRemoteControlPanel::CanFindInContentBrowser() const
{
	return !!GetPreset();
}

void SRemoteControlPanel::FindInContentBrowser() const
{
	if (const URemoteControlPreset* Preset = GetPreset())
	{
		GEditor->SyncBrowserToObject(Preset);
	}
}

bool SRemoteControlPanel::ShouldForceSmallIcons()
{
	// Find the DockTab that houses this RemoteControlPreset widget in it.
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<FTabManager> EditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

	if (TSharedPtr<SDockTab> Tab = EditorTabManager->FindExistingLiveTab(FRemoteControlUIModule::RemoteControlPanelTabName))
	{
		if (TSharedPtr<SWindow> Window = Tab->GetParentWindow())
		{
			const FVector2D& ActualWindowSize = Window->GetSizeInScreen() / Window->GetDPIScaleFactor();

			// Need not to check for less than the minimum value as user can never go beyond that limit while resizing the parent window.
			return ActualWindowSize.X == SRemoteControlPanel::MinimumPanelWidth ? true : false;
		}
	}

	return false;
}

TSharedPtr<class SRCLogicPanelBase> SRemoteControlPanel::GetActiveLogicPanel() const
{
	if (ControllerPanel->IsListFocused())
	{
		return ControllerPanel;
	}
	else if (BehaviorPanel->IsListFocused())
	{
		return BehaviorPanel;
	}
	else if (ActionPanel->IsListFocused())
	{
		return ActionPanel;
	}
	else if (SignaturePanel->IsListFocused())
	{
		return SignaturePanel;
	}

	return nullptr;
}

void SRemoteControlPanel::DeleteEntity_Execute()
{
	// Currently used  as common entry point of Delete UI command for both RC Entity and Logic Items.
	// This could potentially be moved out if the Logic panels are moved to a separate tab.

	// ~ Delete Logic Item ~
	//
	// If the user focus is currently active on a Logic panel then route the Delete command to it and return.
	if (TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		ActiveLogicPanel->RequestDeleteSelectedItem();

		return; // handled
	}

	// ~ Delete Entity ~

	if (LastSelectedEntity->GetRCType() == SRCPanelTreeNode::FieldChild) // Field Child does not contain entity ID, that is why it should not be processed
	{
		return;
	}

	URemoteControlPreset* const Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	if (LastSelectedEntity->GetRCType() == SRCPanelTreeNode::Group)
	{
		FScopedTransaction Transaction(LOCTEXT("DeleteGroup", "Delete Group"));
		Preset->Modify();
		Preset->Layout.DeleteGroup(LastSelectedEntity->GetRCId());
	}
	else
	{
		FScopedTransaction Transaction(LOCTEXT("UnexposeFunction", "Unexpose remote control entity"));
		Preset->Modify();

		TArray<TSharedPtr<SRCPanelTreeNode>> EntitiesToDelete = EntityList->GetSelectedEntities();

		// Reverse the list as items will be processed from the back
		Algo::Reverse(EntitiesToDelete);

		while (!EntitiesToDelete.IsEmpty())
		{
			TSharedPtr<SRCPanelTreeNode> EntityToDelete = EntitiesToDelete.Pop();
			if (!EntityToDelete.IsValid())
			{
				continue;
			}

			const SRCPanelTreeNode::ENodeType RCType = EntityToDelete->GetRCType();
			if (RCType == SRCPanelTreeNode::FieldGroup)
			{
				// Deleting a Field Group would mean that all its children get deleted/unexposed.
				TArray<TSharedPtr<SRCPanelTreeNode>> Children;
				EntityToDelete->GetNodeChildren(Children);
				EntitiesToDelete.Append(MoveTemp(Children));
			}
			else if (RCType != SRCPanelTreeNode::FieldChild)
			{
				Preset->Unexpose(EntityToDelete->GetRCId());
			}
		}
	}

	EntityList->Refresh();
}

bool SRemoteControlPanel::CanDeleteEntity() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if (const TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		return !ActiveLogicPanel->GetSelectedLogicItems().IsEmpty(); // User has focus on a logic panel
	}

	const URemoteControlPreset* const Preset = GetPreset();
	if (LastSelectedEntity.IsValid() && Preset)
	{
		// Do not allow default group to be deleted.
		return !Preset->Layout.IsDefaultGroup(LastSelectedEntity->GetRCId());
	}

	return false;
}

void SRemoteControlPanel::RenameEntity_Execute() const
{
	if (ControllerPanel->IsListFocused())
	{
		ControllerPanel->EnterRenameMode();
		return;
	}

	if (SignaturePanel->IsListFocused())
	{
		SignaturePanel->EnterRenameMode();
		return;
	}

	if (LastSelectedEntity->GetRCType() == SRCPanelTreeNode::FieldChild ||
		LastSelectedEntity->GetRCType() == SRCPanelTreeNode::FieldGroup) // Field Child/Group does not contain entity ID, that is why it should not be processed
	{
		return;
	}

	LastSelectedEntity->EnterRenameMode();
}

bool SRemoteControlPanel::CanRenameEntity() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if (ControllerPanel->IsListFocused())
	{
		return true;
	}

	if (SignaturePanel->IsListFocused())
	{
		return true;
	}

	const URemoteControlPreset* const Preset = GetPreset();
	if (LastSelectedEntity.IsValid() && Preset)
	{
		// Do not allow default group to be renamed.
		return !Preset->Layout.IsDefaultGroup(LastSelectedEntity->GetRCId());
	}

	return false;
}

int32 SRemoteControlPanel::NumControllerItems() const
{
	if (ControllerPanel.IsValid())
	{
		return ControllerPanel->NumControllerItems();
	}
	return INDEX_NONE;
}

void SRemoteControlPanel::ChangePropertyId_Execute() const
{
	if (!LastSelectedEntity.IsValid())
	{
		return;
	}

	if (LastSelectedEntity->GetRCType() == SRCPanelTreeNode::Field)
	{
		LastSelectedEntity->FocusPropertyIdWidget();
	}
}

bool SRemoteControlPanel::CanChangePropertyId() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if (!LastSelectedEntity.IsValid())
	{
		return false;
	}

	if (LastSelectedEntity->GetRCType() != SRCPanelTreeNode::Field)
	{
		return false;
	}

	return true;
}

void SRemoteControlPanel::SetLogicClipboardItems(const TArray<UObject*>& InItems, const TSharedPtr<SRCLogicPanelBase>& InSourcePanel)
{
	LogicClipboardItems = InItems;
	LogicClipboardItemSource = InSourcePanel;
}

void SRemoteControlPanel::CopyItem_Execute()
{
	if (const TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		ActiveLogicPanel->CopySelectedPanelItems();
	}
}

bool SRemoteControlPanel::CanCopyItem() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if (const TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		return ActiveLogicPanel->CanCopyItems();
	}

	return false;
}

void SRemoteControlPanel::PasteItem_Execute()
{
	if (const TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		ActiveLogicPanel->PasteItemsFromClipboard();
	}
}

bool SRemoteControlPanel::CanPasteItem() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if(!LogicClipboardItems.IsEmpty())
	{
		if (const TSharedPtr<SRCLogicPanelBase>& ActiveLogicPanel = GetActiveLogicPanel())
		{
			// Currently we only support pasting items between panels of exactly the same type.
			if (LogicClipboardItemSource == ActiveLogicPanel)
			{
				return ActiveLogicPanel->CanPasteClipboardItems(LogicClipboardItems);
			}
		}
	}

	return false;
}

void SRemoteControlPanel::DuplicateItem_Execute()
{
	if (const TSharedPtr<SRCLogicPanelBase>& ActiveLogicPanel = GetActiveLogicPanel())
	{
		ActiveLogicPanel->DuplicateSelectedPanelItems();

		return;
	}
}

bool SRemoteControlPanel::CanDuplicateItem() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}

	if (const TSharedPtr<SRCLogicPanelBase>& ActiveLogicPanel = GetActiveLogicPanel())
	{
		return ActiveLogicPanel->CanDuplicateItems();
	}

	return false;
}

void SRemoteControlPanel::UpdateValue_Execute()
{
	if (const TSharedPtr<SRCLogicPanelBase>& ActiveLogicPanel = GetActiveLogicPanel())
	{
		ActiveLogicPanel->UpdateValue();
	}
}

bool SRemoteControlPanel::CanUpdateValue() const
{
	if (IsModeActive(ERCPanelMode::Live))
	{
		return false;
	}
	
	if (const TSharedPtr<SRCLogicPanelBase> ActiveLogicPanel = GetActiveLogicPanel())
	{
		return ActiveLogicPanel->CanUpdateValue();
	}

	return false;
}

void SRemoteControlPanel::LoadSettings(const FGuid& InInstanceId) const
{
	const FString SettingsString = InInstanceId.ToString();

	// Load all our data using the settings string as a key in the user settings ini.
	const TSharedPtr<SRCPanelFilter> FilterPtr = EntityList->GetFilterPtr();
	if (FilterPtr.IsValid())
	{
		FilterPtr->LoadSettings(GEditorPerProjectIni, IRemoteControlUIModule::SettingsIniSection, SettingsString);
	}
}

void SRemoteControlPanel::SaveSettings()
{
	URemoteControlPreset* const Preset = GetPreset();

	const TSharedPtr<SRCPanelFilter> FilterPtr = EntityList->GetFilterPtr();
	if (Preset && FilterPtr.IsValid())
	{
		const FString SettingsString = Preset->GetPresetId().ToString();

		// Save all our data using the settings string as a key in the user settings ini.
		FilterPtr->SaveSettings(GEditorPerProjectIni, IRemoteControlUIModule::SettingsIniSection, SettingsString);
	}
}

TSharedRef<SWidget> SRemoteControlPanel::OnGetSelectedWorldButtonContent()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ensure(ToolMenus))
	{
		return SNullWidget::NullWidget;
	}

	if (!ToolMenus->IsMenuRegistered(TargetWorldRemoteControlPanelMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(TargetWorldRemoteControlPanelMenuName, NAME_None, EMultiBoxType::Menu);

		if (!ensure(Menu))
		{
			return SNullWidget::NullWidget;
		}

		Menu->AddDynamicSection(TEXT("Worlds"), FNewToolMenuDelegate::CreateStatic(&CreateTargetWorldButtonDynamicEntries));
	}

	URemoteControlPanelMenuContext* RemoteControlContext = NewObject<URemoteControlPanelMenuContext>();
	RemoteControlContext->RemoteControlPanel = SharedThis(this);

	const FToolMenuContext Context(RemoteControlContext);
	return ToolMenus->GenerateWidget(TargetWorldRemoteControlPanelMenuName, Context);
}

void SRemoteControlPanel::UpdatePanelForWorld(const UWorld* InWorld)
{
	URemoteControlPreset* const Preset = GetPreset();
	if (!IsValid(InWorld) || (Preset && InWorld == Preset->SelectedWorld.Get()))
	{
		return;
	}

	if (!Preset || Preset->IsEmbeddedPreset())
	{
		OpenPanelForEmbeddedPreset(InWorld);
	}
	else
	{
		// Map of old world objects to new target world objects
		TMap<UObject*, UObject*> OldToNewObject;
		for (const TObjectPtr<URemoteControlBinding>& Binding : Preset->Bindings)
		{
			UObject* OldObject = Binding->ResolveForWorld(Preset->SelectedWorld.Get());
			UObject* NewObject = Binding->ResolveForWorld(InWorld);
			OldToNewObject.Add(OldObject, NewObject);
		}

		for (TWeakPtr<FRemoteControlField> FieldWeak : Preset->GetExposedEntities<FRemoteControlField>())
		{
			if (const TSharedPtr<FRemoteControlField> Field = FieldWeak.Pin())
			{
				// Get the bound object in the correct world
				UObject* BoundObject = Field->GetBoundObjectForWorld(Preset->SelectedWorld.Get());
				if (UObject** NewObject = OldToNewObject.Find(BoundObject))
				{
					WidgetRegistry->UpdateGeneratorAndTreeCache(BoundObject, *NewObject, Field->FieldPathInfo.ToPathPropertyString());
				}
			}
		}

		if (EntityList.IsValid())
		{
			EntityList->Refresh();
		}

		SelectedWorldName = InWorld->StreamingLevelsPrefix + InWorld->GetName();
		Preset->SelectedWorld = InWorld;
	}
}

void SRemoteControlPanel::OpenPanelForEmbeddedPreset(const UWorld* InWorld)
{
	UAssetEditorSubsystem* const AssetSubsystem = GEditor ? GEditor->GetEditorSubsystem<UAssetEditorSubsystem>() : nullptr;
	if (!AssetSubsystem)
	{
		return;
	}

	IRemoteControlModule& RCModule = IRemoteControlModule::Get();

	TArray<TWeakObjectPtr<URemoteControlPreset>> EmbeddedPresets;
	RCModule.GetEmbeddedPresets(EmbeddedPresets);

	URemoteControlPreset* SelectedEmbeddedPreset = nullptr;

	const ULevel* const SelectedLevel = InWorld ? InWorld->GetCurrentLevel() : nullptr;

	if (SelectedLevel)
	{
		for (TWeakObjectPtr<URemoteControlPreset> EmbeddedPresetWeak : EmbeddedPresets)
		{
			URemoteControlPreset* const EmbeddedPreset = EmbeddedPresetWeak.Get();
			if (EmbeddedPreset && EmbeddedPreset->GetTypedOuter<ULevel>() == SelectedLevel)
			{
				SelectedEmbeddedPreset = EmbeddedPreset;
				break;
			}
		}
	}

	AssetSubsystem->OpenEditorForAsset(SelectedEmbeddedPreset);
}

void SRemoteControlPanel::CreateTargetWorldButtonDynamicEntries(UToolMenu* InMenu)
{
	const URemoteControlPanelMenuContext* Context = InMenu->FindContext<URemoteControlPanelMenuContext>();

	if (!Context || !Context->RemoteControlPanel.IsValid())
	{
		return;
	}

	const TSharedPtr<SRemoteControlPanel> CurrentPanel = Context->RemoteControlPanel.Pin();

	TSet<const UWorld*> Worlds;
	if (GEngine)
	{
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			const UWorld* CurrentWorld = WorldContext.World();
			if (CurrentWorld && (CurrentWorld->WorldType == EWorldType::Editor || CurrentWorld->WorldType == EWorldType::PIE))
			{
				Worlds.Add(CurrentWorld);
			}
		}
	}

	if (!Worlds.IsEmpty())
	{
		FToolMenuSection& Section = InMenu->FindOrAddSection(TEXT("Worlds"));
		for (const UWorld* World : Worlds)
		{
			Section.AddMenuEntry(
				FName(World->StreamingLevelsPrefix + World->GetName()),
				FText::FromString(World->StreamingLevelsPrefix + World->GetName()),
				LOCTEXT("RCSetTargetWorld_Tooltip", "Set this world as the target world for RC"),
				FSlateIcon(),
				FExecuteAction::CreateSP(CurrentPanel.Get(), &SRemoteControlPanel::UpdatePanelForWorld, World)
				);
		}
	}
}

void SRemoteControlPanel::DeleteEntity()
{
	if (CanDeleteEntity())
	{
		DeleteEntity_Execute();
	}
}

void SRemoteControlPanel::RenameEntity()
{
	if (CanRenameEntity())
	{
		RenameEntity_Execute();
	}
}


#undef LOCTEXT_NAMESPACE /*RemoteControlPanel*/
