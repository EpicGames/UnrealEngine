// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Engine/EngineTypes.h"
#include "Engine/EngineTypes.h"
#include "Filters/SFilterBar.h"
#include "IDetailTreeNode.h"
#include "Misc/TextFilter.h"
#include "RemoteControlUIModule.h"
#include "Styling/SlateTypes.h"
#include "RemoteControlFieldPath.h"
#include "UObject/StrongObjectPtr.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

/** The variety of panels we have in the RC Panel. */
enum class ERCPanelMode : uint8
{
	Controller,

	EntityDetails,

	Protocols,

	OutputLog,

	Live,

	Signature,
};

class AActor;
class FExposedEntityDragDrop;
class FRCBehaviourModel;
class FRCControllerModel;
class FRCPanelWidgetRegistry;
class FReply;
class FUICommandList;
class IPropertyHandle;
class IPropertyRowGenerator;
class IToolkitHost;
class SBox;
class SClassViewer;
class SComboButton;
class SRCActionPanel;
class SRCBehaviourPanel;
class SRCControllerPanel;
class SRCLogicPanelBase;
class SRCPanelDrawer;
class SRCPanelExposedEntitiesList;
class SRCPanelFilter;
class SRCPanelFunctionPicker;
class SRCSignaturePanel;
class SRemoteControlPanel;
class SSearchBox;
class STextBlock;
class URCAction;
class URCBehaviour;
class URCController;
class URemoteControlPreset;
struct EVisibility;
struct FAssetData;
struct FListEntry;
struct FRCPanelDrawerArgs;
struct FRCPanelStyle;
struct FRemoteControlEntity;
struct SRCPanelTreeNode;

DECLARE_DELEGATE_TwoParams(FOnLiveModeChange, TSharedPtr<SRemoteControlPanel> /* Panel */, bool /* bEditModeChange */);

/**
 * UI representation of a remote control preset.
 * Allows a user to expose/unexpose properties and functions from actors and blueprint libraries.
 */
class SRemoteControlPanel : public SCompoundWidget, public FGCObject
{
	SLATE_BEGIN_ARGS(SRemoteControlPanel) {}
		SLATE_EVENT(FOnLiveModeChange, OnLiveModeChange)
		SLATE_ARGUMENT(bool, AllowGrouping)
	SLATE_END_ARGS()

public:
	// Remote Control Logic Delegates
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnControllerAdded, const FName& /* InPropertyName */);
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnControllerSelectionChanged, TSharedPtr<FRCControllerModel> /* InControllerItem */, ESelectInfo::Type /* Select Type Info */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnControllerValueChanged, TSharedPtr<FRCControllerModel> /* InControllerItem */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBehaviourAdded, const URCBehaviour* /* InBehaviour */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnBehaviourSelectionChanged, TSharedPtr<FRCBehaviourModel> /* InBehaviourItem */);
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnActionAdded, URCAction* /* InAction */);

	DECLARE_MULTICAST_DELEGATE(FOnEmptyControllers);
	DECLARE_MULTICAST_DELEGATE(FOnEmptyBehaviours);
	DECLARE_MULTICAST_DELEGATE(FOnEmptyActions);

	void Construct(const FArguments& InArgs, URemoteControlPreset* InPreset, TSharedPtr<IToolkitHost> InToolkitHost);
	~SRemoteControlPanel();

	//~ Begin SWidget interface
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime);
	virtual FReply OnKeyDown(const FGeometry& InGeometry, const FKeyEvent& InKeyEvent) override;
	//~ End SWidget interface

	/**
	 * @return The preset represented by the panel.
	 */
	URemoteControlPreset* GetPreset()
	{
		return PresetWeak.Get();
	}

	/**
	 * @return The preset represented by the panel.
	 */
	const URemoteControlPreset* GetPreset() const
	{
		return PresetWeak.Get();
	}

	/**
	 * @param InArgs extension arguments
	 * @return Whether a property is exposed or not.
	 */
	bool IsExposed(const FRCExposesPropertyArgs& InArgs);

	/**
	 * @param InOuterObjects outer objects to check
	 * @param InPropertyPath full property path
	 * @param bUsingDuplicatesInPath whether duplications like property.property[1] should be exists or just property[1]
	 * @return Whether objects is exposed or not.
	 */
	bool IsAllObjectsExposed(TArray<UObject*> InOuterObjects, const FString& InPropertyPath, bool bUsingDuplicatesInPath);

	/**
	 * Exposes or unexposes a property.
	 * @param InArgs The extension arguments of the property to toggle.
	 * @param InDesiredName Desired Name for this property if leaved empty RC will deduce it itself
	 */
	void ExecutePropertyAction(const FRCExposesPropertyArgs& InArgs, const FString& InDesiredName = TEXT(""));

	/**
	 * Get the selected group.
	 */
	FGuid GetSelectedGroup() const;

	bool CanActivateMode(ERCPanelMode InPanelMode) const;
	bool IsModeActive(ERCPanelMode InPanelMode) const;
	void SetActiveMode(ERCPanelMode InPanelMode);

	/**
	 * Get the exposed entity list.
	 */
	TSharedPtr<SRCPanelExposedEntitiesList> GetEntityList() { return EntityList; }

	/** Re-create the sections of the panel. */
	void Refresh();

	/** Adds or removes widgets from the default toolbar in this asset editor */
	void AddToolbarWidget(TSharedRef<SWidget> Widget);
	void RemoveAllToolbarWidgets();

	/** Public Workaround Delete for Key Handling Issues when Docked */
	void DeleteEntity();
	void RenameEntity();

	/** Retrieves the Logic Action panel. */
	TSharedPtr<SRCActionPanel> GetLogicActionPanel()
	{
		return ActionPanel;
	}

	TSharedPtr<FUICommandList> GetCommandList() const
	{
		return CommandList;
	}

	/** Retrieves the number of controllers. */
	int32 NumControllerItems() const;

	/** For Copy UI command - Sets the logic clipboard item and source */
	void SetLogicClipboardItems(const TArray<UObject*>& InItems, const TSharedPtr<SRCLogicPanelBase>& SourcePanel);

	/** Fetches the last UI item copied to Logic clipboard by the user */
	TArray<UObject*> GetLogicClipboardItems()
	{
		return LogicClipboardItems;
	}

	// FGCObject interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

private:
	/** Applies all the protocol bindings within this widget's owning preset to their respective protocols */
	void ApplyProtocolBindings();

	/** Unapplies all the protocol bindings within this widget's owning preset to their respective protocols */
	void UnapplyProtocolBindings();

	/** Returns a Helper widget for entity details view and protocol details view. */
	static TSharedRef<SBox> CreateNoneSelectedWidget();

	TSharedRef<SWidget> BuildLogicModeContent(const TSharedRef<SWidget>& InLogicPanel);
	TSharedRef<SWidget> BuildEntityDetailsModeContent(const TAttribute<float>& InRatioTop, const TAttribute<float>& InRatioBottom);
	TSharedRef<SWidget> BuildProtocolsModeContent(const TAttribute<float>& InRatioTop, const TAttribute<float>& InRatioBottom);
	TSharedRef<SWidget> BuildLiveModeContent(const TSharedRef<SWidget>& InLogicPanel);
	TSharedRef<SWidget> BuildSignaturesModeContent();

	//~ Remote Control Commands
	void BindRemoteControlCommands();

	/** Called when object are replaced, blueprint for example */
	void OnObjectReplaced(const TMap<UObject*, UObject*>& InObjectReplaced);

	/** Called when PIE begins */
	void PostPIEStarted(const bool bInIsSimulating);

	/** Called at the end of LoadMap to keep the remote control preset panel up-to-date with current PIE world */
	void PostLoadMapWithWorld(UWorld* InWorld);

	/** Called when PIE ends */
	void OnEndPIE(const bool bInIsSimulating);

	/** Register editor events needed to handle reloading objects and blueprint libraries. */
	void RegisterEvents();
	/** Unregister editor events */
	void UnregisterEvents();

	/** Unexpose a field from the preset. */
	void Unexpose(const FRCExposesPropertyArgs& InArgs);

	/** Handler called when a blueprint is reinstanced. */
	void OnBlueprintReinstanced();

	/** Expose a property using its handle. */
	void ExposeProperty(UObject* Object, FRCFieldPathInfo FieldPath, FString InDesiredName = TEXT(""));

	/** Exposes a function.  */
	void ExposeFunction(UObject* Object, UFunction* Function);

	/** Handles exposing an actor from asset data. */
	void OnExposeActor(const FAssetData& AssetData);

	/** Handle exposing an actor. */
	void ExposeActor(AActor* Actor);

	/** Handles disabling CPU throttling. */
	FReply OnClickDisableUseLessCPU() const;

	/** Handles ignoring Warnings. */
	FReply OnClickIgnoreWarnings() const;

	/** Creates a widget that warns the user when CPU throttling is enabled.  */
	TSharedRef<SWidget> CreateCPUThrottleWarning() const;

	/** Creates a widget that warns the user when the settings are currently ignoring the Protected flag is enabled. */
	TSharedRef<SWidget> CreateProtectedIgnoredWarning() const;

	/** Creates a widget that warns the user when the settings are currently ignoring the Getter/Setter is enabled. */
	TSharedRef<SWidget> CreateGetterSetterIgnoredWarning() const;

	/** Create expose button, allowing to expose blueprints and actor functions. */
	TSharedRef<SWidget> CreateExposeFunctionsButton();

	/** Create expose button, allowing to expose actors. */
	TSharedRef<SWidget> CreateExposeActorsButton();

	/** Create expose by class menu content */
	TSharedRef<SWidget> CreateExposeByClassWidget();

	/** Cache the classes (and parent classes) of all actors in the level. */
	void CacheLevelClasses();

	//~ Handlers for various level actor events.
	void OnActorAddedToLevel(AActor* Actor);
	void OnLevelActorsRemoved(AActor* Actor);
	void OnLevelActorListChanged();

	/** Handles caching an actor's class and parent classes. */
	void CacheActorClass(AActor* Actor);

	/** Handles refreshing the class picker when the map is changed. */
	void OnMapChange(uint32);

	/** Create the details view for the entity currently selected. */
	TSharedRef<SWidget> CreateEntityDetailsView();

	/** Update the details view following entity selection change.  */
	void UpdateEntityDetailsView(const TSharedPtr<SRCPanelTreeNode>& SelectedNode);

	/** Returns whether the preset has any unbound property or function. */
	void UpdateRebindButtonVisibility();

	/** Handle user clicking on the rebind all button. */
	FReply OnClickRebindAllButton();

	//~ Handlers called in order to clear the exposed property cache.
	void OnEntityExposed(URemoteControlPreset* InPreset, const FGuid& InEntityId);
	void OnEntityUnexposed(URemoteControlPreset* InPreset, const FGuid& InEntityId);

	/** Toggles the logging part of UI */
	void OnLogCheckboxToggle(ECheckBoxState InState);

	/** Triggers a next frame update of the actor function picker to ensure that added actors are valid. */
	void UpdateActorFunctionPicker();

	/** Handle clicking on the setting button. */
	FReply OnClickSettingsButton();

	/** Handle triggers each time when any material has been recompiled */
	void OnMaterialCompiled(class UMaterialInterface* MaterialInterface);

	/* Refreshed called at max once per frame when a material is compiled. */
	void TriggerMaterialCompiledRefresh();

	/** Registers default tool bar */
	static void RegisterDefaultToolBar();

	/** Makes a default asset editing toolbar */
	void GenerateToolbar();

	/** Registers Auxiliary tool bar */
	static void RegisterAuxiliaryToolBar();

	/** Makes a Auxiliary asset editing toolbar */
	void GenerateAuxiliaryToolbar();

	FText HandlePresetName() const;

	/** Called to test if "Save" should be enabled for this asset */
	bool CanSaveAsset() const;

	/** Called when "Save" is clicked for this asset */
	void SaveAsset() const;

	/** Called to test if "Find in Content Browser" should be enabled for this asset */
	bool CanFindInContentBrowser() const;

	/** Called when "Find in Content Browser" is clicked for this asset */
	void FindInContentBrowser() const;

	static bool ShouldForceSmallIcons();

	/** Called when user attempts to delete a group/exposed entity. */
	void DeleteEntity_Execute();

	/** Called to test if user is able to delete a group/exposed entity. */
	bool CanDeleteEntity() const;

	/** Called when user attempts to rename a group/exposed entity. */
	void RenameEntity_Execute() const;

	/** Called to test if user is able to rename a group/exposed entity. */
	bool CanRenameEntity() const;

	/** Called when user attempts to change property Ids. */
	void ChangePropertyId_Execute() const;

	/** Called to test if user is able to change property Ids. */
	bool CanChangePropertyId() const;

	/** Called when user attempts to Copy a logic UI item. */
	void CopyItem_Execute();

	/** Called to test if user is able to Copy a logic UI item. */
	bool CanCopyItem() const;

	/** Called when user attempts to Paste a logic UI item. */
	void PasteItem_Execute();

	/** Called to test if user is able to Paste a logic UI item. */
	bool CanPasteItem() const;

	/** Called when user attempts to Duplicate a logic UI item. */
	void DuplicateItem_Execute();

	/** Called to test if user is able to Duplicate a logic UI item. */
	bool CanDuplicateItem() const;

	/** Called when user attempts to Update a logic UI item. */
	void UpdateValue_Execute();

	/** Called to test if user is able to Update a logic UI item. */
	bool CanUpdateValue() const;

	/** Loads settings from config based on the preset identifier. */
	void LoadSettings(const FGuid& InInstanceId) const;

	/** Saves settings from config based on the preset identifier. */
	void SaveSettings();

	/** Retrieves active logic panel. */
	TSharedPtr<SRCLogicPanelBase> GetActiveLogicPanel() const;

	/** Get the menu content for the SelectedWorld button */
	TSharedRef<SWidget> OnGetSelectedWorldButtonContent();

	/** Update the panel for the new World */
	void UpdatePanelForWorld(const UWorld* InWorld);

	/** Handle the TargetWorld change for embedded presets */
	void OpenPanelForEmbeddedPreset(const UWorld* World);

	/** Handle the TargetWorld ComboButton entries creation */
	static void CreateTargetWorldButtonDynamicEntries(UToolMenu* InMenu);

private:
	static const FName DefaultRemoteControlPanelToolBarName;
	static const FName AuxiliaryRemoteControlPanelToolBarName;
	static const FName TargetWorldRemoteControlPanelMenuName;
	/** Holds a weak reference to the preset asset. */
	TWeakObjectPtr<URemoteControlPreset> PresetWeak;
	/** Command list of this panel */
	TSharedPtr<FUICommandList> CommandList;
	/** Delegate called when the live mode changes. */
	FOnLiveModeChange OnLiveModeChange;
	/** Holds the blueprint library picker */
	TSharedPtr<SRCPanelFunctionPicker> BlueprintPicker;
	/** Holds the actor function picker */
	TSharedPtr<SRCPanelFunctionPicker> ActorFunctionPicker;
	/** Holds the subsystem function picker. */
	TSharedPtr<SRCPanelFunctionPicker> SubsystemFunctionPicker;
	/** Holds the exposed entity list view. */
	TSharedPtr<SRCPanelExposedEntitiesList> EntityList;
	/** Holds the combo button that allows exposing functions. */
	TSharedPtr<SComboButton> ExposeFunctionsComboButton;
	/** Holds the combo button that allows exposing actors. */
	TSharedPtr<SComboButton> ExposeActorsComboButton;
	/** Caches all the classes of actors in the current level. */
	TSet<TWeakObjectPtr<const UClass>> CachedClassesInLevel;
	/** Holds the class picker used to expose all actors of class. */
	TSharedPtr<SClassViewer> ClassPicker;
	/** Holds the field's details. */
	TSharedPtr<class IStructureDetailsView> EntityDetailsView;
	/** Wrapper widget for entity details view. */
	TSharedPtr<SBorder> WrappedEntityDetailsView;
	/** Holds the field's protocol details. */
	TSharedPtr<SBox> EntityProtocolDetails;
	/** Whether to show the rebind all button. */
	bool bShowRebindButton = false;
	/** Cache of exposed property arguments. */
	TSet<FRCExposesPropertyArgs> CachedExposedPropertyArgs;
	/** Holds a cache of widgets. */
	TSharedPtr<FRCPanelWidgetRegistry> WidgetRegistry;
	/** Holds the handle to a timer set for next tick. Used to not schedule more than once event per frame */
	FTimerHandle NextTickTimerHandle;
	/** The toolkit that hosts this panel. */
	TWeakPtr<IToolkitHost> ToolkitHost;
	/** Asset Editor Default Toolbar */
	TSharedPtr<SWidget> Toolbar;
	/** Asset Editor Auxiliary Toolbar */
	TSharedPtr<SWidget> AuxiliaryToolbar;
	/** The widget that will house the default Toolbar widget */
	TSharedPtr<SBorder> ToolbarWidgetContent;
	/** The widget that will house the Auxiliary Toolbar widget */
	TSharedPtr<SBorder> AuxiliaryToolbarWidgetContent;
	/** Additional widgets to be added to the toolbar */
	TArray<TSharedRef<SWidget>> ToolbarWidgets;
	/** Holds a shared pointer reference to the last entity that was selected. */
	TSharedPtr<SRCPanelTreeNode> LastSelectedEntity;
	/** Panel Drawer widget holds all docked panels. */
	TSharedPtr<SRCPanelDrawer> PanelDrawer;
	/** Map of Opened Drawers. */
	TMap<ERCPanelMode, TSharedRef<FRCPanelDrawerArgs>> RegisteredDrawers;
	/** Panel Style reference. */
	const FRCPanelStyle* RCPanelStyle;
	/** Stores the active panel that is drawn. */
	ERCPanelMode ActiveMode = ERCPanelMode::Controller;
	/** Currently selected world name */
	FString SelectedWorldName;
	// ~ Remote Control Logic Panels ~

	/** Controller panel UI widget for Remote Control Logic*/
	TSharedPtr<SRCControllerPanel> ControllerPanel;
	/** Behaviour panel UI widget for Remote Control Logic*/
	TSharedPtr<SRCBehaviourPanel> BehaviorPanel;
	/** Action panel UI widget for Remote Control Logic*/
	TSharedPtr<SRCActionPanel> ActionPanel;
	/** Signature panel UI widget */
	TSharedPtr<SRCSignaturePanel> SignaturePanel;

	/** LogicClipboardItems - Holds the items copied from a Logic panel
	*
	* Note: We track UObjects (Data Model) here rather than the UI Models as the latter are swept away the moment the user navigates to a different Controller.
	* For example if the user copies an action from a behaviour in a given Controller but then navigates to another Controller, we can no longer rely on the previous UI objects
	* as they would have been discarded in favor of a new data set for the actively selected Controller */
	TArray<TObjectPtr<UObject>> LogicClipboardItems;

	// FGCObject interface
	virtual FString GetReferencerName() const override { return "RemoteControlPanel"; }

	/** Keeps track of whether materials were compiled from the current frame. Used to limit the number of UI refresh to once per frame. */
	bool bMaterialsCompiledThisFrame = false;

public:

	static const float MinimumPanelWidth;
	// Global Delegates for Remote Control Logic
	FOnControllerAdded OnControllerAdded;
	FOnBehaviourAdded OnBehaviourAdded;
	FOnActionAdded OnActionAdded;
	FOnControllerSelectionChanged OnControllerSelectionChanged;
	FOnControllerValueChanged OnControllerValueChangedDelegate;
	FOnBehaviourSelectionChanged OnBehaviourSelectionChanged;
	FOnEmptyControllers OnEmptyControllers;
	FOnEmptyBehaviours OnEmptyBehaviours;
	FOnEmptyActions OnEmptyActions;

	/** The panel from which the latest Logic UI item was copied*/
	TSharedPtr<SRCLogicPanelBase> LogicClipboardItemSource;
};
