// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SCompoundWidget.h"
#include "Misc/TextFilter.h"
#include "Styling/AppStyle.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSearchBox.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"

/**
 * Widget that displays a searchable dropdown list.
 */
template <typename ItemType>
class SSearchableItemList : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnItemSelected, ItemType /*Item*/)
	DECLARE_DELEGATE_RetVal_OneParam(FString, FOnGetDisplayName, ItemType /*Item*/)

	SLATE_BEGIN_ARGS(SSearchableItemList) {}
		SLATE_EVENT(FOnGetDisplayName, OnGetDisplayName)
		SLATE_EVENT(FOnItemSelected, OnItemSelected)
		SLATE_ARGUMENT(TArray<ItemType>, Items)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs)
	{
		bNeedsRefresh = true;
		bNeedsFocus = true;
		Items = InArgs._Items;

		OnItemSelected = InArgs._OnItemSelected;
		OnGetDisplayName = InArgs._OnGetDisplayName;
		ListView = SNew(SListView<ItemType>)
			.ItemHeight(24)
			.ListItemsSource(&FilteredItems)
			.OnGenerateRow(this, &SSearchableItemList::OnGenerateRow)
			.OnSelectionChanged(this, &SSearchableItemList::OnSelectionChanged)
			.SelectionMode(ESelectionMode::Single);

		SearchBoxFilter = MakeShared<TTextFilter<ItemType>>(TTextFilter<ItemType>::FItemToStringArray::CreateSP(this, &SSearchableItemList::TransformElementToString));

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(7.0f, 6.0f)
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &SSearchableItemList::OnFilterTextChanged)
				.OnKeyDownHandler(this, &SSearchableItemList::OnKeyDown)
			]

			+ SVerticalBox::Slot()
			[
				ListView.ToSharedRef()
			]
		];
	}

	virtual void Tick(const FGeometry&, const double, const float DeltaTime) override
	{
		if (bNeedsFocus)
		{
			bNeedsFocus = false;
			FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
		}

		if (bNeedsRefresh)
		{
			bNeedsRefresh = false;
			Populate();
		}
	}

private:
	/** Triggers a refresh on the next tick. */
	void Refresh()
	{
		bNeedsRefresh = true;
	}

	/** Create the initial list of items to be displayed. */
	void Populate()
	{
		FilteredItems.Empty();

		for (const ItemType& Object : Items)
		{
			ConstructRow(Object);
		}
		ListView->RequestListRefresh();
	}

	/** Generates the rows. */
	TSharedRef<ITableRow> OnGenerateRow(ItemType InObject, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow<ItemType>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("GraphEditor.Function_16x"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.HighlightText(this, &SSearchableItemList::GetFilterHighlightText)
				.Text(FText::FromString(OnGetDisplayName.Execute(MoveTemp(InObject))))
			]
		];
	}

	/** Filter text changed handler. */
	void OnFilterTextChanged(const FText& Text)
	{
		SearchBoxFilter->SetRawFilterText(Text);
		Refresh();
	}
	
	/** Add a row if it's not filtered. */
	void ConstructRow(ItemType Object)
	{
		if (SearchBoxFilter->PassesFilter(Object))
		{
			FilteredItems.Add(Object);
		}
	}

	/** Handle selection item selection changed. */
	void OnSelectionChanged(ItemType SelectedObject, ESelectInfo::Type SelectInfo)
	{
		if (SelectInfo != ESelectInfo::OnNavigation && SelectInfo != ESelectInfo::Direct)
		{
			OnItemSelected.ExecuteIfBound(SelectedObject);
			ListView->ClearSelection();
		}
	}

	/** Create the row's display name using the delegate. */
	void TransformElementToString(ItemType InObject, TArray<FString>& OutStrings)
	{
		OutStrings.Add(OnGetDisplayName.Execute(InObject));
	}

	/** Handler for getting the rows highlight text. */
	FText GetFilterHighlightText() const
	{
		return SearchBoxFilter->GetRawFilterText();
	}

	/** Handler for key down events. */
	FReply OnKeyDown(const FGeometry&, const FKeyEvent& KeyEvent)
	{
		if (KeyEvent.GetKey() == EKeys::Escape)
		{
			FSlateApplication::Get().DismissAllMenus();
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Enter)
		{
			if (FilteredItems.Num() > 0)
			{
				OnSelectionChanged(FilteredItems[0], ESelectInfo::Type::Direct);
			}
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

private:
	/** Item selected delegate. */
	FOnItemSelected OnItemSelected;

	/** Display name generator delegate. */
	FOnGetDisplayName OnGetDisplayName;

	/** Holds the search box filter. */
	TSharedPtr<TTextFilter<ItemType>> SearchBoxFilter;

	/** Holds the list view widget. */
	TSharedPtr<SListView<ItemType>> ListView;

	/** Holds the search box widget. */
	TSharedPtr<SSearchBox> SearchBox;

	/** Holds the unfiltered item list. */
	TArray<ItemType> Items;

	/** Holds the filtered item list. */
	TArray<ItemType> FilteredItems;

	/** Whether the list should be refreshed on the next tick. */
	bool bNeedsRefresh;

	/** Whether the searchbox needs to be focused on the next frame. */
	bool bNeedsFocus;
};	