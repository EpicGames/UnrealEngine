// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/TextFilter.h"
#include "Styling/AppStyle.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Images/SImage.h"

/**
 * Widget that displays a searchable tree view.
 */
template <typename ItemType>
class SSearchableTreeView : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_TwoParams(FOnItemSelected, ItemType /* Item */, ItemType /* ItemParent */)
	DECLARE_DELEGATE_RetVal_OneParam(FText, FOnGetDisplayName, ItemType /* Item */)
	DECLARE_DELEGATE_TwoParams(FOnGetChildren, ItemType /* Item */, TArray<ItemType>& /* OutChildren */)
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsSelectable, ItemType /* Item */)
	
	SLATE_BEGIN_ARGS(SSearchableTreeView) {}
		SLATE_EVENT(FOnGetDisplayName, OnGetDisplayName)
		SLATE_EVENT(FOnItemSelected, OnItemSelected)
		SLATE_EVENT(FOnGetChildren, OnGetChildren)
		SLATE_EVENT(FIsSelectable, IsSelectable)
		SLATE_ARGUMENT(TArray<ItemType>*, Items)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs)
	{
		bNeedsRefresh = true;
		bNeedsFocus = true;
		Items = InArgs._Items;

		checkSlow(Items);

		OnItemSelected = InArgs._OnItemSelected;
		OnGetDisplayName = InArgs._OnGetDisplayName;
		OnGetChildrenDelegate = InArgs._OnGetChildren;
		IsSelectable = InArgs._IsSelectable;
		
		TreeView = SNew(STreeView<ItemType>)
			.TreeItemsSource(&FilteredItems)
			.OnGenerateRow(this, &SSearchableTreeView::OnGenerateRow)
			.OnGetChildren(this, &SSearchableTreeView::OnGetChildren)
			.OnSelectionChanged(this, &SSearchableTreeView::OnSelectionChanged)
			.OnMouseButtonClick(this, &SSearchableTreeView::OnMouseClick);
			
		SearchBoxFilter = MakeShared<TTextFilter<ItemType>>(TTextFilter<ItemType>::FItemToStringArray::CreateSP(this, &SSearchableTreeView::TransformElementToString));

		ChildSlot
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.Padding(7.0f, 6.0f)
			.AutoHeight()
			[
				SAssignNew(SearchBox, SSearchBox)
				.OnTextChanged(this, &SSearchableTreeView::OnFilterTextChanged)
				.OnKeyDownHandler(this, &SSearchableTreeView::OnKeyDown)
			]

			+ SVerticalBox::Slot()
			[
				TreeView.ToSharedRef()
			]
		];
		SearchBox->SetSelectAllTextWhenFocused(false);
	}

	virtual void Tick(const FGeometry&, const double, const float DeltaTime) override
	{
		if (bNeedsRefresh)
		{
			bNeedsRefresh = false;
			Populate();
		}

		if (bNeedsFocus)
		{
			bNeedsFocus = false;
			FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
		}
	}

	/** Triggers a refresh on the next tick. */
	void Refresh()
	{
		bNeedsRefresh = true;
	}

	/** Focuses the search text box. */
	void Focus()
	{
		bNeedsFocus = true;
	}

	/** Clears this TreeView's search box. */
	void ClearSearchBox()
	{
		SearchBox->SetText(FText());
		Refresh();
	}
private:

	/** Create the initial list of items to be displayed. */
	void Populate()
	{
		FilteredItems.Empty();
		if (Items)
		{
			for (const ItemType& Object : *Items)
			{
				ConstructRow(Object);
			}
		}
		TreeView->RequestListRefresh();
	}

	/** Generates the rows. */
	TSharedRef<ITableRow> OnGenerateRow(ItemType InObject, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(STableRow<ItemType>, OwnerTable)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.0f, 0, 0, 0)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("GraphEditor.Function_16x"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.0f)
			[
				SNew(STextBlock)
				.HighlightText(this, &SSearchableTreeView::GetFilterHighlightText)
				.Text(OnGetDisplayName.Execute(MoveTemp(InObject)))
			]
		];
	}

	void OnGetChildren(ItemType Item, TArray<ItemType>& OutChildren) const
	{
		TArray<ItemType> UnfilteredChildren;
		OnGetChildrenDelegate.ExecuteIfBound(Item, UnfilteredChildren);
		TArrayView<ItemType> FilteredChildren(UnfilteredChildren);
		if (SearchBoxFilter->PassesFilter(Item))
		{
			OutChildren.Append(UnfilteredChildren);
		}
		else
		{
			OutChildren.Append(FilteredChildren.FilterByPredicate([this](const ItemType& ChildItem){ return SearchBoxFilter->PassesFilter(ChildItem); }));
		}
	}

	/** Filter text changed handler. */
	void OnFilterTextChanged(const FText& Text)
	{
		SearchBoxFilter->SetRawFilterText(Text);
		Refresh();
	}
	
	/** Add a row if it's not filtered. */
	void ConstructRow(ItemType Object)
	{
		bool ChildPassesFilter = false;

		TArray<ItemType> Children;
		OnGetChildrenDelegate.ExecuteIfBound(Object, Children);
		
		if (!SearchBoxFilter->GetRawFilterText().IsEmpty())
		{
			ChildPassesFilter = Children.ContainsByPredicate([this](const ItemType& InItem)
				{
					return SearchBoxFilter->PassesFilter(InItem);
				});
		}

		if (SearchBoxFilter->PassesFilter(Object))
		{
			FilteredItems.Add(Object);
		}

		if (ChildPassesFilter)
		{
			FilteredItems.AddUnique(Object);
			constexpr bool bInShouldExpandItem = true;
			TreeView->SetItemExpansion(Object, bInShouldExpandItem);
		}
	}

	/** Handle selection item selection changed. */
	void OnSelectionChanged(ItemType SelectedObject, ESelectInfo::Type SelectInfo)
	{
		if (SelectInfo != ESelectInfo::OnNavigation && SelectedObject)
		{
			if (IsSelectable.IsBound() && IsSelectable.Execute(SelectedObject))
			{
				OnItemSelected.ExecuteIfBound(SelectedObject, GetParent(SelectedObject));
				TreeView->ClearSelection();
			}
			else
			{
				TreeView->SetItemExpansion(SelectedObject, !TreeView->IsItemExpanded(SelectedObject));
			}
		}
	}

	/** Handle an item being clicked */
	void OnMouseClick(ItemType SelectedObject)
	{
		// Handle the case where the user clicks an item that's already selected.
		if (TreeView->GetNumItemsSelected() == 1 && SelectedObject)
		{
			TArray<ItemType> SelectedObjects;
			TreeView->GetSelectedItems(SelectedObjects);
			if (SelectedObject == SelectedObjects[0])
			{
				TreeView->SetItemExpansion(SelectedObject, false);
			}
		}
	}

	/** Create the row's display name using the delegate. */
	void TransformElementToString(ItemType InObject, TArray<FString>& OutStrings)
	{
		OutStrings.Add(OnGetDisplayName.Execute(InObject).ToString());
	}

	/** Handler for getting the rows highlight text. */
	FText GetFilterHighlightText() const
	{
		return SearchBoxFilter->GetRawFilterText();
	}

	/** Handler for key down events. */
	virtual FReply OnKeyDown(const FGeometry&, const FKeyEvent& KeyEvent) override
	{
		if (KeyEvent.GetKey() == EKeys::Escape)
		{
			FSlateApplication::Get().DismissAllMenus();
			return FReply::Handled();
		}
		else if (KeyEvent.GetKey() == EKeys::Enter)
		{
			if (FilteredItems.Num() > 0)
			{
				TArray<ItemType> SelectedObjects;
				TreeView->GetSelectedItems(SelectedObjects);
				if (SelectedObjects.Num() == 1)
				{
					OnSelectionChanged(SelectedObjects[0], ESelectInfo::Type::Direct);
				}
			}
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	ItemType GetParent(const ItemType& InItem)
	{
		if (Items)
		{
			for (const ItemType& Item : *Items)
			{
				TArray<ItemType> Children;
				OnGetChildrenDelegate.ExecuteIfBound(Item, Children);

				for (const ItemType& Child : Children)
				{
					if (Child == InItem)
					{
						return Item;
					}
				}
			}
		}

		return ItemType();
	}

private:
	/** Item selected delegate. */
	FOnItemSelected OnItemSelected;
	
	/** Checks if an item is selectable. */
	FIsSelectable IsSelectable;

	/** Display name generator delegate. */
	FOnGetDisplayName OnGetDisplayName;

	/** Get an item's children delegate. */
	FOnGetChildren OnGetChildrenDelegate;

	/** Holds the search box filter. */
	TSharedPtr<TTextFilter<ItemType>> SearchBoxFilter;

	/** Holds the list view widget. */
	TSharedPtr<STreeView<ItemType>> TreeView;

	/** Holds the search box widget. */
	TSharedPtr<SSearchBox> SearchBox;

	/** Holds the unfiltered item list. */
	TArray<ItemType>* Items;

	/** Holds the filtered item list. */
	TArray<ItemType> FilteredItems;

	/** Whether the list should be refreshed on the next tick. */
	bool bNeedsRefresh;

	/** Whether the searchbox needs to be focused on the next frame. */
	bool bNeedsFocus;
};	