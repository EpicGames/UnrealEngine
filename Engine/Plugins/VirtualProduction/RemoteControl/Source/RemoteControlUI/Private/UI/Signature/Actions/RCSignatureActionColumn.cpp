// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureActionColumn.h"
#include "RCSignatureAction.h"
#include "RCSignatureActionType.h"
#include "SRCSignatureActionBox.h"
#include "StructUtils/InstancedStruct.h"
#include "UI/Signature/Items/RCSignatureTreeFieldItem.h"
#include "UObject/Class.h"
#include "UObject/UObjectIterator.h"

#define LOCTEXT_NAMESPACE "RCSignatureActionColumn"

FRCSignatureActionColumn::FRCSignatureActionColumn(const TAttribute<bool>& InLiveMode)
	: LiveMode(InLiveMode)
{
}

FName FRCSignatureActionColumn::GetColumnId() const
{
	return TEXT("FRCSignatureActionColumn");
}

bool FRCSignatureActionColumn::ShouldShowColumnByDefault() const
{
	return true;
}

SHeaderRow::FColumn::FArguments FRCSignatureActionColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(0.5f)
		.DefaultLabel(LOCTEXT("DisplayName", "Actions"));
}

TSharedRef<SWidget> FRCSignatureActionColumn::ConstructRowWidget(TSharedPtr<FRCSignatureTreeItemBase> InItem, const TSharedRef<SRCSignatureTree>& InList, const TSharedRef<SRCSignatureRow>& InRow)
{
	return SNew(SRCSignatureActionBox, InItem.ToSharedRef(), InRow)
		.LiveMode(LiveMode)
		.ActionTypesSource(&ActionTypes)
		.OnRefreshActionTypes(this, &FRCSignatureActionColumn::RefreshActionTypes);
}

void FRCSignatureActionColumn::RefreshActionTypes(const TSharedRef<FRCSignatureTreeFieldItem>& InFieldItem)
{
	ActionTypes.Reset();

	if (!UObjectInitialized())
	{
		return;
	}

	const FRCSignatureField* Field = InFieldItem->FindField();
	if (!Field)
	{
		return;
	}

	const FName HiddenMetaData(TEXT("Hidden"));

	for (UScriptStruct* ScriptStruct : TObjectRange<UScriptStruct>())
	{
		if (ScriptStruct->HasMetaData(HiddenMetaData))
		{
			continue;				
		}

		if (ScriptStruct->IsChildOf(TBaseStructure<FRCSignatureAction>::Get()))
		{
			// Initialize a Temp Instance to determine if it's supported by the Tree Item
			TInstancedStruct<FRCSignatureAction> Instance;
			Instance.InitializeAsScriptStruct(ScriptStruct, /*StructMemory*/nullptr);

			const FRCSignatureAction& Action = Instance.Get();

			if (!Action.IsSupported(*Field))
			{
				continue;
			}

			TSharedRef<FRCSignatureActionType> ActionType = MakeShared<FRCSignatureActionType>();
			ActionType->Type = ScriptStruct;
			ActionType->Title = ScriptStruct->GetDisplayNameText();
			ActionType->Icon = Action.GetIcon();

			ActionTypes.Add(MoveTemp(ActionType));
		}
	}
}

#undef LOCTEXT_NAMESPACE
