// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "RCSignatureAction.h"
#include "Textures/SlateIcon.h"

class UScriptStruct;

/** Struct to describe an Item in the Action Type combo box */
struct FRCSignatureActionType
{
	/** Struct type of the Action to create if selected */
	const UScriptStruct* Type = nullptr;

	/** Display Name Text in the combo box entry */
	FText Title;

	/** Icon and color in the combo box entry */
	FRCSignatureActionIcon Icon;
};
