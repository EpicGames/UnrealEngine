// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureAction.h"
#include "RCSignatureAction.h"
#include "SRCSignatureActionIcon.h"
#include "Styling/AppStyle.h"
#include "Styling/ToolBarStyle.h"
#include "UI/Signature/Items/RCSignatureTreeActionItem.h"
#include "UI/Signature/Items/RCSignatureTreeItemBase.h"
#include "Widgets/Layout/SBorder.h"

void SRCSignatureAction::Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeActionItem>& InActionItem)
{
	ActionItemWeak = InActionItem;
	CheckBoxStyle = &FAppStyle::Get().GetWidgetStyle<FToolBarStyle>(TEXT("SlimToolBar")).ToggleButton;

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(this, &SRCSignatureAction::GetBackgroundBrush)
		.Padding(5.f, 4.f)
		[
			SAssignNew(ActionImage, SRCSignatureActionIcon)
			.ActionIcon(InActionItem->GetIcon())
		]
	];
}

FReply SRCSignatureAction::OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	if (InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		SetPressed(true);

		TSharedRef<SRCSignatureAction> This = SharedThis(this);

		return FReply::Handled()
			.DetectDrag(This, InMouseEvent.GetEffectingButton())
			.CaptureMouse(This)
			.SetUserFocus(This, EFocusCause::Mouse);
	}
	return FReply::Unhandled();
}

FReply SRCSignatureAction::OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent)
{
	FReply Reply = FReply::Unhandled();

	if (IsPressed() && InMouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		SetPressed(false);

		bool bEventOverButton = IsHovered();
		if (!bEventOverButton && InMouseEvent.IsTouchEvent())
		{
			bEventOverButton = InGeometry.IsUnderLocation(InMouseEvent.GetScreenSpacePosition());
		}

		if (bEventOverButton && HasMouseCapture())
		{
			if (TSharedPtr<FRCSignatureTreeActionItem> ActionItem = ActionItemWeak.Pin())
			{
				bool bShouldSelect = true;

				const bool bIsMultiSelection = InMouseEvent.IsControlDown();

				// if Multi-selecting and already selected, toggle off
				if (bIsMultiSelection && ActionItem->IsSelected())
				{
					bShouldSelect = false;
				}

				ActionItem->SetSelected(bShouldSelect, bIsMultiSelection);
			}
			Reply = FReply::Handled();
		}
	}

	if (HasMouseCapture())
	{
		Reply.ReleaseMouseCapture();
	}

	return Reply;
}

void SRCSignatureAction::OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent)
{
	SetPressed(false);
}

bool SRCSignatureAction::IsPressed() const
{
	return bIsPressed;
}

void SRCSignatureAction::SetPressed(bool bInPressed)
{
	bIsPressed = bInPressed;
}

const FSlateBrush* SRCSignatureAction::GetBackgroundBrush() const
{
	check(CheckBoxStyle);

	TSharedPtr<FRCSignatureTreeActionItem> ActionItem = ActionItemWeak.Pin();

	const bool bIsSelected = ActionItem.IsValid() && ActionItem->IsSelected();

	if (IsPressed())
	{
		return bIsSelected
			? &CheckBoxStyle->CheckedPressedImage
			: &CheckBoxStyle->UncheckedPressedImage;
	}

	if (IsHovered())
	{
		return bIsSelected
			? &CheckBoxStyle->CheckedHoveredImage
			: &CheckBoxStyle->UncheckedHoveredImage;
	}

	return bIsSelected
		? &CheckBoxStyle->CheckedImage
		: &CheckBoxStyle->UncheckedImage;
}
