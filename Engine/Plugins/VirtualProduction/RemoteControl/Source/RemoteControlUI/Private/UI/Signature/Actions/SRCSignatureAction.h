// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FRCSignatureTreeActionItem;
class SRCSignatureActionIcon;
struct FCheckBoxStyle;

class SRCSignatureAction : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureAction) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeActionItem>& InActionItem);

protected:
	//~ Begin SWidget
	virtual FReply OnMouseButtonDown(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& InGeometry, const FPointerEvent& InMouseEvent) override;
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& InCaptureLostEvent) override;
	//~ End SWidget

private:
	bool IsPressed() const;
	void SetPressed(bool bInPressed);

	const FSlateBrush* GetBackgroundBrush() const;

	TWeakPtr<FRCSignatureTreeActionItem> ActionItemWeak;

	TSharedPtr<SRCSignatureActionIcon> ActionImage;

	const FCheckBoxStyle* CheckBoxStyle = nullptr;

	bool bIsPressed = false;
};
