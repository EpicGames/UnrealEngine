// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureActionBox.h"
#include "DetailLayoutBuilder.h"
#include "RCSignature.h"
#include "RCSignatureActionType.h"
#include "SRCSignatureAction.h"
#include "SRCSignatureActionIcon.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/Signature/Items/RCSignatureTreeActionItem.h"
#include "UI/Signature/Items/RCSignatureTreeFieldItem.h"
#include "UI/Signature/Items/RCSignatureTreeItemBase.h"
#include "UI/Signature/SRCSignatureRow.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SScrollBox.h"

void SRCSignatureActionBox::Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeItemBase>& InItem, const TSharedRef<SRCSignatureRow>& InRow)
{
	ItemWeak = InItem;
	LiveMode = InArgs._LiveMode;
	OnRefreshActionTypes = InArgs._OnRefreshActionTypes;

	IsHovered = TAttribute<bool>(InRow, &SRCSignatureRow::IsHovered);
	IsSelected = TAttribute<bool>(InRow, &SRCSignatureRow::IsSelected);

	ActionTypesComboBox = SNew(SComboBox<TSharedPtr<FRCSignatureActionType>>)
		.IsEnabled(this, &SRCSignatureActionBox::CanAddAction)
		.Visibility(this, &SRCSignatureActionBox::GetAddActionVisibility)
		.ComboBoxStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("SimpleComboBox"))
		.OptionsSource(InArgs.GetActionTypesSource())
		.OnComboBoxOpening(this, &SRCSignatureActionBox::OnComboBoxOpening)
		.OnGenerateWidget(this, &SRCSignatureActionBox::GenerateActionTypeWidget)
		.OnSelectionChanged(this, &SRCSignatureActionBox::OnActionTypeSelected)
		.ContentPadding(0)
		.HasDownArrow(false)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			.DesiredSizeOverride(FVector2D(SRCSignatureActionIcon::IconSize))
		];

	ChildSlot
		[
			SAssignNew(ActionListBox, SScrollBox)
			.ConsumeMouseWheel(EConsumeMouseWheel::Always)
			.Orientation(EOrientation::Orient_Horizontal)
			.ScrollBarThickness(FVector2D(2.f))
		];

	Refresh();
}

void SRCSignatureActionBox::Refresh()
{
	ActionListBox->ClearChildren();

	TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	TSharedPtr<FRCSignatureTreeFieldItem> FieldItem = Item->MutableCast<FRCSignatureTreeFieldItem>();
	if (!FieldItem.IsValid())
	{
		return;
	}

	// Force rebuild children to discover any new actions
	FieldItem->RebuildChildren();

	TConstArrayView<TSharedPtr<FRCSignatureTreeItemBase>> FieldChildren = Item->GetChildren();

	for (const TSharedPtr<FRCSignatureTreeItemBase>& FieldChild : FieldChildren)
	{
		TSharedPtr<FRCSignatureTreeActionItem> ActionItem = FieldChild->MutableCast<FRCSignatureTreeActionItem>();
		if (!ActionItem.IsValid())
		{
			continue;
		}

		ActionListBox->AddSlot()
			.Padding(0.f, 1.f)
			[
				SNew(SRCSignatureAction, ActionItem.ToSharedRef())
			];
	}

	// Add (+) button at the end to add new actions
	ActionListBox->AddSlot()
		[
			ActionTypesComboBox.ToSharedRef()
		];
}

void SRCSignatureActionBox::OnComboBoxOpening()
{
	if (!OnRefreshActionTypes.IsBound())
	{
		return;
	}

	TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
	if (!Item.IsValid())
	{
		return;	
	}

	TSharedPtr<FRCSignatureTreeFieldItem> FieldItem = Item->MutableCast<FRCSignatureTreeFieldItem>();
	if (!FieldItem.IsValid())
	{
		return;
	}

	OnRefreshActionTypes.Execute(FieldItem.ToSharedRef());
}

bool SRCSignatureActionBox::CanAddAction() const
{
	const bool bIsLive = LiveMode.Get(false);
	return !bIsLive;
}

EVisibility SRCSignatureActionBox::GetAddActionVisibility() const
{
	if (!CanAddAction())
	{
		return EVisibility::Collapsed;
	}

	if (ActionTypesComboBox->IsOpen())
	{
		return EVisibility::Visible;
	}

	if (IsSelected.Get(false) || IsHovered.Get(false))
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

TSharedRef<SWidget> SRCSignatureActionBox::GenerateActionTypeWidget(TSharedPtr<FRCSignatureActionType> InActionType)
{
	check(InActionType.IsValid());
	return SNew(SBox)
		.WidthOverride(200.f)
		.Padding(0.f, 5.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(5.f, 0.f)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SRCSignatureActionIcon)
				.ActionIcon(InActionType->Icon)
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(5.f, 0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(InActionType->Title)
				.Justification(ETextJustify::Left)
			]
		];
}

void SRCSignatureActionBox::OnActionTypeSelected(TSharedPtr<FRCSignatureActionType> InActionType, ESelectInfo::Type InSelectType)
{
	if (!InActionType.IsValid() || !ensure(InActionType->Type))
	{
		return;
	}

	TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
	if (!Item.IsValid())
	{
		return;
	}

	TSharedPtr<FRCSignatureTreeFieldItem> FieldItem = Item->MutableCast<FRCSignatureTreeFieldItem>();
	if (!FieldItem.IsValid())
	{
		return;
	}

	FieldItem->AddAction(InActionType->Type);
	ActionTypesComboBox->ClearSelection();
	Refresh();
}
