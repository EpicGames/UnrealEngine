// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/IItemsSource.h"

class FRCSignatureTreeFieldItem;
class FRCSignatureTreeItemBase;
class SRCSignatureRow;
class SScrollBox;
struct FRCSignatureActionType;
template<typename OptionType> class SComboBox;

class SRCSignatureActionBox : public SCompoundWidget
{
public:
	using FOnRefreshActionTypes = TDelegate<void(const TSharedRef<FRCSignatureTreeFieldItem>&)>;

	SLATE_BEGIN_ARGS(SRCSignatureActionBox) {}
		SLATE_ATTRIBUTE(bool, LiveMode)
		SLATE_ITEMS_SOURCE_ARGUMENT(TSharedPtr<FRCSignatureActionType>, ActionTypesSource)
		SLATE_EVENT(FOnRefreshActionTypes, OnRefreshActionTypes)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeItemBase>& InItem, const TSharedRef<SRCSignatureRow>& InRow);

private:
	void Refresh();

	void OnComboBoxOpening();

	bool CanAddAction() const;

	EVisibility GetAddActionVisibility() const;

	TSharedRef<SWidget> GenerateActionTypeWidget(TSharedPtr<FRCSignatureActionType> InActionType);

	void OnActionTypeSelected(TSharedPtr<FRCSignatureActionType> InActionType, ESelectInfo::Type InSelectType);

	TSharedPtr<SComboBox<TSharedPtr<FRCSignatureActionType>>> ActionTypesComboBox;

	TWeakPtr<FRCSignatureTreeItemBase> ItemWeak;

	TSharedPtr<SScrollBox> ActionListBox;

	FOnRefreshActionTypes OnRefreshActionTypes;

	TAttribute<bool> IsHovered;
	TAttribute<bool> IsSelected;
	TAttribute<bool> LiveMode;
};
