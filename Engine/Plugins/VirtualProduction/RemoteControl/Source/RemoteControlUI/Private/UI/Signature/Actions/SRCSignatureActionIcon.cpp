// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureActionIcon.h"
#include "RCSignatureAction.h"
#include "Textures/SlateIcon.h"
#include "Widgets/Images/SLayeredImage.h"

namespace UE::RemoteControlUI::Private
{
	FSlateIcon GetActionIcon(const FRCSignatureActionIcon& In)
	{
		return FSlateIcon(In.StyleSetName, In.StyleName, NAME_None, In.OverlayStyleName);
	}
}

void SRCSignatureActionIcon::Construct(const FArguments& InArgs)
{
	ChildSlot
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SAssignNew(LayeredImage, SLayeredImage)
			.DesiredSizeOverride(FVector2D(SRCSignatureActionIcon::IconSize))
		];

	SetActionIcon(InArgs._ActionIcon);
}

void SRCSignatureActionIcon::SetActionIcon(const FRCSignatureActionIcon& InActionIcon)
{
	const FSlateIcon Icon = UE::RemoteControlUI::Private::GetActionIcon(InActionIcon);

	LayeredImage->RemoveAllLayers();

	// Set Layer 0 (Base) image & color
	LayeredImage->SetImage(Icon.GetIcon());
	LayeredImage->SetLayerColor(0, InActionIcon.BaseColor);

	// Set Layer 1 (Overlay) image & color
	if (const FSlateBrush* OverlayIcon = Icon.GetOverlayIcon())
	{
		LayeredImage->AddLayer(OverlayIcon);

		// Overlay color defaults to IconColor if not set
		LayeredImage->SetLayerColor(1, InActionIcon.OverlayColor.Get(InActionIcon.BaseColor));
	}
}
