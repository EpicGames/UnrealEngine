// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureAction.h"
#include "Widgets/SCompoundWidget.h"

class SLayeredImage;

class SRCSignatureActionIcon : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureActionIcon) {}
		SLATE_ARGUMENT(FRCSignatureActionIcon, ActionIcon)
	SLATE_END_ARGS()

	static constexpr float IconSize = 16.f;

	void Construct(const FArguments& InArgs);

	void SetActionIcon(const FRCSignatureActionIcon& InActionIcon);

private:
	TSharedPtr<SLayeredImage> LayeredImage;
};
