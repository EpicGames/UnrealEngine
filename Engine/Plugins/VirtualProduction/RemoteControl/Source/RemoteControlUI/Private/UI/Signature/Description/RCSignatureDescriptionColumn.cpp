// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureDescriptionColumn.h"
#include "SRCSignatureDescription.h"

#define LOCTEXT_NAMESPACE "RCSignatureDescriptionColumn"

FName FRCSignatureDescriptionColumn::GetColumnId() const
{
	return TEXT("FRCSignatureDescriptionColumn");
}

bool FRCSignatureDescriptionColumn::ShouldShowColumnByDefault() const
{
	return false;
}

SHeaderRow::FColumn::FArguments FRCSignatureDescriptionColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(0.5f)
		.DefaultLabel(LOCTEXT("DisplayName", "Description"));
}

TSharedRef<SWidget> FRCSignatureDescriptionColumn::ConstructRowWidget(TSharedPtr<FRCSignatureTreeItemBase> InItem
	, const TSharedRef<SRCSignatureTree>& InList
	, const TSharedRef<SRCSignatureRow>& InRow)
{
	return SNew(SRCSignatureDescription, InItem.ToSharedRef());
}

#undef LOCTEXT_NAMESPACE
