// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureDescription.h"
#include "UI/Signature/Items/RCSignatureTreeItemBase.h"
#include "Widgets/Text/STextBlock.h"

void SRCSignatureDescription::Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeItemBase>& InItem)
{
	ItemWeak = InItem;

	ChildSlot
	.VAlign(VAlign_Center)
	.Padding(5.f, 0.f, 0.f, 0.f)
	[
		SNew(STextBlock)
		.Text(this, &SRCSignatureDescription::GetDescription)
		.Justification(ETextJustify::Left)
	];
}

FText SRCSignatureDescription::GetDescription() const
{
	if (!CachedDescription.IsSet())
	{
		TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
		CachedDescription = Item.IsValid() ? Item->GetDescription() : FText::GetEmpty();
	}
	return *CachedDescription;
}
