// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FRCSignatureTreeItemBase;

class SRCSignatureDescription : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureDescription) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<FRCSignatureTreeItemBase>& InItem);

private:
	FText GetDescription() const;

	TWeakPtr<FRCSignatureTreeItemBase> ItemWeak;

	mutable TOptional<FText> CachedDescription;
};
