// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureDetails.h"
#include "DetailsViewArgs.h"
#include "IStructureDataProvider.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "RCSignatureRegistry.h"
#include "ScopedTransaction.h"
#include "UI/Signature/Items/RCSignatureTreeRootItem.h"
#include "UI/Signature/RCSignatureTreeItemSelection.h"

#define LOCTEXT_NAMESPACE "SRCSignatureDetails"

void SRCSignatureDetails::Construct(const FArguments& InArgs, URCSignatureRegistry* InSignatureRegistry, const TSharedRef<FRCSignatureTreeItemSelection>& InSelection)
{
	SignatureRegistryWeak = InSignatureRegistry;
	SelectionWeak = InSelection;

	InSelection->OnSelectionChanged().AddSP(this, &SRCSignatureDetails::Refresh);

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.NotifyHook = this;

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	StructDetailsView = PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs, FStructureDetailsViewArgs(), /*StructOnScope*/nullptr);
	StructDetailsView->GetOnFinishedChangingPropertiesDelegate().AddSP(this, &SRCSignatureDetails::OnFinishedChangingProperties);

	ChildSlot
	[
		StructDetailsView->GetWidget().ToSharedRef()
	];

	Refresh();
}

SRCSignatureDetails::~SRCSignatureDetails()
{
	if (TSharedPtr<FRCSignatureTreeItemSelection> Selection = SelectionWeak.Pin())
	{
		Selection->OnSelectionChanged().RemoveAll(this);
	}
}

void SRCSignatureDetails::Refresh()
{
	TSharedRef<FStructOnScopeStructureDataProvider> StructProvider = MakeShared<FStructOnScopeStructureDataProvider>();

	TArray<TSharedPtr<FStructOnScope>> StructOnScopes;
	GatherStructOnScopes(StructOnScopes, ViewedItems);

	StructProvider->SetStructData(StructOnScopes);

	StructDetailsView->SetStructureProvider(StructProvider);
}

void SRCSignatureDetails::GatherStructOnScopes(TArray<TSharedPtr<FStructOnScope>>& OutStructOnScopes, TArray<TWeakPtr<FRCSignatureTreeItemBase>>& OutItems) const
{
	OutStructOnScopes.Reset();
	OutItems.Reset();

	TSharedPtr<FRCSignatureTreeItemSelection> Selection = SelectionWeak.Pin();
	if (!Selection.IsValid())
	{
		return;
	}

	TConstArrayView<TWeakPtr<FRCSignatureTreeItemBase>> SelectedItems = Selection->GetSelectedItemsView();

	OutStructOnScopes.Reserve(SelectedItems.Num());
	OutItems.Reserve(SelectedItems.Num());

	for (const TWeakPtr<FRCSignatureTreeItemBase>& SelectedItemWeak : SelectedItems)
	{
		TSharedPtr<FRCSignatureTreeItemBase> SelectedItem = SelectedItemWeak.Pin();
		if (!SelectedItem.IsValid())
		{
			continue;
		}

		TSharedPtr<FStructOnScope> StructOnScope = SelectedItem->MakeSelectionStruct();
		if (!StructOnScope.IsValid())
		{
			continue;
		}

		OutStructOnScopes.Add(MoveTemp(StructOnScope));
		OutItems.Add(SelectedItem);
	}
}

void SRCSignatureDetails::OnFinishedChangingProperties(const FPropertyChangedEvent& InChangeEvent)
{
	// End Transaction
	CurrentTransaction.Reset();
}

void SRCSignatureDetails::NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange)
{
	if (URCSignatureRegistry* SignatureRegistry = SignatureRegistryWeak.Get())
	{
		// Begin Transaction, if not already in one
		if (!CurrentTransaction.IsValid())
		{
			CurrentTransaction = MakeShared<FScopedTransaction>(LOCTEXT("EditSignature", "Edit Signature"));
		}
		SignatureRegistry->Modify();
	}
}

void SRCSignatureDetails::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	for (TWeakPtr<FRCSignatureTreeItemBase> ViewedItemWeak : ViewedItems)
	{
		if (TSharedPtr<FRCSignatureTreeItemBase> ViewedItem = ViewedItemWeak.Pin())
		{
			ViewedItem->NotifyPostChange(InPropertyChangedEvent, InPropertyThatChanged);
		}
	}
}

#undef LOCTEXT_NAMESPACE
