// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Misc/NotifyHook.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/SCompoundWidget.h"

class FRCSignatureTreeItemBase;
class FRCSignatureTreeItemSelection;
class FScopedTransaction;
class FStructOnScope;
class IStructureDetailsView;
class URCSignatureRegistry;
struct FPropertyChangedEvent;

class SRCSignatureDetails : public SCompoundWidget, public FNotifyHook
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureDetails) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, URCSignatureRegistry* InSignatureRegistry, const TSharedRef<FRCSignatureTreeItemSelection>& InSelection);

	virtual ~SRCSignatureDetails() override;

	void Refresh();

private:
	void GatherStructOnScopes(TArray<TSharedPtr<FStructOnScope>>& OutStructOnScopes, TArray<TWeakPtr<FRCSignatureTreeItemBase>>& OutItems) const;

	void OnFinishedChangingProperties(const FPropertyChangedEvent& InChangeEvent);

	//~ Begin FNotifyHook
	virtual void NotifyPreChange(FEditPropertyChain* InPropertyAboutToChange) override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged) override;
	//~ End FNotifyHook

	TSharedPtr<IStructureDetailsView> StructDetailsView;

	TSharedPtr<FScopedTransaction> CurrentTransaction;

	TWeakObjectPtr<URCSignatureRegistry> SignatureRegistryWeak;

	TWeakPtr<FRCSignatureTreeItemSelection> SelectionWeak;

	/** Currently viewed items in the Details Panel */
	TArray<TWeakPtr<FRCSignatureTreeItemBase>> ViewedItems;
};
