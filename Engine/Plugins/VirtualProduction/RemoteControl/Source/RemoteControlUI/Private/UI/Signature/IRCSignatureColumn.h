// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/Views/SHeaderRow.h"

class FRCSignatureTreeItemBase;
class SRCSignatureRow;
class SRCSignatureTree;

class IRCSignatureColumn : public TSharedFromThis<IRCSignatureColumn>
{
public:
	virtual ~IRCSignatureColumn() = default;

	virtual FName GetColumnId() const = 0;

	/**
	 * Determines whether the Column should be Showing by Default while still be able to toggle it on/off.
	 * Used when calling SHeaderRow::SetShowGeneratedColumn (requires ShouldGenerateWidget to not be set).
	 */
	virtual bool ShouldShowColumnByDefault() const
	{
		return false;
	}

	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() = 0;

	virtual TSharedRef<SWidget> ConstructRowWidget(TSharedPtr<FRCSignatureTreeItemBase> InItem
		, const TSharedRef<SRCSignatureTree>& InList
		, const TSharedRef<SRCSignatureRow>& InRow) = 0;
};
