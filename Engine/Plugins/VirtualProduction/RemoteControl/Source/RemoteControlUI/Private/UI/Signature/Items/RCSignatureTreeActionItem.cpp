// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeActionItem.h"
#include "RCSignature.h"
#include "RCSignatureAction.h"
#include "RCSignatureRegistry.h"
#include "RCSignatureTreeFieldItem.h"
#include "RCSignatureTreeRootItem.h"
#include "ScopedTransaction.h"
#include "UI/Signature/RCSignatureTreeItemSelection.h"

#define LOCTEXT_NAMESPACE "RCSignatureTreeActionItem"

FRCSignatureTreeActionItem::FRCSignatureTreeActionItem(int32 InActionIndex, const TSharedPtr<SRCSignatureTree>& InSignatureTree)
	: FRCSignatureTreeItemBase(InSignatureTree)
	, ActionIndex(InActionIndex)
{
	// Action Items are hidden in Tree View and instead shown in a Horizontal List next to its Parent
	AddTreeViewFlags(ERCSignatureTreeItemViewFlags::Hidden);
}

const FRCSignatureActionInstance* FRCSignatureTreeActionItem::FindActionInstance() const
{
	TSharedPtr<FRCSignatureTreeFieldItem> FieldItem = GetParentFieldItem();
	if (!FieldItem.IsValid())
	{
		return nullptr;
	}

	const FRCSignatureField* Field = FieldItem->FindField();
	if (!Field || !Field->Actions.IsValidIndex(ActionIndex))
	{
		return nullptr;
	}

	return &Field->Actions[ActionIndex];
}

FRCSignatureActionInstance* FRCSignatureTreeActionItem::FindActionInstanceMutable()
{
	FRCSignatureField* Field = FindParentFieldMutable();
	if (!Field || !Field->Actions.IsValidIndex(ActionIndex))
	{
		return nullptr;
	}

	return &Field->Actions[ActionIndex];
}

FRCSignatureActionIcon FRCSignatureTreeActionItem::GetIcon() const
{
	if (const FRCSignatureActionInstance* ActionInstance = FindActionInstance())
	{
		if (const FRCSignatureAction* Action = ActionInstance->GetAction())
		{
			return Action->GetIcon();
		}
	}
	return FRCSignatureActionIcon();
}

void FRCSignatureTreeActionItem::BuildPathSegment(FStringBuilderBase& InStringBuilder) const
{
	InStringBuilder << ActionIndex;
}

TOptional<bool> FRCSignatureTreeActionItem::IsEnabled() const
{
	return true;
}

int32 FRCSignatureTreeActionItem::RemoveFromRegistry()
{
	URCSignatureRegistry* Registry;

	FRCSignatureField* Field = FindParentFieldMutable(&Registry);
	if (!Field || !Field->Actions.IsValidIndex(ActionIndex))
	{
		return 0;
	}

	check(Registry);

	FScopedTransaction Transaction(LOCTEXT("RemoveAction", "Remove Action"));
	Registry->Modify();
	Field->Actions.RemoveAt(ActionIndex);
	return 1;
}

FText FRCSignatureTreeActionItem::GetDisplayNameText() const
{
	// Unused, as Action Items are hidden in Tree View
	return FText::GetEmpty();
}

FText FRCSignatureTreeActionItem::GetDescription() const
{
	// Unused, as Action Items are hidden in Tree View
	return FText::GetEmpty();
}

TSharedPtr<FStructOnScope> FRCSignatureTreeActionItem::MakeSelectionStruct()
{
	if (FRCSignatureActionInstance* ActionInstance = FindActionInstanceMutable())
	{
		return ActionInstance->MakeStructOnScope();
	}
	return nullptr;
}

void FRCSignatureTreeActionItem::NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
{
	if (FRCSignatureActionInstance* ActionInstance = FindActionInstanceMutable())
	{
		ActionInstance->PostEditChange(InPropertyChangedEvent, InPropertyThatChanged);
	}
}

TSharedPtr<FRCSignatureTreeFieldItem> FRCSignatureTreeActionItem::GetParentFieldItem() const
{
	if (TSharedPtr<FRCSignatureTreeItemBase> Parent = GetParent())
	{
		return Parent->MutableCast<FRCSignatureTreeFieldItem>();
	}
	return nullptr;
}

FRCSignatureField* FRCSignatureTreeActionItem::FindParentFieldMutable(URCSignatureRegistry** OutRegistry)
{
	if (TSharedPtr<FRCSignatureTreeFieldItem> ParentFieldItem = GetParentFieldItem())
	{
		return ParentFieldItem->FindFieldMutable(OutRegistry);
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
