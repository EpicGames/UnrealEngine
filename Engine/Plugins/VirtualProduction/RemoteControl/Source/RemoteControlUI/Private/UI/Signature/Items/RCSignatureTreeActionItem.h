// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureTreeItemBase.h"

class FRCSignatureTreeFieldItem;
class URCSignatureRegistry;
struct FRCSignatureActionInstance;
struct FRCSignatureActionIcon;
struct FRCSignatureField;

/** Item class representing a Field owned by a Signature */
class FRCSignatureTreeActionItem : public FRCSignatureTreeItemBase
{
public:
	static constexpr ERCSignatureTreeItemType StaticItemType = ERCSignatureTreeItemType::Action;

	explicit FRCSignatureTreeActionItem(int32 InActionIndex, const TSharedPtr<SRCSignatureTree>& InSignatureTree);

	const FRCSignatureActionInstance* FindActionInstance() const;

	FRCSignatureActionInstance* FindActionInstanceMutable();

	FRCSignatureActionIcon GetIcon() const;

protected:
	//~ Begin FRCSignatureTreeItemBase
	virtual void BuildPathSegment(FStringBuilderBase& InStringBuilder) const override;
	virtual TOptional<bool> IsEnabled() const override;
	virtual int32 RemoveFromRegistry() override;
	virtual FText GetDisplayNameText() const override;
	virtual FText GetDescription() const override;
	virtual TSharedPtr<FStructOnScope> MakeSelectionStruct() override;
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged) override;
	virtual ERCSignatureTreeItemType GetItemType() const override { return StaticItemType; }
	//~ End FRCSignatureTreeItemBase

private:
	TSharedPtr<FRCSignatureTreeFieldItem> GetParentFieldItem() const;

	FRCSignatureField* FindParentFieldMutable(URCSignatureRegistry** OutRegistry = nullptr);

	int32 ActionIndex;
};
