// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeFieldItem.h"
#include "RCSignature.h"
#include "RCSignatureRegistry.h"
#include "RCSignatureTreeActionItem.h"
#include "RCSignatureTreeSignatureItem.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "RCSignatureTreeFieldItem"

FRCSignatureTreeFieldItem::FRCSignatureTreeFieldItem(int32 InFieldIndex, const TSharedPtr<SRCSignatureTree>& InSignatureTree)
	: FRCSignatureTreeItemBase(InSignatureTree)
	, FieldIndex(InFieldIndex)
{
}

const FRCSignatureField* FRCSignatureTreeFieldItem::FindField() const
{
	TSharedPtr<FRCSignatureTreeSignatureItem> ParentSignatureItem = GetParentSignatureItem();
	if (!ParentSignatureItem.IsValid())
	{
		return nullptr;
	}

	const FRCSignature* Signature = ParentSignatureItem->FindSignature();
	if (!Signature || !Signature->Fields.IsValidIndex(FieldIndex))
	{
		return nullptr;
	}

	return &Signature->Fields[FieldIndex];
}

FRCSignatureField* FRCSignatureTreeFieldItem::FindFieldMutable(URCSignatureRegistry** OutRegistry)
{
	FRCSignature* Signature = FindParentSignature(OutRegistry);
	if (!Signature || !Signature->Fields.IsValidIndex(FieldIndex))
	{
		return nullptr;
	}
	return &Signature->Fields[FieldIndex];
}

void FRCSignatureTreeFieldItem::AddAction(const UScriptStruct* InActionType)
{
	URCSignatureRegistry* Registry;

	FRCSignatureField* Field = FindFieldMutable(&Registry);
	if (!Field)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("AddSignatureAction", "Add Signature Action"));
	check(Registry);
	Registry->Modify();
	Field->Actions.Emplace(InActionType, *Field);
}

void FRCSignatureTreeFieldItem::BuildPathSegment(FStringBuilderBase& InBuilder) const
{
	InBuilder << FieldIndex;
}

TOptional<bool> FRCSignatureTreeFieldItem::IsEnabled() const
{
	if (const FRCSignatureField* Field = FindField())
	{
		return Field->bEnabled;
	}
	return TOptional<bool>();
}

void FRCSignatureTreeFieldItem::SetEnabled(bool bInEnabled)
{
	URCSignatureRegistry* Registry;

	FRCSignatureField* Field = FindFieldMutable(&Registry);
	if (!Field || Field->bEnabled == bInEnabled)
	{
		return;
	}

	check(Registry);

	FScopedTransaction Transaction(bInEnabled
		? LOCTEXT("EnableField", "Enable Field")
		: LOCTEXT("DisableField", "Disable Field"));

	Registry->Modify();
	Field->bEnabled = bInEnabled;
}

FText FRCSignatureTreeFieldItem::GetDisplayNameText() const
{
	if (const FRCSignatureField* Field = FindField())
	{
		return FText::FromName(Field->FieldPath.GetFieldName());
	}
	return FText::GetEmpty();
}

FText FRCSignatureTreeFieldItem::GetDescription() const
{
	const FRCSignatureField* Field = FindField();
	if (!Field)
	{
		return FText::GetEmpty();
	}

	FFormatNamedArguments Arguments;
	Arguments.Add(TEXT("ObjectPath"), FText::FromString(Field->ObjectRelativePath));
	Arguments.Add(TEXT("ClassPath"), FText::FromString(Field->SupportedClass.ToString()));

	return FText::Format(LOCTEXT("DescriptionFormat", "{ObjectPath} ({ClassPath})"), MoveTemp(Arguments));
}

int32 FRCSignatureTreeFieldItem::RemoveFromRegistry()
{
	URCSignatureRegistry* Registry;

	FRCSignature* Signature = FindParentSignature(&Registry);
	if (!Signature || !Signature->Fields.IsValidIndex(FieldIndex))
	{
		return 0;
	}

	check(Registry);

	FScopedTransaction Transaction(LOCTEXT("RemoveField", "Remove Field"));
	Registry->Modify();
	Signature->Fields.RemoveAt(FieldIndex);

	// Update the field indices of sibling field items that have a greater field index than the index being removed 
	ForEachSiblingFieldItem(
		[RemovedFieldIndex = FieldIndex](const TSharedPtr<FRCSignatureTreeFieldItem>& InSiblingItem)
		{
			if (InSiblingItem->FieldIndex > RemovedFieldIndex)
			{
				--InSiblingItem->FieldIndex;
				InSiblingItem->RebuildPath();
			}
		});

	return 1;
}

void FRCSignatureTreeFieldItem::GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const
{
	const FRCSignatureField* Field = FindField();
	if (!Field)
	{
		return;
	}

	const TSharedPtr<SRCSignatureTree> SignatureTree = GetSignatureTree();

	OutChildren.Reserve(OutChildren.Num() + Field->Actions.Num());
	for (int32 ActionIndex = 0; ActionIndex < Field->Actions.Num(); ++ActionIndex)
	{
		OutChildren.Add(MakeShared<FRCSignatureTreeActionItem>(ActionIndex, SignatureTree));
	}
}

TSharedPtr<FRCSignatureTreeSignatureItem> FRCSignatureTreeFieldItem::GetParentSignatureItem() const
{
	if (TSharedPtr<FRCSignatureTreeItemBase> Parent = GetParent())
	{
		return Parent->MutableCast<FRCSignatureTreeSignatureItem>();
	}
	return nullptr;
}

FRCSignature* FRCSignatureTreeFieldItem::FindParentSignature(URCSignatureRegistry** OutRegistry)
{
	TSharedPtr<FRCSignatureTreeSignatureItem> ParentSignatureItem = GetParentSignatureItem();
	if (!ParentSignatureItem.IsValid())
	{
		return nullptr;
	}

	URCSignatureRegistry* Registry = ParentSignatureItem->GetRegistry();

	FRCSignature* Signature = ParentSignatureItem->FindSignatureMutable(Registry);
	if (!Signature)
	{
		return nullptr;
	}

	if (OutRegistry)
	{
		*OutRegistry = Registry;	
	}
	return Signature;
}

void FRCSignatureTreeFieldItem::ForEachSiblingFieldItem(TFunctionRef<void(const TSharedPtr<FRCSignatureTreeFieldItem>&)> InCallable)
{
	if (TSharedPtr<FRCSignatureTreeSignatureItem> ParentSignatureItem = GetParentSignatureItem())
	{
		ParentSignatureItem->VisitChildren(
			[&InCallable](const TSharedPtr<FRCSignatureTreeItemBase>& InChildItem)
			{
				if (const TSharedPtr<FRCSignatureTreeFieldItem> FieldItem = InChildItem->MutableCast<FRCSignatureTreeFieldItem>())
				{
					InCallable(FieldItem);
				}
				return true; // Continue iteration
			}
			, /*bRecursive*/ false);
	}
}

#undef LOCTEXT_NAMESPACE
