// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureTreeItemBase.h"

class FRCSignatureTreeSignatureItem;
class URCSignatureRegistry;
struct FRCSignature;
struct FRCSignatureField;

/** Item class representing a Field owned by a Signature */
class FRCSignatureTreeFieldItem : public FRCSignatureTreeItemBase
{
public:
	static constexpr ERCSignatureTreeItemType StaticItemType = ERCSignatureTreeItemType::Field;

	explicit FRCSignatureTreeFieldItem(int32 InFieldIndex, const TSharedPtr<SRCSignatureTree>& InSignatureTree);

	const FRCSignatureField* FindField() const;

	FRCSignatureField* FindFieldMutable(URCSignatureRegistry** OutRegistry);

	void AddAction(const UScriptStruct* InActionType);

protected:
	//~ Begin FRCSignatureTreeItemBase
	virtual void BuildPathSegment(FStringBuilderBase& InStringBuilder) const override;
	virtual TOptional<bool> IsEnabled() const override;
	virtual void SetEnabled(bool bInEnabled) override;
	virtual FText GetDisplayNameText() const override;
	virtual FText GetDescription() const override;
	virtual int32 RemoveFromRegistry() override;
	virtual ERCSignatureTreeItemType GetItemType() const override { return StaticItemType; }
	virtual void GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const override;
	//~ End FRCSignatureTreeItemBase

private:
	TSharedPtr<FRCSignatureTreeSignatureItem> GetParentSignatureItem() const;

	FRCSignature* FindParentSignature(URCSignatureRegistry** OutRegistry);

	/** Iterates each field item sibling */
	void ForEachSiblingFieldItem(TFunctionRef<void(const TSharedPtr<FRCSignatureTreeFieldItem>&)> InCallable);

	int32 FieldIndex;
};
