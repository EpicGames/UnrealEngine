// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeItemBase.h"
#include "RCSignatureTreeRootItem.h"
#include "UI/Signature/RCSignatureTreeItemSelection.h"
#include "UI/Signature/SRCSignatureTree.h"

const FLazyName FRCSignatureTreeItemBase::SignatureTreeItemName = "SignatureTreeItem";

FRCSignatureTreeItemBase::FRCSignatureTreeItemBase(const TSharedPtr<SRCSignatureTree>& InSignatureTree)
	: FRCLogicModeBase(InSignatureTree ? InSignatureTree->GetRemoteControlPanel() : nullptr)
	, SignatureTreeWeak(InSignatureTree)
{
}

void FRCSignatureTreeItemBase::AddTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags)
{
	EnumAddFlags(TreeViewFlags, InFlags);
}

void FRCSignatureTreeItemBase::RemoveTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags)
{
	EnumRemoveFlags(TreeViewFlags, InFlags);
}

bool FRCSignatureTreeItemBase::HasAnyTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags) const
{
	return EnumHasAnyFlags(TreeViewFlags, InFlags);
}

void FRCSignatureTreeItemBase::SetSelected(bool bInSelected, bool bInIsMultiSelection)
{
	if (TSharedPtr<FRCSignatureTreeItemSelection> Selection = GetRootSelection())
	{
		Selection->SetSelected(SharedThis(this), bInSelected, bInIsMultiSelection);
	}
}

bool FRCSignatureTreeItemBase::IsSelected() const
{
	if (TSharedPtr<FRCSignatureTreeItemSelection> Selection = GetRootSelection())
	{
		return Selection->IsSelected(SharedThis(this));
	}
	return false;
}

void FRCSignatureTreeItemBase::RebuildChildren()
{
	// Save current children in a map for restoring the flags
	TMap<FName, TSharedPtr<FRCSignatureTreeItemBase>> OldChildren;
	OldChildren.Reserve(Children.Num());
	for (const TSharedPtr<FRCSignatureTreeItemBase>& Child : Children)
	{
		OldChildren.Add(Child->GetPathId(), Child);
	}

	Children.Reset();
	GenerateChildren(Children);

	TSharedRef<FRCSignatureTreeItemBase> This = SharedThis(this);
	for (const TSharedPtr<FRCSignatureTreeItemBase>& Child : Children)
	{
		Child->Initialize(This);
		if (TSharedPtr<FRCSignatureTreeItemBase>* OldChild = OldChildren.Find(Child->GetPathId()))
		{
			Child->RestoreFrom(*OldChild);
		}
		Child->RebuildChildren();
	}

	PostChildrenRebuild();
}

void FRCSignatureTreeItemBase::VisitChildren(TFunctionRef<bool(const TSharedPtr<FRCSignatureTreeItemBase>&)> InCallable, bool bInRecursive)
{
	for (const TSharedPtr<FRCSignatureTreeItemBase>& Child : Children)
	{
		if (!InCallable(Child))
		{
			break;
		}

		if (bInRecursive)
		{
			Child->VisitChildren(InCallable, /*bRecursive*/true);
		}
	}
}

void FRCSignatureTreeItemBase::RebuildPath()
{
	Path = BuildPath();

	// Rebuilds path for each item recursively (VisitChildren isn't setup to be recursive, but RebuildPath is)
	VisitChildren(
		[](const TSharedPtr<FRCSignatureTreeItemBase>& InChild)
		{
			InChild->RebuildPath();
			return true; // continue iteration
		}
		, /*bRecursive*/false);
}

void FRCSignatureTreeItemBase::Initialize(const TSharedPtr<FRCSignatureTreeItemBase>& InParent)
{
	ParentWeak = InParent;
	SelectionWeak = GetRootSelection();
	Path = BuildPath();
}

void FRCSignatureTreeItemBase::RestoreFrom(const TSharedPtr<FRCSignatureTreeItemBase>& InOldItem)
{
	if (InOldItem.IsValid())
	{
		// The only important things in restoration are flags & children.
		// Children are only restored so they can restore their flags
		TreeViewFlags = InOldItem->GetTreeViewFlags();
		Children = InOldItem->GetChildren();
	}
}

TSharedPtr<FRCSignatureTreeItemSelection> FRCSignatureTreeItemBase::GetRootSelection() const
{
	TSharedPtr<FRCSignatureTreeItemSelection> Selection = SelectionWeak.Pin();
	if (Selection.IsValid())
	{
		return Selection;
	}

	TSharedPtr<const FRCSignatureTreeItemBase> Parent = GetParent();
	if (!Parent.IsValid())
	{
		return nullptr;
	}

	while (Parent->GetParent().IsValid())
	{
		Parent = Parent->GetParent();
	}

	TSharedPtr<const FRCSignatureTreeRootItem> RootItem = Parent->Cast<FRCSignatureTreeRootItem>();
	if (!RootItem.IsValid())
	{
		return nullptr;
	}

	Selection = RootItem->GetSelection();
	SelectionWeak = Selection;
	return Selection;
}

FName FRCSignatureTreeItemBase::BuildPath() const
{
	// For now, Signatures would have at most 2 items here, and Fields 3.
	TArray<const FRCSignatureTreeItemBase*, TInlineAllocator<3>> AncestorItems;

	// Generate Ancestor Path Array
	{
		const FRCSignatureTreeItemBase* CurrentItem = this;
		while (CurrentItem)
		{
			AncestorItems.Add(CurrentItem);
			CurrentItem = CurrentItem->GetParent().Get();
		}
	}

	// Expectation is that signature items will generate a path from Guid (32 characters)
	// with a first dot to the field path (index : 1-2 characters for the numbers in most use cases)
	// and a second dot with the action path (index : 1-2 characters)
	// the expected size of the string is 36 characters... rounded up to a multiple of 8 => 40
	TStringBuilder<40> PathBuilder;

	// Build Path starting from the farthest Ancestor.
	for (const FRCSignatureTreeItemBase* Item : ReverseIterate(AncestorItems))
	{
		if (PathBuilder.Len() > 0)
		{
			PathBuilder << TEXT(".");
		}
		Item->BuildPathSegment(PathBuilder);
	}

	return PathBuilder.ToString();
}

int32 FRCSignatureTreeItemBase::RemoveModel()
{
	return RemoveFromRegistry();
}
