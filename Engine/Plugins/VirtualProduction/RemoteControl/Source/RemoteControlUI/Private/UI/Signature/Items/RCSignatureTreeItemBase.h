// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/BaseLogicUI/RCLogicModeBase.h"

class FEditPropertyChain;
class FRCSignatureTreeItemSelection;
class FStructOnScope;
class SRCSignatureTree;
struct FPropertyChangedEvent;

enum class ERCSignatureTreeItemViewFlags : uint8
{
	None = 0,
	Expanded = 1 << 0,
	Hidden   = 1 << 1,
};
ENUM_CLASS_FLAGS(ERCSignatureTreeItemViewFlags)

enum class ERCSignatureTreeItemType : uint8
{
	Undefined,
	Root,
	Signature,
	Field,
	Action,
};

/** Base class for any Item represented in the Signature Tree */
class FRCSignatureTreeItemBase : public FRCLogicModeBase
{
	friend class FRCSignatureTreeRootItem;

public:
	static const FLazyName SignatureTreeItemName;

	/**
	 * Creates an Item and Initializes it with the given Parent
	 * NOTE: this does not build/make child items. RebuildChildren would need to be called separately if necessary.
	 */
	template<typename T, typename... ArgTypes UE_REQUIRES(std::is_base_of_v<FRCSignatureTreeItemBase, T>)>
	static TSharedRef<T> Create(const TSharedPtr<FRCSignatureTreeItemBase>& InParent, ArgTypes&&... InArgs)
	{
		TSharedRef<T> Item = MakeShared<T>(Forward<ArgTypes>(InArgs)...);
		Item->Initialize(InParent);
		return Item;
	}

	explicit FRCSignatureTreeItemBase(const TSharedPtr<SRCSignatureTree>& InSignatureTree);

	virtual FName GetModelType() const override
	{
		return SignatureTreeItemName;
	}

	virtual int32 RemoveModel() override;

	virtual TOptional<bool> IsEnabled() const
	{
		return TOptional<bool>();
	}

	virtual void SetEnabled(bool bInEnabled)
	{
	}

	FName GetPathId() const
	{
		return Path;
	}

	ERCSignatureTreeItemViewFlags GetTreeViewFlags() const
	{
		return TreeViewFlags;
	}

	void AddTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags);

	void RemoveTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags);

	bool HasAnyTreeViewFlags(ERCSignatureTreeItemViewFlags InFlags) const;

	virtual FText GetDisplayNameText() const
	{
		return FText::GetEmpty();
	}

	virtual void SetRenaming(bool bInRenaming)
	{
	}

	/**
	 * Gets the Rename Delegate
	 * @return the delegate to when renaming state changes, or nullptr if item does not support renaming
	 */
	virtual TMulticastDelegateRegistration<void(bool)>* GetOnRenameStateChanged() const
	{
		return nullptr;
	}

	virtual void SetDisplayNameText(const FText& InText)
	{
	}

	virtual FText GetDescription() const
	{
		return FText::GetEmpty();
	}

	virtual int32 RemoveFromRegistry()
	{
		return 0;
	}

	void SetSelected(bool bInSelected, bool bInIsMultiSelection);

	bool IsSelected() const;

	/** Optional: makes a struct on scope for selection visualization in Details Panel */
	virtual TSharedPtr<FStructOnScope> MakeSelectionStruct()
	{
		return nullptr;
	}

	/** Called when the Selection Struct created for this Item is changed via the Details Panel */
	virtual void NotifyPostChange(const FPropertyChangedEvent& InPropertyChangedEvent, FEditPropertyChain* InPropertyThatChanged)
	{
	}

	virtual ERCSignatureTreeItemType GetItemType() const
	{
		return ERCSignatureTreeItemType::Undefined;
	}

	template<typename InItemType>
	TSharedPtr<InItemType> MutableCast()
	{
		if (GetItemType() == InItemType::StaticItemType)
		{
			return StaticCastSharedRef<InItemType>(SharedThis(this));
		}
		return nullptr;
	}

	template<typename InItemType>
	TSharedPtr<const InItemType> Cast() const
	{
		if (GetItemType() == InItemType::StaticItemType)
		{
			return StaticCastSharedRef<const InItemType>(SharedThis(this));
		}
		return nullptr;
	}

	TConstArrayView<TSharedPtr<FRCSignatureTreeItemBase>> GetChildren() const
	{
		return Children;
	}

	TSharedPtr<FRCSignatureTreeItemBase> GetParent() const
	{
		return ParentWeak.Pin();
	}

	TSharedPtr<SRCSignatureTree> GetSignatureTree() const
	{
		return SignatureTreeWeak.Pin();
	}

	void RebuildChildren();

	void VisitChildren(TFunctionRef<bool(const TSharedPtr<FRCSignatureTreeItemBase>&)> InCallable, bool bInRecursive);

protected:
	/** Called when needing to recalculate a new path to the item */
	void RebuildPath();

	virtual void BuildPathSegment(FStringBuilderBase& InBuilder) const
	{
	}

	virtual void GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const
	{
	}

	virtual void PostChildrenRebuild()
	{
	}

private:
	void Initialize(const TSharedPtr<FRCSignatureTreeItemBase>& InParent);

	void RestoreFrom(const TSharedPtr<FRCSignatureTreeItemBase>& InOldItem);

	TSharedPtr<FRCSignatureTreeItemSelection> GetRootSelection() const;

	/** Builds the path from the root to the item. Each item will be its own segment delimited by a dot */
	FName BuildPath() const;

	/** Unique path from the root to the item. Used to identify items in the Signature Tree */
	FName Path;

	TArray<TSharedPtr<FRCSignatureTreeItemBase>> Children;

	TWeakPtr<FRCSignatureTreeItemBase> ParentWeak;

	TWeakPtr<SRCSignatureTree> SignatureTreeWeak;

	ERCSignatureTreeItemViewFlags TreeViewFlags = ERCSignatureTreeItemViewFlags::Expanded;

	/** Cached weak pointer to the selection object to avoid having to go up the hierarchy to find it */
	mutable TWeakPtr<FRCSignatureTreeItemSelection> SelectionWeak;
};
