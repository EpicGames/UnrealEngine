// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeRootItem.h"
#include "RCSignature.h"
#include "RCSignatureRegistry.h"
#include "RCSignatureTreeSignatureItem.h"
#include "UI/Signature/RCSignatureTreeItemSelection.h"
#include "UI/Signature/SRCSignatureTree.h"

FRCSignatureTreeRootItem::FRCSignatureTreeRootItem(const TSharedPtr<SRCSignatureTree>& InSignatureTree)
	: FRCSignatureTreeItemBase(InSignatureTree)
	, Selection(MakeShared<FRCSignatureTreeItemSelection>())
{
	SelectionWeak = Selection;
}

void FRCSignatureTreeRootItem::GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const
{
	TSharedPtr<SRCSignatureTree> SignatureTree = GetSignatureTree();
	if (!SignatureTree.IsValid())
	{
		return;
	}

	URCSignatureRegistry* SignatureRegistry = SignatureTree->GetSignatureRegistry();
	if (!SignatureRegistry)
	{
		return;
	}

	TConstArrayView<FRCSignature> Signatures = SignatureRegistry->GetSignatures();
	OutChildren.Reserve(OutChildren.Num() + Signatures.Num());

	for (const FRCSignature& Signature : Signatures)
	{
		OutChildren.Add(MakeShared<FRCSignatureTreeSignatureItem>(Signature, SignatureTree));
	}
}

void FRCSignatureTreeRootItem::PostChildrenRebuild()
{
	Selection->RecacheSelectedItems(SharedThis(this));
}
