// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCSignatureTreeItemBase.h"

class FRCSignatureTreeItemSelection;

/**
 * Item class for the Root of all items
 * This is primarily so that the Top level items can have the same set of functionalities as the rest of the items in the tree 
 */
class FRCSignatureTreeRootItem : public FRCSignatureTreeItemBase
{
public:
	static constexpr ERCSignatureTreeItemType StaticItemType = ERCSignatureTreeItemType::Root;

	explicit FRCSignatureTreeRootItem(const TSharedPtr<SRCSignatureTree>& InSignatureTree);

	TArray<TSharedPtr<FRCSignatureTreeItemBase>>& GetChildrenMutable()
	{
		return Children;
	}

	const TSharedRef<FRCSignatureTreeItemSelection>& GetSelection() const
	{
		return Selection;
	}

protected:
	//~ Begin FRCSignatureTreeItemBase
	virtual void BuildPathSegment(FStringBuilderBase& InBuilder) const override {}
	virtual ERCSignatureTreeItemType GetItemType() const { return StaticItemType; }
	virtual void GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const override;
	virtual void PostChildrenRebuild() override;
	//~ End FRCSignatureTreeItemBase

	TSharedRef<FRCSignatureTreeItemSelection> Selection;
};
