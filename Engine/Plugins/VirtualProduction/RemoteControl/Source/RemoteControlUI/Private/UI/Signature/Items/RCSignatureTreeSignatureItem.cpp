// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeSignatureItem.h"
#include "Framework/Notifications/NotificationManager.h"
#include "IRemoteControlUIModule.h"
#include "RCSignatureRegistry.h"
#include "RCSignatureTreeFieldItem.h"
#include "RemoteControlPreset.h"
#include "ScopedTransaction.h"
#include "UI/SRemoteControlPanel.h"
#include "UI/Signature/SRCSignatureTree.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "RCSignatureTreeSignatureItem"

FRCSignatureTreeSignatureItem::FRCSignatureTreeSignatureItem(const FRCSignature& InSignature, const TSharedPtr<SRCSignatureTree>& InSignatureTree)
	: FRCSignatureTreeItemBase(InSignatureTree)
	, SignatureId(InSignature.Id)
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		RegistryWeak = Preset->GetSignatureRegistry();
	}
}

const FGuid& FRCSignatureTreeSignatureItem::GetSignatureId() const
{
	return SignatureId;
}

URCSignatureRegistry* FRCSignatureTreeSignatureItem::GetRegistry() const
{
	return RegistryWeak.Get();
}

const FRCSignature* FRCSignatureTreeSignatureItem::FindSignature() const
{
	if (URCSignatureRegistry* SignatureRegistry = GetRegistry())
	{
		return SignatureRegistry->FindSignature(SignatureId);
	}
	return nullptr;
}

FRCSignature* FRCSignatureTreeSignatureItem::FindSignatureMutable(URCSignatureRegistry* InRegistry)
{
	if (InRegistry)
	{
		return InRegistry->FindSignatureMutable(SignatureId);
	}
	return nullptr;
}

bool FRCSignatureTreeSignatureItem::AddField(URCSignatureRegistry* InRegistry, const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	FRCSignature* Signature = FindSignatureMutable(InRegistry);
	if (!Signature)
	{
		return false;
	}

	TArray<FRCSignatureField, TInlineAllocator<1>> Fields;
	{
		TArray<UObject*> OuterObjects;
		InPropertyHandle->GetOuterObjects(OuterObjects);
		Fields.Reserve(OuterObjects.Num());

		const FRCFieldPathInfo PathInfo(InPropertyHandle->GeneratePathToProperty(), /*bSkipDuplicates*/true);
		const FProperty* Property = InPropertyHandle->GetProperty();

		for (UObject* OuterObject : OuterObjects)
		{
			Fields.Emplace(FRCSignatureField::CreateField(PathInfo, OuterObject, Property));
		}
	}

	return Signature->AddFields(Fields) > 0;
}

void FRCSignatureTreeSignatureItem::ApplySignature(TConstArrayView<TWeakObjectPtr<UObject>> InObjects)
{
	if (InObjects.IsEmpty())
	{
		return;
	}

	URemoteControlPreset* Preset = GetPreset();
	if (!Preset)
	{
		return;
	}

	const FRCSignature* Signature = FindSignature();
	if (!Signature || !Signature->bEnabled)
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("ApplySignature", "Apply Signature"));

	int32 AffectedCount = Signature->ApplySignature(Preset, InObjects);

	if (AffectedCount > 0)
	{
		const FText MessageFormat = LOCTEXT("SignatureAppliedMessage", "Signature applied to {0} property entities.");

		FNotificationInfo NotificationInfo(FText::Format(MessageFormat, FText::AsNumber(AffectedCount)));
		NotificationInfo.ExpireDuration = 3.f;
		NotificationInfo.bFireAndForget = true;

		FSlateNotificationManager::Get().AddNotification(NotificationInfo);
	}
	else
	{
		// Nothing was exposed, cancel transaction
		Transaction.Cancel();
	}
}

void FRCSignatureTreeSignatureItem::BuildPathSegment(FStringBuilderBase& InBuilder) const
{
	SignatureId.AppendString(InBuilder, EGuidFormats::DigitsLower);
}

TOptional<bool> FRCSignatureTreeSignatureItem::IsEnabled() const
{
	if (const FRCSignature* Signature = FindSignature())
	{
		return Signature->bEnabled;
	}
	return TOptional<bool>();
}

void FRCSignatureTreeSignatureItem::SetEnabled(bool bInEnabled)
{
	URCSignatureRegistry* SignatureRegistry = GetRegistry();

	FRCSignature* Signature = FindSignatureMutable(SignatureRegistry);
	if (!Signature || Signature->bEnabled == bInEnabled)
	{
		return;
	}

	FScopedTransaction Transaction(bInEnabled
		? LOCTEXT("EnableSignature", "Enable Signature")
		: LOCTEXT("DisableSignature", "Disable Signature"));

	SignatureRegistry->Modify();
	Signature->bEnabled = bInEnabled;
}

FText FRCSignatureTreeSignatureItem::GetDisplayNameText() const
{
	if (const FRCSignature* Signature = FindSignature())
	{
		return Signature->DisplayName;
	}
	return FText::GetEmpty();
}

void FRCSignatureTreeSignatureItem::SetRenaming(bool bInRenaming)
{
	if (bRenaming != bInRenaming)
	{
		bRenaming = bInRenaming;
		OnRenameStateChangedDelegate.Broadcast(bInRenaming);
	}
}

TMulticastDelegateRegistration<void(bool)>* FRCSignatureTreeSignatureItem::GetOnRenameStateChanged() const
{
	return &OnRenameStateChangedDelegate;
}

void FRCSignatureTreeSignatureItem::SetDisplayNameText(const FText& InText)
{
	URCSignatureRegistry* SignatureRegistry = GetRegistry();

	FRCSignature* Signature = FindSignatureMutable(SignatureRegistry);
	if (!Signature || Signature->DisplayName.EqualTo(InText))
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("SetSignatureName", "Set Signature Name"));
	SignatureRegistry->Modify();
	Signature->DisplayName = InText;
}

FText FRCSignatureTreeSignatureItem::GetDescription() const
{
	return FText::GetEmpty();
}

int32 FRCSignatureTreeSignatureItem::RemoveFromRegistry()
{
	URCSignatureRegistry* SignatureRegistry = GetRegistry();
	if (!SignatureRegistry)
	{
		return 0;
	}

	// Remove Signatures from Preset
	FScopedTransaction Transaction(LOCTEXT("RemoveSignature", "Remove Signature"));
	SignatureRegistry->Modify();
	return SignatureRegistry->RemoveSignature(SignatureId);
}

void FRCSignatureTreeSignatureItem::GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const
{
	const FRCSignature* Signature = FindSignature();
	if (!Signature)
	{
		return;
	}

	const TSharedPtr<SRCSignatureTree> SignatureTree = GetSignatureTree();

	OutChildren.Reserve(OutChildren.Num() + Signature->Fields.Num());
	for (int32 FieldIndex = 0; FieldIndex < Signature->Fields.Num(); ++FieldIndex)
	{
		OutChildren.Add(MakeShared<FRCSignatureTreeFieldItem>(FieldIndex, SignatureTree));
	}
}

#undef LOCTEXT_NAMESPACE
