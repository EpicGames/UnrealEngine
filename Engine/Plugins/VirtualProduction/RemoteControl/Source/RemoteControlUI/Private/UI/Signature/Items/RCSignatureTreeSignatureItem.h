// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "RCSignatureTreeItemBase.h"
#include "UI/Signature/IRCSignatureItem.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class IPropertyHandle;
class URCSignatureRegistry;
struct FRCSignature;

/** Item class representing an RC Signature */
class FRCSignatureTreeSignatureItem : public FRCSignatureTreeItemBase, public IRCSignatureItem
{
public:
	static constexpr ERCSignatureTreeItemType StaticItemType = ERCSignatureTreeItemType::Signature;

	explicit FRCSignatureTreeSignatureItem(const FRCSignature& InSignature, const TSharedPtr<SRCSignatureTree>& InSignatureTree);

	const FGuid& GetSignatureId() const;

	URCSignatureRegistry* GetRegistry() const;

	const FRCSignature* FindSignature() const;

	FRCSignature* FindSignatureMutable(URCSignatureRegistry* InRegistry);

	bool AddField(URCSignatureRegistry* InRegistry, const TSharedRef<IPropertyHandle>& InPropertyHandle);

	//~ Begin IRCSignatureItem
	virtual void ApplySignature(TConstArrayView<TWeakObjectPtr<UObject>> InObjects) override;
	//~ End IRCSignatureItem

protected:
	//~ Begin FRCSignatureTreeItem
	virtual void BuildPathSegment(FStringBuilderBase& InBuilder) const override;
	virtual TOptional<bool> IsEnabled() const override;
	virtual void SetEnabled(bool bInEnabled) override;
	virtual FText GetDisplayNameText() const override;
	virtual void SetRenaming(bool bInRenaming) override;
	virtual TMulticastDelegateRegistration<void(bool)>* GetOnRenameStateChanged() const override;
	virtual void SetDisplayNameText(const FText& InText) override;
	virtual FText GetDescription() const override;
	virtual int32 RemoveFromRegistry() override;
	virtual ERCSignatureTreeItemType GetItemType() const override { return StaticItemType; }
	virtual void GenerateChildren(TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const override;
	//~ End FRCSignatureTreeItem

private:
	TWeakObjectPtr<URCSignatureRegistry> RegistryWeak;

	FGuid SignatureId;

	/** delegate to call when entering/exiting rename mode */
	mutable TMulticastDelegate<void(bool)> OnRenameStateChangedDelegate;

	/** true if currently in rename mode */
	bool bRenaming = false;
};
