// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureLabelColumn.h"
#include "SRCSignatureLabel.h"

#define LOCTEXT_NAMESPACE "RCSignatureLabelColumn"

FName FRCSignatureLabelColumn::GetColumnId() const
{
	return TEXT("FRCSignatureLabelColumn");
}

SHeaderRow::FColumn::FArguments FRCSignatureLabelColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetColumnId())
		.FillWidth(0.5f)
		.DefaultLabel(LOCTEXT("DisplayName", "Label"))
		.ShouldGenerateWidget(true);
}

TSharedRef<SWidget> FRCSignatureLabelColumn::ConstructRowWidget(TSharedPtr<FRCSignatureTreeItemBase> InItem
	, const TSharedRef<SRCSignatureTree>& InList
	, const TSharedRef<SRCSignatureRow>& InRow)
{
	return SNew(SRCSignatureLabel, InItem.ToSharedRef(), InRow);
}

#undef LOCTEXT_NAMESPACE
