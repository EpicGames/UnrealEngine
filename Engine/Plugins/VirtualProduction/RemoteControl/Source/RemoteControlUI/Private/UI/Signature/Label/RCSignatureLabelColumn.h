// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/Signature/IRCSignatureColumn.h"

class FRCSignatureLabelColumn : public IRCSignatureColumn
{
	//~ Begin IRCSignatureColumn
	virtual FName GetColumnId() const override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual TSharedRef<SWidget> ConstructRowWidget(TSharedPtr<FRCSignatureTreeItemBase> InItem, const TSharedRef<SRCSignatureTree>& InList, const TSharedRef<SRCSignatureRow>& InRow) override;
	//~ End IRCSignatureColumn
};
