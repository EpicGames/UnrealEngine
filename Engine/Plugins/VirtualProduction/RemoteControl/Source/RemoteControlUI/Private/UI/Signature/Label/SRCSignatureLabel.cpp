// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureLabel.h"
#include "Styling/SlateTypes.h"
#include "UI/Signature/Items/RCSignatureTreeItemBase.h"
#include "UI/Signature/SRCSignatureRow.h"
#include "UI/Signature/SRCSignatureTree.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Views/SExpanderArrow.h"

#define LOCTEXT_NAMESPACE "SRCSignatureLabel"

void SRCSignatureLabel::Construct(const FArguments& InArgs
	, const TSharedRef<FRCSignatureTreeItemBase>& InItem
	, const TSharedRef<SRCSignatureRow>& InRow)
{
	ItemWeak = InItem;

	SignatureTreeWeak = InItem->GetSignatureTree();

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredHeight(25)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(6.f, 0.f, 0.f, 0.f)
			[
				SNew(SExpanderArrow, InRow)
				.IndentAmount(12)
				.ShouldDrawWires(true)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SCheckBox)
				.IsChecked(this, &SRCSignatureLabel::GetItemEnabledState)
				.OnCheckStateChanged(this, &SRCSignatureLabel::SetItemEnabledState)
				.ToolTipText(LOCTEXT("ItemEnableCheckBoxTooltip", "Determines whether the entry is enabled or not"))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(6.f, 0.f)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				CreateTextBlock(InItem, InRow)
			]
		]
	];
}

SRCSignatureLabel::~SRCSignatureLabel()
{
	if (TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin())
	{
		if (TMulticastDelegateRegistration<void(bool)>* OnRenameStateChanged = Item->GetOnRenameStateChanged())
		{
			OnRenameStateChanged->Remove(OnRenameStateChangedHandle);
			OnRenameStateChangedHandle.Reset();
		}
		Item->SetRenaming(false);
	}
}

TSharedRef<SWidget> SRCSignatureLabel::CreateTextBlock( const TSharedRef<FRCSignatureTreeItemBase>& InItem, const TSharedRef<SRCSignatureRow>& InRow)
{
	// Item can be renamed if it has a valid rename delegate
	if (TMulticastDelegateRegistration<void(bool)>* OnRenameStateChanged = InItem->GetOnRenameStateChanged())
	{
		OnRenameStateChangedHandle = OnRenameStateChanged->AddSP(this, &SRCSignatureLabel::OnItemRenameStateChanged);

		return SAssignNew(EditableTextBlock, SInlineEditableTextBlock)
			.Text(this, &SRCSignatureLabel::GetSignatureDisplayName)
			.OnTextCommitted(this, &SRCSignatureLabel::OnSignatureDisplayNameCommitted)
			.Justification(ETextJustify::Left)
			.IsSelected(InRow, &SRCSignatureRow::IsSelected)
			.OnEnterEditingMode(this, &SRCSignatureLabel::SetEditMode, /*bEditMode*/true)
			.OnExitEditingMode(this, &SRCSignatureLabel::SetEditMode, /*bEditMode*/false);
	}

	// Else resort to just showing the items display name without option to edit it
	return SNew(STextBlock)
		.Text(this, &SRCSignatureLabel::GetSignatureDisplayName)
		.Justification(ETextJustify::Left);
}

void SRCSignatureLabel::OnItemRenameStateChanged(bool bInRenaming)
{
	// Skip if the Edit Mode is already in the desired state
	if (bEditMode == bInRenaming || !EditableTextBlock.IsValid())
	{
		return;
	}

	if (bInRenaming)
	{
		EditableTextBlock->EnterEditingMode();
	}
	else
	{
		EditableTextBlock->ExitEditingMode();
	}
}

void SRCSignatureLabel::SetEditMode(bool bInEditMode)
{
	bEditMode = bInEditMode;

	if (TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin())
	{
		Item->SetRenaming(bInEditMode);
	}
}

ECheckBoxState SRCSignatureLabel::GetItemEnabledState() const
{
	if (!CachedCheckBoxState.IsSet())
	{
		TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
		TOptional<bool> ItemEnabled = Item.IsValid() ? Item->IsEnabled() : TOptional<bool>();

		if (ItemEnabled.IsSet())
		{
			CachedCheckBoxState = *ItemEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		}
		else
		{
			CachedCheckBoxState = ECheckBoxState::Undetermined;
		}
	}
	return *CachedCheckBoxState;
}

void SRCSignatureLabel::SetItemEnabledState(ECheckBoxState InState)
{
	if (TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin())
	{
		Item->SetEnabled(InState == ECheckBoxState::Checked);
		CachedCheckBoxState.Reset();
	}
}

FText SRCSignatureLabel::GetSignatureDisplayName() const
{
	if (!CachedDisplayName.IsSet())
	{
		TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();
		CachedDisplayName = Item.IsValid() ? Item->GetDisplayNameText() : FText::GetEmpty();
	}
	return *CachedDisplayName;
}

void SRCSignatureLabel::OnSignatureDisplayNameCommitted(const FText& InText, ETextCommit::Type InCommitType)
{
	if (TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin())
	{
		Item->SetDisplayNameText(InText);
		CachedDisplayName.Reset();
	}

	if (InCommitType == ETextCommit::OnEnter)
	{
		if (TSharedPtr<SRCSignatureTree> SignatureTree = SignatureTreeWeak.Pin())
		{
			SignatureTree->ProcessRenameQueue();
		}
	}
}

#undef LOCTEXT_NAMESPACE
