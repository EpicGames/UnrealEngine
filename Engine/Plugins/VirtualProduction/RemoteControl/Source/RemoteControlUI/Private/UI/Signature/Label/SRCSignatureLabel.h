// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class FRCSignatureTreeItemBase;
class SInlineEditableTextBlock;
class SRCSignatureRow;
class SRCSignatureTree;
enum class ECheckBoxState : uint8;

class SRCSignatureLabel : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureLabel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedRef<FRCSignatureTreeItemBase>& InItem
		, const TSharedRef<SRCSignatureRow>& InRow);

	virtual ~SRCSignatureLabel() override;

private:
	TSharedRef<SWidget> CreateTextBlock(const TSharedRef<FRCSignatureTreeItemBase>& InItem, const TSharedRef<SRCSignatureRow>& InRow);

	/** Called when the Item rename state changes */
	void OnItemRenameStateChanged(bool bInRenaming);

	void SetEditMode(bool bInEditMode);

	ECheckBoxState GetItemEnabledState() const;

	void SetItemEnabledState(ECheckBoxState InState);

	FText GetSignatureDisplayName() const;

	void OnSignatureDisplayNameCommitted(const FText& InText, ETextCommit::Type InCommitType);

	TWeakPtr<FRCSignatureTreeItemBase> ItemWeak;

	TWeakPtr<SRCSignatureTree> SignatureTreeWeak;

	TSharedPtr<SInlineEditableTextBlock> EditableTextBlock;

	mutable TOptional<ECheckBoxState> CachedCheckBoxState;

	mutable TOptional<FText> CachedDisplayName;

	FDelegateHandle OnRenameStateChangedHandle;

	bool bEditMode = false;
};
