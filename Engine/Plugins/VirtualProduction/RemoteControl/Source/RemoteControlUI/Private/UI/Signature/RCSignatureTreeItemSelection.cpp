// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCSignatureTreeItemSelection.h"
#include "Items/RCSignatureTreeItemBase.h"

FRCSignatureTreeItemSelection::FSelectionScope::FSelectionScope(FRCSignatureTreeItemSelection* InSelection)
	: Selection(InSelection)
{
	check(Selection);
	OldValue = Selection->bAllowNotifications;
	Selection->bAllowNotifications = false;
}

FRCSignatureTreeItemSelection::FSelectionScope::~FSelectionScope()
{
	Selection->bAllowNotifications = OldValue;
	Selection->NotifySelectionChanged();
}

FRCSignatureTreeItemSelection::FSelectionScope FRCSignatureTreeItemSelection::CreateSelectionScope()
{
	return FSelectionScope(this);
}

void FRCSignatureTreeItemSelection::SetSelected(const TSharedRef<FRCSignatureTreeItemBase>& InItem, bool bInSelected, bool bInMultiSelection)
{
	FSelectionScope SelectionScope = CreateSelectionScope();

	if (bInSelected)
	{
		if (!bInMultiSelection)
		{
			ClearSelection();
		}

		SelectedPaths.Add(InItem->GetPathId());
		CachedSelectedItems.AddUnique(InItem);
	}
	else
	{
		SelectedPaths.Remove(InItem->GetPathId());
		CachedSelectedItems.Remove(InItem);
	}
}

bool FRCSignatureTreeItemSelection::IsSelected(const TSharedPtr<const FRCSignatureTreeItemBase>& InItem) const
{
	return InItem.IsValid() && SelectedPaths.Contains(InItem->GetPathId());
}

void FRCSignatureTreeItemSelection::ClearSelection()
{
	FSelectionScope SelectionScope = CreateSelectionScope();

	CachedSelectedItems.Empty();
	SelectedPaths.Empty();
}

void FRCSignatureTreeItemSelection::NotifySelectionChanged()
{
	if (bAllowNotifications)
	{
		OnSelectionChangedDelegate.Broadcast();
	}
}

TArray<TSharedPtr<FRCSignatureTreeItemBase>> FRCSignatureTreeItemSelection::GetSelectedItems() const
{
	TArray<TSharedPtr<FRCSignatureTreeItemBase>> SelectedItems;
	SelectedItems.Reserve(CachedSelectedItems.Num());

	for (const TWeakPtr<FRCSignatureTreeItemBase>& SelectedItemWeak : CachedSelectedItems)
	{
		if (TSharedPtr<FRCSignatureTreeItemBase> SelectedItem = SelectedItemWeak.Pin())
		{
			SelectedItems.Add(MoveTemp(SelectedItem));
		}
	}

	return SelectedItems;
}

void FRCSignatureTreeItemSelection::RecacheSelectedItems(const TSharedRef<FRCSignatureTreeItemBase>& InRoot)
{
	// No Selection Scope is set here because this is not changing the Selected Paths
	// which is what truly determines whether an item is selected.
	CachedSelectedItems.Empty();

	auto ProcessItem = [&SelectedItems = CachedSelectedItems, &SelectedPaths = SelectedPaths](const TSharedPtr<FRCSignatureTreeItemBase>& InItem)->bool
	{
		if (SelectedPaths.Contains(InItem->GetPathId()))
		{
			SelectedItems.Add(InItem);
		}
		return true;
	};

	ProcessItem(InRoot);
	InRoot->VisitChildren(ProcessItem, /*bRecursive*/true);
}
