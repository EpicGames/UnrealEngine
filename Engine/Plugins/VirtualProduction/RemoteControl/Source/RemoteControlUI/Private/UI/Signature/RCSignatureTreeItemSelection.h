// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class FRCSignatureTreeItemBase;

class FRCSignatureTreeItemSelection : public TSharedFromThis<FRCSignatureTreeItemSelection>
{
public:
	/** Scope class to ensure Notify Selection Changed gets called at most once for the duration of the scope */
	struct FSelectionScope
	{
		explicit FSelectionScope(FRCSignatureTreeItemSelection* InSelection);
		~FSelectionScope();

	private:
		FRCSignatureTreeItemSelection* Selection;
		bool OldValue = false;
	};

	FSelectionScope CreateSelectionScope();

	TMulticastDelegateRegistration<void()>& OnSelectionChanged() const
	{
		return OnSelectionChangedDelegate;
	}

	void SetSelected(const TSharedRef<FRCSignatureTreeItemBase>& InItem, bool bInSelected, bool bInMultiSelection);

	bool IsSelected(const TSharedPtr<const FRCSignatureTreeItemBase>& InItem) const;

	void ClearSelection();

	void NotifySelectionChanged();

	TConstArrayView<TWeakPtr<FRCSignatureTreeItemBase>> GetSelectedItemsView() const
	{
		return CachedSelectedItems;
	}

	TArray<TSharedPtr<FRCSignatureTreeItemBase>> GetSelectedItems() const;

	/**
	 * Clears the current selection items (not paths!) and rebuilds a new selection list of all the items that match one of the selected paths.
	 * @param InRoot the root item to iterate
	 */
	void RecacheSelectedItems(const TSharedRef<FRCSignatureTreeItemBase>& InRoot);

private:
	TSet<FName> SelectedPaths;

	TArray<TWeakPtr<FRCSignatureTreeItemBase>> CachedSelectedItems;

	mutable TMulticastDelegate<void()> OnSelectionChangedDelegate;

	bool bAllowNotifications = true;
};
