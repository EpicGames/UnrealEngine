// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignaturePanel.h"
#include "Actions/RCSignatureActionColumn.h"
#include "Description/RCSignatureDescriptionColumn.h"
#include "Details/SRCSignatureDetails.h"
#include "Items/RCSignatureTreeRootItem.h"
#include "Items/RCSignatureTreeSignatureItem.h"
#include "Label/RCSignatureLabelColumn.h"
#include "Misc/MessageDialog.h"
#include "RCSignatureRegistry.h"
#include "RemoteControlPreset.h"
#include "SRCSignatureTree.h"
#include "ScopedTransaction.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/Panels/SRCDockPanel.h"
#include "UI/RemoteControlPanelStyle.h"
#include "UI/SRemoteControlPanel.h"

#define LOCTEXT_NAMESPACE "SRCSignaturePanel"

void SRCSignaturePanel::Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel)
{
	SRCLogicPanelBase::Construct(SRCLogicPanelBase::FArguments(), InPanel);

	const FRCPanelStyle& RCPanelStyle = FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.MinorPanel");

	// Signature Tree Panel
	TSharedRef<SRCMinorPanel> SignaturesTreePanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("SignaturesLabel", "Signatures"))
		.EnableFooter(false)
		[
			SAssignNew(SignatureTreeView, SRCSignatureTree, SharedThis(this), InPanel)
			.Columns(
				{
					MakeShared<FRCSignatureLabelColumn>(),
					MakeShared<FRCSignatureDescriptionColumn>(),
					MakeShared<FRCSignatureActionColumn>(InArgs._LiveMode),
				})
		];

	// Details Panel
	TSharedRef<SRCMinorPanel> SignaturesDetailsPanel = SNew(SRCMinorPanel)
		.HeaderLabel(LOCTEXT("DetailsLabel", "Details"))
		.EnableFooter(false)
		[
			SAssignNew(SignatureDetails, SRCSignatureDetails
				, GetSignatureRegistry()
				, SignatureTreeView->GetRootItem()->GetSelection())
		];

	constexpr float ContentPaddingY = 2.f;

	// Add New Signature Button
	TSharedRef<SButton> AddSignatureButton = SNew(SButton)
		.AddMetaData<FTagMetaData>(FTagMetaData(TEXT("Add Signature")))
		.IsEnabled_Lambda([LiveMode = InArgs._LiveMode]() { return !LiveMode.Get(); })
		.OnClicked(this, &SRCSignaturePanel::OnAddButtonClicked)
		.ForegroundColor(FSlateColor::UseForeground())
		.ButtonStyle(&RCPanelStyle.FlatButtonStyle)
		.ContentPadding(FMargin(4.f, ContentPaddingY))
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Content()
		[
			SNew(SBox)
			.WidthOverride(RCPanelStyle.IconSize.X)
			.HeightOverride(RCPanelStyle.IconSize.Y)
			[
				SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(FAppStyle::GetBrush("Icons.PlusCircle"))
			]
		];

	SignaturesTreePanel->AddHeaderToolbarItem(EToolbar::Left, AddSignatureButton);

	// Header Toolbar Heights are not fixed and vary depending on what is inside it
	// Since the Details Panel header has nothing but text in it (for now),
	// a box of a calculated height is added to compensate and match the height of the Signatures Panel
	{
		const float PanelStyleIconSize = RCPanelStyle.IconSize.Y;
		const float ButtonStylePadding = RCPanelStyle.FlatButtonStyle.NormalPadding.Bottom + RCPanelStyle.FlatButtonStyle.NormalPadding.Top;
		constexpr float ContentPadding = 2.f * ContentPaddingY;

		SignaturesDetailsPanel->AddHeaderToolbarItem(EToolbar::Left, SNew(SBox)
			.HeightOverride(PanelStyleIconSize + ButtonStylePadding + ContentPaddingY));
	}

	ChildSlot
		.Padding(RCPanelStyle.PanelPadding)
		[
			SNew(SSplitter)
			.Orientation(Orient_Horizontal)
			+ SSplitter::Slot()
			.Value(0.6f)
			[
				SignaturesTreePanel
			]
			+ SSplitter::Slot()
			.Value(0.4f)
			[
				SignaturesDetailsPanel
			]
		];
}

URCSignatureRegistry* SRCSignaturePanel::GetSignatureRegistry() const
{
	if (URemoteControlPreset* Preset = GetPreset())
	{
		return Preset->GetSignatureRegistry();
	}
	return nullptr;
}

void SRCSignaturePanel::AddToSignature(const FRCExposesPropertyArgs& InPropertyArgs)
{
	if (!SignatureTreeView.IsValid())
	{
		return;
	}

	URCSignatureRegistry* Registry = GetSignatureRegistry();
	if (!Registry)
	{
		return;
	}

	if (!InPropertyArgs.PropertyHandle.IsValid())
	{
		return;
	}

	TSharedRef<IPropertyHandle> PropertyHandle = InPropertyArgs.PropertyHandle.ToSharedRef();

	FScopedTransaction Transaction(LOCTEXT("AddToSignatureTransaction", "Add to Signature"));
	Registry->Modify();

	bool bFieldsAdded = false;

	TArray<TSharedPtr<FRCSignatureTreeItemBase>> SelectedItems = SignatureTreeView->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		TSharedRef<FRCSignatureTreeRootItem> RootItem = SignatureTreeView->GetRootItem();

		FRCSignature& Signature = Registry->AddSignature();

		// Make a new signature with temp view model item to add the field
		TSharedRef<FRCSignatureTreeSignatureItem> SignatureItem = FRCSignatureTreeItemBase::Create<FRCSignatureTreeSignatureItem>(RootItem, Signature, SignatureTreeView);
		if (SignatureItem->AddField(Registry, PropertyHandle))
		{
			// Select the Signature Item
			SignatureItem->SetSelected(/*bSelected*/true, /*bMultiSelection*/false);
			bFieldsAdded = true;
		}
	}
	else
	{
		for (const TSharedPtr<FRCSignatureTreeItemBase>& Item : SelectedItems)
		{
			if (TSharedPtr<FRCSignatureTreeSignatureItem> SignatureItem = Item->MutableCast<FRCSignatureTreeSignatureItem>())
			{
				if (SignatureItem->AddField(Registry, PropertyHandle))
				{
					bFieldsAdded = true;
				}
			}
		}
	}

	if (bFieldsAdded)
	{
		Refresh();
	}
	else
	{
		Transaction.Cancel();
	}
}

bool SRCSignaturePanel::IsListFocused() const
{
	return SignatureTreeView.IsValid() && SignatureTreeView->IsListFocused();
}

void SRCSignaturePanel::EnterRenameMode()
{
	if (SignatureTreeView.IsValid())
	{
		return SignatureTreeView->EnterRenameMode();
	}
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCSignaturePanel::GetSelectedLogicItems() const
{
	if (SignatureTreeView.IsValid())
	{
		return SignatureTreeView->GetSelectedLogicItems();
	}
	return {};
}

FReply SRCSignaturePanel::RequestDeleteSelectedItem()
{
	DeleteSelectedPanelItems();
	return FReply::Handled();
}

FReply SRCSignaturePanel::RequestDeleteAllItems()
{
	if (!SignatureTreeView.IsValid())
	{
		return FReply::Unhandled();
	}

	const EAppReturnType::Type UserResponse = FMessageDialog::Open(EAppMsgType::YesNo
		, FText::Format(LOCTEXT("DeleteAllWarning", "You are about to delete {0} signatures. Are you sure you want to proceed?"), SignatureTreeView->Num()));

	if (UserResponse == EAppReturnType::Yes)
	{
		return DeleteAllItems();
	}

	return FReply::Handled();
}

bool SRCSignaturePanel::CanCopyItems() const
{
	// todo: Copy/Paste Signatures
	return false;
}

bool SRCSignaturePanel::CanDuplicateItems() const
{
	// todo: Duplicate Signatures
	return false;
}

void SRCSignaturePanel::DeleteSelectedPanelItems()
{
	if (SignatureTreeView.IsValid())
	{
		SignatureTreeView->DeleteSelectedPanelItems();
		Refresh();
	}
}

void SRCSignaturePanel::PostUndo(bool bInSuccess)
{
	Refresh();
}

void SRCSignaturePanel::PostRedo(bool bInSuccess)
{
	Refresh();
}

void SRCSignaturePanel::Refresh()
{
	if (SignatureTreeView.IsValid())
	{
		SignatureTreeView->Refresh();
	}

	if (SignatureDetails.IsValid())
	{
		SignatureDetails->Refresh();
	}
}

FReply SRCSignaturePanel::OnAddButtonClicked()
{
	URCSignatureRegistry* SignatureRegistry = GetSignatureRegistry();
	if (!SignatureRegistry)
	{
		return FReply::Unhandled();
	}

	// Add new signature
	{
		FScopedTransaction Transaction(LOCTEXT("NewSignature", "New Signature"));
		SignatureRegistry->Modify();
		SignatureRegistry->AddSignature();
	}

	Refresh();
	return FReply::Handled();
}

FReply SRCSignaturePanel::DeleteAllItems()
{
	URCSignatureRegistry* SignatureRegistry = GetSignatureRegistry();
	if (!SignatureRegistry)
	{
		return FReply::Unhandled();
	}

	// Don't generate a transaction + modify if the signature container is already empty
	if (SignatureRegistry->GetSignatures().IsEmpty())
	{
		return FReply::Handled();
	}

	FScopedTransaction Transaction(LOCTEXT("EmptySignatures", "Empty Signatures"));
	SignatureRegistry->Modify();
	SignatureRegistry->EmptySignatures();

	Refresh();
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE 
