// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/BaseLogicUI/SRCLogicPanelBase.h"

class SRCSignatureDetails;
class SRCSignatureTree;
class URCSignatureRegistry;
struct FRCExposesPropertyArgs;

class SRCSignaturePanel : public SRCLogicPanelBase, public FSelfRegisteringEditorUndoClient
{
public:
	SLATE_BEGIN_ARGS(SRCSignaturePanel) {}
		SLATE_ATTRIBUTE(bool, LiveMode)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SRemoteControlPanel>& InPanel);

	/** Gets the Signature Registry from the RC Preset */
	URCSignatureRegistry* GetSignatureRegistry() const;

	void AddToSignature(const FRCExposesPropertyArgs& InPropertyArgs);

	/** Whether this widget currently has focus */
	bool IsListFocused() const;

	void EnterRenameMode();

	//~ Begin SRCLogicPanelBase
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() const override;
	virtual FReply RequestDeleteSelectedItem() override;
	virtual FReply RequestDeleteAllItems() override;
	virtual bool CanCopyItems() const override;
	virtual bool CanDuplicateItems() const override;
	virtual void DeleteSelectedPanelItems() override;
	//~ End SRCLogicPanelBase

protected:
	//~ Begin FEditorUndoClient
	virtual void PostUndo(bool bInSuccess) override;
	virtual void PostRedo(bool bInSuccess) override;
	//~ End FEditorUndoClient

private:
	void Refresh();

	FReply OnAddButtonClicked();

	FReply DeleteAllItems();

	TSharedPtr<SRCSignatureTree> SignatureTreeView;

	TSharedPtr<SRCSignatureDetails> SignatureDetails;
};
