// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureRow.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"
#include "EditorActorFolders.h"
#include "GameFramework/Actor.h"
#include "IRCSignatureColumn.h"
#include "Items/RCSignatureTreeItemBase.h"
#include "Items/RCSignatureTreeSignatureItem.h"
#include "RemoteControlUIModule.h"
#include "SRCSignatureTree.h"
#include "UI/SRCPanelExposedEntity.h"
#include "UI/Signature/IRCSignatureCustomization.h"
#include "Widgets/SNullWidget.h"

void SRCSignatureRow::Construct(const FArguments& InArgs
	, const TSharedPtr<FRCSignatureTreeItemBase>& InItem
	, const TSharedRef<SRCSignatureTree>& InSignatureTree
	, const TSharedRef<STableViewBase>& InTableView)
{
	ItemWeak = InItem;
	SignatureTreeWeak = InSignatureTree;

	SMultiColumnTableRow::Construct(FSuperRowType::FArguments()
		.ShowWires(true)
		.OnCanAcceptDrop(this, &SRCSignatureRow::OnRowCanAcceptDrop)
		.OnAcceptDrop(this, &SRCSignatureRow::OnRowAcceptDrop)
		, InTableView);
}

TSharedRef<SWidget> SRCSignatureRow::GenerateWidgetForColumn(const FName& InColumnName)
{
	TSharedPtr<SRCSignatureTree> SignatureTree = SignatureTreeWeak.Pin();
	TSharedPtr<FRCSignatureTreeItemBase> Item = ItemWeak.Pin();

	if (!SignatureTree.IsValid() || !Item.IsValid())
	{
		return SNullWidget::NullWidget;
	}

	const TSharedPtr<IRCSignatureColumn> Column = SignatureTree->FindColumn(InColumnName);
	if (ensureMsgf(Column.IsValid(), TEXT("Column %s was unexpectedly not found."), *InColumnName.ToString()))
	{
		return Column->ConstructRowWidget(Item.ToSharedRef(), SignatureTree.ToSharedRef(), SharedThis(this));
	}

	return SNullWidget::NullWidget;
}

TOptional<EItemDropZone> SRCSignatureRow::OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, TSharedPtr<FRCSignatureTreeItemBase> InItem)
{
	if (!InItem.IsValid())
	{
		return TOptional<EItemDropZone>();
	}

	if (FRCSignatureTreeSignatureItem* SignatureItem = InItem->MutableCast<FRCSignatureTreeSignatureItem>().Get())
	{
		// External Customizations to handle drag drops first
		for (TSharedRef<IRCSignatureCustomization> Customization : FRemoteControlUIModule::Get().GetSignatureCustomizations())
		{
			if (Customization->CanAcceptDrop(InDragDropEvent, SignatureItem))
			{
				return EItemDropZone::OntoItem;;
			}
		}

		// Entity Drag Drop
		if (InDragDropEvent.GetOperationAs<FExposedEntityDragDrop>())
		{
			return EItemDropZone::OntoItem;
		}

		// Actor Drag Drop (only Actors)
		if (InDragDropEvent.GetOperationAs<FActorDragDropOp>())
		{
			return EItemDropZone::OntoItem;
		}

		// Folder Drag Drop (only folders)
		if (InDragDropEvent.GetOperationAs<FFolderDragDropOp>())
		{
			return EItemDropZone::OntoItem;
		}

		// Composite Drag and Drop (mix of Actors and Folders)
		if (TSharedPtr<FCompositeDragDropOp> CompositeDragDropOp = InDragDropEvent.GetOperationAs<FCompositeDragDropOp>())
		{
			if (CompositeDragDropOp->GetSubOp<FActorDragDropOp>())
			{
				return EItemDropZone::OntoItem;
			}

			if (CompositeDragDropOp->GetSubOp<FFolderDragDropOp>())
			{
				return EItemDropZone::OntoItem;
			}
		}
	}

	return TOptional<EItemDropZone>();
}

FReply SRCSignatureRow::OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent
	, EItemDropZone InDropZone
	, TSharedPtr<FRCSignatureTreeItemBase> InItem)
{
	if (!InItem.IsValid())
	{
		return FReply::Unhandled();
	}

	if (FRCSignatureTreeSignatureItem* SignatureItem = InItem->MutableCast<FRCSignatureTreeSignatureItem>().Get())
	{
		// External Customizations to handle drops first
		for (TSharedRef<IRCSignatureCustomization> Customization : FRemoteControlUIModule::Get().GetSignatureCustomizations())
		{
			if (Customization->AcceptDrop(InDragDropEvent, SignatureItem).IsEventHandled())
			{
				return FReply::Handled();
			}
		}

		// Actor Drag Drop (Only Actors dragged)
		if (TSharedPtr<FActorDragDropOp> ActorDragDropOp = InDragDropEvent.GetOperationAs<FActorDragDropOp>())
		{
			HandleActorDragDrop(SignatureItem, *ActorDragDropOp);
			return FReply::Handled();
		}

		// Folder Drag Drop (Only Folders dragged)
		if (TSharedPtr<FFolderDragDropOp> FolderDragDropOp = InDragDropEvent.GetOperationAs<FFolderDragDropOp>())
		{
			HandleFolderDragDrop(SignatureItem, *FolderDragDropOp);
			return FReply::Handled();
		}

		// Composite Drag and Drop (mix of Actors and Folders)
		if (TSharedPtr<FCompositeDragDropOp> CompositeDragDropOp = InDragDropEvent.GetOperationAs<FCompositeDragDropOp>())
		{
			if (TSharedPtr<FActorDragDropOp> ActorDragDropOp = CompositeDragDropOp->GetSubOp<FActorDragDropOp>())
			{
				HandleActorDragDrop(SignatureItem, *ActorDragDropOp);
				return FReply::Handled();
			}

			if (TSharedPtr<FFolderDragDropOp> FolderDragDropOp = CompositeDragDropOp->GetSubOp<FFolderDragDropOp>())
			{
				HandleFolderDragDrop(SignatureItem, *FolderDragDropOp);
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

void SRCSignatureRow::HandleActorDragDrop(IRCSignatureItem* InSignatureItem, const FActorDragDropOp& InActorDragDropOp)
{
	TArray<TWeakObjectPtr<UObject>> DragDropObjects(InActorDragDropOp.Actors);
	InSignatureItem->ApplySignature(DragDropObjects);
}

void SRCSignatureRow::HandleFolderDragDrop(IRCSignatureItem* InSignatureItem, const FFolderDragDropOp& InFolderDragDropOp)
{
	if (UWorld* World = InFolderDragDropOp.World.Get())
	{
		TArray<TWeakObjectPtr<AActor>> Actors;
		FActorFolders::GetWeakActorsFromFolders(*World, InFolderDragDropOp.Folders, Actors);

		TArray<TWeakObjectPtr<UObject>> DragDropObjects(MoveTemp(Actors));
		InSignatureItem->ApplySignature(DragDropObjects);
	}
}
