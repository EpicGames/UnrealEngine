// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FActorDragDropOp;
class FFolderDragDropOp;
class FRCSignatureTreeItemBase;
class IRCSignatureItem;
class SRCSignatureTree;

class SRCSignatureRow : public SMultiColumnTableRow<TSharedPtr<FRCSignatureTreeItemBase>>
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureRow) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs
		, const TSharedPtr<FRCSignatureTreeItemBase>& InItem
		, const TSharedRef<SRCSignatureTree>& InSignatureTree
		, const TSharedRef<STableViewBase>& InTableView);

	//~ Begin SMultiColumnTableRow
	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& InColumnName) override;
	//~ End SMultiColumnTableRow

private:
	TOptional<EItemDropZone> OnRowCanAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FRCSignatureTreeItemBase> InItem);

	FReply OnRowAcceptDrop(const FDragDropEvent& InDragDropEvent, EItemDropZone InDropZone, TSharedPtr<FRCSignatureTreeItemBase> InItem);

	void HandleActorDragDrop(IRCSignatureItem* InSignatureItem, const FActorDragDropOp& InActorDragDropOp);

	void HandleFolderDragDrop(IRCSignatureItem* InSignatureItem, const FFolderDragDropOp& InFolderDragDropOp);

	TWeakPtr<FRCSignatureTreeItemBase> ItemWeak;

	TWeakPtr<SRCSignatureTree> SignatureTreeWeak;
};
