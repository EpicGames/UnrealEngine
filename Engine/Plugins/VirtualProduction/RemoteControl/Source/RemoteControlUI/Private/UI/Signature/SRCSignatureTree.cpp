// Copyright Epic Games, Inc. All Rights Reserved.

#include "SRCSignatureTree.h"
#include "Algo/Reverse.h"
#include "IRCSignatureColumn.h"
#include "Items/RCSignatureTreeItemBase.h"
#include "Items/RCSignatureTreeRootItem.h"
#include "Items/RCSignatureTreeSignatureItem.h"
#include "RCSignatureRegistry.h"
#include "RCSignatureTreeItemSelection.h"
#include "RemoteControlPreset.h"
#include "SRCSignaturePanel.h"
#include "SRCSignatureRow.h"
#include "ScopedTransaction.h"
#include "Styling/RemoteControlStyles.h"
#include "UI/RemoteControlPanelStyle.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STreeView.h"

#define LOCTEXT_NAMESPACE "SRCSignatureTreeView"

void SRCSignatureTree::Construct(const FArguments& InArgs, const TSharedRef<SRCSignaturePanel>& InSignaturePanel, const TSharedRef<SRemoteControlPanel>& InRCPanel)
{
	SRCLogicPanelListBase::Construct(SRCLogicPanelListBase::FArguments(), InSignaturePanel, InRCPanel);
	SignaturePanelWeak = InSignaturePanel;

	RootItem = MakeShared<FRCSignatureTreeRootItem>(SharedThis(this));
	RootItem->GetSelection()->OnSelectionChanged().AddSP(this, &SRCSignatureTree::UpdateTreeViewSelection);

	const FRCPanelStyle* RCPanelStyle = &FRemoteControlPanelStyle::Get()->GetWidgetStyle<FRCPanelStyle>("RemoteControlPanel.LogicControllersPanel");

	HeaderRow = SNew(SHeaderRow)
		.Style(&RCPanelStyle->HeaderRowStyle)
		.Visibility(EVisibility::Visible)
		.CanSelectGeneratedColumn(true);

	ConstructColumns(InArgs._Columns);

	ChildSlot
	[
		SAssignNew(SignatureTreeView, STreeView<TSharedPtr<FRCSignatureTreeItemBase>>)
		.TreeItemsSource(&RootItem->GetChildrenMutable())
		.HeaderRow(HeaderRow)
		.OnGetChildren(this, &SRCSignatureTree::OnGetChildren)
		.OnGenerateRow(this, &SRCSignatureTree::OnGenerateRow)
		.OnExpansionChanged(this, &SRCSignatureTree::OnItemExpansionChanged)
		.OnSelectionChanged(this, &SRCSignatureTree::OnItemSelectionChanged)
		.OnContextMenuOpening(this, &SRCLogicPanelListBase::GetContextMenuWidget)
		.SelectionMode(ESelectionMode::Multi)
		.HighlightParentNodesForSelection(true)
	];

	Refresh();
}

TSharedRef<FRCSignatureTreeRootItem> SRCSignatureTree::GetRootItem() const
{
	return RootItem.ToSharedRef();
}

URCSignatureRegistry* SRCSignatureTree::GetSignatureRegistry() const
{
	TSharedPtr<SRCSignaturePanel> SignaturePanel = SignaturePanelWeak.Pin();
	if (!SignaturePanel.IsValid())
	{
		return nullptr;
	}

	if (URemoteControlPreset* Preset = SignaturePanel->GetPreset())
	{
		return Preset->GetSignatureRegistry();
	}
	return nullptr;
}

TSharedPtr<IRCSignatureColumn> SRCSignatureTree::FindColumn(FName InColumnName) const
{
	if (const TSharedRef<IRCSignatureColumn>* Column = Columns.Find(InColumnName))
	{
		return *Column;
	}
	return nullptr;
}

void SRCSignatureTree::Refresh()
{
	TSharedRef<FRCSignatureTreeItemSelection> Selection = RootItem->GetSelection();

	RootItem->RebuildChildren();
	RootItem->VisitChildren([&TreeView = SignatureTreeView, &Selection](const TSharedPtr<FRCSignatureTreeItemBase>& InItem)->bool
		{
			TreeView->SetItemExpansion(InItem, InItem->HasAnyTreeViewFlags(ERCSignatureTreeItemViewFlags::Expanded));
			TreeView->SetItemSelection(InItem, Selection->IsSelected(InItem));
			return true;
		}
		, /*bRecursive*/true);

	SignatureTreeView->RequestTreeRefresh();
}

void SRCSignatureTree::EnterRenameMode()
{
	TArray<TSharedPtr<FRCSignatureTreeItemBase>> SelectedItems = GetSelectedItems();

	// Remove all the items that can't be renamed
	SelectedItems.RemoveAll([](const TSharedPtr<FRCSignatureTreeItemBase>& InItem)
		{
			return !InItem->GetOnRenameStateChanged();
		});

	RenameQueue.Reset();
	RenameQueue.Append(MoveTemp(SelectedItems));

	// Reverse as items will be removed from the end (pop) to avoid shifting items every dequeue
	Algo::Reverse(RenameQueue);

	ProcessRenameQueue();
}

void SRCSignatureTree::ProcessRenameQueue()
{
	if (TSharedPtr<FRCSignatureTreeItemBase> CurrentItemRenaming = CurrentItemRenamingWeak.Pin())
	{
		CurrentItemRenaming->SetRenaming(false);
	}

	CurrentItemRenamingWeak.Reset();

	if (RenameQueue.IsEmpty())
	{
		return;
	}

	// Dequeue until a valid item is gotten
	// Most likely it will be the first item
	TSharedPtr<FRCSignatureTreeItemBase> ItemToRename;
	do
	{
		ItemToRename = RenameQueue.Pop().Pin();
	}
	while (!RenameQueue.IsEmpty() && !ItemToRename.IsValid());

	if (!ItemToRename.IsValid())
	{
		return;
	}

	CurrentItemRenamingWeak = ItemToRename;
	ItemToRename->SetRenaming(true);
}

TArray<TSharedPtr<FRCSignatureTreeItemBase>> SRCSignatureTree::GetSelectedItems() const
{
	return RootItem->GetSelection()->GetSelectedItems();
}

URemoteControlPreset* SRCSignatureTree::GetPreset()
{
	if (TSharedPtr<SRCSignaturePanel> SignaturePanel = SignaturePanelWeak.Pin())
	{
		return SignaturePanel->GetPreset();
	}
	return nullptr;
}

TArray<TSharedPtr<FRCLogicModeBase>> SRCSignatureTree::GetSelectedLogicItems()
{
	return TArray<TSharedPtr<FRCLogicModeBase>>(GetSelectedItems());
}

bool SRCSignatureTree::IsEmpty() const
{
	return RootItem->GetChildren().IsEmpty();
}

bool SRCSignatureTree::IsListFocused() const
{
	return SignatureTreeView->HasAnyUserFocus().IsSet() || ContextMenuWidgetCached.IsValid();
}

int32 SRCSignatureTree::Num() const
{
	return RootItem->GetChildren().Num();
}

int32 SRCSignatureTree::NumSelectedLogicItems() const
{
	return SignatureTreeView->GetNumItemsSelected();
}

void SRCSignatureTree::DeleteSelectedPanelItems()
{
	const TArray<TSharedPtr<FRCSignatureTreeItemBase>> SelectedItems = RootItem->GetSelection()->GetSelectedItems();
	if (SelectedItems.IsEmpty())
	{
		return;
	}

	FScopedTransaction Transaction(LOCTEXT("RemoveSelectedItems", "Remove Selected Items"));
	if (!DeleteItemsFromLogicPanel<FRCSignatureTreeItemBase>(RootItem->GetChildrenMutable(), SelectedItems))
	{
		Transaction.Cancel();
	}

	RootItem->GetSelection()->ClearSelection();
}

void SRCSignatureTree::Reset()
{
	Refresh();
}

void SRCSignatureTree::ConstructColumns(TConstArrayView<TSharedRef<IRCSignatureColumn>> InColumns)
{
	HeaderRow->ClearColumns();
	Columns.Empty(InColumns.Num());

	for (const TSharedRef<IRCSignatureColumn>& Column : InColumns)
	{
		Columns.Add(Column->GetColumnId(), Column);
		HeaderRow->AddColumn(Column->ConstructHeaderRowColumn());
		HeaderRow->SetShowGeneratedColumn(Column->GetColumnId(), Column->ShouldShowColumnByDefault());
	}
}

TSharedRef<ITableRow> SRCSignatureTree::OnGenerateRow(TSharedPtr<FRCSignatureTreeItemBase> InItem, const TSharedRef<STableViewBase>& InTableView)
{
	check(InItem.IsValid());
	return SNew(SRCSignatureRow, InItem, SharedThis(this), InTableView);
}

void SRCSignatureTree::OnGetChildren(TSharedPtr<FRCSignatureTreeItemBase> InItem, TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const
{
	if (InItem.IsValid())
	{
		OutChildren.Append(InItem->GetChildren());

		// Remove all children that are not meant to be shown in the Tree View
		OutChildren.RemoveAll([](const TSharedPtr<FRCSignatureTreeItemBase>& InItem)
			{
				return InItem->HasAnyTreeViewFlags(ERCSignatureTreeItemViewFlags::Hidden);
			});
	}
}

void SRCSignatureTree::OnItemExpansionChanged(TSharedPtr<FRCSignatureTreeItemBase> InItem, bool bInIsExpanded)
{
	if (InItem.IsValid())
	{
		if (bInIsExpanded)
		{
			InItem->AddTreeViewFlags(ERCSignatureTreeItemViewFlags::Expanded);
		}
		else
		{
			InItem->RemoveTreeViewFlags(ERCSignatureTreeItemViewFlags::Expanded);
		}
	}
}

void SRCSignatureTree::UpdateTreeViewSelection()
{
	// Skip if already Syncing Selection to the Selection object
	if (bSyncingSelection)
	{
		return;
	}

	TArray<TSharedPtr<FRCSignatureTreeItemBase>> Items = RootItem->GetSelection()->GetSelectedItems();
	SignatureTreeView->ClearSelection();
	SignatureTreeView->SetItemSelection(Items, true);
}

void SRCSignatureTree::OnItemSelectionChanged(TSharedPtr<FRCSignatureTreeItemBase> InItem, ESelectInfo::Type InSelectionType)
{
	// Skip if already syncing Selection or if the selection wasn't done by the user, as these direct selections go through the Selection object already
	if (bSyncingSelection || InSelectionType == ESelectInfo::Direct)
	{
		return;
	}

	TGuardValue<bool> SyncSelectionGuard(bSyncingSelection, true);

	TSharedRef<FRCSignatureTreeItemSelection> Selection = RootItem->GetSelection();

	FRCSignatureTreeItemSelection::FSelectionScope SelectionScope = Selection->CreateSelectionScope();

	Selection->ClearSelection();

	for (const TSharedPtr<FRCSignatureTreeItemBase>& SelectedItem : SignatureTreeView->GetSelectedItems())
	{
		Selection->SetSelected(SelectedItem.ToSharedRef(), /*bSelected*/true, /*bMultiSelection*/true);
	}
}

#undef LOCTEXT_NAMESPACE
