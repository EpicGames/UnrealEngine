// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/BaseLogicUI/SRCLogicPanelListBase.h"

class FRCSignatureTreeItemBase;
class FRCSignatureTreeRootItem;
class IRCSignatureColumn;
class SHeaderRow;
class SRCSignaturePanel;
class URCSignatureRegistry;
template<typename ItemType> class STreeView;

class SRCSignatureTree : public SRCLogicPanelListBase
{
public:
	SLATE_BEGIN_ARGS(SRCSignatureTree) {}
		SLATE_ARGUMENT(TArray<TSharedRef<IRCSignatureColumn>>, Columns)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<SRCSignaturePanel>& InSignaturePanel, const TSharedRef<SRemoteControlPanel>& InRCPanel);

	TSharedRef<FRCSignatureTreeRootItem> GetRootItem() const;

	URCSignatureRegistry* GetSignatureRegistry() const;

	TSharedPtr<IRCSignatureColumn> FindColumn(FName InColumnName) const;

	void Refresh();

	void EnterRenameMode();

	void ProcessRenameQueue();

	TArray<TSharedPtr<FRCSignatureTreeItemBase>> GetSelectedItems() const;

	//~ Begin SRCLogicPanelListBase
	virtual URemoteControlPreset* GetPreset() override;
	virtual TArray<TSharedPtr<FRCLogicModeBase>> GetSelectedLogicItems() override;
	virtual bool IsEmpty() const override;
	virtual bool IsListFocused() const override;
	virtual int32 Num() const override;
	virtual int32 NumSelectedLogicItems() const override;
	virtual void DeleteSelectedPanelItems() override;
	virtual void BroadcastOnItemRemoved() override {}
	virtual void Reset() override;
	//~ End SRCLogicPanelListBase

private:
	void ConstructColumns(TConstArrayView<TSharedRef<IRCSignatureColumn>> InColumns);

	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FRCSignatureTreeItemBase> InItem, const TSharedRef<STableViewBase>& InTableView);

	void OnGetChildren(TSharedPtr<FRCSignatureTreeItemBase> InItem, TArray<TSharedPtr<FRCSignatureTreeItemBase>>& OutChildren) const;

	void OnItemExpansionChanged(TSharedPtr<FRCSignatureTreeItemBase> InItem, bool bInIsExpanded);

	void UpdateTreeViewSelection();

	void OnItemSelectionChanged(TSharedPtr<FRCSignatureTreeItemBase> InItem, ESelectInfo::Type InSelectionType);

	TSharedPtr<FRCSignatureTreeRootItem> RootItem;

	TMap<FName, TSharedRef<IRCSignatureColumn>> Columns;

	TSharedPtr<STreeView<TSharedPtr<FRCSignatureTreeItemBase>>> SignatureTreeView;

	TSharedPtr<SHeaderRow> HeaderRow;

	TWeakPtr<SRCSignaturePanel> SignaturePanelWeak;

	/** Items that are pending to be renamed */
	TArray<TWeakPtr<FRCSignatureTreeItemBase>> RenameQueue;

	/** Current item being renamed */
	TWeakPtr<FRCSignatureTreeItemBase> CurrentItemRenamingWeak;

	bool bSyncingSelection = false;
};
