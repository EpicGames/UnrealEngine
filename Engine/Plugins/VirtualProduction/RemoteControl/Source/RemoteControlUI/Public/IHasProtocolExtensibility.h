// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class SWidget;

/**
 * A simple interface provides protocol specific utilities.
 */
class IHasProtocolExtensibility
{
public:

	/**
	 * Retrieves the protocol specific widget if protocol is specified, column specific widget otherwise.
	 */
	virtual TSharedRef<SWidget> GetProtocolWidget(const FName ForColumnName, const FName InProtocolName = NAME_None) = 0;
	
	/**
	 * Ensures the derived classes supports protocols.
	 */
	virtual const bool HasProtocolExtension() const = 0;
	
	/**
	 * Returns true when the derived classes has two or more protocol bindings.
	 */
	virtual const bool GetProtocolBindingsNum() const = 0;
	
	/**
	 * Callback for the editable text's OnTextChanged event.
	 */
	virtual void OnProtocolTextChanged(const FText& InText, const FName InProtocolName) = 0;

	/**
	 * Determines whether the derived class supports the given protocol.
	 */
	virtual const bool SupportsProtocol(const FName& InProtocolName) const = 0;
};
