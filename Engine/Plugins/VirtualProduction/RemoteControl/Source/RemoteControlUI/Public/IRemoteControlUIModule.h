// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/ContainersFwd.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Layout/SSplitter.h"

class FDelegateHandle;
class FPropertyPath;
class FRCPanelWidgetRegistry;
class FUICommandList;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class IPropertyHandle;
class IRCExposedEntitiesPanelExtender;
class IRCPanelExposedEntitiesGroupWidgetFactory;
class IRCPanelExposedEntitiesListSettingsForProtocol;
class IRCPanelExposedEntityWidgetFactory;
class IRCSignatureCustomization;
class URCVirtualPropertyBase;
class URemoteControlPreset;
struct FOnGenerateGlobalRowExtensionArgs;
struct FRemoteControlEntity;
struct FRemoteControlProperty;
struct SRCPanelTreeNode;

struct FRCColumnSizeData
{
	TAttribute<float> LeftColumnWidth;
	TAttribute<float> RightColumnWidth;
	SSplitter::FOnSlotResized OnWidthChanged;

	void SetColumnWidth(float InWidth) { OnWidthChanged.ExecuteIfBound(InWidth); }
};

struct FGenerateWidgetArgs
{
	URemoteControlPreset* Preset = nullptr;
	FRCColumnSizeData ColumnSizeData;
	TSharedPtr<FRemoteControlEntity> Entity;
	TAttribute<bool> bIsInLiveMode;
	TWeakPtr<FRCPanelWidgetRegistry> WidgetRegistry;
	TSharedPtr<FUICommandList> CommandList;
	FText HighlightText;
};

/**
 * Exposed property arguments
 * The struct is wrapper struct to generate Remote Control Entity from IPropertyHandle or OwnerObject with Property Path and Property instance 
 */
struct FRCExposesPropertyArgs
{
	/**
	 * Type of the Extension arguments
	 */
	enum class EType : uint8
	{
		E_Handle,
		E_OwnerObject,
		E_None
	};

	FRCExposesPropertyArgs();
	FRCExposesPropertyArgs(const FOnGenerateGlobalRowExtensionArgs& InExtensionArgs);
	FRCExposesPropertyArgs(FOnGenerateGlobalRowExtensionArgs&& InExtensionArgs);
	FRCExposesPropertyArgs(TSharedPtr<IPropertyHandle>& InPropertyHandle);
	FRCExposesPropertyArgs(const TSharedPtr<IPropertyHandle>& InPropertyHandle);
	FRCExposesPropertyArgs(UObject* InOwnerObject, const FString& InPropertyPath, FProperty* InProperty);


	/** whether extensions arguments valid*/
	bool IsValid() const;

	/* Get Type of the Extension Arguments */
	EType GetType() const;

	/** Get property on the exposed arguments type */
	FProperty* GetProperty() const;

	/** Checked version of Get Property */
	FProperty* GetPropertyChecked() const;

	friend inline bool operator==(const FRCExposesPropertyArgs& LHS, const FRCExposesPropertyArgs& RHS)
	{
		return LHS.Id == RHS.Id;
	}

	friend inline bool operator!=(const FRCExposesPropertyArgs& LHS, const FRCExposesPropertyArgs& RHS)
	{
		return LHS.Id != RHS.Id;
	}

	friend inline uint32 GetTypeHash(const FRCExposesPropertyArgs& InArgs)
	{
		return GetTypeHash(InArgs.Id);
	}

public:
	/** The detail row's property handle. */
	TSharedPtr<IPropertyHandle> PropertyHandle;

	/** Owner object for the property extension */
	TWeakObjectPtr<UObject> OwnerObject;

	/** Path of the exposed property */
	FString PropertyPath;

	/** Exposed property */
	TWeakFieldPtr<FProperty> Property;

private:
	/** Unique generated ID of the struct */
	FGuid Id;
};

DECLARE_DELEGATE_RetVal_OneParam(TSharedPtr<SRCPanelTreeNode>, FOnGenerateRCWidget, const FGenerateWidgetArgs& /*Args*/);

DECLARE_MULTICAST_DELEGATE_OneParam(FOnGenerateExtensions, TArray<TSharedRef<class SWidget>>& /*OutExtensions*/);

/**
 * Filter queried in order to determine if a property should be displayed.
 */
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnDisplayExposeIcon, const FRCExposesPropertyArgs& /*InPropertyArgs*/);

/**
 * Callback called to customize the display of a metadata entry for entities.
 */
DECLARE_DELEGATE_FourParams(FOnCustomizeMetadataEntry, URemoteControlPreset* /*Preset*/, const FGuid& /*DisplayedEntityId*/, IDetailLayoutBuilder& /*LayoutBuilder*/, IDetailCategoryBuilder& /*CategoryBuilder*/);

/** A struct used for controllers list items columns customization */
struct FRCControllerExtensionWidgetsInfo
{
	FRCControllerExtensionWidgetsInfo(URCVirtualPropertyBase* InController)
		: Controller(InController)
	{
	}

	/** Add a new widget, and associate it to the specified custom column */
	bool AddColumnWidget(const FName& InColumnName, const TSharedRef<SWidget>& InWidget)
	{
		if (CustomWidgetsMap.Contains(InColumnName))
		{
			return false;
		}
		
		CustomWidgetsMap.Add(InColumnName, InWidget);

		return true;
	}

	/** The controller being customized */
	URCVirtualPropertyBase* Controller;

	/** A map associating each column with a widget */
	TMap<FName, TSharedRef<SWidget>> CustomWidgetsMap;
};

/**
 * Callback called to customize the extensible widget for controllers in controllers list
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGenerateControllerExtensionsWidgets, FRCControllerExtensionWidgetsInfo& /*OutWidgetsInfo*/);

/**
 * Called when setting up / resetting the Controller panel list. Register to add custom column.
 */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnAddControllerExtensionColumn, TArray<FName>& /*OutColumnNames*/);

/**
 * Callback for custom resolution of exposed property sources.
 * Returns true if the property is resolved / dealt with.
 * When successfully resolving, bound objects can be replaced.
 */
DECLARE_DELEGATE_RetVal_ThreeParams(bool, FRCPropertyResolver, const TSharedRef<FRemoteControlProperty>& /* In Property */, TArray<UObject*>& /* In/Out Bound Objects */, TSharedPtr<FPropertyPath>& /* Out Property Path */)

/**
 * A Remote Control module that allows exposing objects and properties from the editor.
 */
class IRemoteControlUIModule : public IModuleInterface
{
public:	
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IRemoteControlUIModule& Get()
	{
		static const FName ModuleName = "RemoteControlUI";
		return FModuleManager::LoadModuleChecked<IRemoteControlUIModule>(ModuleName);
	}
	
	/** Delegate called when a Remote Control Preset panel is created/opened for the specified Remote Control Preset */
    DECLARE_MULTICAST_DELEGATE_OneParam(FOnRemoteControlPresetOpened, URemoteControlPreset* /*RemoteControlPreset*/);
    virtual FOnRemoteControlPresetOpened& OnRemoteControlPresetOpened() = 0;

	/** Delegate called when a Remote Control Preset panel is created/opened for the specified Remote Control Preset */
	virtual FOnGenerateControllerExtensionsWidgets& OnGenerateControllerExtensionsWidgets() = 0;

	/** Delegate called when a Remote Control Preset panel is created/opened for the specified Remote Control Preset */
	virtual FOnAddControllerExtensionColumn& OnAddControllerExtensionColumn() = 0;
	
	/** 
	 * Get the toolbar extension generators.
	 * Usage: Bind a handler that adds a widget to the out array parameter.
	 */
	virtual FOnGenerateExtensions& GetExtensionGenerators() = 0;

	/**
	 * Add a property filter that indicates if the property handle should be displayed or not.
	 * When queried, returning true will allow the expose icon to be displayed in the details panel, false will hide it.
	 * @Note This filter will be queried after the RemoteControlModule's own filters.
	 * @param OnDisplayExposeIcon The delegate called to determine whether to display the icon or not.
	 * @return A handle to the delegate, used to unregister the delegate with the module.
	 */
	virtual FDelegateHandle AddPropertyFilter(FOnDisplayExposeIcon OnDisplayExposeIcon) = 0;

	/**
	 * Remove a property filter using its id.
	 */
	virtual void RemovePropertyFilter(const FDelegateHandle& FilterDelegateHandle) = 0;

	/**
	 * Register a delegate to customize how an entry is displayed in the entity details panel.
	 * @param MetadataKey The metadata map entry to customize.
	 * @param OnCustomizeCallback The handler called to handle customization for the entry's details panel row.
	 */
	virtual void RegisterMetadataCustomization(FName MetadataKey, FOnCustomizeMetadataEntry OnCustomizeCallback) = 0;

	/**
     * Unregister the delegate used to customize how an entry is displayed in the entity details panel.
     * @param MetadataKey The metadata map entry to unregister the customization for.
     */
	virtual void UnregisterMetadataCustomization(FName MetadataKey) = 0;

	/**
	 * Get the preset currently being edited in the editor.
	 */
	virtual URemoteControlPreset* GetActivePreset() const = 0;

	/**
	 * Retrieves the advanced asset category type registered by Remote Control Module.
	 */
	virtual uint32 GetRemoteControlAssetCategory() const = 0;

	/**
	 * Register the given signature customization to extend the Signature Panel capabilities like Drag/Drop handling
	 * @param InCustomization	The signature customization instance to register
	 */
	virtual void RegisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization) = 0;

	/**
	 * Unregisters the given signature customization
	 * @param InCustomization	The signature customization instance to unregister
	 */
	virtual void UnregisterSignatureCustomization(const TSharedPtr<IRCSignatureCustomization>& InCustomization) = 0;

	/**
	 * Registers the given settings for an exposed entities list.
	 * @param InSettings		The exposed entities list settings instance to register.
	 */
	virtual void RegisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings) = 0;
	
	/**
	 * Unregisters the given settings for an exposed entities list.
	 * @param InSettings		The exposed entities list settings instance to unregister.
	 */
	virtual void UnregisterExposedEntitiesListSettingsForProtocol(const TSharedRef<IRCPanelExposedEntitiesListSettingsForProtocol>& InSettings) = 0;

	/**
	 * Registers the given customization for an exposed entities panel.
	 * @param InExtender		The exposed entities panel extender instance to register.
	 */
	virtual void RegisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender) = 0;

	/**
	 * Unregisters the given exposed entities panel customization.
	 * @param InExtender		The exposed entities panel extender instance to unregister.
	 */
	virtual void UnregisterExposedEntitiesPanelExtender(const TSharedRef<IRCExposedEntitiesPanelExtender>& InExtender) = 0;

	/**
	 * Registers a widget factory a column in an exposed entities group row.
	 * @param InFactory			The widget factory to register.
	 */
	virtual void RegisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory) = 0;

	/**
	 * Unregisters the given widget factory.
	 * @param InFactory			The factory to unregister.
	 */
	virtual void UnregisterExposedEntitiesGroupWidgetFactory(const TSharedRef<IRCPanelExposedEntitiesGroupWidgetFactory>& InFactory) = 0;

	/**
	 * Registers a widget factory a column in an exposed entity row.
	 * @param InFactory			The widget factory to register.
	 */
	virtual void RegisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory) = 0;

	/**
	 * Unregisters the given customization for a exposed entity.
	 * @param InFactory			The factory to unregister.
	 */
	virtual void UnregisterExposedEntityWidgetFactory(const TSharedRef<IRCPanelExposedEntityWidgetFactory>& InFactory) = 0;

	/**
	 * Register a widget factory to handle creating a widget in the control panel.
	 */
	virtual void RegisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType, const FOnGenerateRCWidget& OnGenerateRCWidgetDelegate) = 0;

	/**
	 * Unregister a previously registered widget factory.
	 */
	virtual void UnregisterWidgetFactoryForType(UScriptStruct* RemoteControlEntityType) = 0;

	/**
	 * Registers a property source resolution callback.
	 */
	virtual void RegisterPropertyResolver(FName InResolverId, const FRCPropertyResolver& InResolver) = 0;

	/**
	 * Unregisters a property source resolution callback.
	 */
	virtual void UnregisterPropertyResolver(FName InResolverId) = 0;

	/** The section of EditorPerProjectUserSettings in which to save filter settings */
	static const FString SettingsIniSection;

	/**
	 * If a details panel is open, highlight a property specified by the path in it.
	 */
	virtual void HighlightPropertyInDetailsPanel(const FPropertyPath& Path) const = 0;

	/**
	 * Set the list of selected objects.
	 */
	virtual void SelectObjects(const TArray<UObject*>& Objects) const = 0;

	/**
	 * Tries to retrieve a Custom Controller Widget for the specified Controller
	 */
	virtual TSharedPtr<SWidget> CreateCustomControllerWidget(URCVirtualPropertyBase* InController, TSharedPtr<IPropertyHandle> InOriginalPropertyHandle) const = 0;
};
