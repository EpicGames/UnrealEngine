// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "RCFilter.h"
#include "UObject/UnrealType.h"

class FFieldClass;
class FFilterCategory;
struct FSlateIcon;
class IAssetTypeActions;
class UClass;

/**
 * A custom view model to represent each type of filters we have.
 */
class FCustomTypeFilterData
{
public:
	
	FCustomTypeFilterData(EEntityTypeCategories::Type InEntityTypeCategory, FFieldClass* InEntityType, FText InFilterDisplayName, const FName& InCustomTypeName = NAME_None)
		: Class(nullptr)
		, EntityType(InEntityType)
		, EntityTypeCategory(InEntityTypeCategory)
		, FilterDisplayName(InFilterDisplayName)
		, Color(FLinearColor::White)
		, CustomTypeName(InCustomTypeName)
		, AssetTypeActions(nullptr)
	{
	};
	
	FCustomTypeFilterData(EEntityTypeCategories::Type InEntityTypeCategory, FFieldClass* InEntityType, FText InEntityTypeDisplayName, TSharedPtr<FFilterCategory> InCategory, const FName& InCustomTypeName = NAME_None)
		: Class(nullptr)
		, EntityType(InEntityType)
		, EntityTypeCategory(InEntityTypeCategory)
		, FilterDisplayName(InEntityTypeDisplayName)
		, Color(FLinearColor::White)
		, CustomTypeName(InCustomTypeName)
		, AssetTypeActions(nullptr)
	{
		Categories.Add(InCategory);
	};

	/* You can provide an IAssetTypeActions for your type to get all the information from that */
	FCustomTypeFilterData(TWeakPtr<IAssetTypeActions> InAssetTypeAction)
		: Class(nullptr)
		, EntityType(FObjectProperty::StaticClass())
		, EntityTypeCategory(EEntityTypeCategories::Assets)
		, Color(FLinearColor::White)
		, CustomTypeName(NAME_None)
		, AssetTypeActions(InAssetTypeAction)
	{
	}

	/* Or you can provide a UClass, a Category and a Color to identify the filter */
	FCustomTypeFilterData(UClass* InClass, TSharedPtr<FFilterCategory> InCategory, FLinearColor InColor)
		: Class(InClass)
		, EntityType(FObjectProperty::StaticClass())
		, EntityTypeCategory(EEntityTypeCategories::Assets)
		, Color(InColor)
		, CustomTypeName(NAME_None)
		, AssetTypeActions(nullptr)
	{
		Categories.Add(InCategory);
	}

	/* Unlike normal filters, Type Filters are allowed to belong to multiple categories */
	void AddCategory(TSharedPtr<FFilterCategory> InCategory);

	/** Add this filter to the input Backend Filter */
	void BuildBackendFilter(FRCFilter& OutFilter);

	/* Get the AssetTypeActions associated with this filter if they exist */
	TSharedPtr<IAssetTypeActions> GetAssetTypeActions() const;

	/* Get the Catgories this filter belongs to */
	TArray<TSharedPtr<FFilterCategory>> GetCategories() const;

	/* Get the UClass associated with this filter */
	UClass* GetClass() const;

	/* Get the Property Type associated with this filter */
	FFieldClass* GetEntityType() const;

	/* Get the Entity Type Catgories this filter belongs to */
	EEntityTypeCategories::Type GetEntityTypeCategory() const;

	/* Get the color of this filter */
	FLinearColor GetColor() const;

	/** Get the logical name of this filter class */
	FText GetName() const;

	/** Get the display name of this filter class */
	FString GetFilterName() const;

	/** Retrieves the appropriate icon for the filter. */
	FSlateIcon GetSlateIcon() const;

	/** Remove this filter from the input Backend Filter */
	void ResetBackendFilter(FRCFilter& OutFilter);

protected:

	/** Returns proper name of the entity type to be displayed. */
	FText GetProperEntityDisplayName() const;
	
	/** Returns proper icon of the entity type to be displayed. */
	FSlateIcon GetProperEntityIcon() const;

private:

	/** The Class the filter is associated with (if it does not have an AssetTypeAction) */
	UClass* Class;

	/** The Entity Type the filter is associated with. */
	FFieldClass* EntityType;

	/** The Entity Type the filter is associated with. */
	EEntityTypeCategories::Type EntityTypeCategory;

	/** The entity type display name */
	FText FilterDisplayName;

	/** The Categories the filter will show up under */
	TArray<TSharedPtr<FFilterCategory>> Categories;

	/** The Color of the filter. */
	FLinearColor Color;

	/** If property name could not be deducted from entity type then this will be used instead. */
	FName CustomTypeName;

	/** An optional AssetTypeAction to get information about the filter class */
	TWeakPtr<IAssetTypeActions> AssetTypeActions;

	/** Empty Slate Icon. */
	static FSlateIcon EmptyIcon;
	
	/** Entity Slate Icon. */
	static FSlateIcon EntityIcon;

	/** Array Slate Icon. */
	static FSlateIcon ArrayIcon;

	/** Map Slate Icon. */
	static FSlateIcon MapIcon;
	
	/** Set Slate Icon. */
	static FSlateIcon SetIcon;
};
