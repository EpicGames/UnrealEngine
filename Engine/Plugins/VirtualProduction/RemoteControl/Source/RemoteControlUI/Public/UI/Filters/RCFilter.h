// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FFieldClass;
struct SRCPanelTreeNode;

using FEntityFilterType = TSharedRef<const SRCPanelTreeNode>;

namespace RemoteControlTypes
{
	static const FName NAME_RCActors = "Exposed Actors";

	static const FName NAME_RCFunctions = "Exposed Functions";
}

/**
 * The list of categories for Entity Types.
 */
namespace EEntityTypeCategories
{
	enum Type
	{
		None = 0,

		Core = 1 << 0,

		Strings = 1 << 1,

		Structs = 1 << 2,

		Containers = 1 << 3,

		Objects = 1 << 4,

		References = 1 << 5,

		UserDefined = 1 << 6,

		Assets = 1 << 7,

		Primary = 1 << 8,
	};
}

/**
 * A data model to represent combined backend filter.
 */
class FRCFilter
{
public:

	/** Adds an entity type to the filters list. */
	void AddTypeFilter(FFieldClass* InEntityTypeFilter);

	/** Adds a custom type to the filters list. */
	void AddTypeFilter(const FName InCustomTypeFilter);

	/** Adds an asset type to the filters list. */
	void AddTypeFilter(UClass* InAssetTypeFilter);
	
	/** Returns true if the given item passes atleast one filter. */
	bool DoesPassFilters(FEntityFilterType InEntityItem);
	
	/** Returns true if atleast one of the type filters are populated. */
	bool HasAnyActiveFilters();

	/** Removes an entity type from the filters list. */
	void RemoveTypeFilter(FFieldClass* InEntityTypeFilter);

	/** Removes a custom type from the filters list. */
	void RemoveTypeFilter(const FName& InCustomTypeFilter);
	
	/** Removes an asset type from the filters list. */
	void RemoveTypeFilter(UClass* InAssetTypeFilter);

private:

	/** The list of active entity types. */
	TSet<FFieldClass*> EntityTypeFilters;
	
	/** The list of active asset types. */
	TSet<UClass*> AssetTypeFilters;

	/** The list of active custom types. */
	TSet<FName> CustomTypeFilters;
};
