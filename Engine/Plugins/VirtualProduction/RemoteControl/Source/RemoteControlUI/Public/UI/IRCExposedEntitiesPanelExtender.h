// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

class SWidget;
class URemoteControlPreset;

/** An externder for the exposed entities panel. Draws a widget above the exposed entities list. */
class IRCExposedEntitiesPanelExtender
{
public:
	struct FArgs
	{
		TAttribute<FName> ActiveProtocolAttribute;
	};

	/** Creates a widget above the exposed entities list. */
	virtual TSharedRef<SWidget> MakeWidget(URemoteControlPreset* Preset, const FArgs& Args) = 0;
};
