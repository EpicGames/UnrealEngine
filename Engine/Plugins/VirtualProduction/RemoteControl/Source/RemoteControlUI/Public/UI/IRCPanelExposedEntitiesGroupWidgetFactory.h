// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UI/RCFieldGroupType.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class SWidget;
class URemoteControlPreset;
struct FRemoteControlProperty;

/** Arguments provided by the exposed entities group widget factory */
struct FRCPanelExposedEntitiesGroupWidgetFactoryArgs
{
	FRCPanelExposedEntitiesGroupWidgetFactoryArgs(
		const TWeakObjectPtr<URemoteControlPreset> InWeakPreset,
		const TArray<TSharedRef<FRemoteControlProperty>> InChildProperties)
		: WeakPreset(InWeakPreset)
		, ChildProperties(InChildProperties)
	{}

	/** The preset for which this exposed entities group widget is painted */
	TWeakObjectPtr<URemoteControlPreset> WeakPreset;

	/** The child properties that reside in this exposed entities group. */
	TArray<TSharedRef<FRemoteControlProperty>> ChildProperties;
};

/** Factory for a widget in a column in an exposed entities group row */
class IRCPanelExposedEntitiesGroupWidgetFactory
{
public:
	/** Creates a widget in a column in an exposed entities group row. */
	virtual TSharedRef<SWidget> MakeWidget(const FRCPanelExposedEntitiesGroupWidgetFactoryArgs& Args) = 0;

	/** Returns the column in which the widget should be painted */
	virtual FName GetColumnName() const = 0;

	/** Returns the protocol name, or name none if the widget factory is not protocol specific. */
	virtual FName GetProtocolName() const { return NAME_None; }
};
