// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UI/RCPanelExposedEntitiesListSettingsData.h"
#include "UObject/NameTypes.h"

class URemoteControlPreset;

/** Interface to store and recall settings for an entity list for a specific protocol */
class IRCPanelExposedEntitiesListSettingsForProtocol
{
public:
	/** Returns the protocol name */
	virtual FName GetProtocolName() const = 0;

	/** Returns the list settings that should be applied to the exposed entities list */
	virtual FRCPanelExposedEntitiesListSettingsData GetListSettings(URemoteControlPreset* Preset) const = 0;

	/**
	 * Called when the corresponding entity list changed. Useful to store the current settings.
	 *
	 * @param Preset				The preset for which the list is painted.
	 * @param ListSettings			The current list settings.
	 */
	virtual void OnSettingsChanged(URemoteControlPreset* Preset, const FRCPanelExposedEntitiesListSettingsData& ListSettings) = 0;
};
