// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class SWidget;
class URemoteControlPreset;
enum class ERCFieldGroupType : uint8;
struct FRemoteControlProperty;

/** Arguments to construct an exposed property widget */
struct FRCPanelExposedPropertyWidgetArgs
{
	FRCPanelExposedPropertyWidgetArgs(
		const TWeakObjectPtr<URemoteControlPreset> InWeakPreset,
		const TSharedRef<FRemoteControlProperty> InProperty)
		: WeakPreset(InWeakPreset)
		, Property(InProperty)
	{}

	/** The preset for which this exposed entities group widget is painted */
	const TWeakObjectPtr<URemoteControlPreset> WeakPreset;

	/** The property of shown in this widget. */
	const TSharedRef<FRemoteControlProperty> Property;
};

/** Factory for a widget in a column of an exposed entity row */
class IRCPanelExposedEntityWidgetFactory
{
public:
	/** Creates a widget for a property in a column of an exposed entity row */
	virtual TSharedRef<SWidget> MakePropertyWidget(const FRCPanelExposedPropertyWidgetArgs& Args) = 0;

	/** Returns the column in which the widget should be drawn */
	virtual FName GetColumnName() const = 0;

	/** Returns the protocol name, or name none if the widget factory is not protocol specific. */
	virtual FName GetProtocolName() const { return NAME_None; }
};
