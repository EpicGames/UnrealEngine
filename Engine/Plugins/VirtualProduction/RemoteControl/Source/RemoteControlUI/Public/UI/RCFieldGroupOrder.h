// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCFieldGroupOrder.generated.h"

/** Order of remote control field groups */
UENUM()
enum class ERCFieldGroupOrder : uint8
{
	None,
	Ascending,
	Descending
};
