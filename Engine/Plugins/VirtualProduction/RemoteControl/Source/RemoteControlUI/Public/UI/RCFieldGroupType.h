// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCFieldGroupType.generated.h"

/** Types of remote control field groups */
UENUM()
enum class ERCFieldGroupType : uint8
{
	/** No Grouping. Will be shown as a flat list */
	None,

	/** Fields with same property id are grouped together */
	PropertyId,

	/** Fields with same owner are grouped together */
	Owner,
};

ENUM_CLASS_FLAGS(ERCFieldGroupType)

