// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RCFieldGroupType.h"
#include "RCFieldGroupOrder.h"

#include "RCPanelExposedEntitiesListSettingsData.generated.h"

/** Data for exposed entities list settings */
USTRUCT()
struct FRCPanelExposedEntitiesListSettingsData
{
	GENERATED_BODY()

	/** The field group type for the entity list. */
	UPROPERTY()
	ERCFieldGroupType FieldGroupType = ERCFieldGroupType::None;

	/** Whether the field groups are expanded */
	UPROPERTY()
	ERCFieldGroupOrder FieldGroupOrder = ERCFieldGroupOrder::Ascending;
};
