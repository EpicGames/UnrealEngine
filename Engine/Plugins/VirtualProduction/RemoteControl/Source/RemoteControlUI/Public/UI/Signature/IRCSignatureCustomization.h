// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

class FDragDropEvent;
class FReply;
class IRCSignatureItem;

/** Customization Interface to extend the capabilities of RC Signature UI */
class IRCSignatureCustomization
{
public:
	virtual bool CanAcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const = 0;

	virtual FReply AcceptDrop(const FDragDropEvent& InDragDropEvent, IRCSignatureItem* InSignatureItem) const = 0;
};
