// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "UObject/WeakObjectPtrTemplatesFwd.h"

class UObject;

/** Interface for external implementations to interact with the Signature View Model */
class IRCSignatureItem
{
public:
	virtual void ApplySignature(TConstArrayView<TWeakObjectPtr<UObject>> InObjects) = 0;
};
