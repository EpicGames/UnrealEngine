// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

public class RemoteControlUI : ModuleRules
{
	public RemoteControlUI(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"ApplicationCore",
				"AssetDefinition",
				"AssetRegistry",
				"AssetTools",
				"BlueprintGraph",
				"ClassViewer",
				"Core",
				"CoreUObject",
				"ContentBrowser",
				"DesktopWidgets",
				"DeveloperSettings",
				"EditorWidgets",
				"EditorStyle",
				"Engine",
				"GraphEditor",
				"HotReload",
				"InputCore",
				"LevelEditor",
				"MainFrame",
				"MessageLog",
				"Projects",
				"PropertyEditor",
				"RemoteControl",
				"RemoteControlCommon",
				"RemoteControlLogic",				
				"RemoteControlProtocol",
				"RemoteControlProtocolWidgets",
				"SceneOutliner",
				"Serialization",
				"Slate",
				"SlateCore",
				"ToolMenus",
				"ToolWidgets",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UnrealEd",
				"WidgetRegistration",
				"WorkspaceMenuStructure",
			}
		);
    }
}
