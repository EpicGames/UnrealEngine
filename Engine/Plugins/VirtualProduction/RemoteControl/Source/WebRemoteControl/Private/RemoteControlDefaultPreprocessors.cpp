// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlDefaultPreprocessors.h"

#define LOCTEXT_NAMESPACE "WebRemoteControl"

namespace UE::WebRemoteControl
{
	/** Utility function to wrap a preprocessor handler to a http request handler than the HttpRouter can take. */
	FHttpRequestHandler MakeHttpRequestHandler(FRCPreprocessorHandler Handler)
	{
		return FHttpRequestHandler::CreateLambda([WrappedHandler = MoveTemp(Handler)](const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
		{
			FPreprocessorResult Result = WrappedHandler(Request);
			if (Result.Result == EPreprocessorResult::RequestPassthrough)
			{
				return false; // Request not handled.
			}
			else
			{
				OnComplete(MoveTemp(Result.OptionalResponse));
				return true; // Request handled.
			}
		});
	}

#if WITH_EDITOR
	/** Notification prompt used when a remote client attempts to connect to unreal without a passphrase. */
	TWeakPtr<SNotificationItem> NoPassphrasePrompt;
	/** Message displayed in the notification prompt when no action was taken. */
	FText NoActionTakenText = LOCTEXT("NoActionTaken", "No action was taken, further requests from this IP will be denied.");

	/** Attempt saving the remote control config, checkouting it if needed.*/
	void SaveRemoteControlConfig()
	{
		FString ConfigFilename = FPaths::ConvertRelativePathToFull(URemoteControlSettings::StaticClass()->GetConfigName());
		
		if (!FPlatformFileManager::Get().GetPlatformFile().IsReadOnly(*ConfigFilename))
		{
			GetMutableDefault<URemoteControlSettings>()->SaveConfig();
		}
		else
		{
			if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*ConfigFilename))
			{
				if (ISourceControlModule::Get().IsEnabled())
				{
					if (SettingsHelpers::IsSourceControlled(ConfigFilename))
					{
						FText DisplayMessage = LOCTEXT("SaveConfigCheckoutMessage", "The configuration file for these settings is currently not checked out. Would you like to check it out from revision control?");
						if (FMessageDialog::Open(EAppMsgType::YesNo, DisplayMessage) == EAppReturnType::Yes)
						{
							constexpr bool bForceSourceControlUpdate = true;
							constexpr bool bShowErrorInNotification = true;
							SettingsHelpers::CheckOutOrAddFile(ConfigFilename, bForceSourceControlUpdate, bShowErrorInNotification);
							GetMutableDefault<URemoteControlSettings>()->SaveConfig();
							return;
						}
					}
				}

				FText DisplayMessage = LOCTEXT("MakeConfigWritable", "The configuration file for these settings is currently read only. Would you like to make it writable?");
				if (FMessageDialog::Open(EAppMsgType::YesNo, DisplayMessage) == EAppReturnType::Yes)
				{
					if (FPlatformFileManager::Get().GetPlatformFile().SetReadOnly(*ConfigFilename, false))
					{
						GetMutableDefault<URemoteControlSettings>()->SaveConfig();
					}
					else
					{
						FText ErrorMessage = FText::Format(LOCTEXT("FailedToMakeWritable", "Could not make {0} writable."), FText::FromString(ConfigFilename));;

						FNotificationInfo MakeWritableNotification(MoveTemp(ErrorMessage));
						MakeWritableNotification.ExpireDuration = 3.0f;
						FSlateNotificationManager::Get().AddNotification(MakeWritableNotification);
					}
				}
			}
		}
	}

	/** Add IP to the list of IPs that should be let through by RC. */
	void AddIPToAllowlist(FString IPAddress)
	{
		if (TSharedPtr<SNotificationItem> Notification = NoPassphrasePrompt.Pin())
		{
			Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Pending);

			GetMutableDefault<URemoteControlSettings>()->AllowClient(IPAddress);
			SaveRemoteControlConfig();
			
			Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
			Notification->ExpireAndFadeout();
		}
	}

	/** Prompts the user to create a passphrase.  */
	void CreatePassphrase()
	{
		if (TSharedPtr<SNotificationItem> Notification = NoPassphrasePrompt.Pin())
		{
			Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Pending);

			TSharedPtr<SEditableTextBox> PassphraseIdentifierTextbox;
			TSharedPtr<SEditableTextBox> PassphraseTextbox;

			TSharedPtr<SWindow> Window = SNew(SWindow)
				.Title(LOCTEXT("RemoteControlCreatePassphrase", "Create Passphrase"))
				.SizingRule(ESizingRule::Autosized)
				.SupportsMaximize(false)
				.SupportsMinimize(false)
				.FocusWhenFirstShown(true);

			auto OnPassphraseEntered = [Window, Notification](const FText& PassphraseIdentifier, const FText& PassphraseText)
			{
				FRCPassphrase PassphraseStruct;
				PassphraseStruct.Identifier = PassphraseIdentifier.ToString();
				PassphraseStruct.Passphrase = FMD5::HashAnsiString(*PassphraseText.ToString());
				GetMutableDefault<URemoteControlSettings>()->Passphrases.Add(MoveTemp(PassphraseStruct));
				SaveRemoteControlConfig();

				Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
				Notification->ExpireAndFadeout();

				if (Window)
				{
					Window->RequestDestroyWindow();
				}
			};

			Window->SetContent(
				SNew(SBorder)
				.Padding(4.f)
				.VAlign(VAlign_Center)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.Padding(0.f, 5.f)
					.AutoHeight()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("CreatePassphraseDescription", "Create a passphrase for the client to use."))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(PassphraseIdentifierTextbox, SEditableTextBox)
						.HintText(LOCTEXT("PassphraseIdentifier", "Passphrase Identifier (Optional)"))
					]
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(PassphraseTextbox, SEditableTextBox)
						.HintText(LOCTEXT("Passphrase", "Passphrase"))
						.OnTextCommitted_Lambda([PassphraseIdentifierTextbox, OnPassphraseEntered](const FText& InText, ETextCommit::Type CommitType)
						{
							if (PassphraseIdentifierTextbox && !InText.IsEmpty() && CommitType == ETextCommit::OnEnter)
							{
								OnPassphraseEntered(PassphraseIdentifierTextbox->GetText(), InText);
							}
						})
					]
					+ SVerticalBox::Slot()
					.HAlign(HAlign_Right)
					.AutoHeight()
					.Padding(0.0f, 2.0f, 0.0f, 0.0f)
					[
						SNew(SButton)
						.Text(LOCTEXT("OK", "OK"))
						.OnClicked_Lambda([PassphraseTextbox, PassphraseIdentifierTextbox, OnPassphraseEntered]()
						{
							if (PassphraseIdentifierTextbox && PassphraseTextbox && !PassphraseTextbox->GetText().IsEmpty())
							{
								OnPassphraseEntered(PassphraseIdentifierTextbox->GetText(), PassphraseTextbox->GetText());
							}

							return FReply::Handled();
						})
					]
				]
			);

			Window->GetOnWindowClosedEvent().AddLambda([Notification](const TSharedRef<SWindow>&)
			{
				if (Notification->GetCompletionState() == SNotificationItem::ECompletionState::CS_Pending)
				{
					Notification->SetText(NoActionTakenText);
					Notification->SetSubText(FText::GetEmpty());
					Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
					Notification->ExpireAndFadeout();
				}
			});

			Window->GetOnWindowActivatedEvent().AddLambda([PassphraseTextbox]()
			{
				if (PassphraseTextbox)
				{
					constexpr uint32 UserIndex = 0;
					FSlateApplication::Get().SetUserFocus(UserIndex, PassphraseTextbox, EFocusCause::SetDirectly);
				}
			});

			GEditor->EditorAddModalWindow(Window.ToSharedRef());
		}
	}

	/** Disables remote passphrase enforcement by modifying a RC project setting. */
	void DisableRemotePassphrases()
	{
		if (TSharedPtr<SNotificationItem> Notification = NoPassphrasePrompt.Pin())
		{
			Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Pending);
			
			if (FMessageDialog::Open(EAppMsgType::YesNo, LOCTEXT("DisableRemotePassphrases", "Are you sure you want to disable passphrase requirement for remote clients? This will allow anyone on your network to access the remote control servers and could open you up to vulnerabilities.")) == EAppReturnType::Yes)
			{
				GetMutableDefault<URemoteControlSettings>()->bEnforcePassphraseForRemoteClients = false;
				SaveRemoteControlConfig();
				
				Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Success);
                Notification->ExpireAndFadeout();
			}
			else
			{
				Notification->SetText(NoActionTakenText);
				Notification->SetSubText(FText::GetEmpty());
				Notification->SetCompletionState(SNotificationItem::ECompletionState::CS_Fail);
				Notification->ExpireAndFadeout();
			}
		}
	}
#endif

	// Notifies the editor if a client tries to access the RC server without a passphrase.
	FPreprocessorResult RemotePassphraseEnforcementPreprocessor(const FHttpServerRequest& Request)
	{
		if (GetDefault<URemoteControlSettings>()->bRestrictServerAccess)
		{
			if (Request.PeerAddress)
			{
				constexpr bool bAppendPort = false;
				FString PeerAddress = Request.PeerAddress->ToString(bAppendPort);

				if (!GetDefault<URemoteControlSettings>()->bEnforcePassphraseForRemoteClients)
				{
					return FPreprocessorResult::Passthrough();
				}

				auto IsLocal = [](const FString& Address)
				{
					return Address.Contains(TEXT("127.0.0.1")) || Address.Contains(TEXT("localhost"));
				};

				bool bContainsForwardedAddress = false;
				if (IsLocal(PeerAddress))
				{
					if (const TArray<FString>* ForwardedIP = Request.Headers.Find(WebRemoteControlInternalUtils::ForwardedIPHeader))
					{
						if (ForwardedIP->Num())
						{
							PeerAddress = ForwardedIP->Last();

							if (GetDefault<URemoteControlSettings>()->IsClientAllowed(PeerAddress) || IsLocal(PeerAddress))
							{
								return FPreprocessorResult::Passthrough();
							}
							else
							{
								// Let the forwarded IP go through the rest of the validations if it's not 
								bContainsForwardedAddress = true;
							}
						}
						else
						{
							return FPreprocessorResult::Passthrough();
						}
					}
					else
					{
						return FPreprocessorResult::Passthrough();
					}
				}

				// We've already attempted to validate a forwarded address so just do it if it's a direct remote address.
				if (!bContainsForwardedAddress && GetDefault<URemoteControlSettings>()->IsClientAllowed(PeerAddress))
				{
					return FPreprocessorResult::Passthrough();
				}

				if (GetDefault<URemoteControlSettings>()->Passphrases.Num())
				{
					const TArray<FString>* PassphraseHeader = Request.Headers.Find(WebRemoteControlInternalUtils::PassphraseHeader);
					if (!PassphraseHeader || PassphraseHeader->Num() == 0)
					{
						return FPreprocessorResult::Deny(TEXT("Remote passphrase enforcement is enabled but no passphrase was specified!"));
					}

					if (!WebRemoteControlInternalUtils::CheckPassphrase(PassphraseHeader->Last()))
					{
						return FPreprocessorResult::Deny(WebRemoteControlInternalUtils::InvalidPassphraseError);
					}
					else
					{
						return FPreprocessorResult::Passthrough();
					}
				}
				else
				{
					/* Prompt user, checkout settings if needed,
					 * Either
					 * 1: Explicitely add peer IP to Allowlist
					 * 2: Create a passphrase, tell user he must enter that on the app he's using or put it in the header
					 * 3: Disable remote passphrase enforcement. (Warn that this is dangerous)
					 */
#if WITH_EDITOR
					if (GEditor)
					{
						if (!NoPassphrasePrompt.IsValid() || NoPassphrasePrompt.Pin()->GetCompletionState() == SNotificationItem::CS_None)
						{
							FNotificationInfo Info(LOCTEXT("NoPassphraseNotificationHeader", "Remote control request denied!"));
							Info.SubText = FText::Format(LOCTEXT("NoPassphraseNotificationSubtext", "A remote control request was made by an external client ({0}) without a passphrase."), FText::FromString(PeerAddress));
							Info.bFireAndForget = false;
							Info.FadeInDuration = 0.5f;
							Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("AllowListIP", "AllowList IP"), LOCTEXT("AllowListTooltip", "Add this IP to the list of allowed IPs that can make remote control requests without a passphrase."), FSimpleDelegate::CreateStatic(&AddIPToAllowlist, PeerAddress)));
							Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("CreatePassphrase", "Create Passphrase"), LOCTEXT("CreatePassPhraseTooltip", "Create a passphrase for this client to use."), FSimpleDelegate::CreateStatic(&CreatePassphrase)));
							Info.ButtonDetails.Add(FNotificationButtonInfo(LOCTEXT("DisableRemotePassphrase", "Disable Passphrases"), LOCTEXT("DisableRemotePassphrasesTooltip", "Disable the requirement for remote control requests coming from external clients to have a passphrase.\nWarning: This should only be done as a last resort since all clients on the network will be able to access your servers."), FSimpleDelegate::CreateStatic(&DisableRemotePassphrases)));
							Info.WidthOverride = 450.f;
							NoPassphrasePrompt = FSlateNotificationManager::Get().AddNotification(MoveTemp(Info));
							NoPassphrasePrompt.Pin()->SetCompletionState(SNotificationItem::CS_Pending);
							return FPreprocessorResult::Deny(TEXT("A passphrase for remote control is required. See editor for resolving this."));
						}
						else
						{
							return FPreprocessorResult::Deny(TEXT("A passphrase for remote control is required. See editor for resolving this."));
						}
					}
					else
#endif
					{
						// In -game or packaged, we won't prompt the user so just deny the request.
						return FPreprocessorResult::Deny(TEXT("A passphrase for remote control is required but we can't create one in -game or packaged."));
					}
				}
			}
		}

		return FPreprocessorResult::Passthrough();
	}
	
	/** Checks whether a request has a valid passphrase when passphrases are enabled for this editor. */
	FPreprocessorResult PassphrasePreprocessor(const FHttpServerRequest& Request)
	{
		TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

		TArray<FString> ValueArray = {};
		if (Request.Headers.Find(WebRemoteControlInternalUtils::PassphraseHeader))
		{
			ValueArray = Request.Headers[WebRemoteControlInternalUtils::PassphraseHeader];
		}
		
		const FString Passphrase = !ValueArray.IsEmpty() ? ValueArray.Last() : "";

		if (!WebRemoteControlInternalUtils::CheckPassphrase(Passphrase))
		{
			return FPreprocessorResult::Deny(WebRemoteControlInternalUtils::InvalidPassphraseError);
		}

		return FPreprocessorResult::Passthrough();
	}

	/** Checks whether an IP is in a valid range for making remote control requests. 
	 * Also checks for a valid origin to block malicious requests from web browsers.
	 * Can be controlled using the Allowed Origin and AllowedIP remote control settings.
	 */
	FPreprocessorResult IPValidationPreprocessor(const FHttpServerRequest& Request)
	{
		if (GetDefault<URemoteControlSettings>()->bRestrictServerAccess)
		{
			TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

			FString OriginHeader;
			if (const TArray<FString>* OriginHeaders = Request.Headers.Find(WebRemoteControlInternalUtils::OriginHeader))
			{
				if (OriginHeaders->Num())
				{
					OriginHeader = (*OriginHeaders)[0];
				}
			}

			OriginHeader.RemoveSpacesInline();
			OriginHeader.TrimStartAndEndInline();
		
			auto SimplifyAddress = [] (FString Address)
			{
				Address.RemoveFromStart(TEXT("https://www."));
				Address.RemoveFromStart(TEXT("http://www."));
				Address.RemoveFromStart(TEXT("https://"));
				Address.RemoveFromStart(TEXT("http://"));
				Address.RemoveFromEnd(TEXT("/"));
				return Address;
			};

			const FString SimplifiedOrigin = SimplifyAddress(OriginHeader);
			const FWildcardString SimplifiedAllowedOrigin = SimplifyAddress(GetDefault<URemoteControlSettings>()->AllowedOrigin);
			if (!SimplifiedOrigin.IsEmpty() && GetDefault<URemoteControlSettings>()->AllowedOrigin != TEXT("*"))
			{
				if (!SimplifiedAllowedOrigin.IsMatch(SimplifiedOrigin))
				{
					return FPreprocessorResult::Deny(FString::Printf(TEXT("Client origin %s does not respect the allowed origin set in Remote Control Settings."), *OriginHeader));
				}
			}

			if (Request.PeerAddress)
			{
				constexpr bool bAppendPort = false;
				FString ClientIP = Request.PeerAddress->ToString(bAppendPort);
				
				// Allow requests from localhost
				if (ClientIP != TEXT("localhost") && ClientIP != TEXT("127.0.0.1"))
				{
					if (!GetDefault<URemoteControlSettings>()->IsClientAllowed(ClientIP))
					{
						return FPreprocessorResult::Deny(FString::Printf(TEXT("Client IP %s does not respect the allowed IP set in Remote Control Settings."), *ClientIP));
					}
				}
			}
		}

		return FPreprocessorResult::Passthrough();
	}
}

#undef LOCTEXT_NAMESPACE /* WebRemoteControl */
