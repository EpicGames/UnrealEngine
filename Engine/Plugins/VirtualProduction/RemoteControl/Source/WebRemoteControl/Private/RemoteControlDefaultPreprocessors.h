// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HttpResultCallback.h"
#include "IPAddress.h"
#include "HttpServerRequest.h"
#include "Misc/WildcardString.h"
#include "RemoteControlSettings.h"
#include "WebRemoteControlInternalUtils.h"

#if WITH_EDITOR
#include "Dialogs/Dialogs.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "HAL/PlatformFileManager.h"
#include "ISourceControlModule.h"
#include "Misc/MessageDialog.h"
#include "SSettingsEditorCheckoutNotice.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/Text/STextBlock.h"
#endif

#define LOCTEXT_NAMESPACE "WebRemoteControl"

namespace UE::WebRemoteControl
{
	/** Indicates whether a request has been handled (OnComplete callback called) or if it just passed through the preprocessor. */
	enum class EPreprocessorResult : uint8
	{
		RequestPassthrough,
		RequestHandled
	};

	/** Result of a preprocessor. When failed, it will automatically respond to the client request.
	 * Usage:
	 *	 Deny a request:
	 *		return FPreprocessorResult::Deny(TEXT("My preprocessor error message"));
	 * 
	 *	or let it through:
	 *		return FPreprocessorResult::Passthrough();
	 * 
	 * 
	 */
	struct FPreprocessorResult
	{
		/** Let request pass. */
		static FPreprocessorResult Passthrough()
		{
			return FPreprocessorResult();
		}

		/** Deny request and respond with error message. */
		static FPreprocessorResult Deny(const FString& ErrorMessage)
		{
			UE_LOGF(LogRemoteControl, Error, "%ls", *ErrorMessage);
			IRemoteControlModule::BroadcastError(ErrorMessage);

			TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(ErrorMessage, Response->Body);
			Response->Code = EHttpServerResponseCodes::Denied;

			return FPreprocessorResult{ EPreprocessorResult::RequestHandled, MoveTemp(Response) };
		}

		/** Holds the preprocessor result. */
		EPreprocessorResult Result = EPreprocessorResult::RequestPassthrough;
		/** If denied, holds the response to be sent to the client. */
		TUniquePtr<FHttpServerResponse> OptionalResponse;

	private:
		FPreprocessorResult() = default;
		FPreprocessorResult(EPreprocessorResult InResult, TUniquePtr<FHttpServerResponse> InOptionalResponse)
			: Result(InResult), OptionalResponse(MoveTemp(InOptionalResponse))
		{}
	};
	
	using FRCPreprocessorHandler = TFunction<FPreprocessorResult(const FHttpServerRequest& Request)>;

	/** Utility function to wrap a preprocessor handler to a http request handler than the HttpRouter can take. */
	FHttpRequestHandler MakeHttpRequestHandler(FRCPreprocessorHandler Handler);

#if WITH_EDITOR
	/** Attempt saving the remote control config, checkouting it if needed.*/
	void SaveRemoteControlConfig();

	/** Add IP to the list of IPs that should be let through by RC. */
	void AddIPToAllowlist(FString IPAddress);

	/** Prompts the user to create a passphrase.  */
	void CreatePassphrase();

	/** Disables remote passphrase enforcement by modifying a RC project setting. */
	void DisableRemotePassphrases();
#endif

	// Notifies the editor if a client tries to access the RC server without a passphrase.
	FPreprocessorResult RemotePassphraseEnforcementPreprocessor(const FHttpServerRequest& Request);
	
	/** Checks whether a request has a valid passphrase when passphrases are enabled for this editor. */
	FPreprocessorResult PassphrasePreprocessor(const FHttpServerRequest& Request);

	/** Checks whether an IP is in a valid range for making remote control requests. 
	 * Also checks for a valid origin to block malicious requests from web browsers.
	 * Can be controlled using the Allowed Origin and AllowedIP remote control settings.
	 */
	FPreprocessorResult IPValidationPreprocessor(const FHttpServerRequest& Request);
}

#undef LOCTEXT_NAMESPACE /* WebRemoteControl */
