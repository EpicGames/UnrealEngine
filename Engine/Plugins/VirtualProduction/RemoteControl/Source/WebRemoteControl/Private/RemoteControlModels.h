// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "CoreMinimal.h"
#include "Algo/Transform.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/ARFilter.h"
#include "RCVirtualProperty.h"
#include "GameFramework/Actor.h"
#include "RemoteControlActor.h"
#include "RemoteControlField.h"
#include "RemoteControlPreset.h"

#include "RemoteControlModels.generated.h"

namespace RemoteControlModels
{
	// Names used to sanitize property/function metadata
	static FName Name_UIMin("UIMin");
	static FName Name_UIMax("UIMax");
	static FName Name_ClampMin("ClampMin");
	static FName Name_ClampMax("ClampMax");
	static FName Name_ToolTip("ToolTip");
	static FName Name_EnumValues("EnumValues");

	// Names used to sanitize asset metadata
	static FName NAME_FiBData("FiBData");
	static FName NAME_ClassFlags("ClassFlags");
	static FName NAME_AssetImportData("AssetImportData");

	static TMap<FName, FString> SanitizeMetadata(const TMap<FName, FString>& InMetadata)
	{
		TMap<FName, FString> OutMetadata;
		OutMetadata.Reserve(InMetadata.Num());
		auto IsValid = [] (const TTuple<FName, FString>& InTuple)
			{
				return InTuple.Key == Name_UIMin
					|| InTuple.Key == Name_UIMax
					|| InTuple.Key == Name_ClampMin
					|| InTuple.Key == Name_ClampMax
					|| InTuple.Key == Name_ToolTip;
			};

		Algo::TransformIf(InMetadata, OutMetadata, IsValid, [](const TTuple<FName, FString>& InTuple) { return InTuple; });
		return OutMetadata;
	}

	static TMap<FName, FString> SanitizeAssetMetadata(const FAssetDataTagMap& InAssetMetadata)
	{
		TMap<FName, FString> OutMetadata;
		OutMetadata.Reserve(InAssetMetadata.Num());
		auto IsValid = [] (const TTuple<FName, FString>& InTuple)
		{
			return InTuple.Key != NAME_FiBData
				&& InTuple.Key != NAME_ClassFlags
				&& InTuple.Key != NAME_AssetImportData;
		};

		Algo::TransformIf(InAssetMetadata, OutMetadata, IsValid, [](const TTuple<FName, FString>& InTuple) { return InTuple; });
		return OutMetadata;
	}
}

/**
 * Short description of an unreal object.
 */
USTRUCT()
struct FRCObjectDescription
{
	GENERATED_BODY()

	FRCObjectDescription() = default;

	FRCObjectDescription(UObject* InObject)
	{
		checkSlow(InObject);
		
#if WITH_EDITOR
		if (AActor* Actor = Cast<AActor>(InObject))
		{
			Name = Actor->GetActorLabel();
		}
		else
#endif
		{
			// Get the class name when dealing with BP libraries and subsystems. 
			Name = InObject->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject) ? InObject->GetClass()->GetName() : InObject->GetName();
		}
		
		Class = InObject->GetClass()->GetName();
		Path = InObject->GetPathName();
	}

	/** Name of the object. */
	UPROPERTY()
	FString Name;

	/** Class of the object. */
	UPROPERTY()
	FString Class;

	/** Path of the object. */
	UPROPERTY()
	FString Path;
};

USTRUCT()
struct FRCPropertyDescription
{
	GENERATED_BODY()
	
	FRCPropertyDescription() = default;
	
	FRCPropertyDescription(const FProperty* Property)
	{
		checkSlow(Property);

		Name = Property->GetName();

#if WITH_EDITORONLY_DATA
		DisplayName = Property->GetDisplayNameText();
#else
		DisplayName = FText::FromString(Name);
#endif

		const FProperty* ValueProperty = Property;
		
		if (const FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Property))
		{
			ContainerType = Property->GetCPPType();
			ValueProperty = ArrayProperty->Inner;
		}
		else if (const FSetProperty* SetProperty = CastField<FSetProperty>(Property))
		{
			ContainerType = Property->GetCPPType();
			ValueProperty = SetProperty->ElementProp;
		}
		else if (const FMapProperty* MapProperty = CastField<FMapProperty>(Property))
		{
			ContainerType = Property->GetCPPType();
			KeyType = MapProperty->KeyProp->GetCPPType();
			ValueProperty = MapProperty->ValueProp;
		}
		else if (Property->ArrayDim > 1)
		{
			ContainerType = TEXT("CArray");
		}

		//Write the type name
		Type = ValueProperty->GetCPPType();

		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(ValueProperty))
		{
			if (UClass* Class = ObjectProperty->PropertyClass)
			{
				Type = FString::Printf(TEXT("%s%s"), Class->GetPrefixCPP(), *Class->GetName());
				TypePath = *Class->GetPathName();
			}
		}

#if WITH_EDITOR
		Metadata = Property->GetMetaDataMap() ? RemoteControlModels::SanitizeMetadata(*Property->GetMetaDataMap()) : TMap<FName, FString>();
		Description = Property->GetMetaData("ToolTip");
#endif

		//Fill Enum choices metadata
		UEnum* Enum = nullptr;

		if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(ValueProperty))
		{
			Enum = EnumProperty->GetEnum();
		}

		if (const FByteProperty* ByteProperty = CastField<FByteProperty>(ValueProperty))
		{
			Enum = ByteProperty->Enum;
		}

		if (Enum)
		{
			const int32 EnumCount = Enum->NumEnums() - 1; //Don't list the _MAX entry
			TStringBuilder<256> Builder;
			for (int32 Index = 0; Index < EnumCount; ++Index)
			{
				FString Text = Enum->GetDisplayNameTextByIndex(Index).ToString();
				if (Text.IsEmpty())
				{
					Text = Enum->GetNameStringByIndex(Index);
				}

				if (Index > 0)
				{
					Builder.Append(TEXT(", "));
				}
				Builder.Append(Text);
			}

			if (Builder.Len() > 0)
			{
				Metadata.FindOrAdd(RemoteControlModels::Name_EnumValues) = Builder.ToString();
			}
		}
	}

	/** Name of the exposed property */
	UPROPERTY()
	FString Name;

	/** Display name of the exposed property */
	UPROPERTY()
	FText DisplayName;
	
	/** Description of the exposed property */
	UPROPERTY()
	FString Description;
	
	/** Type of the property value (If an array, this will be the content of the array) */
	UPROPERTY()
	FString Type;

	/** Type of the property value (If an array, this will be the content of the array) */
	UPROPERTY()
	FName TypePath;

	/** Type of the container (TMap, TArray, CArray, TSet) or empty if none */
	UPROPERTY()
	FString ContainerType;

	/** Key type if container is a map */
	UPROPERTY()
	FString KeyType;

	/** Metadata for this exposed property */
	UPROPERTY()
	TMap<FName, FString> Metadata;
};

USTRUCT()
struct FRCFunctionDescription
{
	GENERATED_BODY()

	FRCFunctionDescription() = default;

	FRCFunctionDescription(const UFunction* Function)
		: Name(Function->GetName())
	{
		checkSlow(Function);
#if WITH_EDITOR
		Description = Function->GetMetaData("ToolTip");
#endif
		for (TFieldIterator<FProperty> It(Function); It; ++It)
		{
			if (It && It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm))
			{
				Arguments.Emplace(*It);
			}
		}
	}
	
	/** Name of the function. */
	UPROPERTY()
	FString Name;
	
	/** Description for the function. */
	UPROPERTY()
	FString Description;
	
	/** The function's arguments. */
	UPROPERTY()
	TArray<FRCPropertyDescription> Arguments;
};

USTRUCT()
struct FRCExposedPropertyDescription
{
	GENERATED_BODY()

	FRCExposedPropertyDescription() = default;
	
	FRCExposedPropertyDescription(const FRemoteControlProperty& RCProperty)
		: DisplayName(RCProperty.GetLabel())
	{
		UnderlyingProperty = RCProperty.GetProperty();
		Metadata = RCProperty.GetMetadata();
		ID = RCProperty.GetId().ToString();
		OwnerObjects.Append(RCProperty.GetBoundObjects());
	}

	/** The label displayed in the remote control panel for this exposed property. */
	UPROPERTY()
	FName DisplayName;

	/** Unique identifier for the exposed property. */
	UPROPERTY()
	FString ID;

	/** The underlying exposed property. */
	UPROPERTY()
	FRCPropertyDescription UnderlyingProperty;

	/** Metadata specific to this exposed property. */
	UPROPERTY()
	TMap<FName, FString> Metadata;

	/** The objects that own the underlying property. */
	UPROPERTY()
	TArray<FRCObjectDescription> OwnerObjects; 
};

USTRUCT()
struct FRCExposedFunctionDescription
{
	GENERATED_BODY()
	
	FRCExposedFunctionDescription() = default;

	FRCExposedFunctionDescription(const FRemoteControlFunction& Function)
		: DisplayName(Function.GetLabel())
		, ID(Function.GetId().ToString())
		, UnderlyingFunction(Function.GetFunction())
	{
		OwnerObjects.Append(Function.GetBoundObjects());
	}

	/** The label displayed in the remote control panel for this exposed property. */
	UPROPERTY()
	FName DisplayName;

	/** Unique identifier for the exposed function. */
	UPROPERTY()
	FString ID;
	
	/** The underlying exposed function. */
	UPROPERTY()
	FRCFunctionDescription UnderlyingFunction;

	/** The objects that own the underlying function. */
	UPROPERTY()
	TArray<FRCObjectDescription> OwnerObjects;
};

USTRUCT()
struct FRCExposedActorDescription
{
	GENERATED_BODY()

	FRCExposedActorDescription() = default;

	FRCExposedActorDescription(const FRemoteControlActor& InExposedActor)
		: DisplayName(InExposedActor.GetLabel())
	{
		if (AActor* Actor = Cast<AActor>(InExposedActor.Path.ResolveObject()))
		{
			UnderlyingActor = FRCObjectDescription{ Actor };
		}

		ID = InExposedActor.GetId().ToString();
	}

	/** The label displayed in the remote control panel for this exposed actor. */
	UPROPERTY()
	FName DisplayName;
	
	/** Unique identifier for the exposed actor. */
	UPROPERTY()
	FString ID;

	/** The underlying exposed actor. */
	UPROPERTY()
	FRCObjectDescription UnderlyingActor;
};

USTRUCT()
struct FRCControllerDescription
{
	GENERATED_BODY()

	FRCControllerDescription() = default;

	FRCControllerDescription(const URCVirtualPropertyBase* InController)
	{
		if (!InController || !InController->GetProperty())
		{
			return;
		}

		DisplayName = InController->DisplayName;
		Description = InController->Description;
		ID = InController->Id.ToString();
		Path = InController->GetPathName();
		Type = InController->GetProperty()->GetCPPType();
		Metadata = InController->Metadata;
	}
	
	/** The label displayed in the remote control panel for this controller. */
	UPROPERTY()
	FName DisplayName;

	/** Controller description. */
	UPROPERTY()
	FText Description;

	/** Unique identifier for the controller. */
	UPROPERTY()
	FString ID;

	/** Type of the Controller */
	UPROPERTY()
	FString Type;

	/** Path of the Controller */
	UPROPERTY()
	FString Path;

	/** Metadata of the Controller. Initially Empty. */
	UPROPERTY()
	TMap<FName, FString> Metadata;
};

USTRUCT()
struct FRCControllerModifiedDescription
{
	GENERATED_BODY()

	FRCControllerModifiedDescription () = default;

	FRCControllerModifiedDescription(const URemoteControlPreset* InPreset, const TArray<FGuid>& InModifiedControllers)
	{
		if (!InPreset)
		{
			return;
		}

		for (const FGuid& ControllerId : InModifiedControllers)
		{
			AddModifiedController(InPreset, ControllerId);
		}
		
	}

public:
	/** The list of modified RC controllers. */
	UPROPERTY()
	TArray<FRCControllerDescription> Controllers;

	UPROPERTY()
	TArray<FString> ChangedValues;
	
private:
	void AddModifiedController(const URemoteControlPreset* InPreset, const FGuid& InControllerId)
	{
		if (!InPreset)
		{
			return;
		}

		if (URCVirtualPropertyBase* Controller = InPreset->GetController(InControllerId))
		{
			Controllers.Emplace(Controller);
			ChangedValues.Emplace(Controller->GetDisplayValueAsString());
		}
	}
};

USTRUCT()
struct FRCPresetLayoutGroupDescription
{
	GENERATED_BODY()

	FRCPresetLayoutGroupDescription() = default;

	FRCPresetLayoutGroupDescription(const URemoteControlPreset* Preset, const FRemoteControlPresetGroup& Group)
		: Name(Group.Name)
	{
		checkSlow(Preset);
		for (const FGuid& EntityId : Group.GetFields())
		{
			AddExposedField(Preset, EntityId);
		}
	}

	FRCPresetLayoutGroupDescription(const URemoteControlPreset* Preset, const FRemoteControlPresetGroup& Group, const TArray<FGuid>& EntityIds)
		: Name(Group.Name)
	{
		checkSlow(Preset);
		for (const FGuid& Id : EntityIds)
		{
			AddExposedField(Preset, Id);
		}
	}

public:
	UPROPERTY()
	FName Name;

	UPROPERTY()
	TArray<FRCExposedPropertyDescription> ExposedProperties;

	UPROPERTY()
	TArray<FRCExposedFunctionDescription> ExposedFunctions;

	UPROPERTY()
	TArray<FRCExposedActorDescription> ExposedActors;

private:
	/** Add an exposed field to this group description. */
	void AddExposedField(const URemoteControlPreset* Preset, const FGuid& EntityId)
	{
		if (TSharedPtr<const FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(EntityId).Pin())
		{
			if (FProperty* Property = RCProperty->GetProperty())
			{
				ExposedProperties.Emplace(*RCProperty);	
			}
		}
		else if (TSharedPtr<const FRemoteControlFunction> RCFunction = Preset->GetExposedEntity<FRemoteControlFunction>(EntityId).Pin())
		{
			if (RCFunction->GetFunction())
			{
				ExposedFunctions.Emplace(*RCFunction);
			}
		}
		else if (TSharedPtr<const FRemoteControlActor> RCActor = Preset->GetExposedEntity<FRemoteControlActor>(EntityId).Pin())
		{
			if (RCActor->GetActor())
			{
				ExposedActors.Emplace(*RCActor);
			}
		}
	}
};

/**
 * Holds lists of modified RC entities.
 * @Note that this does not mean the underlying property/function/actor was modified,
 * but rather that the entity structure itself was modified in some way.
 */
USTRUCT()
struct FRCPresetModifiedEntitiesDescription
{
	GENERATED_BODY()

	FRCPresetModifiedEntitiesDescription() = default;
	
	FRCPresetModifiedEntitiesDescription(const URemoteControlPreset* Preset, const TArray<FGuid>& EntityIds)
	{
		checkSlow(Preset);
		for (const FGuid& Id : EntityIds)
		{
			AddModifiedEntity(Preset, Id);
		}
	}

public:
	/** The list of modified RC properties. */
	UPROPERTY()
	TArray<FRCExposedPropertyDescription> ModifiedRCProperties;

	/** The list of modified RC functions. */
	UPROPERTY()
	TArray<FRCExposedFunctionDescription> ModifiedRCFunctions;

	/** The list of modified RC actors. */
	UPROPERTY()
	TArray<FRCExposedActorDescription> ModifiedRCActors;

private:
	/** Add an exposed field to this group description. */
	void AddModifiedEntity(const URemoteControlPreset* Preset, const FGuid& EntityId)
	{
		if (TSharedPtr<const FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(EntityId).Pin())
		{
			if (FProperty* Property = RCProperty->GetProperty())
			{
				ModifiedRCProperties.Emplace(*RCProperty);	
			}
		}
		else if (TSharedPtr<const FRemoteControlFunction> RCFunction = Preset->GetExposedEntity<FRemoteControlFunction>(EntityId).Pin())
		{
			if (RCFunction->GetFunction())
			{
				ModifiedRCFunctions.Emplace(*RCFunction);
			}
		}
		else if (TSharedPtr<const FRemoteControlActor> RCActor = Preset->GetExposedEntity<FRemoteControlActor>(EntityId).Pin())
		{
			if (RCActor->GetActor())
			{
				ModifiedRCActors.Emplace(*RCActor);
			}
		}
	}
};

USTRUCT()
struct FRCPresetDescription
{
	GENERATED_BODY()
	
	FRCPresetDescription() = default;

	FRCPresetDescription(const URemoteControlPreset* Preset)
	{
		checkSlow(Preset)

		Name = Preset->GetPresetName().ToString();
		Path = Preset->GetPathName();
		ID = Preset->GetPresetId().ToString();

		Algo::Transform(Preset->Layout.GetGroups(), Groups, [Preset](const FRemoteControlPresetGroup& Group) { return FRCPresetLayoutGroupDescription{ Preset, Group }; });
		Algo::Transform(Preset->GetControllers(), Controllers, [Preset](const URCVirtualPropertyBase* Controller){ return FRCControllerDescription{ Controller };});
	}

	/**
	 * Name of the preset.
	 */
	UPROPERTY()
	FString Name;

	/**
	 * Name of the preset.
	 */
	UPROPERTY()
	FString Path;

	/**
	 * Unique identifier for the preset, can be used to make requests to the API.
	 */
	UPROPERTY()
	FString ID;

	/**
	 * The groups containing exposed entities.
	 */
	UPROPERTY()
	TArray<FRCPresetLayoutGroupDescription> Groups;

	/**
	 * The controllers held by the Preset (no groups)
	 */
	UPROPERTY()
	TArray<FRCControllerDescription> Controllers;
};

USTRUCT()
struct FRCShortPresetDescription
{
	GENERATED_BODY()
 
	FRCShortPresetDescription() = default;

	FRCShortPresetDescription(const FAssetData& PresetAsset)
	{
		Name = PresetAsset.AssetName;
		FGuid PresetId = PresetAsset.GetTagValueRef<FGuid>(FName("PresetId"));

		if (!PresetId.IsValid())
		{
			// Load the object to attempt to get a valid ID.
			if (URemoteControlPreset* Preset = Cast<URemoteControlPreset>(PresetAsset.GetAsset()))
			{
				PresetId = Preset->GetPresetId();
			}
		}

		ID = PresetId.ToString();
		Path = *PresetAsset.GetObjectPathString();
	}

	FRCShortPresetDescription(const URemoteControlPreset* InPreset)
	{
		if (InPreset)
		{
			Name = InPreset->GetPresetName();
			ID = InPreset->GetPresetId().ToString();
			Path = *InPreset->GetPathName();
		}
	}

	FRCShortPresetDescription(const TWeakObjectPtr<URemoteControlPreset> InPreset)
	{
		if (InPreset.IsValid())
		{
			Name = InPreset->GetPresetName();
			ID = InPreset->GetPresetId().ToString();
			Path = *InPreset->GetPathName();
		}
	}

	/**
	 * Name of the preset.
	 */
	UPROPERTY()
	FName Name;

	/**
	 * Unique identifier for the preset, can be used to make requests to the API.
	 */
	UPROPERTY()
	FString ID;

	/**
	 * Object path of the preset.
	 */
	UPROPERTY()
	FName Path;
};

USTRUCT()
struct FRCAssetDescription
{
	GENERATED_BODY()

	FRCAssetDescription() = default;
	FRCAssetDescription(const FAssetData& InAsset)
		: Name(InAsset.AssetName)
		, Class(*InAsset.AssetClassPath.ToString())
		, Path(*InAsset.GetObjectPathString())
	{
		Metadata = RemoteControlModels::SanitizeAssetMetadata(InAsset.TagsAndValues.CopyMap());
	}

	UPROPERTY()
	FName Name;

	UPROPERTY()
	FName Class;

	UPROPERTY()
	FName Path;

	UPROPERTY()
	TMap<FName, FString> Metadata;
};

USTRUCT()
struct FRCPresetFieldRenamed
{
	GENERATED_BODY()

	FRCPresetFieldRenamed() = default;

	FRCPresetFieldRenamed(const TTuple<FName, FName>& RenamedField)
		: OldFieldLabel(RenamedField.Key)
		, NewFieldLabel(RenamedField.Value)
	{
	}

	UPROPERTY()
	FName OldFieldLabel;

	UPROPERTY()
	FName NewFieldLabel;
};

USTRUCT()
struct FRCAssetFilter
{
	GENERATED_BODY()

	FRCAssetFilter() = default;

	FARFilter ToARFilter() const
	{
		auto ToAssetPath = [](FName InClassName) { return FTopLevelAssetPath(InClassName.ToString()); };

		FARFilter Filter;
		Filter.PackageNames = PackageNames;

		Filter.ClassPaths.Reserve(ClassNames.Num());
		Algo::Transform(ClassNames, Filter.ClassPaths, ToAssetPath);

		Algo::Transform(RecursiveClassesExclusionSet, Filter.RecursiveClassPathsExclusionSet, ToAssetPath);
		Filter.bRecursiveClasses = RecursiveClasses;
		Filter.PackagePaths = PackagePaths;

		// Default to a recursive search at root if no filter is specified.
		if (Filter.PackageNames.Num() == 0
			&& Filter.ClassPaths.Num() == 0
			&& Filter.PackagePaths.Num() == 0)
		{
			Filter.PackagePaths = { FName("/Game") };
			Filter.bRecursivePaths = true;
		}
		else
		{
			Filter.PackagePaths = PackagePaths;
			Filter.bRecursivePaths = RecursivePaths;
		}

		return Filter;
	}

	/** The filter component for package names */
	UPROPERTY()
	TArray<FName> PackageNames;

	/** The filter component for package paths */
	UPROPERTY()
	TArray<FName> PackagePaths;

	/** The filter component for class names. Instances of the specified classes, but not subclasses (by default), will be included. Derived classes will be included only if bRecursiveClasses is true. */
	UPROPERTY()
	TArray<FName> ClassNames;

	/** Only if bRecursiveClasses is true, the results will exclude classes (and subclasses) in this list */
	UPROPERTY()
	TSet<FName> RecursiveClassesExclusionSet;

	/** Only if EnableBlueprintNativeClassFiltering is true, resulting asset will be filtered for dependants of classes in this list. */
	UPROPERTY()
	TArray<FName> NativeParentClasses;

	/** If true, subclasses of ClassNames will also be included and RecursiveClassesExclusionSet will be excluded. */
	UPROPERTY()
	bool RecursiveClasses = false;

	/** If true, PackagePath components will be recursive */
	UPROPERTY()
	bool RecursivePaths = false;

	/** When dealing with blueprint classes, you might want to filter for a base class which can't be picked by asset registry if you derive from a blueprint class */
	UPROPERTY()
	bool EnableBlueprintNativeClassFiltering = false;
};

/**
 * A description of an actor that can both uniquely identify it and provide a user-friendly name.
 */
USTRUCT()
struct FRCActorDescription
{
	GENERATED_BODY()

	FRCActorDescription() = default;

	FRCActorDescription(const AActor* InActor)
		: Name(InActor->GetActorNameOrLabel())
		, Path(InActor->GetPathName())
	{
	}

	inline bool operator==(const FRCActorDescription& Other) const
	{
		return Path == Other.Path && Name == Other.Name;
	}

	/** A user-friendly name for the actor. */
	UPROPERTY()
	FString Name;

	/** The path to the actor. */
	UPROPERTY()
	FString Path;
};