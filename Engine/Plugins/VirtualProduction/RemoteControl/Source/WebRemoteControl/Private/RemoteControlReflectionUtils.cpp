// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlReflectionUtils.h"

#include "Engine/Engine.h"
#include "IRemoteControlModule.h"
#include "UObject/Package.h"
#include "UObject/StructOnScope.h"

UScriptStruct* UWebRCStructRegistry::GenerateStruct(FName StructureName, const FWebRCGenerateStructArgs& Args)
{
	if (UScriptStruct* CachedStruct = CachedStructs.FindRef(StructureName))
	{
		return CachedStruct;
	}

	return CachedStructs.Add(StructureName, GenerateStructInternal(StructureName, Args));
}

UScriptStruct* UWebRCStructRegistry::GenerateStructInternal(FName StructureName, const FWebRCGenerateStructArgs& Args)
{
	UScriptStruct* NewStruct = NewObject<UScriptStruct>(GetTransientPackage(), *StructureName.ToString(), RF_NoFlags);

	for (FName ParamName : Args.StringProperties)
	{
		NewStruct->AddCppProperty(new FStrProperty(NewStruct, *ParamName.ToString()));
	}

	for (const TPair<FName, UScriptStruct*>& Pair : Args.ArrayProperties)
	{
		FArrayProperty* NewArrayProperty = new FArrayProperty(NewStruct, Pair.Key);

		FName InnerPathPropName(Pair.Key.ToString() + TEXT("_Inner"));
		FStructProperty* NewInnerProp = new FStructProperty(NewArrayProperty, InnerPathPropName);
		NewInnerProp->Struct = Pair.Value;
		NewArrayProperty->Inner = NewInnerProp;

		NewStruct->AddCppProperty(NewArrayProperty);
	}

	for (const TPair<FName, FProperty*>& Pair : Args.GenericProperties)
	{
		check(Pair.Value);
		FProperty* DuplicatedField = CastField<FProperty>(FField::Duplicate(Pair.Value, NewStruct, Pair.Key));
		NewStruct->AddCppProperty(DuplicatedField);
	}

	for (const TPair<FName, UScriptStruct*>& Pair : Args.StructProperties)
	{
		check(Pair.Value);
		FStructProperty* StructProperty = new FStructProperty(NewStruct, Pair.Key);
		StructProperty->Struct = Pair.Value;
		NewStruct->AddCppProperty(StructProperty);
	}

	// Finalize the struct
	NewStruct->Bind();
	NewStruct->PrepareCppStructOps();
	NewStruct->StaticLink(/*bRelinkExistingProperties*/true);

	return NewStruct;
}

UScriptStruct* UE::WebRCReflectionUtils::GenerateStruct(FName StructureName, const FWebRCGenerateStructArgs& Args)
{
	check(IsInGameThread());

	UWebRCStructRegistry* StructRegistry = GEngine->GetEngineSubsystem<UWebRCStructRegistry>();
	return StructRegistry->GenerateStruct(StructureName, Args);
}

void UE::WebRCReflectionUtils::SetStringPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const FString& Value)
{
	FProperty* Prop = TargetStruct.GetStruct()->FindPropertyByName(PropertyName);
	check(Prop);
	*Prop->ContainerPtrToValuePtr<FString>((void*)TargetStruct.GetStructMemory()) = Value;
}

void UE::WebRCReflectionUtils::CopyPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const FRCObjectReference& SourceObject)
{
	FProperty* TargetValueProp = TargetStruct.GetStruct()->FindPropertyByName(PropertyName);
	const FProperty* SourceValueProp = SourceObject.Property.Get();

	if (TargetValueProp && SourceValueProp && TargetValueProp->GetClass() == SourceValueProp->GetClass() && TargetValueProp->GetSize() == SourceValueProp->GetSize())
	{
		void* DestPtr = TargetValueProp->ContainerPtrToValuePtr<void>((void*)TargetStruct.GetStructMemory());
		if (!DestPtr)
		{
			checkNoEntry();
			return;
		}

		if (SourceValueProp->HasGetter() && SourceObject.Object.IsValid())
		{
			SourceObject.Property->PerformOperationWithGetter(SourceObject.Object.Get(), nullptr,
				[TargetValueProp, DestPtr](const void* InValuePtr)
				{
					if (InValuePtr)
					{
						TargetValueProp->CopyCompleteValue(DestPtr, InValuePtr);
						return;
					}
					checkNoEntry();
				});
			return;
		}

		const void* SrcPtr = SourceObject.Property->ContainerPtrToValuePtr<void>(SourceObject.ContainerAdress);
		if (SrcPtr)
		{
			TargetValueProp->CopyCompleteValue(DestPtr, SrcPtr);
			return;
		}
	}

	checkNoEntry();
}

void UE::WebRCReflectionUtils::SetStructArrayPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const TArray<FStructOnScope>& ArrayElements)
{
	FArrayProperty* ArrayProp = CastFieldChecked<FArrayProperty>(TargetStruct.GetStruct()->FindPropertyByName(PropertyName));
	FScriptArrayHelper_InContainer Helper{ ArrayProp, TargetStruct.GetStructMemory() };
	for (int32 Index = 0; Index < ArrayElements.Num(); Index++)
	{
		const FStructOnScope& ArrayElement = ArrayElements[Index];
		Helper.AddValue();
		void* NewPosPtr = Helper.GetRawPtr(Index);
		((UScriptStruct*)ArrayElement.GetStruct())->CopyScriptStruct(NewPosPtr, ArrayElement.GetStructMemory());
	}
}

void UE::WebRCReflectionUtils::SetStructPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const UScriptStruct* Struct, void* StructData)
{
	FProperty* DestProp = TargetStruct.GetStruct()->FindPropertyByName(PropertyName);
	check(DestProp);
	void* DestPtr = DestProp->ContainerPtrToValuePtr<void>((void*)TargetStruct.GetStructMemory());
	check(DestPtr);
	check(StructData);
	Struct->CopyScriptStruct(DestPtr, StructData);
}
