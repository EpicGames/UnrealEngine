// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "UObject/Class.h"

#include "RemoteControlReflectionUtils.generated.h"

struct FRCObjectReference;
class FStructOnScope;

struct FWebRCGenerateStructArgs
{
	/** String properties. */
	TArray<FName> StringProperties;

	/** Struct properties. */
	TArray<TPair<FName, UScriptStruct*>> StructProperties;

	/** Properties with an unknown type at compile time. */
	TArray<TPair<FName, FProperty*>> GenericProperties;

	/** List of array properties and the type they should hold. */
	TArray<TPair<FName, UScriptStruct*>> ArrayProperties;
};

/** A subsystem to provide and cache dynamically created ustructs. */
UCLASS()
class UWebRCStructRegistry : public UEngineSubsystem
{
	GENERATED_BODY()
	
public:
	/** 
	 * Generates a struct based on provided arguments.
	 * @param StructureName The name of the structure to generate.
	 * @param Args the properties that should populate the structure.
	 * @return The generated structure.
	 */
	UScriptStruct* GenerateStruct(FName StructureName, const FWebRCGenerateStructArgs& Args);

private:	
	/** Creates a new struct based on provided arguments. */
	UScriptStruct* GenerateStructInternal(FName StructureName, const FWebRCGenerateStructArgs& Args);

private:
	/** Map of cached structs. */
	UPROPERTY(Transient)
	TMap<FName, TObjectPtr<UScriptStruct>> CachedStructs;
};

namespace UE
{
	namespace WebRCReflectionUtils
	{
		/** 
		 * Generates a struct based on provided arguments.
		 * @param StructureName The name of the structure to generate.
		 * @param Args the properties that should populate the structure.
		 * @return The generated structure.
		 */
		UScriptStruct* GenerateStruct(FName StructureName, const FWebRCGenerateStructArgs& Args);
		
		/**
		 * Set a string property value in a provided struct on scope.
		 * @param PropertyName the name of the property to set.
		 * @param TargetStruct the structure that contains the property to modify.
		 * @param Value the new property value.
		 */
		void SetStringPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const FString& Value);

		/**
		 * Copy a property value from an object to a struct on scope.
		 * @param PropertyName the name of the property to copy.
		 * @param TargetStruct the structure that contains the property to overwrite.
		 * @param SourceObject the object to copy the property from.
		 */
		void CopyPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const FRCObjectReference& SourceObject);

		/**
		 * Sets an array of structures in a struct on scope.
		 * @param PropertyName the name of the property to set.
		 * @param TargetStruct the structure that contains the property to modify.
		 * @param ArrayElements the array that should be copied into the target structure.
		 */
		void SetStructArrayPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const TArray<FStructOnScope>& ArrayElements);

		/**
		 * Sets a struct property in the target struct.
		 * @param PropertyName the name of the property to set.
		 * @param TargetStruct the structure that contains the property to modify.
		 * @param Struct the type of the structure to set.
		 * @param StructData the data that should be copied into the target structure.
		 */
		void SetStructPropertyValue(FName PropertyName, const FStructOnScope& TargetStruct, const UScriptStruct* Struct, void* StructData);
	}
}

