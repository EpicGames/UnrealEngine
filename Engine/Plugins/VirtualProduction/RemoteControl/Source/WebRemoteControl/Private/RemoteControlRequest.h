// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "CoreMinimal.h"
#include "IRemoteControlModule.h"
#include "RemoteControlModels.h"
#include "RemoteControlWebsocketRoute.h"

#include "RemoteControlRequest.generated.h"

USTRUCT()
struct FRCRequestWrapper : public FRCRequest
{
	GENERATED_BODY()
	
	FRCRequestWrapper()
		: RequestId(-1)
	{
		AddStructParameter(BodyLabel());
	}

	/**
	 * Get the label for the parameters struct.
	 */
	static FString BodyLabel() { return TEXT("Body"); }
	
	UPROPERTY()
	FString URL;

	UPROPERTY()
	FName Verb;

	UPROPERTY()
	int32 RequestId;
};

/**
 * Holds a request that wraps multiple requests..
 */
USTRUCT()
struct FRCBatchRequest : public FRCRequest
{
	GENERATED_BODY()

	/**
	 * The list of batched requests.
	 */
	UPROPERTY()
	TArray<FRCRequestWrapper> Requests;
};

UENUM()
enum class ERemoteControlEvent : uint8
{
	PreObjectPropertyChanged = 0,
	ObjectPropertyChanged,
	EventCount,
};

/**
 * Holds a request to create an event hook.
 */
USTRUCT()
struct FRemoteControlObjectEventHookRequest : public FRCRequest
{
	GENERATED_BODY()

	FRemoteControlObjectEventHookRequest()
		: EventType(ERemoteControlEvent::EventCount)
	{}

	/**
	 * What type of event should be listened to.
	 */
	UPROPERTY()
	ERemoteControlEvent EventType;

	/**
	 * The path of the target object.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * The name of the property to watch for changes.
	 */
	UPROPERTY()
	FString PropertyName;
};


/**
 * Holds a request to call a function
 */
USTRUCT()
struct FRCCallRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCCallRequest()
	{
		AddStructParameter(ParametersLabel());
	}

	/**
	 * Get the label for the parameters struct.
	 */
	static FString ParametersLabel() { return TEXT("Parameters"); }

	/**
	 * The path of the target object.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * The name of the function to call.
	 */
	UPROPERTY()
	FString FunctionName;

	/**
	 * Whether a transaction should be created for the call.
	 */
	UPROPERTY()
	bool GenerateTransaction = false;
};

/**
 * Holds a request to access an object
 */
USTRUCT()
struct FRCObjectRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCObjectRequest()
	{
		AddStructParameter(PropertyValueLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString PropertyValueLabel() { return TEXT("PropertyValue"); }

	/**
	 * Get whether the property should be writen or read and if a transaction should be created.
	 */
	ERCAccess GetAccessValue() const
	{
		ERCAccess RCAccess = Access;
		if (RCAccess == ERCAccess::NO_ACCESS)
		{
			RCAccess = ERCAccess::READ_ACCESS;
			// Use read access by default when no access is specified, but use write access if property value is specified
			if (StructParameters.FindChecked(TEXT("PropertyValue")).BlockStart > 0)
			{
				if (GenerateTransaction)
				{
					RCAccess = ERCAccess::WRITE_TRANSACTION_ACCESS;
				}
				else
				{
					RCAccess = ERCAccess::WRITE_ACCESS;
				}
			}
		}
		
		return RCAccess;
	}

public:
	/**
	 * The path of the target object.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * The property to read or modify.
	 */
	UPROPERTY()
	FString PropertyName;

	/**
	 * Whether the property should be reset to default.
	 */
	UPROPERTY()
	bool ResetToDefault = false;

	/**
	 * Whether a transaction should be created for the call.
	 */
	UPROPERTY()
	bool GenerateTransaction = false;

	/**
	 * Which type of operation should be performed if this is modifying a property.
	 */
	UPROPERTY()
	ERCModifyOperation Operation = ERCModifyOperation::EQUAL;

private:
	/**
	 * Indicates if the property should be read or written to.
	 */
	UPROPERTY()
	ERCAccess Access = ERCAccess::NO_ACCESS;
};

/**
 * Holds a request to set a property on a preset
 */
USTRUCT()
struct FRCPresetSetPropertyRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCPresetSetPropertyRequest()
	{
		AddStructParameter(PropertyValueLabel());
	}

	/**
	 * Get the label for the PropertyValue struct.
	 */
	static FString PropertyValueLabel() { return TEXT("PropertyValue"); }

	/**
	 * Which type of operation should be performed on the value of the property.
	 * This will be ignored if ResetToDefault is true.
	 */
	UPROPERTY()
	ERCModifyOperation Operation = ERCModifyOperation::EQUAL;

	/**
	 * Whether a transaction should be created for the call.
	 */
	UPROPERTY()
	bool GenerateTransaction = false;

	UPROPERTY()
	bool ResetToDefault = false;
};

/**
 * Holds a request to call a function on a preset
 */
USTRUCT()
struct FRCPresetCallRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCPresetCallRequest()
	{
		AddStructParameter(ParametersLabel());
	}

	/**
	 * Get the label for the parameters struct.
	 */
	static FString ParametersLabel() { return TEXT("Parameters"); }

	/**
	 * Whether a transaction should be created for the call.
	 */
	UPROPERTY()
	bool GenerateTransaction = false;
};

/**
 * Holds a request to expose a property on a preset
 */
USTRUCT()
struct FRCPresetExposePropertyRequest : public FRCRequest
{
	GENERATED_BODY()

	/**
	 * The path of the target object.
	 */
	UPROPERTY()
	FString ObjectPath;

	/**
	 * The property to expose.
	 */
	UPROPERTY()
	FString PropertyName;

	/**
	 * The label to give the new exposed property (optional).
	 */
	UPROPERTY()
	FString Label;

	/**
	 * The name of the group in which to place the new exposed property (optional).
	 */
	UPROPERTY()
	FString GroupName;

	/**
	 * Whether to automatically enable the edit condition for the exposed property.
	 */
	UPROPERTY()
	bool EnableEditCondition = true;
};

/**
 * Holds a request to describe an object using its path.
 */
USTRUCT()
struct FDescribeObjectRequest: public FRCRequest
{
	GENERATED_BODY()

	FDescribeObjectRequest()
	{
	}

	/**
	 * The target object's path.
	 */
	UPROPERTY()
	FString ObjectPath;
};

/**
 * Holds a request to search for an asset.
 */
USTRUCT()
struct FSearchAssetRequest : public FRCRequest
{
	GENERATED_BODY()

	FSearchAssetRequest()
		: Limit(100)
	{
	}

	/**
	 * The search query which will be compared with the asset names.
	 */
	UPROPERTY()
	FString Query;

	/*
	 * The filter applied to this search.
	 */
	UPROPERTY()
	FRCAssetFilter Filter;
	
	/**
	 * The maximum number of search results returned.
	 */
	UPROPERTY()
	int32 Limit;
};

/**
 * Holds a request to search for an actor.
 */
USTRUCT()
struct FSearchActorRequest : public FRCRequest
{
	GENERATED_BODY()

	FSearchActorRequest()
		: Limit(100)
	{
	}

	/*
	 * The search query.
	 */
	UPROPERTY()
	FString Query;

	/**
	 * The target actor's class. 
	 */
	UPROPERTY()
	FString Class;

	/**
	 * The maximum number of search results returned.
	 */
	UPROPERTY()
	int32 Limit;
};

/**
 * Holds a request to search for an asset.
 */
USTRUCT()
struct FSearchObjectRequest : public FRCRequest
{
	GENERATED_BODY()

	FSearchObjectRequest()
		: Limit(100)
	{
	}

	/*
	 * The search query.
	 */
	UPROPERTY()
	FString Query;

	/**
	 * The target object's class.
	 */
	UPROPERTY()
	FString Class;

	/**
	 * The search target's outer object.
	 */
	UPROPERTY()
	FString Outer;

	/**
	 * The maximum number of search results returned.
	 */
	UPROPERTY()
	int32 Limit;
};


/**
 * Holds a request to set a metadata field.
 */
USTRUCT()
struct FSetPresetMetadataRequest : public FRCRequest
{
	GENERATED_BODY()

	FSetPresetMetadataRequest() = default;

	/**
	 * The new value for the metadata field.
	 */
	UPROPERTY()
	FString Value;
};

/**
 * Holds a request to set a metadata field.
 */
USTRUCT()
struct FSetEntityMetadataRequest : public FRCRequest
{
	GENERATED_BODY()

	FSetEntityMetadataRequest() = default;

	/**
	 * The new value for the metadata field.
	 */
	UPROPERTY()
	FString Value;
};

/**
 * Holds a request to set an entity's label.
 */
USTRUCT()
struct FSetEntityLabelRequest : public FRCRequest
{
	GENERATED_BODY()

	/**
	 * The new label to assign.
	 */
	UPROPERTY()
	FString NewLabel;
};

/**
 * Holds a request to get an asset's thumbnail.
 */
USTRUCT()
struct FGetObjectThumbnailRequest : public FRCRequest
{
	GENERATED_BODY()

	FGetObjectThumbnailRequest() = default;

	/**
	 * The target object's path.
	 */
	UPROPERTY()
	FString ObjectPath;
};

/**
 * Holds a request made for web socket.
 */
USTRUCT()
struct FRCWebSocketRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketRequest()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * Name of the websocket message.
	 */
	UPROPERTY()
	FString MessageName;

	/**
	 * (Optional) Id of the incoming message, used to identify a deferred response to the clients.
	 */
	UPROPERTY()
	int32 Id = INDEX_NONE;

	/**
	 * (Optional) If the request was forwared for a remote client, this will contain the forwarded IP.
	 */
	UPROPERTY()
	FString ForwardedFor;
};

/**
 * Holds a request that wraps multiple requests..
 */
USTRUCT()
struct FRCWebSocketBatchRequest : public FRCRequest
{
	GENERATED_BODY()

	/**
	 * The list of batched requests.
	 */
	UPROPERTY()
	TArray<FRCWebSocketRequest> Requests;
};

/**
 * Holds a request made via websocket to register for events about a given preset.
 */
USTRUCT()
struct FRCWebSocketPresetRegisterBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketPresetRegisterBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * Name of the preset its registering.
	 */
	UPROPERTY()
	FString PresetName;

	/** Whether changes to properties triggered remotely should fire an event. */
	UPROPERTY()
	bool IgnoreRemoteChanges = false;
};

/**
 * Holds a request made via websocket to automatically destroy a transient preset when the calling client disconnects.
 */
USTRUCT()
struct FRCWebSocketTransientPresetAutoDestroyBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketTransientPresetAutoDestroyBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * Name of the transient preset to mark for automatic destruction.
	 */
	UPROPERTY()
	FString PresetName;
};

/**
 * Holds a request made via websocket to register for spawn/destroy events about a given actor type.
 */
USTRUCT()
struct FRCWebSocketActorRegisterBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketActorRegisterBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * Name of the actor class to register for.
	 */
	UPROPERTY()
	FName ClassName;
};

/**
 * Holds a request made via websocket to modify a property exposed in a preset.
 */
USTRUCT()
struct FRCWebSocketPresetSetPropertyBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketPresetSetPropertyBody()
	{
		AddStructParameter(PropertyValueLabel());
	}

	/**
	 * Get the label for the PropertyValue struct.
	 */
	static FString PropertyValueLabel() { return TEXT("PropertyValue"); }

	/**
	 * The name of the remote control preset to which the property belongs.
	 */
	UPROPERTY()
	FName PresetName;

	/**
	 * The label of the property to modify.
	 */
	UPROPERTY()
	FName PropertyLabel;

	/**
	 * Which type of operation should be performed on the value of the property.
	 * This will be ignored if ResetToDefault is true.
	 */
	UPROPERTY()
	ERCModifyOperation Operation = ERCModifyOperation::EQUAL;

	/**
	 * How to handle generating transactions for this property change.
	 * If NONE, don't generate a transaction immediately.
	 * If AUTOMATIC, let the Remote Control system automatically start and end the transaction after enough time passes.
	 * If MANUAL, TransactionId must also be set and the changes will only be applied if that transaction is still active.
	 */
	UPROPERTY()
	ERCTransactionMode TransactionMode = ERCTransactionMode::NONE;

	/**
	 * The ID of the transaction with which to associate these changes. Must be provided if TransactionMode is Manual.
	 */
	UPROPERTY()
	int32 TransactionId = -1;

	/**
	 * If true, ignore the other parameters and just reset the property to its default value.
	 */
	UPROPERTY()
	bool ResetToDefault = false;

	/**
	 * The sequence number of this change. The highest sequence number received from this client will be
	 * sent back to the client in future PresetFieldsChanged events.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};

/**
 * Holds a request made via websocket to call an exposed function on an object.
 */
USTRUCT()
struct FRCWebSocketCallBody : public FRCCallRequest
{
	GENERATED_BODY()

	FRCWebSocketCallBody()
	: FRCCallRequest()
	{
	}

	/**
	 * How to handle generating transactions for this property change.
	 * If NONE, don't generate a transaction immediately.
	 * If AUTOMATIC, let the Remote Control system automatically start and end the transaction after enough time passes.
	 * If MANUAL, TransactionId must also be set and the changes will only be applied if that transaction is still active.
	 * If bGenerateTransaction is true, this value will be treated as if it was AUTOMATIC.
	 */
	UPROPERTY()
	ERCTransactionMode TransactionMode = ERCTransactionMode::NONE;

	/**
	 * The ID of the transaction with which to associate these changes. Must be provided if TransactionMode is Manual.
	 */
	UPROPERTY()
	int32 TransactionId = -1;

	/**
	 * The sequence number of this change. The highest sequence number received from this client will be
	 * sent back to the client in future PresetFieldsChanged events.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};

/**
 * Holds a request made via websocket to start a transaction.
 */
USTRUCT()
struct FRCWebSocketTransactionStartBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketTransactionStartBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The description of the transaction.
	 */
	UPROPERTY()
	FString Description;

	/**
	 * The ID that will be used to refer to the transaction in future messages.
	 */
	UPROPERTY()
	int32 TransactionId = -1;
};


/**
 * Holds a request made via websocket to end a transaction.
 */
USTRUCT()
struct FRCWebSocketTransactionEndBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketTransactionEndBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The ID of the transaction. If this doesn't match the current editor transaction, it won't be ended.
	 */
	UPROPERTY()
	int32 TransactionId = -1;
};

/**
 * Holds a request made via websocket to change the compression mode.
 */
USTRUCT()
struct FRCWebSocketCompressionChangeBody : public FRCRequest
{
	GENERATED_BODY()

	FRCWebSocketCompressionChangeBody()
	{
		AddStructParameter(ParametersFieldLabel());
	}

	/**
	 * Get the label for the property value struct.
	 */
	static FString ParametersFieldLabel() { return TEXT("Parameters"); }

	/**
	 * The compression mode to use.
	 */
	UPROPERTY()
	ERCWebSocketCompressionMode Mode = ERCWebSocketCompressionMode::NONE;
};

/**
 * Struct representation of SetPresetController HTTP request
 */
USTRUCT()
struct FRCPresetSetControllerRequest : public FRCRequest
{
	GENERATED_BODY()

	FRCPresetSetControllerRequest()
	{
		AddStructParameter(PropertyValueLabel());
	}

	/**
	 * Get the label for the PropertyValue struct.
	 */
	static FString PropertyValueLabel() { return TEXT("PropertyValue"); }

public:

	/**
	 * The name of the Controller being set (for a given Remote Control Preset asset)
	 */
	UPROPERTY()
	FString ControllerName;
};

