// Copyright Epic Games, Inc. All Rights Reserved.
#pragma  once

#include "AssetRegistry/AssetData.h"
#include "CoreMinimal.h"
#include "IRemoteControlModule.h"
#include "RemoteControlField.h"
#include "RemoteControlModels.h"
#include "RemoteControlPreset.h"
#include "RemoteControlRoute.h"


#include "RemoteControlResponse.generated.h"

USTRUCT()
struct FAPIInfoResponse
{
	GENERATED_BODY()
	
	FAPIInfoResponse() = default;

	FAPIInfoResponse(const TArray<FRemoteControlRoute>& InRoutes, bool bInPackaged, URemoteControlPreset* InActivePreset)
		: IsPackaged(bInPackaged)
		, ActivePreset(InActivePreset) 
	{
		HttpRoutes.Append(InRoutes);
	}

private:
	/**
	 * Whether this is a packaged build or not.
	 */
	bool IsPackaged = false;

	/**
	 * Descriptions for all the routes that make up the remote control API.
	 */
	UPROPERTY()
	TArray<FRemoteControlRouteDescription> HttpRoutes;

	UPROPERTY()
	FRCShortPresetDescription ActivePreset;
};

USTRUCT()
struct FListPresetsResponse
{
	GENERATED_BODY()
	
	FListPresetsResponse() = default;

	FListPresetsResponse(const TArray<FAssetData>& InPresets, const TArray<TWeakObjectPtr<URemoteControlPreset>> InEmbeddedPresets)
	{
		Presets.Append(InPresets);
		Presets.Append(InEmbeddedPresets);
	}

	/**
	 * The list of available remote control presets. 
	 */
	UPROPERTY()
	TArray<FRCShortPresetDescription> Presets;
};

USTRUCT()
struct FGetPresetResponse
{
	GENERATED_BODY()

	FGetPresetResponse() = default;

	FGetPresetResponse(const URemoteControlPreset* InPreset)
		: Preset(InPreset)
	{}

	UPROPERTY()
	FRCPresetDescription Preset;
};

USTRUCT()
struct FCheckPassphraseResponse
{
	GENERATED_BODY()

	FCheckPassphraseResponse() = default;

	FCheckPassphraseResponse(bool bInKeyCorrect)
		: keyCorrect(bInKeyCorrect)
	{}

	UPROPERTY()
	bool keyCorrect = false;
};

USTRUCT()
struct FDescribeObjectResponse
{
	GENERATED_BODY()

	FDescribeObjectResponse() = default;

	FDescribeObjectResponse(UObject* Object)
		: Name(Object->GetName())
		, Class(Object->GetClass())
	{
		for (TFieldIterator<FProperty> It(Object->GetClass()); It; ++It)
		{
			if (!It->HasAnyPropertyFlags(CPF_NativeAccessSpecifierProtected | CPF_NativeAccessSpecifierPrivate | CPF_DisableEditOnInstance))
			{
				Properties.Emplace(*It);
			}
		}

		for (TFieldIterator<UFunction> It(Object->GetClass()); It; ++It)
		{
			if (It->HasAnyFunctionFlags(FUNC_BlueprintCallable | FUNC_Public))
			{
				Functions.Emplace(*It);
			}
		}
	}

	UPROPERTY()
	FString Name;

	UPROPERTY()
	TObjectPtr<UClass> Class = nullptr;

	UPROPERTY()
	TArray<FRCPropertyDescription> Properties;

	UPROPERTY()
	TArray<FRCFunctionDescription> Functions;
};


USTRUCT()
struct FSearchAssetResponse
{
	GENERATED_BODY()

	FSearchAssetResponse() = default;

	FSearchAssetResponse(const TArray<FAssetData>& InAssets)
	{
		Assets.Append(InAssets);
	}

	UPROPERTY()
	TArray<FRCAssetDescription> Assets;
};


USTRUCT()
struct FSearchActorResponse
{
	GENERATED_BODY()

	FSearchActorResponse() = default;

	FSearchActorResponse(const TArray<AActor*>& InActors)
	{
		Actors.Append(InActors);
	}

	UPROPERTY()
	TArray<FRCObjectDescription> Actors;
};

USTRUCT()
struct FGetMetadataFieldResponse
{
	GENERATED_BODY()

	FGetMetadataFieldResponse() = default;

	FGetMetadataFieldResponse(FString InValue)
		: Value(MoveTemp(InValue))
	{}

	/** The metadata value for a given key. */
	UPROPERTY()
	FString Value;
};


USTRUCT()
struct FGetMetadataResponse
{
	GENERATED_BODY()

	FGetMetadataResponse() = default;

	FGetMetadataResponse(TMap<FString, FString> InMetadata)
		: Metadata(MoveTemp(InMetadata))
	{}

	UPROPERTY()
	TMap<FString, FString> Metadata;
};

USTRUCT()
struct FSetEntityLabelResponse
{
	GENERATED_BODY()

	FSetEntityLabelResponse() = default;

	FSetEntityLabelResponse(FString&& InString)
		: AssignedLabel(MoveTemp(InString))
	{
	}

	/** The label that was assigned when requesting to modify an entity's label. */
	UPROPERTY()
	FString AssignedLabel;
};

USTRUCT()
struct FRCPresetFieldsRenamedEvent
{
	GENERATED_BODY()

	FRCPresetFieldsRenamedEvent() = default;

	FRCPresetFieldsRenamedEvent(FName InPresetName, FGuid InPresetId, TArray<TTuple<FName, FName>> InRenamedFields)
		: Type(TEXT("PresetFieldsRenamed"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, RenamedFields(MoveTemp(InRenamedFields))
	{
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	TArray<FRCPresetFieldRenamed> RenamedFields;
};

USTRUCT()
struct FRCPresetMetadataModified
{
	GENERATED_BODY()

	FRCPresetMetadataModified() = default;

	FRCPresetMetadataModified(URemoteControlPreset* InPreset)
		: Type(TEXT("PresetMetadataModified"))
	{
		if (InPreset)
		{
			PresetName = InPreset->GetPresetName();
			PresetId = InPreset->GetPresetId().ToString();
			Metadata = InPreset->Metadata;
		}
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	TMap<FString, FString> Metadata;
};


USTRUCT()
struct FRCPresetLayoutModified
{
	GENERATED_BODY()

	FRCPresetLayoutModified() = default;

	FRCPresetLayoutModified(URemoteControlPreset* InPreset)
		: Type(TEXT("PresetLayoutModified"))
		, Preset(InPreset)
	{
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FRCPresetDescription Preset;
};

USTRUCT()
struct FRCPresetFieldsRemovedEvent
{
	GENERATED_BODY()

	FRCPresetFieldsRemovedEvent() = default;

	FRCPresetFieldsRemovedEvent(FName InPresetName, FGuid InPresetId, TArray<FName> InRemovedFields, const TArray<FGuid>& InRemovedFieldIDs)
		: Type(TEXT("PresetFieldsRemoved"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, RemovedFields(MoveTemp(InRemovedFields))
	{
		Algo::Transform(InRemovedFieldIDs, RemovedFieldIds, [](const FGuid& Id){ return Id.ToString(); });
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	TArray<FName> RemovedFields;

	UPROPERTY()
	TArray<FString> RemovedFieldIds;
};

USTRUCT()
struct FRCPresetFieldsAddedEvent
{
	GENERATED_BODY()

	FRCPresetFieldsAddedEvent() = default;

	FRCPresetFieldsAddedEvent(FName InPresetName, FGuid InPresetId, FRCPresetDescription InPresetAddition)
		: Type(TEXT("PresetFieldsAdded"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, Description(MoveTemp(InPresetAddition))
	{
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	FRCPresetDescription Description;
};

/**
 * Event triggered when an exposed entity struct is modified.
 */
USTRUCT()
struct FRCPresetEntitiesModifiedEvent
{
	GENERATED_BODY()

	FRCPresetEntitiesModifiedEvent() = default;

	FRCPresetEntitiesModifiedEvent(URemoteControlPreset* InPreset, const TArray<FGuid>& InModifiedEntities)
		: Type(TEXT("PresetEntitiesModified"))
	{
		checkSlow(InPreset);
		PresetName = InPreset->GetPresetName();
		PresetId = InPreset->GetPresetId().ToString();
		ModifiedEntities = FRCPresetModifiedEntitiesDescription{InPreset, InModifiedEntities};
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * Name of the preset which contains the modified entities.
	 */
	UPROPERTY()
	FName PresetName;
	
	/**
	 * ID of the preset that contains the modified entities.
	 */
	UPROPERTY()
	FString PresetId;

	/**
	 * The entities that were modified in the last frame.
	 */
	UPROPERTY()
	FRCPresetModifiedEntitiesDescription ModifiedEntities;
};

USTRUCT()
struct FRCPresetControllersRenamedEvent
{
	GENERATED_BODY()

	FRCPresetControllersRenamedEvent() = default;

	FRCPresetControllersRenamedEvent(FName InPresetName, FGuid InPresetId, TArray<TTuple<FName, FName>> InRenamedControllers)
		: Type(TEXT("PresetControllersRenamed"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, RenamedControllers(InRenamedControllers)
	{
	}
		
	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	TArray<FRCPresetFieldRenamed> RenamedControllers;
};

USTRUCT()
struct FRCPresetControllersRemovedEvent
{
	GENERATED_BODY()

	FRCPresetControllersRemovedEvent() = default;

	FRCPresetControllersRemovedEvent(FName InPresetName, FGuid InPresetId, TArray<FName> InRemoveControllers, const TArray<FGuid>& InRemovedControllerIds)
		: Type(TEXT("PresetControllersRemoved"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, RemovedControllers(MoveTemp(InRemoveControllers))
	{
		Algo::Transform(InRemovedControllerIds, RemovedControllerIds, [](const FGuid& Id){ return Id.ToString();});
	}

	UPROPERTY()
	FString Type;

	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	TArray<FName> RemovedControllers;

	UPROPERTY()
	TArray<FString> RemovedControllerIds;
};

USTRUCT()
struct FRCPresetControllersAddedEvent
{
	GENERATED_BODY()

	FRCPresetControllersAddedEvent() = default;

	FRCPresetControllersAddedEvent(FName InPresetName, FGuid InPresetId, FRCPresetDescription InPresetDescription)
		: Type(TEXT("PresetControllersAdded"))
		, PresetName(InPresetName)
		, PresetId(InPresetId.ToString())
		, Description(MoveTemp(InPresetDescription))
	{
	}

	UPROPERTY()
	FString Type;
	
	UPROPERTY()
	FName PresetName;

	UPROPERTY()
	FString PresetId;

	UPROPERTY()
	FRCPresetDescription Description;
};

/** Event which is triggered whenever a Controller is modified. */
USTRUCT()
struct FRCPresetControllersModifiedEvent
{
	GENERATED_BODY()

	FRCPresetControllersModifiedEvent() = default;

	FRCPresetControllersModifiedEvent(const URemoteControlPreset* InPreset, const TArray<FGuid>& InModifiedControllers)
		: Type(TEXT("PresetControllersModified"))
	{
		checkSlow(InPreset);
		PresetName = InPreset->GetPresetName();
		PresetId = InPreset->GetPresetId().ToString();
		ModifiedControllers = FRCControllerModifiedDescription(InPreset, InModifiedControllers);
		
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * Name of the preset which contains the modified controller.
	 */
	UPROPERTY()
	FName PresetName;
	
	/**
	 * ID of the preset that contains the modified controller.
	 */
	UPROPERTY()
	FString PresetId;

	/**
	 * The controllers that were modified in the last frame.
	 */
	UPROPERTY()
	FRCControllerModifiedDescription ModifiedControllers;
};

/**
 * Data about actors that have changed in the scene.
 */
USTRUCT()
struct FRCActorsChangedData
{
	GENERATED_BODY()

	/** Actors that were added. */
	UPROPERTY()
	TArray<FRCActorDescription> AddedActors;

	/** Actors that were renamed. */
	UPROPERTY()
	TArray<FRCActorDescription> RenamedActors;

	/** Actors that were deleted. */
	UPROPERTY()
	TArray<FRCActorDescription> DeletedActors;
};

/**
 * Event triggered when the list of actors in the current scene (or their names) changes.
 */
USTRUCT()
struct FRCActorsChangedEvent
{
	GENERATED_BODY()

	FRCActorsChangedEvent()
		: Type(TEXT("ActorsChanged"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * Map from class name to changes in actors of that type.
	 */
	UPROPERTY()
	TMap<FString, FRCActorsChangedData> Changes;
};

/**
 * Event sent to a client that contributed to a transaction, indicating that the transaction was either cancelled or finalized.
 */
USTRUCT()
struct FRCTransactionEndedEvent
{
	GENERATED_BODY()

	FRCTransactionEndedEvent()
	: Type(TEXT("TransactionEnded"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The client-specific ID of the transaction that was ended.
	 */
	UPROPERTY()
	int32 TransactionId = -1;

	/**
	 * The highest sequence number received from the receiving client at the time that the transaction ended.
	 */
	UPROPERTY()
	int64 SequenceNumber = -1;
};

/**
 * Event sent to confirm that the WebSocket compression mode has changed.
 * This will be sent using the previous compression mode, and messages after this point will use the new one indicated in this event.
 */
USTRUCT()
struct FRCCompressionChangedEvent
{
	GENERATED_BODY()

	FRCCompressionChangedEvent()
	: Type(TEXT("CompressionChanged"))
	{
	}

	/**
	 * Type of the event.
	 */
	UPROPERTY()
	FString Type;

	/**
	 * The new compression mode which will be used by future messages.
	 */
	UPROPERTY()
	ERCWebSocketCompressionMode Mode = ERCWebSocketCompressionMode::NONE;
};

