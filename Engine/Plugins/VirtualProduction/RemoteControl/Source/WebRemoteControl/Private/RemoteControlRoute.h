// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HttpResultCallback.h"
#include "HttpServerResponse.h"
#include "HttpServerRequest.h"
#include "HttpPath.h"
#include "RemoteControlWebsocketRoute.h"

#include "RemoteControlRoute.generated.h"

struct FRemoteControlRoute
{
	FRemoteControlRoute(FString InRouteDescription, FHttpPath InPath, EHttpServerRequestVerbs InVerb, FHttpRequestHandler InHandler)
		: RouteDescription(MoveTemp(InRouteDescription))
		, Path(MoveTemp(InPath))
		, Verb(InVerb)
		, Handler(MoveTemp(InHandler))
	{
	}
	/** A description of how the route should be used. */
	FString RouteDescription;
	/** Relative path (ie. /remote/object) */
	FHttpPath Path;
	/** The desired HTTP verb (ie. GET, PUT..) */
	EHttpServerRequestVerbs Verb = EHttpServerRequestVerbs::VERB_GET;
	/** The handler called when the route is accessed. */
	FHttpRequestHandler Handler;

	friend uint32 GetTypeHash(const FRemoteControlRoute& Route) { return HashCombine(GetTypeHash(Route.Path), GetTypeHash(Route.Verb)); }
	friend bool operator==(const FRemoteControlRoute& LHS, const FRemoteControlRoute& RHS) { return LHS.Path == RHS.Path && LHS.Verb == RHS.Verb; }
};

UENUM()
enum class ERemoteControlHttpVerbs : uint16
{
	None = 0,
	Get = 1 << 0,
	Post = 1 << 1,
	Put = 1 << 2,
	Patch = 1 << 3,
	Delete = 1 << 4,
	Options = 1 << 5
};

/**
 * Utility struct to create a textual representation of an http route.
 */
USTRUCT()
struct FRemoteControlRouteDescription
{
	GENERATED_BODY()

	FRemoteControlRouteDescription() = default;

	FRemoteControlRouteDescription(const FRemoteControlRoute& Route)
		: Path(Route.Path.GetPath())
		, Verb((ERemoteControlHttpVerbs)Route.Verb)
		, Description(Route.RouteDescription)
	{}

	UPROPERTY(EditAnywhere, Category = Test)
	FString Path;

	UPROPERTY()
	ERemoteControlHttpVerbs Verb = ERemoteControlHttpVerbs::None;

	UPROPERTY()
	FString Description;
};

