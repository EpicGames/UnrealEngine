// Copyright Epic Games, Inc. All Rights Reserved.

#include "RemoteControlWebSocketServer.h"

#include "Containers/Ticker.h"
#include "IPAddress.h"
#include "IRemoteControlModule.h"
#include "IWebSocketNetworkingModule.h"
#include "Misc/Compression.h"
#include "Misc/WildcardString.h"
#include "RemoteControlRequest.h"
#include "RemoteControlSettings.h"
#include "SocketSubsystem.h"
#include "Sockets.h"
#include "WebRemoteControlInternalUtils.h"
#include "WebSocketNetworkingDelegates.h"

#define LOCTEXT_NAMESPACE "RCWebSocketServer"

static TAutoConsoleVariable<int32> CVarWebSocketMaxUncompressedMessageSize(
	TEXT("WebSocket.MaxUncompressedMessageSize"),
	1024 * 1024 * 256,
	TEXT("If a compressed WebSocket message reports an uncompressed size larger than this, reject the message.")
);

namespace RemoteControlWebSocketServer
{
	static const FString MessageNameFieldName = TEXT("MessageName");
	static const FString PayloadFieldName = TEXT("Parameters");

	TOptional<FRemoteControlWebSocketMessage> ParseWebsocketMessage(TArrayView<uint8> InPayload)
	{
		FRCWebSocketRequest Request;
		bool bSuccess = WebRemoteControlInternalUtils::DeserializeRequestPayload(InPayload, nullptr, Request);

		FString ErrorText;
		if (Request.MessageName.IsEmpty())
		{
			ErrorText = TEXT("Missing MessageName field.");
		}

		FBlockDelimiters PayloadDelimiters = Request.GetParameterDelimiters(FRCWebSocketRequest::ParametersFieldLabel());
		if (PayloadDelimiters.BlockStart == PayloadDelimiters.BlockEnd)
		{
			ErrorText = FString::Printf(TEXT("Missing %s field."), *FRCWebSocketRequest::ParametersFieldLabel());
		}

		TOptional<FRemoteControlWebSocketMessage> ParsedMessage;
		if (!bSuccess)
		{
			UE_LOGF(LogRemoteControl, Error, "%ls", *FString::Format(TEXT("Encountered error while deserializing websocket message. \r\n{0}"), { *ErrorText }));
		}
		else
		{
			FRemoteControlWebSocketMessage Message;
			Message.MessageId = Request.Id;
			if (PayloadDelimiters.BlockStart != PayloadDelimiters.BlockEnd)
			{
				Message.RequestPayload = MakeArrayView(InPayload).Slice(PayloadDelimiters.BlockStart, PayloadDelimiters.BlockEnd - PayloadDelimiters.BlockStart);
			}
			if (!Request.Passphrase.IsEmpty())
			{
				Message.Header.FindOrAdd(WebRemoteControlInternalUtils::PassphraseHeader) = { Request.Passphrase };
			}
			if (!Request.ForwardedFor.IsEmpty())
			{
				Message.Header.FindOrAdd(WebRemoteControlInternalUtils::ForwardedIPHeader) = { Request.ForwardedFor };
			}
			Message.MessageName = MoveTemp(Request.MessageName);
			ParsedMessage = MoveTemp(Message);
		}

		if (!ErrorText.IsEmpty())
		{
			IRemoteControlModule::BroadcastError(ErrorText);
		}

		return ParsedMessage;
	}
}

void FWebsocketMessageRouter::Dispatch(const FRemoteControlWebSocketMessage& Message)
{
	if (FWebSocketMessageDelegate* Callback = DispatchTable.Find(Message.MessageName))
	{
		Callback->ExecuteIfBound(Message);
	}
}

void FWebsocketMessageRouter::AddPreDispatch(TFunction<bool(const FRemoteControlWebSocketMessage& Message)> WebsocketPreprocessor)
{
	DispatchPreProcessor.Add(WebsocketPreprocessor);
}

void FWebsocketMessageRouter::AttemptDispatch(const struct FRemoteControlWebSocketMessage& Message)
{
	if (!PreDispatch(Message))
	{
		return;
	}

	Dispatch(Message);
}

bool FWebsocketMessageRouter::PreDispatch(const FRemoteControlWebSocketMessage& Message) const
{
	for (TFunction<bool(const FRemoteControlWebSocketMessage&)> PreprocessFunction : DispatchPreProcessor)
	{
		if (!PreprocessFunction(Message))
		{
			return false;
		}
	}
	return true;
}

void FWebsocketMessageRouter::BindRoute(const FString& MessageName, FWebSocketMessageDelegate OnMessageReceived)
{
	DispatchTable.Add(MessageName, MoveTemp(OnMessageReceived));
}

void FWebsocketMessageRouter::UnbindRoute(const FString& MessageName)
{
	DispatchTable.Remove(MessageName);
}

bool FRCWebSocketServer::Start(uint32 Port, TSharedPtr<FWebsocketMessageRouter> InRouter)
{
	FWebSocketClientConnectedCallBack CallBack;
	CallBack.BindRaw(this, &FRCWebSocketServer::OnWebSocketClientConnected);

	Server = FModuleManager::Get().LoadModuleChecked<IWebSocketNetworkingModule>(TEXT("WebSocketNetworking")).CreateServer();
	
	if (!Server || !Server->Init(Port, CallBack, GetDefault<URemoteControlSettings>()->RemoteControlWebsocketServerBindAddress))
	{
		Server.Reset();
		return false;
	}

	FWebSocketFilterConnectionCallback FilterCallback;
	FilterCallback.BindRaw(this, &FRCWebSocketServer::FilterConnection);
	Server->SetFilterConnectionCallback(MoveTemp(FilterCallback));
	
	Router = MoveTemp(InRouter);
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FRCWebSocketServer::Tick));

	return true;
}

bool FRCWebSocketServer::IsPortAvailable(uint32 Port) const
{
	if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	{
		TSharedRef<FInternetAddr> Addr = SocketSubsystem->CreateInternetAddr();
		Addr->SetAnyAddress();
		Addr->SetPort(Port);

		if (FUniqueSocket Socket = SocketSubsystem->CreateUniqueSocket(NAME_Stream, TEXT("TemporarySocket")))
		{
			if (Socket->Bind(*Addr))
			{
				return true;
			}
		}
	}
	
	return false;
}

void FRCWebSocketServer::Stop()
{
	if (TickerHandle.IsValid())
	{
		FTSTicker::RemoveTicker(TickerHandle);
		TickerHandle.Reset();
	}
	Router.Reset();
	Server.Reset();
}

FRCWebSocketServer::~FRCWebSocketServer()
{
	Stop();
}

void FRCWebSocketServer::Broadcast(const TArray<uint8>& InUTF8Payload)
{
	for (FWebSocketConnection& Connection : Connections)
	{
		SendOnConnection(Connection, InUTF8Payload);
	}
}

void FRCWebSocketServer::Send(const FGuid& InTargetClientId, const TArray<uint8>& InUTF8Payload)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCWebSocketServer::Send);
	if (FWebSocketConnection* Connection = GetClientById(InTargetClientId))
	{
		SendOnConnection(*Connection, InUTF8Payload);
	}
}

bool FRCWebSocketServer::IsRunning() const
{
	return !!Server;
}

void FRCWebSocketServer::SetClientCompressionMode(const FGuid& ClientId, ERCWebSocketCompressionMode Mode)
{
	for (FWebSocketConnection& Connection : Connections)
	{
		if (ClientId != Connection.Id)
		{
			continue;
		}

		Connection.CompressionMode = Mode;
		break;
	}
}

bool FRCWebSocketServer::Tick(float DeltaTime)
{
	Server->Tick();
	return true;
}

void FRCWebSocketServer::OnWebSocketClientConnected(INetworkingWebSocket* Socket)
{
	if (ensureMsgf(Socket, TEXT("Socket was null while creating a new websocket connection.")))
	{
		FWebSocketConnection Connection = FWebSocketConnection{ Socket };
			
		FWebSocketPacketReceivedCallBack ReceiveCallBack;
		ReceiveCallBack.BindRaw(this, &FRCWebSocketServer::ReceivedRawPacket, Connection.Id, Connection.PeerAddress);
		Socket->SetReceiveCallBack(ReceiveCallBack);

		FWebSocketInfoCallBack CloseCallback;
		CloseCallback.BindRaw(this, &FRCWebSocketServer::OnSocketClose, Socket);
		Socket->SetSocketClosedCallBack(CloseCallback);

		OnConnectionOpened().Broadcast(Connection.Id);
		Connections.Add(MoveTemp(Connection));
	}
}

void FRCWebSocketServer::ReceivedRawPacket(void* Data, int32 Size, FGuid ClientId, TSharedPtr<FInternetAddr> PeerAddress)
{
	if (!Router)
	{
		return;
	}
	
	TRACE_CPUPROFILER_EVENT_SCOPE(FRCWebSocketServer::ReceivedRawPacket);

	TArray<uint8> Payload;

	if (FWebSocketConnection* Connection = GetClientById(ClientId))
	{
		switch (Connection->CompressionMode)
		{
			case ERCWebSocketCompressionMode::ZLIB:
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(DecompressWebSocketData);
				
				// Read the header containing the message's uncompressed size
				const TArrayView<uint8> CompressedData = MakeArrayView(static_cast<uint8*>(Data), Size);
				FMemoryReaderView Archive(CompressedData);

#if !PLATFORM_LITTLE_ENDIAN
				// Use canonical little-endian ordering
				Archive.SetByteSwapping(true);
#endif

				int64 HeaderSize;
				int32 UncompressedSize;
				Archive << UncompressedSize;
				HeaderSize = Archive.Tell();

				if (UncompressedSize <= 0)
				{
					// Invalid size; don't try to allocate this or we'll crash
					return;
				}

				if (UncompressedSize > CVarWebSocketMaxUncompressedMessageSize.GetValueOnGameThread())
				{
					return;
				}

				TArray<uint8> UncompressedData;
				UncompressedData.SetNumUninitialized(UncompressedSize);

				const bool bUncompressOk = FCompression::UncompressMemory(
					NAME_Zlib,
					UncompressedData.GetData(), UncompressedSize,
					CompressedData.GetData() + HeaderSize, CompressedData.Num() - HeaderSize
				);

				if (!bUncompressOk)
				{
					return;
				}

				WebRemoteControlUtils::ConvertToTCHAR(UncompressedData, Payload);
			}
			break;

			default:
				WebRemoteControlUtils::ConvertToTCHAR(MakeArrayView(static_cast<uint8*>(Data), Size), Payload);
				break;
		}
	}

	if (TOptional<FRemoteControlWebSocketMessage> Message = RemoteControlWebSocketServer::ParseWebsocketMessage(Payload))
	{
		Message->ClientId = ClientId;
		Message->PeerAddress = PeerAddress;
	
		Router->AttemptDispatch(*Message);
	}
}

void FRCWebSocketServer::OnSocketClose(INetworkingWebSocket* Socket)
{
	int32 Index = Connections.IndexOfByPredicate([Socket](const FWebSocketConnection& Connection) { return Connection.Socket == Socket; });
	if (Index != INDEX_NONE)
	{
		OnConnectionClosed().Broadcast(Connections[Index].Id);
		Connections.RemoveAtSwap(Index);
	}
}

FRCWebSocketServer::FWebSocketConnection* FRCWebSocketServer::GetClientById(const FGuid& Id)
{
	return Connections.FindByPredicate([&Id](const FWebSocketConnection& InConnection) { return InConnection.Id == Id; });
}

void FRCWebSocketServer::SendOnConnection(FWebSocketConnection& Connection, const TArray<uint8>& InUTF8Payload)
{
	if (!Connection.Socket)
	{
		return;
	}

	switch (Connection.CompressionMode)
	{
	case ERCWebSocketCompressionMode::ZLIB:
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(CompressWebSocketData);

			int32 CompressedSize = InUTF8Payload.Num();

			TArray<uint8> CompressedPayload;
			CompressedPayload.SetNumUninitialized(CompressedSize);

			const bool bCompressOk = FCompression::CompressMemory(
				NAME_Zlib,
				CompressedPayload.GetData(), CompressedSize,
				InUTF8Payload.GetData(), InUTF8Payload.Num()
			);

			if (bCompressOk && CompressedSize < InUTF8Payload.Num())
			{
				Connection.Socket->Send(CompressedPayload.GetData(), CompressedSize, /*PrependSize=*/false);
				return;
			}
		}
		break;
	}

	// Send uncompressed data
	Connection.Socket->Send(InUTF8Payload.GetData(), InUTF8Payload.Num(), /*PrependSize=*/false);
}

EWebsocketConnectionFilterResult FRCWebSocketServer::FilterConnection(FString OriginHeader, FString ClientIP) const
{
	const URemoteControlSettings* Settings = GetDefault<URemoteControlSettings>();
	if (Settings->bRestrictServerAccess)
	{
		OriginHeader.RemoveSpacesInline();
		OriginHeader.TrimStartAndEndInline();

		auto SimplifyAddress = [](FString Address)
		{
			Address.RemoveFromStart(TEXT("https://www."));
			Address.RemoveFromStart(TEXT("http://www."));
			Address.RemoveFromStart(TEXT("https://"));
			Address.RemoveFromStart(TEXT("http://"));
			Address.RemoveFromEnd(TEXT("/"));
			return Address;
		};

		const FString SimplifiedOrigin = SimplifyAddress(OriginHeader);
		const FWildcardString SimplifiedAllowedOrigin = SimplifyAddress(GetDefault<URemoteControlSettings>()->AllowedOrigin);

		if (!SimplifiedOrigin.IsEmpty() && GetDefault<URemoteControlSettings>()->AllowedOrigin != TEXT("*"))
		{
			if (!SimplifiedAllowedOrigin.IsMatch(SimplifiedOrigin))
			{
				return EWebsocketConnectionFilterResult::ConnectionRefused;
			}

			// Allow requests from localhost
			if (ClientIP != TEXT("localhost") && ClientIP != TEXT("127.0.0.1"))
			{
				if (!Settings->IsClientAllowed(ClientIP))
				{
					return EWebsocketConnectionFilterResult::ConnectionRefused;
				}
			}
		}
	}

	return EWebsocketConnectionFilterResult::ConnectionAccepted;
}

#undef LOCTEXT_NAMESPACE /* FRCWebSocketServer */

