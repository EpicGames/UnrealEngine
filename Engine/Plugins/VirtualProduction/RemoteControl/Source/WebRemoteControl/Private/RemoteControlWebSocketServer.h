// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WebSocketNetDriver.h"

#include "Containers/Ticker.h"
#include "INetworkingWebSocket.h"
#include "IRemoteControlModule.h"
#include "IWebRemoteControlModule.h"
#include "IWebSocketServer.h"
#include "RemoteControlRoute.h"
#include "SocketSubsystem.h"
#include "UObject/StrongObjectPtr.h"

/**
 * Router used to dispatch messages received by a WebSocketServer.
 */
class FWebsocketMessageRouter
{
public:
	/**
	 * Add a handler to the Router's dispatch table.
	 * @param MessageName the name of the message to handle.
	 * @param FWebSocketMessageDelegate the handler to called upon receiving a message with the right name.
	 */
	void BindRoute(const FString& MessageName, FWebSocketMessageDelegate OnMessageReceived);

	/**
	 * Remove a route from the dispatch table.
	 * @param MessageName the name of the message handler to remove.
	 */
	void UnbindRoute(const FString& MessageName);

	/**
	 * Adds a Preprocessor called before the Dispatch of the Route
	 * @param WebsocketPreprocessor The Preprocessor Function to register 
	 * @note: If the predispatch returns false, the actual request will not be executed.
	 */
	void AddPreDispatch(TFunction<bool(const struct FRemoteControlWebSocketMessage& Message)> WebsocketPreprocessor);

	/**
	 * Attempt to dispatch a WebSocket message if its PreDispatch passes.
	 */
	void AttemptDispatch(const struct FRemoteControlWebSocketMessage& Message);

private:
	/** Preprocessors for the Dispatch Function */
	TArray<TFunction<bool(const struct FRemoteControlWebSocketMessage& Message)>> DispatchPreProcessor;
	
	/**
	 * Check if the Message to be dispatched can be executed
	 * @param MessageName the name of the message to dispatch, used to find its handler
	 * @param TCHARMessage the payload to dispatch.
	 * @return true if can be dispatched, false if not
	 */
	bool PreDispatch(const struct FRemoteControlWebSocketMessage& Message) const;
	
	/**
	 * Invoke the handler bound to a message name.
	 * @param MessageName the name of the message to dispatch, used to find its handler.
	 * @param TCHARMessage the payload to dispatch.
	 */
	void Dispatch(const struct FRemoteControlWebSocketMessage& Message);

private:
	/** The dispatch table used to keep track of message handlers. */
	TMap<FString, FWebSocketMessageDelegate> DispatchTable;

	friend class FRCWebSocketServer;
};

/**
 * WebSocket server that allows handling and sending WebSocket messages.
 */
class FRCWebSocketServer
{
public:

	FRCWebSocketServer() = default;
	~FRCWebSocketServer();

	/**
	 * Start listening for WebSocket messages.
	 * @param Port the port to listen on.
	 * @param InRouter the router used to dispatch received messages.
	 * @return whether the server was successfully started. 
	 */
	bool Start(uint32 Port, TSharedPtr<FWebsocketMessageRouter> InRouter);
	
	/**
	 * Stop listening for WebSocket messages.
	 */
	void Stop();

	/**
	 * Send a message to all clients currently connected to the server.
	 * @param InUTF8Payload the payload to broadcast to connected clients.
	 */
	void Broadcast(const TArray<uint8>& InUTF8Payload);

	/**
	 * Send a message to a client.
	 * @param InTargetClientId the target client's id.
	 * @param InUTF8Payload the payload to send.
	 */
	void Send(const FGuid& InTargetClientId, const TArray<uint8>& InUTF8Payload);

	/** Returns whether the server is currently listening for messages. */
	bool IsRunning() const;

	/** Callback when a socket is opened */
	FOnWebSocketConnectionOpened& OnConnectionOpened() { return OnConnectionOpenedDelegate; }
	
	/** Callback when a socket is closed */
	FOnWebSocketConnectionClosed& OnConnectionClosed() { return OnConnectionClosedDelegate; }

	/** Set the compression mode for a client by its GUID. */
	void SetClientCompressionMode(const FGuid& ClientId, ERCWebSocketCompressionMode Mode);

private:
	class FWebSocketConnection;

	bool Tick(float DeltaTime);

	/** Handles a new client connecting. */
	void OnWebSocketClientConnected(INetworkingWebSocket* Socket);

	/** Handles sending the received packet to the message router. */
	void ReceivedRawPacket(void* Data, int32 Size, FGuid ClientId, TSharedPtr<FInternetAddr> PeerAddress);

	void OnSocketClose(INetworkingWebSocket* Socket);

	/** Given a client ID, find the corresponding client, or null if it's not connected to this server. */
	FWebSocketConnection* GetClientById(const FGuid& Id);

	/** Send a message on a specific connection */
	void SendOnConnection(FWebSocketConnection& Connection, const TArray<uint8>& InUTF8Payload);
	
	/** Handle rejecting the websocket connection if it doesn't respect the user's CORS policy. */
	EWebsocketConnectionFilterResult FilterConnection(FString OriginHeader, FString ClientIP) const;

	/** Returns whether the port is available. */
	bool IsPortAvailable(uint32 Port) const;

private:
	/** Holds a web socket connection to a client. */
	class FWebSocketConnection
	{
	public:

		explicit FWebSocketConnection(INetworkingWebSocket* InSocket)
			: Socket(InSocket)
			, Id(FGuid::NewGuid())
		{
			constexpr bool bAppendPort = false;
			if (ISocketSubsystem* SocketSubsystem = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
			{
				PeerAddress = SocketSubsystem->GetAddressFromString(InSocket->RemoteEndPoint(bAppendPort));
			}
		}

		FWebSocketConnection(FWebSocketConnection&& WebSocketConnection)
			: Id(WebSocketConnection.Id)
		{
			Socket = WebSocketConnection.Socket;
			PeerAddress = WebSocketConnection.PeerAddress;
			WebSocketConnection.Socket = nullptr;
		}

		~FWebSocketConnection()
		{
			if (Socket)
			{
				delete Socket;
				Socket = nullptr;
			}
		}

		FWebSocketConnection(const FWebSocketConnection&) = delete;
		FWebSocketConnection& operator=(const FWebSocketConnection&) = delete;
		FWebSocketConnection& operator=(FWebSocketConnection&&) = delete;

		/** Underlying WebSocket. */
		INetworkingWebSocket* Socket = nullptr;

		/** Generated ID for this client. */
		FGuid Id;

		/** IP Address of the client. */
		TSharedPtr<FInternetAddr> PeerAddress;
		
		/** Compression mode to use for this client. */
		ERCWebSocketCompressionMode CompressionMode = ERCWebSocketCompressionMode::NONE;
	};

private:
	/** Handle to the tick delegate. */
	FTSTicker::FDelegateHandle TickerHandle;
 
	/** Holds the LibWebSocket wrapper. */
	TUniquePtr<IWebSocketServer> Server;

	/** Holds all active connections. */
	TArray<FWebSocketConnection> Connections;

	/** Holds the router responsible for dispatching messages received by this server. */
	TSharedPtr<FWebsocketMessageRouter> Router;

	/** Delegate triggered when a connection is opened */
	FOnWebSocketConnectionOpened OnConnectionOpenedDelegate;

	/** Delegate triggered when a connection is closed */
	FOnWebSocketConnectionClosed OnConnectionClosedDelegate;
};
