// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCJsonStructDeserializerBackend.h"
#include "IRemoteControlModule.h"
#include "UObject/EnumProperty.h"
#include "UObject/UnrealType.h"

#include <limits>

namespace
{
	static void* GetPropertyValuePtr(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
	{
		check(Property);

		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(Outer))
		{
			if (ArrayProperty->Inner != Property)
			{
				return nullptr;
			}

			FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, Data);
			int32 Index = ArrayHelper.AddValue();

			return ArrayHelper.GetRawPtr(Index);
		}
		else if (Outer && (Outer->IsA<FMapProperty>() || Outer->IsA<FSetProperty>()))
		{
			if (Property->IsA<FEnumProperty>())
			{
				ensureAlwaysMsgf(false, TEXT("Set or Map property containing enums is not supported by Remote Control."));
				return nullptr;
			}
		}

		if (ArrayIndex >= Property->ArrayDim)
		{
			return nullptr;
		}

		return Property->ContainerPtrToValuePtr<void>(Data, ArrayIndex);
	}

	UEnum* GetEnum(FProperty* Property)
	{
		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			return ByteProperty->Enum;
		}
		else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			return EnumProperty->GetEnum();
		}
		else
		{
			return nullptr;
		}
	}

	int32 GetEnumIndexByDisplayName(const FString& DisplayName, UEnum* Enum)
	{
		int32 EnumIndex = INDEX_NONE;
		// NumEnums - 1 to avoid MAX_VALUE
		for (int32 Index = 0; Index < Enum->NumEnums() - 1; Index++)
		{
			if (DisplayName == Enum->GetDisplayNameTextByIndex(Index).ToString())
			{
				EnumIndex = Index;
				break;
			}
		}

		if (EnumIndex == INDEX_NONE)
		{
			EnumIndex = Enum->GetIndexByNameString(DisplayName);
		}

		return EnumIndex;
	}

	bool SetEnumValueByIndex(void* ValuePtr, FProperty* Property, UEnum* Enum, const int32 Index)
	{
		if (FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			ByteProperty->SetPropertyValue(ValuePtr, (uint8)Enum->GetValueByIndex(Index));
		}
		else if (FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			EnumProperty->GetUnderlyingProperty()->SetIntPropertyValue(ValuePtr, Enum->GetValueByIndex(Index));
		}
		else
		{
			checkNoEntry();
		}

		return true;
	}

	bool ReadEnum(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex, const FString& StringValue)
	{
		UEnum* Enum = GetEnum(Property);

		if (ensure(Enum))
		{
			const int32 EnumIndex = GetEnumIndexByDisplayName(StringValue, Enum);

			if (EnumIndex != INDEX_NONE)
			{
				if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
				{
					return SetEnumValueByIndex(Ptr, Property, Enum, EnumIndex);
				}
			}
		}
		return false;
	}

	template<typename PropertyType>
	static bool HandleObjectProperty(PropertyType* ObjectProperty, FProperty* Outer, void* Data, int32 ArrayIndex, const FString& StringValue)
	{
		constexpr UObject* ObjectOuter = nullptr;
		constexpr TCHAR* Filename = nullptr;
		UObject* LoadedObject = StaticLoadObject(ObjectProperty->PropertyClass, ObjectOuter, *StringValue, Filename, LOAD_NoWarn);
		if (LoadedObject == nullptr)
		{
			UE_LOGF(LogRemoteControl, Warning, "Deserialization error: Could not load object %ls for property %ls.", *StringValue, *ObjectProperty->GetName());
		}

		if (void* Ptr = GetPropertyValuePtr(ObjectProperty, Outer, Data, ArrayIndex))
		{
			ObjectProperty->SetPropertyValue(Ptr, typename PropertyType::TCppType(LoadedObject));
			return true;
		}

		return false;
	}
}

bool FRCJsonStructDeserializerBackend::ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex)
{
	//Whether we handled the property read or we need to user parent class version
	TOptional<bool> HandledResult;

	if (GetLastNotation() == EJsonNotation::String)
	{
		const FString& StringValue = GetReader()->GetValueAsString();

		if (Property->IsA<FByteProperty>() || Property->IsA<FEnumProperty>())
		{
			HandledResult = ReadEnum(Property, Outer, Data, ArrayIndex, StringValue);
			if (HandledResult.GetValue() == false)
			{
				UE_LOGF(LogRemoteControl, Error, "Deserialization error: %ls does not contain value %ls", *Property->GetName(), *StringValue);
			}
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if (IsInGameThread())
			{
				HandledResult = HandleObjectProperty(ObjectProperty, Outer, Data, ArrayIndex, StringValue);
			}
			else
			{
				UE_LOGF(LogRemoteControl, Verbose, "ObjectProperty: Deserializing RemoteControl payload from another thread. Can't preload object %ls in property %ls.", *StringValue, *Property->GetName());
			}
		}
		else if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
		{
			if (IsInGameThread())
			{
				HandledResult = HandleObjectProperty(WeakObjectProperty, Outer, Data, ArrayIndex, StringValue);
			}
			else
			{
				UE_LOGF(LogRemoteControl, Verbose, "WeakObjectProperty: Deserializing RemoteControl payload from another thread. Can't preload object %ls in property %ls.", *StringValue, *Property->GetName());
			}
		}
	}
	else if (GetLastNotation() == EJsonNotation::Number)
	{
		if (FFloatProperty* FloatProperty = CastField<FFloatProperty>(Property))
		{
			if (void* Ptr = GetPropertyValuePtr(Property, Outer, Data, ArrayIndex))
			{
				const double& DoubleValue = GetReader()->GetValueAsNumber();
				if (DoubleValue > std::numeric_limits<float>::max())
				{
					FloatProperty->SetPropertyValue(Ptr, std::numeric_limits<float>::max());
					HandledResult = true;
				}
			}
			else
			{
				HandledResult = false;
			}
		}
	}

	if (HandledResult.IsSet() == false)
	{
		return FJsonStructDeserializerBackend::ReadProperty(Property, Outer, Data, ArrayIndex);
	}
	else
	{
		return HandledResult.GetValue();
	}
}
