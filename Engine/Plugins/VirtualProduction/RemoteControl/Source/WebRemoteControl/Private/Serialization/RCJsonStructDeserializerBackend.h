// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Backends/JsonStructDeserializerBackend.h"

/**
 * Implements a custom reader for UStruct serialization using Json.
 * Tries to read enum values using the display name.
 */
class FRCJsonStructDeserializerBackend
	: public FJsonStructDeserializerBackend
{
public:

	/**
	 * Initialize the deserializer
	 * @param InArchive The archive to deserialize from.
	 */
	FRCJsonStructDeserializerBackend(FArchive& InArchive)
		: FJsonStructDeserializerBackend(InArchive)
	{ }


	// IStructDeserializerBackend interface
	virtual bool ReadProperty(FProperty* Property, FProperty* Outer, void* Data, int32 ArrayIndex) override;
};
