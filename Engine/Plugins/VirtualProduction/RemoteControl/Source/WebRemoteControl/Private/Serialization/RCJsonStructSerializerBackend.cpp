// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCJsonStructSerializerBackend.h"
#include "UObject/EnumProperty.h"
#include <limits>

void FRCJsonStructSerializerBackend::WriteProperty(const FStructSerializerState& State, int32 ArrayIndex /*= 0*/)
{
	if (State.FieldType == FByteProperty::StaticClass())
	{
		FByteProperty* ByteProperty = CastFieldChecked<FByteProperty>(State.ValueProperty);

		if (ByteProperty->IsEnum())
		{
			const uint8 PropertyValue = ByteProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
			const FText DisplayName = ByteProperty->Enum->GetDisplayNameTextByValue(PropertyValue);
			WritePropertyValue(State, DisplayName.ToString());
			return;
		}
	}
	else if (State.FieldType == FEnumProperty::StaticClass())
	{
		FEnumProperty* EnumProperty = CastFieldChecked<FEnumProperty>(State.ValueProperty);
		const void* PropertyValuePtr = EnumProperty->ContainerPtrToValuePtr<void>(State.ValueData, ArrayIndex);
		const int64 Value = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(PropertyValuePtr);
		const FText DisplayName = EnumProperty->GetEnum()->GetDisplayNameTextByValue(Value);
		WritePropertyValue(State, DisplayName.ToString());
		return;
	}
	else if (State.FieldType == FFloatProperty::StaticClass())
	{
		FFloatProperty* FloatProperty = CastFieldChecked<FFloatProperty>(State.ValueProperty);
		const float Value = FloatProperty->GetPropertyValue_InContainer(State.ValueData, ArrayIndex);
		if (Value >= std::numeric_limits<float>::max())
		{
			WritePropertyValue(State, std::numeric_limits<float>::max());
		}
		else
		{
			WritePropertyValue(State, Value);
		}
		return;
	}

	FJsonStructSerializerBackend::WriteProperty(State, ArrayIndex);
}
