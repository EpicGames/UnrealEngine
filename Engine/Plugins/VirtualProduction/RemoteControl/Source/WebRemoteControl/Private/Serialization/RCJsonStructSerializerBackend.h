// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Backends/JsonStructSerializerBackend.h"

/**
 * Implements a custom writer for UStruct serialization using Json.
 * Tries to serialize enum values with their display name.
 */
class FRCJsonStructSerializerBackend
	: public FJsonStructSerializerBackend
{
public:
	/** Default flags for serializing structures in Web RC. */
	static const EStructSerializerBackendFlags DefaultSerializerFlags = EStructSerializerBackendFlags::WriteByteArrayAsByteStream;

	/**
	 * Creates and initializes a new instance with the given flags.
	 *
	 * @param InArchive The archive to serialize into.
	 * @param InFlags The flags that control the serialization behavior (typically FRCJsonStructSerializerBackend::DefaultSerializerFlags).
	 */
	FRCJsonStructSerializerBackend(FArchive& InArchive, const EStructSerializerBackendFlags InFlags = DefaultSerializerFlags)
		: FJsonStructSerializerBackend(InArchive, InFlags)
	{ }

	// IStructSerializerBackend interface
	virtual void WriteProperty(const FStructSerializerState& State, int32 ArrayIndex = 0) override;
};
