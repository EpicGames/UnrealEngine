// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRemoteControl.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

#include "IRemoteControlModule.h"
#include "RCVirtualPropertyContainer.h"
#include "RCVirtualProperty.h"
#include "RemoteControlDefaultPreprocessors.h"
#include "RemoteControlReflectionUtils.h"
#include "RemoteControlRoute.h"
#include "RemoteControlSettings.h"
#include "RemoteControlPreset.h"
#include "RemoteControlWebsocketRoute.h"
#include "WebRemoteControlInternalUtils.h"
#include "WebRemoteControlExternalLogger.h"
#include "WebRemoteControlUtils.h"
#include "WebSocketMessageHandler.h"

#if WITH_EDITOR
// Settings
#include "Editor.h"
#include "IRemoteControlUIModule.h"
#include "Framework/Notifications/NotificationManager.h"
#include "ScopedTransaction.h"
#include "Widgets/Notifications/SNotificationList.h"
#endif

// Serialization
#include "Serialization/RCJsonStructDeserializerBackend.h"
#include "Serialization/RCJsonStructSerializerBackend.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

// Http server
#include "HttpServerModule.h"
#include "IHttpRouter.h"
#include "HttpServerRequest.h"
#include "HttpRequestHandler.h"
#include "HttpServerConstants.h"

// Commands
#include "HAL/IConsoleManager.h"
#include "UObject/StructOnScope.h"

// Requests, Models, Responses
#include "RemoteControlRequest.h"
#include "RemoteControlResponse.h"
#include "RemoteControlModels.h"

// Asset registry
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"

// Miscelleanous
#include "Blueprint/BlueprintSupport.h"
#include "Misc/App.h"
#include "Misc/WildcardString.h"
#include "UObject/UnrealType.h"
#include "Templates/UnrealTemplate.h"

#define LOCTEXT_NAMESPACE "WebRemoteControl"

// Boot the server on startup flag
static TAutoConsoleVariable<int32> CVarWebControlStartOnBoot(TEXT("WebControl.EnableServerOnStartup"), 0, TEXT("Enable the Web Control servers (web and websocket) on startup."));

// Enable experimental remote routes
static TAutoConsoleVariable<int32> CVarWebControlEnableExperimentalRoutes(TEXT("WebControl.EnableExperimentalRoutes"), 0, TEXT("Enable the Web Control server experimental routes."));

namespace WebRemoteControlStructUtils
{
	FName Struct_PropertyValue = "WebRCPropertyValue";
	FName Prop_ObjectPath = "ObjectPath";
	FName Prop_PropertyValue = "PropertyValue";

	FName Struct_GetPropertyResponse = "WebRCGetPropertyResponse";
	FName Prop_PropertyValues = "PropertyValues";
	FName Prop_ExposedPropertyDescription = "ExposedPropertyDescription";
	
	UScriptStruct* CreatePropertyValueContainer(FProperty* InValueSrcProperty)
	{
		check(InValueSrcProperty);

		static FGuid PropertyValueGuid = FGuid::NewGuid();

		FWebRCGenerateStructArgs Args;
		Args.GenericProperties.Emplace(Prop_PropertyValue, InValueSrcProperty);
		Args.StringProperties.Add(Prop_ObjectPath);
		
		const FString StructName = FString::Format(TEXT("{0}_{1}_{2}_{3}"), { *Struct_PropertyValue.ToString(), *InValueSrcProperty->GetClass()->GetName(), *InValueSrcProperty->GetName(), PropertyValueGuid.ToString() });
		return UE::WebRCReflectionUtils::GenerateStruct(*StructName, Args);
	}

	UScriptStruct* CreateActorPropertyValueContainer(FProperty* InValueSrcProperty)
	{
		check(InValueSrcProperty);

		static FGuid ActorPropertyValueGuid = FGuid::NewGuid();

		FWebRCGenerateStructArgs Args;
		Args.GenericProperties.Emplace(Prop_PropertyValue, InValueSrcProperty);

		const FString StructName = FString::Format(TEXT("{0}_{1}_{2}_{4}"), { *Struct_PropertyValue.ToString(), *InValueSrcProperty->GetClass()->GetName(), *InValueSrcProperty->GetName(), ActorPropertyValueGuid.ToString() });
		return UE::WebRCReflectionUtils::GenerateStruct(*StructName, Args);
	}

	FStructOnScope CreatePropertyValueOnScope(const TSharedPtr<FRemoteControlProperty>& RCProperty, const FRCObjectReference& ObjectReference)
	{
		check(ObjectReference.IsValid());

		UScriptStruct* Struct = CreatePropertyValueContainer(ObjectReference.Property.Get());
		FStructOnScope StructOnScope{ Struct };

		UE::WebRCReflectionUtils::SetStringPropertyValue(Prop_ObjectPath, StructOnScope, ObjectReference.Object->GetPathName());
		UE::WebRCReflectionUtils::CopyPropertyValue(Prop_PropertyValue, StructOnScope, ObjectReference);

		return StructOnScope;
	}

	UScriptStruct* CreateGetPropertyResponseStruct(UScriptStruct* CustomContainer)
	{
		check(CustomContainer);

		FWebRCGenerateStructArgs Args;
		Args.ArrayProperties.Emplace(Prop_PropertyValues, CustomContainer);
		Args.StructProperties.Emplace(Prop_ExposedPropertyDescription, FRCExposedPropertyDescription::StaticStruct());
		
		const FString StructName = FString::Format(TEXT("{0}_{1}"), { *Struct_GetPropertyResponse.ToString(), *CustomContainer->GetFName().ToString()});
		return UE::WebRCReflectionUtils::GenerateStruct(*StructName, Args);
	}

	FStructOnScope CreateGetPropertyOnScope(const TSharedPtr<FRemoteControlProperty>& RCProperty, const FRCObjectReference& ObjectReference, FStructOnScope&& PropertyValueOnScope)
	{
		check(ObjectReference.IsValid() && RCProperty);

		UScriptStruct* TopLevelStruct = CreateGetPropertyResponseStruct((UScriptStruct*)PropertyValueOnScope.GetStruct());
		FStructOnScope TopLevelOnScope{ TopLevelStruct };

		FRCExposedPropertyDescription PropertyDescription{ *RCProperty };
		UE::WebRCReflectionUtils::SetStructPropertyValue(Prop_ExposedPropertyDescription, TopLevelOnScope, FRCExposedPropertyDescription::StaticStruct(), &PropertyDescription);

		TArray<FStructOnScope> PropertyValueArray {};
		PropertyValueArray.Add(MoveTemp(PropertyValueOnScope));
		UE::WebRCReflectionUtils::SetStructArrayPropertyValue(Prop_PropertyValues, TopLevelOnScope, PropertyValueArray);

		return TopLevelOnScope;
	}

	FStructOnScope CreateActorPropertyOnScope(const FRCObjectReference& ObjectReference)
	{
		check(ObjectReference.IsValid());

		UScriptStruct* Struct = CreateActorPropertyValueContainer(ObjectReference.Property.Get());

		FStructOnScope StructOnScope{ Struct };
		UE::WebRCReflectionUtils::CopyPropertyValue(Prop_PropertyValue, StructOnScope, ObjectReference);

		return StructOnScope;
	}
}

namespace WebRemoteControl
{
	template <typename EntityType>
	TSharedPtr<EntityType> GetRCEntity(URemoteControlPreset* Preset, FString PropertyLabelOrId)
	{
		if (!Preset)
		{
			return nullptr;
		}

		FGuid Id;
		if (FGuid::ParseExact(PropertyLabelOrId, EGuidFormats::Digits, Id))
		{
			if (TSharedPtr<EntityType> Entity = Preset->GetExposedEntity<EntityType>(Id).Pin())
			{
        		return Entity;
			}
		}

		return Preset->GetExposedEntity<EntityType>(Preset->GetExposedEntityId(*PropertyLabelOrId)).Pin();
	}

	URCVirtualPropertyBase* GetController(URemoteControlPreset* Preset, FString PropertyLabelOrId)
	{
		if (!Preset)
		{
			return nullptr;
		}
		
		FGuid Id;
		if (FGuid::ParseExact(PropertyLabelOrId, EGuidFormats::Digits, Id))
		{
			if (URCVirtualPropertyBase* Controller = Preset->GetController(Id))
			{
        		return Controller;
			}
		}

		// Not supporting label, in case Exposed Property and Controller have the same name.
		return nullptr;
	}

	URemoteControlPreset* GetPreset(FString PresetNameOrId)
	{
		FGuid Id;
		if (FGuid::ParseExact(PresetNameOrId, EGuidFormats::Digits, Id))
		{
			if (URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(Id))
			{
				return ResolvedPreset;
			}
		}

		return IRemoteControlModule::Get().ResolvePreset(*PresetNameOrId);
	}
	
	bool IsWebControlEnabledInEditor()
	{
		bool bIsEditor = false;

#if WITH_EDITOR
		bIsEditor = GIsEditor;
#endif

		// By default, web remote control is disabled in -game and packaged game.
		return bIsEditor || FParse::Param(FCommandLine::Get(), TEXT("RCWebControlEnable"));
	}

	void DoNativeClassBlueprintFilter(const FSearchAssetRequest& SearchAssetRequest, TArray<FAssetData>& OutFilteredAssets)
	{
		if (SearchAssetRequest.Filter.EnableBlueprintNativeClassFiltering && SearchAssetRequest.Filter.NativeParentClasses.Num() > 0)
		{
			TSet<UClass*> NativeClasses;
			NativeClasses.Reserve(SearchAssetRequest.Filter.NativeParentClasses.Num());
			for (FName NativeClassName : SearchAssetRequest.Filter.NativeParentClasses)
			{
				if (UClass* Class = FindObject<UClass>(FTopLevelAssetPath{ NativeClassName.ToString()}))
				{
					NativeClasses.Add(Class);
				}
			}

			for (auto It = OutFilteredAssets.CreateIterator(); It; ++It)
			{
				const FAssetData& AssetData = *It;
				const FString NativeParentClassPath = AssetData.GetTagValueRef<FString>(FBlueprintTags::NativeParentClassPath);
				const FSoftClassPath ClassPath(NativeParentClassPath);
				UClass* NativeParentClass = ClassPath.ResolveClass();
				if (NativeParentClass)
				{
					for (UClass* ClassFilter : NativeClasses)
					{
						if (!NativeParentClass->IsChildOf(ClassFilter))
						{
							It.RemoveCurrent();
						}
					}
				}
				else
				{
					It.RemoveCurrent();
				}
			}
		}
	}

	/**
	 * Resolve the object referenced in a WebRemoteControl request.
	 * Returns true if the object was resolved, or else returns false and calls OnComplete with an error response.
	 */
	bool ResolveObjectPropertyForRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete, TUniquePtr<FHttpServerResponse>& InResponse,
		FRCObjectReference& OutObjectRef, FRCObjectRequest& OutDeserializedRequest)
	{
		if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
		{
			return false;
		}

		if (!RemotePayloadSerializer::DeserializeObjectRef(Request, OutObjectRef, OutDeserializedRequest, OnComplete))
		{
			return false;
		}

		// If we haven't found the object, return a not found error code
		if (!OutObjectRef.IsValid())
		{
			InResponse->Code = EHttpServerResponseCodes::NotFound;
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to find the object."), InResponse->Body);
			OnComplete(MoveTemp(InResponse));
			return false;
		}

		return true;
	}

	/**
	 * Create a memory reader for the deserialized property data contained in a request to add an element to an array.
	 */
	FMemoryReader MakePropertyMemoryReaderForArrayAddRequest(FRCObjectRequest& InDeserializedRequest)
	{
		const FBlockDelimiters& PropertyValueDelimiters = InDeserializedRequest.GetStructParameters().FindChecked(FRCObjectRequest::PropertyValueLabel());
		FMemoryReader Reader(InDeserializedRequest.TCHARBody);
		Reader.Seek(PropertyValueDelimiters.BlockStart);
		Reader.SetLimitSize(PropertyValueDelimiters.BlockEnd);

		return MoveTemp(Reader);
	}

	/**
	 * Retrieve the integer "index" query parameter from a request, or if unable to, call OnComplete with an error response and return false.
	 */
	bool GetIndexQueryParamForRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete, TUniquePtr<FHttpServerResponse>& InResponse, int32& OutIndex)
	{
		const FString* IndexParam = Request.QueryParams.Find(TEXT("index"));
		if (!IndexParam) {
			InResponse->Code = EHttpServerResponseCodes::BadRequest;
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("No 'index' query parameter was provided."), InResponse->Body);
			OnComplete(MoveTemp(InResponse));
			return false;
		}

		OutIndex = FCString::Atoi(**IndexParam);
		return true;
	}
}

void FWebRemoteControlModule::StartupModule()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("RCWebControlDisable")) || !FApp::CanEverRender())
	{
		return;
	}

	// By default, disable web remote control in -game, packaged game and on build machines
	if (!WebRemoteControl::IsWebControlEnabledInEditor() || GIsBuildMachine)
	{
		UE_LOGF(LogRemoteControl, Display, "Web remote control is disabled by default when running outside the editor. Use the -RCWebControlEnable flag when launching in order to use it.");
		return;
	}

#if WITH_EDITOR
	RegisterSettings();
#endif

	WebSocketRouter = MakeShared<FWebsocketMessageRouter>();

	HttpServerPort = GetDefault<URemoteControlSettings>()->RemoteControlHttpServerPort;
	WebSocketServerPort = GetDefault<URemoteControlSettings>()->RemoteControlWebSocketServerPort;
	WebsocketServerBindAddress = GetDefault<URemoteControlSettings>()->RemoteControlWebsocketServerBindAddress;

	WebSocketHandler = MakeUnique<FWebSocketMessageHandler>(&WebSocketServer, ActingClientId);

	RegisterConsoleCommands();
	RegisterRoutes();

	if (GetDefault<URemoteControlSettings>()->bAutoStartWebServer || CVarWebControlStartOnBoot.GetValueOnAnyThread() > 0)
	{
		StartHttpServer();
	}

	if (GetDefault<URemoteControlSettings>()->bAutoStartWebSocketServer || CVarWebControlStartOnBoot.GetValueOnAnyThread() > 0)
	{
		StartWebSocketServer();
	}

	RegisterDefaultPreprocessors();
	RegisterExternalPreprocesors();
}

void FWebRemoteControlModule::ShutdownModule()
{
	if (FParse::Param(FCommandLine::Get(), TEXT("RCWebControlDisable")) || !FApp::CanEverRender())
	{
		return;
	}

	// By default, web remote control is disabled in -game and packaged game.
	if (!WebRemoteControl::IsWebControlEnabledInEditor())
	{
		return;
	}

	EditorRoutes.UnregisterRoutes(this);
	if (WebSocketHandler)
	{
		WebSocketHandler->UnregisterRoutes(this);
	}
	StopHttpServer();
	StopWebSocketServer();
	UnregisterConsoleCommands();

#if WITH_EDITOR
	UnregisterSettings();
#endif
}

FDelegateHandle FWebRemoteControlModule::RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor)
{
	FDelegateHandle WebRCHandle{FDelegateHandle::GenerateNewHandle};
	
	PreprocessorsToRegister.Add(WebRCHandle, RequestPreprocessor);
	
	FDelegateHandle HttpRouterHandle;
	if (HttpRouter)
	{
		HttpRouterHandle = HttpRouter->RegisterRequestPreprocessor(MoveTemp(RequestPreprocessor));
	}

	PreprocessorsHandleMappings.Add(WebRCHandle, HttpRouterHandle);
	
	return WebRCHandle;
}

void FWebRemoteControlModule::UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle)
{
	PreprocessorsToRegister.Remove(RequestPreprocessorHandle);
	if (FDelegateHandle* HttpRouterHandle = PreprocessorsHandleMappings.Find(RequestPreprocessorHandle))
	{
		if (HttpRouterHandle->IsValid())
		{
			HttpRouter->UnregisterRequestPreprocessor(*HttpRouterHandle);
		}
		
		PreprocessorsHandleMappings.Remove(RequestPreprocessorHandle);
	}
}

void FWebRemoteControlModule::RegisterRoute(const FRemoteControlRoute& Route)
{
	RegisteredHttpRoutes.Add(Route);

	// If the route is registered after the server is already started.
	if (HttpRouter)
	{
		StartRoute(Route);
	}
}

void FWebRemoteControlModule::UnregisterRoute(const FRemoteControlRoute& Route)
{
	RegisteredHttpRoutes.Remove(Route);
	const uint32 RouteHash = GetTypeHash(Route);
	if (FHttpRouteHandle* Handle = ActiveRouteHandles.Find(RouteHash))
	{
		if (HttpRouter)
		{
			HttpRouter->UnbindRoute(*Handle);
		}
		ActiveRouteHandles.Remove(RouteHash);
	}
}

void FWebRemoteControlModule::RegisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route)
{
	RegisteredWebSocketRoutes.Add(Route);

	if (WebSocketRouter)
	{
		WebSocketRouter->BindRoute(Route.MessageName, Route.Delegate);
	}
}

void FWebRemoteControlModule::UnregisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route)
{
	RegisteredWebSocketRoutes.Remove(Route);

	if (WebSocketRouter)
	{
		WebSocketRouter->UnbindRoute(Route.MessageName);
	}
}

void FWebRemoteControlModule::SendWebsocketMessage(const FGuid& InTargetClientId, const TArray<uint8>& InUTF8Payload)
{
	WebSocketServer.Send(InTargetClientId, InUTF8Payload);
}

void FWebRemoteControlModule::StartHttpServer()
{
	if (!HttpRouter)
	{
		HttpRouter = FHttpServerModule::Get().GetHttpRouter(HttpServerPort, /* bFailOnBindFailure = */ true);
		if (!HttpRouter)
		{
			UE_LOGF(LogRemoteControl, Error, "Web Remote Call server couldn't be started on port %d", HttpServerPort);
			return;
		}

		for (FRemoteControlRoute& Route : RegisteredHttpRoutes)
		{
			StartRoute(Route);
		}
		
		FHttpServerModule::Get().StartAllListeners();

		bIsHttpServerRunning = true;
		OnHttpServerStartedDelegate.Broadcast(HttpServerPort);
	}
}

void FWebRemoteControlModule::StopHttpServer()
{
	if (FHttpServerModule::IsAvailable())
	{
		FHttpServerModule::Get().StopAllListeners();
	}

	if (HttpRouter)
	{
		for (const TPair<uint32, FHttpRouteHandle>& Tuple : ActiveRouteHandles)
		{
			if (Tuple.Key)
			{
				HttpRouter->UnbindRoute(Tuple.Value);
			}
		}

		ActiveRouteHandles.Reset();
	}

	UnregisterAllPreprocessors();

	HttpRouter.Reset();
	bIsHttpServerRunning = false;

	OnHttpServerStoppedDelegate.Broadcast();
}

void FWebRemoteControlModule::StartWebSocketServer()
{
	if (!WebSocketServer.IsRunning())
	{
		if (!WebSocketServer.Start(WebSocketServerPort, WebSocketRouter))
		{
			UE_LOGF(LogRemoteControl, Error, "Web Remote Call WebSocket server couldn't be started on port %d", WebSocketServerPort);
#if WITH_EDITOR
			FNotificationInfo Info{FText::Format(LOCTEXT("FailedStartRemoteControlServer", "Web Remote Call WebSocket server couldn't be started on port {0}"), WebSocketServerPort)};
			Info.bFireAndForget = true;
			Info.ExpireDuration =  3.0f;
			FSlateNotificationManager::Get().AddNotification(MoveTemp(Info));
#endif /*WITH_EDITOR*/

			return;
		}

		UE_LOGF(LogRemoteControl, Log, "Web Remote Control WebSocket server started on port %d", WebSocketServerPort);
		OnWebSocketServerStartedDelegate.Broadcast(WebSocketServerPort);

		for (FRemoteControlWebsocketRoute& Route : RegisteredWebSocketRoutes)
		{
			WebSocketRouter->BindRoute(Route.MessageName, Route.Delegate);
		}
	}
}

void FWebRemoteControlModule::StopWebSocketServer()
{
	WebSocketServer.Stop();
	OnWebSocketServerStoppedDelegate.Broadcast();
}

void FWebRemoteControlModule::SetExternalRemoteWebSocketLoggerConnection(TSharedPtr<INetworkingWebSocket> WebSocketLoggerConnection)
{
	ExternalLogger.Reset();
	
	if (WebSocketLoggerConnection.IsValid())
	{
		ExternalLogger = MakeUnique<FWebRemoteControlExternalLogger>(WebSocketLoggerConnection);
	}
}

void FWebRemoteControlModule::StartRoute(const FRemoteControlRoute& Route)
{
	// The handler is wrapped in a lambda since HttpRouter::BindRoute only accepts TFunctions
	ActiveRouteHandles.Add(GetTypeHash(Route), HttpRouter->BindRoute(Route.Path, Route.Verb, Route.Handler));
}

void FWebRemoteControlModule::RegisterRoutes()
{
	// Misc
	RegisterRoute({
		TEXT("Get information about different routes available on this API."),
		FHttpPath(TEXT("/remote/info")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleInfoRoute)
		});

	RegisterRoute({
		TEXT("Allows cross-origin http requests to the API."),
		FHttpPath(TEXT("/remote")),
		EHttpServerRequestVerbs::VERB_OPTIONS,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleOptionsRoute)
		});

	// Raw API
	RegisterRoute({
		TEXT("Allows batching multiple calls into one request."),
		FHttpPath(TEXT("/remote/batch")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleBatchRequest)
		});

	RegisterRoute({
		TEXT("Call a function on a remote object."),
		FHttpPath(TEXT("/remote/object/call")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleObjectCallRoute)
		});

	RegisterRoute({
		TEXT("Read or write a property on a remote object."),
		FHttpPath(TEXT("/remote/object/property")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleObjectPropertyRoute)
		});

	RegisterRoute({
		TEXT("Append a new item to an array property on a remote object."),
		FHttpPath(TEXT("/remote/object/property/append")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleObjectPropertyAppendRoute)
		});

	RegisterRoute({
		TEXT("Insert a new item into an array property on a remote object."),
		FHttpPath(TEXT("/remote/object/property/insert")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleObjectPropertyInsertRoute)
		});

	RegisterRoute({
		TEXT("Remove an item from an array property on a remote object."),
		FHttpPath(TEXT("/remote/object/property/remove")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleObjectPropertyRemoveRoute)
		});

	RegisterRoute({
		TEXT("Describe an object."),
		FHttpPath(TEXT("/remote/object/describe")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleDescribeObjectRoute)
		});

	// Passphrase Checking
	RegisterRoute({
		TEXT("Check whether or no the given Passphrase is correct"),
		FHttpPath(TEXT("/remote/passphrase/")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePassphraseRoute)
		});

	// Preset API
	RegisterRoute({
		TEXT("Get a remote control preset's content."),
		FHttpPath(TEXT("/remote/preset/:preset")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleGetPresetRoute)
		});

	RegisterRoute({
		TEXT("Get a list of available remote control presets."),
		FHttpPath(TEXT("/remote/presets")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleGetPresetsRoute)
		});

	RegisterRoute({
		TEXT("Call a function on a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/function/:functionname")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetCallFunctionRoute)
		});

	RegisterRoute({
		TEXT("Set a property on a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/property/:propertyname")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetSetPropertyRoute)
		});

	RegisterRoute({
		TEXT("Get a property on a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/property/:propertyname")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetGetPropertyRoute)
		});

	RegisterRoute({
		TEXT("Expose a property on a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/expose/property")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetExposePropertyRoute)
		});

	RegisterRoute({
		TEXT("Unexpose a property on a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/unexpose/property/:property")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetUnexposePropertyRoute)
		});

	RegisterRoute({
		TEXT("Get an exposed actor's properties."),
		FHttpPath(TEXT("/remote/preset/:preset/actor/:actor")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetGetExposedActorPropertiesRoute)
		});

	RegisterRoute({
		TEXT("Get an exposed actor's property."),
		FHttpPath(TEXT("/remote/preset/:preset/actor/:actor/property/:propertyname")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetGetExposedActorPropertyRoute)
		});

	RegisterRoute({
		TEXT("Set an exposed actor's property."),
		FHttpPath(TEXT("/remote/preset/:preset/actor/:actor/property/:propertyname")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetSetExposedActorPropertyRoute)
		});

	RegisterRoute({
		TEXT("Get a preset"),
		FHttpPath(TEXT("/remote/preset/:preset")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleGetPresetRoute)
		});
	
	// Search
	RegisterRoute({
		TEXT("Search for assets"),
		FHttpPath(TEXT("/remote/search/assets")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleSearchAssetRoute)
		});

	// Metadata
	RegisterRoute({
		TEXT("Get a preset's metadata"),
		FHttpPath(TEXT("/remote/preset/:preset/metadata")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleGetMetadataRoute)
		});

	RegisterRoute({
		TEXT("Get/Set/Delete a preset metadata field"),
		FHttpPath(TEXT("/remote/preset/:preset/metadata/:metadatafield")),
		EHttpServerRequestVerbs::VERB_GET | EHttpServerRequestVerbs::VERB_PUT | EHttpServerRequestVerbs::VERB_DELETE,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleMetadataFieldOperationsRoute)
		});

	RegisterRoute({
	    TEXT("Get/Set/Delete an exposed property's metadata field"),
	    FHttpPath(TEXT("/remote/preset/:preset/property/:label/metadata/:key")),
	    EHttpServerRequestVerbs::VERB_GET | EHttpServerRequestVerbs::VERB_PUT | EHttpServerRequestVerbs::VERB_DELETE,
	    FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntityMetadataOperationsRoute)
    });

	RegisterRoute({
        TEXT("Get/Set/Delete an exposed function's metadata field"),
		FHttpPath(TEXT("/remote/preset/:preset/function/:label/metadata/:key")),
        EHttpServerRequestVerbs::VERB_GET | EHttpServerRequestVerbs::VERB_PUT | EHttpServerRequestVerbs::VERB_DELETE,
        FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntityMetadataOperationsRoute)
    });

	RegisterRoute({
        TEXT("Get/Set/Delete an exposed property's metadata field"),
		FHttpPath(TEXT("/remote/preset/:preset/actor/:label/metadata/:key")),
        EHttpServerRequestVerbs::VERB_GET | EHttpServerRequestVerbs::VERB_PUT | EHttpServerRequestVerbs::VERB_DELETE,
        FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntityMetadataOperationsRoute)
    });

	RegisterRoute({
		TEXT("Set an exposed property's label"),
		FHttpPath(TEXT("/remote/preset/:preset/property/:label/label")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntitySetLabelRoute)
	});
	
	RegisterRoute({
		TEXT("Set an exposed function's label"),
		FHttpPath(TEXT("/remote/preset/:preset/function/:label/label")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntitySetLabelRoute)
	});
	
	RegisterRoute({
		TEXT("Set an exposed actor's label"),
		FHttpPath(TEXT("/remote/preset/:preset/actor/:label/label")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleEntitySetLabelRoute)
	});

	RegisterRoute({
		TEXT("Create a temporary preset which won't be visible in the editor or saved as an asset"),
		FHttpPath(TEXT("/remote/preset/transient")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleCreateTransientPresetRoute)
	});

	RegisterRoute({
		TEXT("Delete a transient preset"),
		FHttpPath(TEXT("/remote/preset/transient/:preset")),
		EHttpServerRequestVerbs::VERB_DELETE,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandleDeleteTransientPresetRoute)
	});

	// Remote Control Logic - Controllers
	RegisterRoute({
	TEXT("Set a controller value on a preset."),
	FHttpPath(TEXT("/remote/preset/:preset/controller/:controller")),
	EHttpServerRequestVerbs::VERB_PUT,
	FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetSetControllerRoute)
		});

	RegisterRoute({
		TEXT("Get a controller value from a preset."),
		FHttpPath(TEXT("/remote/preset/:preset/controller/:controller")),
		EHttpServerRequestVerbs::VERB_GET,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlModule::HandlePresetGetControllerRoute)
		});

	//**************************************
	// Special websocket route just using http request
	RegisterWebsocketRoute({
		TEXT("Route a message that targets a http route."),
		TEXT("http"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebRemoteControlModule::HandleWebSocketHttpMessage)
		});

	RegisterWebsocketRoute({
		TEXT("Batch multiple WebSocket messages into one request"),
		TEXT("batch"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebRemoteControlModule::HandleWebSocketBatchMessage)
		});

	WebSocketHandler->RegisterRoutes(this);

	EditorRoutes.RegisterRoutes(this);
}

void FWebRemoteControlModule::RegisterConsoleCommands()
{
	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("WebControl.StartServer"),
		TEXT("Start the http remote control web server"),
		FConsoleCommandDelegate::CreateRaw(this, &FWebRemoteControlModule::StartHttpServer)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("WebControl.StopServer"),
		TEXT("Stop the http remote control web server"),
		FConsoleCommandDelegate::CreateRaw(this, &FWebRemoteControlModule::StopHttpServer)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("WebControl.StartWebSocketServer"),
		TEXT("Start the WebSocket remote control web server"),
		FConsoleCommandDelegate::CreateRaw(this, &FWebRemoteControlModule::StartWebSocketServer)
		));

	ConsoleCommands.Add(MakeUnique<FAutoConsoleCommand>(
		TEXT("WebControl.StopWebSocketServer"),
		TEXT("Stop the WebSocket remote control web server"),
		FConsoleCommandDelegate::CreateRaw(this, &FWebRemoteControlModule::StopWebSocketServer)
		));
}

void FWebRemoteControlModule::UnregisterConsoleCommands()
{
	for (TUniquePtr<FAutoConsoleCommand>& Command : ConsoleCommands)
	{
		Command.Reset();
	}
}

bool FWebRemoteControlModule::HandleInfoRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::Ok);

	
	bool bInPackaged = false;
	URemoteControlPreset* ActivePreset = nullptr; 

#if !WITH_EDITOR
	bInPackaged = true;
#else
	// If we are running an editor, then also add the active preset being edited to the payload.
	IRemoteControlUIModule& RemoteControlUIModule = FModuleManager::Get().LoadModuleChecked<IRemoteControlUIModule>(TEXT("RemoteControlUI"));
	ActivePreset = RemoteControlUIModule.GetActivePreset();
#endif

	FAPIInfoResponse RCResponse{RegisteredHttpRoutes.Array(), bInPackaged, ActivePreset};
	WebRemoteControlUtils::SerializeMessage(MoveTemp(RCResponse), Response->Body);
	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleBatchRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}

	Response->Code = EHttpServerResponseCodes::Ok;

	FRCBatchRequest BatchRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, BatchRequest))
	{
		return true;
	}

	FMemoryWriter Writer(Response->Body);
	TSharedPtr<TJsonWriter<ANSICHAR>> JsonWriter = TJsonWriter<ANSICHAR>::Create(&Writer);

	JsonWriter->WriteObjectStart();
	JsonWriter->WriteIdentifierPrefix("Responses");
	JsonWriter->WriteArrayStart();

	BatchRequest.Passphrase = Request.Headers[WebRemoteControlInternalUtils::PassphraseHeader].Last();

	for (FRCRequestWrapper& Wrapper : BatchRequest.Requests)
	{
		Wrapper.Passphrase = BatchRequest.Passphrase;
		// This makes sure the Json writer is in a good state before writing raw data.
		JsonWriter->WriteRawJSONValue(TEXT(""));
		InvokeWrappedRequest(Wrapper, Writer, &Request);
	}

	JsonWriter->WriteArrayEnd();
	JsonWriter->WriteObjectEnd();

	OnComplete(MoveTemp(Response));

	return true;
}

bool FWebRemoteControlModule::HandleOptionsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	OnComplete(WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::Ok));
	return true;
}

bool FWebRemoteControlModule::HandleObjectCallRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString ErrorText;
	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}

	FRCCall Call;
	if (!RemotePayloadSerializer::DeserializeCall(Request, Call, OnComplete))
	{
		return true;
	}

	// if we haven't resolved the object or function, return not found
	if (!Call.IsValid())
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("The object or function was not found."), Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
		OnComplete(MoveTemp(Response));
		return true;
	}

	IRemoteControlModule::Get().InvokeCall(Call);

	TArray<uint8> WorkingBuffer;
	WorkingBuffer.Empty();
	if (!RemotePayloadSerializer::SerializeCall(Call, WorkingBuffer, true))
	{
		Response->Code = EHttpServerResponseCodes::ServerError;
	}
	else
	{
		WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
		Response->Code = EHttpServerResponseCodes::Ok;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleObjectPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FRCObjectReference ObjectRef;
	FRCObjectRequest DeserializedRequest;
	if (!WebRemoteControl::ResolveObjectPropertyForRequest(Request, OnComplete, Response, ObjectRef, DeserializedRequest))
	{
		return true;
	}

	switch (ObjectRef.Access)
	{
	case ERCAccess::READ_ACCESS:
	{
		TArray<uint8> WorkingBuffer;
		FMemoryWriter Writer(WorkingBuffer);
		FRCJsonStructSerializerBackend SerializerBackend(Writer);
		if (IRemoteControlModule::Get().GetObjectProperties(ObjectRef, SerializerBackend))
		{
			Response->Code = EHttpServerResponseCodes::Ok;
			WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
		}
	}
	break;
	case ERCAccess::WRITE_ACCESS:
	case ERCAccess::WRITE_TRANSACTION_ACCESS:
	case ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS:
	{
		const FBlockDelimiters& PropertyValueDelimiters = DeserializedRequest.GetStructParameters().FindChecked(FRCObjectRequest::PropertyValueLabel());
		if (DeserializedRequest.ResetToDefault)
		{
			constexpr bool bAllowIntercept = true;
			if (IRemoteControlModule::Get().ResetObjectProperties(ObjectRef, bAllowIntercept))
			{
				Response->Code = EHttpServerResponseCodes::Ok;
			}
		}
		else if (PropertyValueDelimiters.BlockStart > 0)
		{
			FMemoryReader Reader(DeserializedRequest.TCHARBody);
			Reader.Seek(PropertyValueDelimiters.BlockStart);
			Reader.SetLimitSize(PropertyValueDelimiters.BlockEnd);
			FRCJsonStructDeserializerBackend DeserializerBackend(Reader);
			// Set a ERCPayloadType and TCHARBody in order to follow the replication path
			if (IRemoteControlModule::Get().SetObjectProperties(ObjectRef, DeserializerBackend, ERCPayloadType::Json, DeserializedRequest.TCHARBody, DeserializedRequest.Operation))
			{
				Response->Code = EHttpServerResponseCodes::Ok;
			}
		}
	}
	break;
	default:
		// Bad request
		break;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleObjectPropertyAppendRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
	
	FRCObjectReference ObjectRef;
	FRCObjectRequest DeserializedRequest;
	if (!WebRemoteControl::ResolveObjectPropertyForRequest(Request,OnComplete, Response, ObjectRef, DeserializedRequest))
	{
		return true;
	}

	FMemoryReader Reader = WebRemoteControl::MakePropertyMemoryReaderForArrayAddRequest(DeserializedRequest);
	FRCJsonStructDeserializerBackend DeserializerBackend(Reader);

	if (IRemoteControlModule::Get().AppendToObjectArrayProperty(ObjectRef, DeserializerBackend, ERCPayloadType::Json, DeserializedRequest.TCHARBody))
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleObjectPropertyInsertRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FRCObjectReference ObjectRef;
	FRCObjectRequest DeserializedRequest;
	if (!WebRemoteControl::ResolveObjectPropertyForRequest(Request, OnComplete, Response, ObjectRef, DeserializedRequest))
	{
		return true;
	}

	FMemoryReader Reader = WebRemoteControl::MakePropertyMemoryReaderForArrayAddRequest(DeserializedRequest);
	FRCJsonStructDeserializerBackend DeserializerBackend(Reader);

	int32 Index;
	if (!WebRemoteControl::GetIndexQueryParamForRequest(Request, OnComplete, Response, Index))
	{
		return true;
	}

	if (IRemoteControlModule::Get().InsertToObjectArrayProperty(Index, ObjectRef, DeserializerBackend, ERCPayloadType::Json, DeserializedRequest.TCHARBody))
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleObjectPropertyRemoveRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FRCObjectReference ObjectRef;
	FRCObjectRequest DeserializedRequest;
	if (!WebRemoteControl::ResolveObjectPropertyForRequest(Request, OnComplete, Response, ObjectRef, DeserializedRequest))
	{
		return true;
	}

	int32 Index;
	if (!WebRemoteControl::GetIndexQueryParamForRequest(Request, OnComplete, Response, Index))
	{
		return true;
	}

	if (IRemoteControlModule::Get().RemoveFromObjectArrayProperty(Index, ObjectRef))
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetCallFunctionRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FResolvePresetFieldArgs Args;
	Args.PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	Args.FieldLabel = Request.PathParams.FindChecked(TEXT("functionname"));

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*Args.PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}
	
	TSharedPtr<FRemoteControlFunction> RCFunction = WebRemoteControl::GetRCEntity<FRemoteControlFunction>(Preset, *Args.FieldLabel);
	
	if (!RCFunction || !RCFunction->GetFunction() || !RCFunction->FunctionArguments || !RCFunction->FunctionArguments->IsValid())
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset field."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	FRCPresetCallRequest CallRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, CallRequest))
	{
		return true;
	}

	FBlockDelimiters Delimiters = CallRequest.GetParameterDelimiters(FRCPresetCallRequest::ParametersLabel());
	const int64 DelimitersSize = Delimiters.GetBlockSize();

	TArray<uint8> OutputBuffer;
	FMemoryWriter Writer{ OutputBuffer };
	TSharedPtr<TJsonWriter<UCS2CHAR>> JsonWriter = TJsonWriter<UCS2CHAR>::Create(&Writer);
	FRCJsonStructSerializerBackend WriterBackend{ Writer };

	JsonWriter->WriteObjectStart();
	JsonWriter->WriteIdentifierPrefix("ReturnedValues");
	JsonWriter->WriteArrayStart();

	bool bSuccess = false;

	if (Delimiters.BlockStart != Delimiters.BlockEnd &&
		CallRequest.TCHARBody.IsValidIndex(Delimiters.BlockStart) &&
		CallRequest.TCHARBody.IsValidIndex(DelimitersSize)
		)
	{
		/**
		 * In order to have a replication payload we need to copy the inner payload from TCHARBody to new buffer
		 * Example:
		 * Original buffer from HTTP rquest: { "Parameters": { "NewLocation": {"X": 0, "Y": 0, "Z": 400} }
		 * New buffer: "NewLocation": {"X": 0, "Y": 0, "Z": 400}
		 */
		TArray<uint8> FunctionPayload;
		FunctionPayload.SetNumUninitialized(DelimitersSize);
		const uint8* DataStart = &CallRequest.TCHARBody[Delimiters.BlockStart];	
		FMemory::Memcpy(FunctionPayload.GetData(), DataStart, DelimitersSize);

		FMemoryReader Reader{ FunctionPayload };
		FRCJsonStructDeserializerBackend ReaderBackend{ Reader };

		// Copy the default arguments.
		FStructOnScope FunctionArgs{ RCFunction->GetFunction() };
		for (TFieldIterator<FProperty> It(RCFunction->GetFunction()); It; ++It)
		{
			if (It->HasAnyPropertyFlags(CPF_Parm) && !It->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm))
			{
				It->CopyCompleteValue_InContainer(FunctionArgs.GetStructMemory(), RCFunction->FunctionArguments->GetStructMemory());
			}
		}

		// Deserialize the arguments passed from the user onto the copy of default arguments.
		if (FStructDeserializer::Deserialize((void*)FunctionArgs.GetStructMemory(), *const_cast<UStruct*>(FunctionArgs.GetStruct()), ReaderBackend, FStructDeserializerPolicies()))
		{
			bSuccess = true;
			for (UObject* Object : RCFunction->GetBoundObjects())
			{
				FRCCallReference CallRef;
				CallRef.Object = Object;
				CallRef.Function = RCFunction->GetFunction();

				FRCCall Call;
				Call.CallRef = MoveTemp(CallRef);
				Call.TransactionMode = CallRequest.GenerateTransaction ? ERCTransactionMode::AUTOMATIC : ERCTransactionMode::NONE;
				Call.ParamStruct = FStructOnScope(FunctionArgs.GetStruct(), FunctionArgs.GetStructMemory());

				FString ErrorText;
				if (!WebRemoteControlInternalUtils::ValidateFunctionCall(Call, &ErrorText))
				{
					const FString FormattedMsg = FString::Printf(TEXT("Web Remote Call deserialization error: %s"), *ErrorText);
					UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
					IRemoteControlModule::BroadcastError(FormattedMsg);
					WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(ErrorText, Response->Body);
					OnComplete(MoveTemp(Response));
					return true; // handled
				}

				// Invoke call with replication payload
				bSuccess &= IRemoteControlModule::Get().InvokeCall(Call, ERCPayloadType::Json, FunctionPayload);

				if (bSuccess)
				{
					FStructOnScope ReturnedStruct{ FunctionArgs.GetStruct() };
					TSet<FProperty*> OutProperties;

					// Only copy the out/return parameters from the StructOnScope resulting from the call.
					for (TFieldIterator<FProperty> It(RCFunction->GetFunction()); It; ++It)
					{
						if (It->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm))
						{
							OutProperties.Add(*It);
							// Copy the out/return values into the returned struct.
							It->CopyCompleteValue_InContainer(ReturnedStruct.GetStructMemory(), FunctionArgs.GetStructMemory());

							// Then clear the out/return values.
							It->ClearValue_InContainer(FunctionArgs.GetStructMemory());
						}
					}

					FStructSerializerPolicies Policies;
					Policies.PropertyFilter = [&OutProperties](const FProperty* CurrentProp, const FProperty* ParentProp) { return OutProperties.Contains(CurrentProp); };
					FStructSerializer::Serialize((void*)ReturnedStruct.GetStructMemory(), *const_cast<UStruct*>(ReturnedStruct.GetStruct()), WriterBackend, Policies);
				}
			}
		}
	}

	JsonWriter->WriteArrayEnd();
	JsonWriter->WriteObjectEnd();

	if (bSuccess)
	{
		Response->Code = EHttpServerResponseCodes::Ok;
		WebRemoteControlUtils::ConvertToUTF8(OutputBuffer, Response->Body);
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to call function %s."), *Args.FieldLabel), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetSetPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}

	FRCPresetSetPropertyRequest SetPropertyRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetPropertyRequest))
	{
		return true;
	}

	FResolvePresetFieldArgs Args;
	Args.PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	Args.FieldLabel = Request.PathParams.FindChecked(TEXT("propertyname"));

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*Args.PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	const TSharedPtr<FRemoteControlProperty> RemoteControlProperty = WebRemoteControl::GetRCEntity<FRemoteControlProperty>(Preset, Args.FieldLabel);

	if (!RemoteControlProperty.IsValid())
	{
		// In case the Property is not found or not valid, see if we do have that Id for the Controllers.
		if (URCVirtualPropertyBase* Controller = WebRemoteControl::GetController(Preset, *Args.FieldLabel))
		{
			TArray<uint8> NewPayload;
			const FName PropertyValueKey = WebRemoteControlStructUtils::Prop_PropertyValue;
			const FName PropertyNameInternal = Controller->GetPropertyName();

			// Objrct Ref Method
			FRCObjectReference ObjectRef;
			ObjectRef.Property = Controller->GetProperty();
			ObjectRef.Access = ERCAccess::WRITE_ACCESS;

			RemotePayloadSerializer::ReplaceFirstOccurence(SetPropertyRequest.TCHARBody, PropertyValueKey.ToString(), PropertyNameInternal.ToString(), NewPayload);
			FMemoryReader Reader(NewPayload);
			FJsonStructDeserializerBackend Backend(Reader);
			
			if (IRemoteControlModule::Get().SetPresetController(*Args.PresetName, Controller, Backend, NewPayload, true))
			{
				Response->Code = EHttpServerResponseCodes::Ok;
			}
			else
			{
				WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to set controller %s."), *Args.FieldLabel), Response->Body);
			}

			OnComplete(MoveTemp(Response));

			return true;
		}
		
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset field."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	FString Error;
	const bool bSuccess = WebRemoteControlInternalUtils::ModifyPropertyUsingPayload(
		*RemoteControlProperty.Get(), SetPropertyRequest, SetPropertyRequest.TCHARBody, ActingClientId, *WebSocketHandler,
		SetPropertyRequest.GenerateTransaction ? ERCAccess::WRITE_TRANSACTION_ACCESS : ERCAccess::WRITE_ACCESS, &Error);

	if (bSuccess)
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to modify property %s.\n%s"), *Args.FieldLabel, *Error), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetGetPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FResolvePresetFieldArgs Args;
	Args.PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	Args.FieldLabel = Request.PathParams.FindChecked(TEXT("propertyname"));
	
	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*Args.PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = WebRemoteControl::GetRCEntity<FRemoteControlProperty>(Preset, Args.FieldLabel);
	if (!RemoteControlProperty)
	{
		// In case we do not find a Property, instead go look for a Controller
		if (URCVirtualPropertyBase* Controller = WebRemoteControl::GetController(Preset, *Args.FieldLabel))
		{
			// Define a StructOnScope for representing our generic value output
			FWebRCGenerateStructArgs StructArgs;
			FName PropertyValueKey = WebRemoteControlStructUtils::Prop_PropertyValue;
			StructArgs.GenericProperties.Emplace(PropertyValueKey, Controller->GetProperty());

			FString& StructName = ControllersSerializerStructNameCache.FindOrAdd(Controller->PropertyName);
			if(StructName.IsEmpty())
			{
				StructName = FString::Format(TEXT("{0}_{1}"), { *PropertyValueKey.ToString(), Controller->Id.ToString() });
				ControllersSerializerStructNameCache.Add(Controller->PropertyName, StructName);
			}
			
			FStructOnScope Result(UE::WebRCReflectionUtils::GenerateStruct(*StructName, StructArgs));

			// Copy raw memory from the controller onto our dynamic struct
			const FProperty* TargetValueProp = Result.GetStruct()->FindPropertyByName(PropertyValueKey);
			uint8* DestPtr = TargetValueProp->ContainerPtrToValuePtr<uint8>((void*)Result.GetStructMemory());

			Controller->CopyCompleteValue(TargetValueProp, DestPtr);

			// Serialize to JSON
			TArray<uint8> JsonBuffer;
			FMemoryWriter Writer(JsonBuffer);
			WebRemoteControlInternalUtils::SerializeStructOnScope(Result, Writer);

			// Finalize HTTP Response
			WebRemoteControlUtils::ConvertToUTF8(JsonBuffer, Response->Body);
			Response->Code = EHttpServerResponseCodes::Ok;

			OnComplete(MoveTemp(Response));

			return true;
		}
		
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the exposed entity."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	if (!RemoteControlProperty->IsBound())
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Exposed entity was found but not bound to any object."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	TArray<uint8> WorkingBuffer;
	FMemoryWriter Writer(WorkingBuffer);

	FRCObjectReference ObjectRef;
	FString Error;

	bool bSuccess = IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, RemoteControlProperty->GetBoundObjects()[0], RemoteControlProperty->FieldPathInfo.ToString(), ObjectRef, &Error);

	if (bSuccess)
	{
		FStructOnScope PropertyValueOnScope = WebRemoteControlStructUtils::CreatePropertyValueOnScope(RemoteControlProperty, ObjectRef);
		FStructOnScope FinalStruct = WebRemoteControlStructUtils::CreateGetPropertyOnScope(RemoteControlProperty, ObjectRef, MoveTemp(PropertyValueOnScope));
		WebRemoteControlInternalUtils::SerializeStructOnScope(FinalStruct, Writer);

		Response->Code = EHttpServerResponseCodes::Ok;
		WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to read property %s.\n%s"), *Args.FieldLabel, *Error), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetExposePropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	const FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*FString::Printf(TEXT("Unable to resolve the preset '%s'."), *PresetName), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	FRCPresetExposePropertyRequest ExposePropertyRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, ExposePropertyRequest))
	{
		return true;
	}

	FRCObjectReference ObjectRef;
	const bool bFoundProperty = IRemoteControlModule::Get().ResolveObject(ERCAccess::WRITE_ACCESS, ExposePropertyRequest.ObjectPath, ExposePropertyRequest.PropertyName, ObjectRef);

	if (!bFoundProperty)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*FString::Printf(TEXT("Unable to resolve the property '%s' on object '%s'."),
			*ExposePropertyRequest.PropertyName, *ExposePropertyRequest.ObjectPath), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	FRemoteControlPresetExposeArgs ExposeArgs;
	ExposeArgs.Label = ExposePropertyRequest.Label;
	ExposeArgs.bEnableEditCondition = ExposePropertyRequest.EnableEditCondition;

	if (ExposePropertyRequest.GroupName.Len() > 0)
	{
		const FRemoteControlPresetGroup* Group = Preset->Layout.GetGroupByName(FName(ExposePropertyRequest.GroupName));
		if (Group == nullptr)
		{
			Response->Code = EHttpServerResponseCodes::NotFound;
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*FString::Printf(TEXT("Unable to resolve the property group '%s' on preset '%s'."),
				*ExposePropertyRequest.GroupName, *PresetName), Response->Body);
			OnComplete(MoveTemp(Response));
			return true;
		}

		ExposeArgs.GroupId = Group->Id;
	}

	TSharedPtr<FRemoteControlProperty> ExposedProperty = Preset->ExposeProperty(ObjectRef.Object.Get(), ObjectRef.PropertyPathInfo, ExposeArgs).Pin();

	if (ExposedProperty.IsValid())
	{
		TArray<uint8> WorkingBuffer;
		FMemoryWriter Writer(WorkingBuffer);

		FStructOnScope PropertyValueOnScope = WebRemoteControlStructUtils::CreatePropertyValueOnScope(ExposedProperty, ObjectRef);
		FStructOnScope FinalStruct = WebRemoteControlStructUtils::CreateGetPropertyOnScope(ExposedProperty, ObjectRef, MoveTemp(PropertyValueOnScope));
		WebRemoteControlInternalUtils::SerializeStructOnScope(FinalStruct, Writer);

		Response->Code = EHttpServerResponseCodes::Ok;
		WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
	}
	else
	{
		Response->Code = EHttpServerResponseCodes::ServerError;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Failed to expose the property."), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}


bool FWebRemoteControlModule::HandlePresetUnexposePropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	const FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*FString::Printf(TEXT("Unable to resolve the preset '%s'."), *PresetName), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	const FString PropertyLabelString = Request.PathParams.FindChecked(TEXT("property"));
	const FName PropertyLabel = FName(PropertyLabelString);
	if (!Preset->GetExposedEntityId(PropertyLabel).IsValid())
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*FString::Printf(TEXT("Exposeed property '%s' not found on preset '%s'."),
			*PropertyLabelString, *PresetName), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	Preset->Unexpose(PropertyLabel);

	Response->Code = EHttpServerResponseCodes::Ok;
	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetGetExposedActorPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString ActorRCLabel = Request.PathParams.FindChecked(TEXT("actor"));
	FString PropertyName = Request.PathParams.FindChecked(TEXT("propertyname"));
	FRCFieldPathInfo FieldPath{PropertyName};

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);

	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	if (TSharedPtr<FRemoteControlActor> RCActor = WebRemoteControl::GetRCEntity<FRemoteControlActor>(Preset, ActorRCLabel))
	{
		if (AActor* Actor = RCActor->GetActor())
		{
			TArray<uint8> WorkingBuffer;
			FMemoryWriter Writer(WorkingBuffer);

			FRCObjectReference ObjectRef;
			FString Error;

			if (IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, Actor, FieldPath, ObjectRef, &Error))
			{
				FStructOnScope ActorPropertyOnScope = WebRemoteControlStructUtils::CreateActorPropertyOnScope(ObjectRef);
				WebRemoteControlInternalUtils::SerializeStructOnScope(ActorPropertyOnScope, Writer);
			
				Response->Code = EHttpServerResponseCodes::Ok;
				WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
			}
			else
			{
				Response->Code = EHttpServerResponseCodes::NotFound;
				WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to read property %s on actor %s.\n%s"), *PropertyName, *ActorRCLabel, *Error), Response->Body);
			}
		}
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetGetExposedActorPropertiesRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString ActorRCLabel = Request.PathParams.FindChecked(TEXT("actor"));

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);

	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	if (TSharedPtr<FRemoteControlActor> RCActor = WebRemoteControl::GetRCEntity<FRemoteControlActor>(Preset, ActorRCLabel))
	{
		if (AActor* Actor = RCActor->GetActor())
		{
			TArray<uint8> WorkingBuffer;
			FMemoryWriter Writer(WorkingBuffer);
			FRCJsonStructSerializerBackend Backend{Writer};

			FRCObjectReference Ref{ ERCAccess::READ_ACCESS, Actor};
			if (IRemoteControlModule::Get().GetObjectProperties(Ref, Backend))
			{
				WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Response->Body);
				Response->Code = EHttpServerResponseCodes::Ok;
			}
			else
			{
				Response->Code = EHttpServerResponseCodes::NotFound;
				WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to serialize exposed actor " + ActorRCLabel), Response->Body);
			}
		}
		else
		{
			Response->Code = EHttpServerResponseCodes::NotFound;
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve object path " + RCActor->Path.ToString()), Response->Body);
		}
	}
	else
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the exposed actor " + ActorRCLabel), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetSetExposedActorPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FRCPresetSetPropertyRequest SetPropertyRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetPropertyRequest))
	{
		return true;
	}

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString ActorRCLabel = Request.PathParams.FindChecked(TEXT("actor"));
	FString PropertyName = Request.PathParams.FindChecked(TEXT("propertyname"));
	FRCFieldPathInfo FieldPath{ PropertyName };

	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	bool bSuccess = true;
	FString Error;
	
	if (TSharedPtr<FRemoteControlActor> RCActor = WebRemoteControl::GetRCEntity<FRemoteControlActor>(Preset, ActorRCLabel))
	{
		if (AActor* Actor = RCActor->GetActor())
		{
			FRCObjectReference ObjectRef;
			ERCAccess Access = SetPropertyRequest.GenerateTransaction ? ERCAccess::WRITE_TRANSACTION_ACCESS : ERCAccess::WRITE_ACCESS;
			bSuccess &= IRemoteControlModule::Get().ResolveObjectProperty(Access, Actor, FieldPath, ObjectRef, &Error);

			if (bSuccess && ObjectRef.Property.IsValid())
			{
				FName ResolvedPropertyName = ObjectRef.Property->GetFName();

				// Replace PropertyValue with the underlying property name.
				TArray<uint8> NewPayload;
				RemotePayloadSerializer::ReplaceFirstOccurence(SetPropertyRequest.TCHARBody, TEXT("PropertyValue"), ResolvedPropertyName.ToString(), NewPayload);

				// Then deserialize the payload onto all the bound objects.
				FMemoryReader NewPayloadReader(NewPayload);
				FRCJsonStructDeserializerBackend Backend(NewPayloadReader);


				if (SetPropertyRequest.ResetToDefault)
				{
					constexpr bool bAllowIntercept = true;
					bSuccess &= IRemoteControlModule::Get().ResetObjectProperties(ObjectRef, bAllowIntercept);
				}
				else
				{
					NewPayloadReader.Seek(0);
					// Set a ERCPayloadType and NewPayload in order to follow the replication path
					bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, Backend, ERCPayloadType::Json, NewPayload, SetPropertyRequest.Operation);
				}
			}
		}
		else
		{
			Response->Code = EHttpServerResponseCodes::NotFound;
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve object path " + RCActor->Path.ToString()), Response->Body);
		}
	}
	else
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the exposed actor " + ActorRCLabel), Response->Body);
	}

	if (bSuccess)
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to modify property %s on actor %s.\n%s"), *PropertyName, *ActorRCLabel, *Error), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleGetPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));

	if (URemoteControlPreset* Preset = WebRemoteControl::GetPreset(PresetName))
	{
		WebRemoteControlUtils::SerializeMessage(FGetPresetResponse{ Preset }, Response->Body);
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Preset %s could not be found."), *PresetName), Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleGetPresetsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::Ok);

	TArray<FAssetData> PresetAssets;
	IRemoteControlModule::Get().GetPresetAssets(PresetAssets, false);

	TArray<TWeakObjectPtr<URemoteControlPreset>> EmbeddedPresets;
	IRemoteControlModule::Get().GetEmbeddedPresets(EmbeddedPresets);

	WebRemoteControlUtils::SerializeMessage(FListPresetsResponse{ PresetAssets, EmbeddedPresets }, Response->Body);

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleDescribeObjectRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FDescribeObjectRequest DescribeRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, DescribeRequest))
	{
		return true;
	}

	FRCObjectReference Ref;
	FString ErrorText;
	if (IRemoteControlModule::Get().ResolveObject(ERCAccess::READ_ACCESS, DescribeRequest.ObjectPath, TEXT(""), Ref, &ErrorText) && Ref.Object.IsValid())
	{
		WebRemoteControlUtils::SerializeMessage(FDescribeObjectResponse{ Ref.Object.Get() }, Response->Body);
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(*ErrorText, Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePassphraseRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
	
	FString Passphrase = "";
	const TArray<FString>* PassphraseHeader = Request.Headers.Find(WebRemoteControlInternalUtils::PassphraseHeader);
	
	if (PassphraseHeader)
	{
		Passphrase = PassphraseHeader->Last();
	}
	
	if (bool bIsCorrect = WebRemoteControlInternalUtils::CheckPassphrase(Passphrase))
	{
		WebRemoteControlUtils::SerializeMessage(FCheckPassphraseResponse{ bIsCorrect}, Response->Body);
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlUtils::SerializeMessage(FCheckPassphraseResponse{ bIsCorrect}, Response->Body);
		Response->Code = EHttpServerResponseCodes::Denied;
	}

	OnComplete(MoveTemp(Response));
	return true;
}


bool FWebRemoteControlModule::HandleSearchActorRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::NotSupported);
	WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Route not implemented."), Response->Body);
	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleSearchAssetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
	FSearchAssetRequest SearchAssetRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SearchAssetRequest))
	{
		return true;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	FARFilter Filter = SearchAssetRequest.Filter.ToARFilter();

	TArray<FAssetData> Assets;
	AssetRegistryModule.Get().GetAssets(Filter, Assets);

	//Do advanced blueprint filtering if required
	WebRemoteControl::DoNativeClassBlueprintFilter(SearchAssetRequest, Assets);

	TArrayView<FAssetData> AssetsView{Assets};
	int32 ArrayEnd = FMath::Min(SearchAssetRequest.Limit, Assets.Num());

	TArray<FAssetData> FilteredAssets;
	FilteredAssets.Reserve(SearchAssetRequest.Limit);

	for (const FAssetData& AssetData : Assets)
	{	
		if (!SearchAssetRequest.Query.IsEmpty())
		{
			if (AssetData.AssetName.ToString().Contains(*SearchAssetRequest.Query))
			{
				FilteredAssets.Add(AssetData);
			}
		}
		else
		{
			FilteredAssets.Add(AssetData);
		}

		if (FilteredAssets.Num() >= SearchAssetRequest.Limit)
		{
			break;
		}
	}

	WebRemoteControlUtils::SerializeMessage(FSearchAssetResponse{ MoveTemp(FilteredAssets) }, Response->Body);
	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleGetMetadataRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));

	if (URemoteControlPreset* Preset = WebRemoteControl::GetPreset(PresetName))
	{
		WebRemoteControlUtils::SerializeMessage(FGetMetadataResponse{Preset->Metadata}, Response->Body);
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Preset %s could not be found."), *PresetName), Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleMetadataFieldOperationsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString MetadataField = Request.PathParams.FindChecked(TEXT("metadatafield"));

	if (URemoteControlPreset* Preset = WebRemoteControl::GetPreset(PresetName))
	{
		if (Request.Verb == EHttpServerRequestVerbs::VERB_GET)
		{
			if (FString* MetadataValue = Preset->Metadata.Find(MetadataField))
			{
				WebRemoteControlUtils::SerializeMessage(FGetMetadataFieldResponse{ *MetadataValue }, Response->Body);
			}
			else
			{
				WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Metadata field %s could not be found."), *MetadataField), Response->Body);
				Response->Code = EHttpServerResponseCodes::NotFound;
			}
		}
		else if (Request.Verb == EHttpServerRequestVerbs::VERB_PUT)
		{
			FSetPresetMetadataRequest SetMetadataRequest;
			if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetMetadataRequest))
			{
				return true;
			}
#if WITH_EDITOR
			FScopedTransaction Transaction(LOCTEXT("ModifyPresetMetadata", "Modify preset metadata"));
#endif
			Preset->Modify();
			FString& MetadataValue = Preset->Metadata.FindOrAdd(MoveTemp(MetadataField));
			MetadataValue = MoveTemp(SetMetadataRequest.Value);
		}
		else if (Request.Verb == EHttpServerRequestVerbs::VERB_DELETE)
		{
#if WITH_EDITOR
			FScopedTransaction Transaction(LOCTEXT("DeletePresetMetadata", "Delete preset metadata entry"));
#endif
			Preset->Modify();
			Preset->Metadata.Remove(MoveTemp(MetadataField));
		}

		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Preset %s could not be found."), *PresetName), Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleSearchObjectRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::NotSupported);
	WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Route not implemented."), Response->Body);
	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleEntityMetadataOperationsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString Label = Request.PathParams.FindChecked(TEXT("label"));
	FString Key = Request.PathParams.FindChecked(TEXT("key"));
	
	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	TSharedPtr<FRemoteControlEntity> Entity = WebRemoteControl::GetRCEntity<FRemoteControlEntity>(Preset, Label);

	if (!Entity.IsValid())
	{
		// Test for Controllers
		if (URCVirtualPropertyBase* Controller = WebRemoteControl::GetController(Preset, Label))
		{
			if (Request.Verb == EHttpServerRequestVerbs::VERB_GET)
			{
				FString MetadataValue = Controller->GetMetadataValue(*Key);
				if (!MetadataValue.IsEmpty())
				{
					WebRemoteControlUtils::SerializeMessage(FGetMetadataFieldResponse{ *MetadataValue }, Response->Body);
				}
				else
				{
					WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Metadata entry %s could not be found."), *Key), Response->Body);
					Response->Code = EHttpServerResponseCodes::NotFound;
				}
			}
			else if (Request.Verb == EHttpServerRequestVerbs::VERB_PUT)
			{
				if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
				{
					return true;
				}
				
				FSetEntityMetadataRequest SetMetadataRequest;
				if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetMetadataRequest))
				{
					return true;
				}

#if WITH_EDITOR
				FScopedTransaction Transaction( LOCTEXT("ModifyEntityMetadata", "Modify exposed entity metadata entry"));
				Preset->Modify();
#endif
				Controller->SetMetadataValue(*Key, SetMetadataRequest.Value);
			}
			else if (Request.Verb == EHttpServerRequestVerbs::VERB_DELETE)
			{
#if WITH_EDITOR
				FScopedTransaction Transaction( LOCTEXT("DeleteEntityMetadata", "Delete exposed entity metadata entry"));
				Preset->Modify();
#endif
				Controller->RemoveMetadataValue(*Key);
			}

			Response->Code = EHttpServerResponseCodes::Ok;

			OnComplete(MoveTemp(Response));
			return true;
		}
		
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset entity."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	if (Request.Verb == EHttpServerRequestVerbs::VERB_GET)
	{
		if (const FString* MetadataValue = Entity->GetMetadata().Find(*Key))
		{
			WebRemoteControlUtils::SerializeMessage(FGetMetadataFieldResponse{ *MetadataValue }, Response->Body);
		}
		else
		{
			WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Metadata entry %s could not be found."), *Key), Response->Body);
			Response->Code = EHttpServerResponseCodes::NotFound;
		}
	}
	else if (Request.Verb == EHttpServerRequestVerbs::VERB_PUT)
	{
		if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
		{
			return true;
		}
		
		FSetEntityMetadataRequest SetMetadataRequest;
		if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetMetadataRequest))
		{
			return true;
		}

#if WITH_EDITOR
		FScopedTransaction Transaction( LOCTEXT("ModifyEntityMetadata", "Modify exposed entity metadata entry"));
		Preset->Modify();
#endif
		Entity->SetMetadataValue(*Key, SetMetadataRequest.Value);
	}
	else if (Request.Verb == EHttpServerRequestVerbs::VERB_DELETE)
	{
#if WITH_EDITOR
		FScopedTransaction Transaction( LOCTEXT("DeleteEntityMetadata", "Delete exposed entity metadata entry"));
		Preset->Modify();
#endif
		Entity->RemoveMetadataEntry(*Key);
	}

	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleEntitySetLabelRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	FString Label = Request.PathParams.FindChecked(TEXT("label"));
	
	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}
	
	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	TSharedPtr<FRemoteControlEntity> Entity = WebRemoteControl::GetRCEntity<FRemoteControlEntity>(Preset, Label);

	if (!Entity.IsValid())
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset entity."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	FSetEntityLabelRequest SetEntityLabelRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetEntityLabelRequest))
	{
		return true;
	}

#if WITH_EDITOR
	FScopedTransaction Transaction( LOCTEXT("SetEntityLabel", "Modify exposed entity's label"));
	Preset->Modify();
#endif
	
	FName AssignedLabel = Entity->Rename(*SetEntityLabelRequest.NewLabel);
	
	WebRemoteControlUtils::SerializeMessage(FSetEntityLabelResponse{ AssignedLabel.ToString() }, Response->Body);
	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleCreateTransientPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	const URemoteControlPreset* Preset = IRemoteControlModule::Get().CreateTransientPreset();
	if (!Preset)
	{
		Response->Code = EHttpServerResponseCodes::ServerError;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to create a transient preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	WebRemoteControlUtils::SerializeMessage(FGetPresetResponse{ Preset }, Response->Body);
	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandleDeleteTransientPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	FString PresetName = Request.PathParams.FindChecked(TEXT("preset"));

	const bool bSuccess = IRemoteControlModule::Get().DestroyTransientPreset(FName(*PresetName));
	if (!bSuccess)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("No transient preset with that name exists."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));
	return true;
}

// Remote Control Logic - Controllers
bool FWebRemoteControlModule::HandlePresetSetControllerRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	// 1. Prepare response object
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	// 2. Validate Content Type
	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}

	// 3. Deserialize Json POST data into request struct
	FRCPresetSetControllerRequest SetControllerRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, SetControllerRequest))
	{
		return true;
	}

	// 4. Resolve url arguments
	FResolvePresetControllerArgs Args;
	Args.PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	Args.ControllerName = Request.PathParams.FindChecked(TEXT("controller"));

	// 5. Acquire Remote Control Preset object
	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*Args.PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	// 6. Acquire Controller
	URCVirtualPropertyBase* Controller = Preset->GetControllerByDisplayName(*Args.ControllerName);
	if (!Controller)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the controller input."), Response->Body);
		OnComplete(MoveTemp(Response));
		return true;
	}

	// 7. Reformat Payload to represent our internal structure
	TArray<uint8> NewPayload;
	const FName PropertyValueKey = WebRemoteControlStructUtils::Prop_PropertyValue;
	const FName PropertyNameInternal = Controller->GetPropertyName();

	RemotePayloadSerializer::ReplaceFirstOccurence(SetControllerRequest.TCHARBody, PropertyValueKey.ToString(), PropertyNameInternal.ToString(), NewPayload);
	FMemoryReader Reader(NewPayload);
	FJsonStructDeserializerBackend Backend(Reader);

	// 8. Invoke via RC Module which provides Interception functionality
	const bool bSuccess = IRemoteControlModule::Get().SetPresetController(*Args.PresetName, Controller, Backend, NewPayload, true /*support interception*/);

	// 9. Finalize HTTP Response
	if (bSuccess)
	{
		Response->Code = EHttpServerResponseCodes::Ok;
	}
	else
	{
		Response->Code = EHttpServerResponseCodes::ServerError;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Error while trying to set controller %s."), *Args.ControllerName), Response->Body);
	}

	OnComplete(MoveTemp(Response));
	return true;
}

bool FWebRemoteControlModule::HandlePresetGetControllerRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	// 1. Prepare response object
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();

	// 2. Resolve url arguments
	FResolvePresetControllerArgs Args;
	Args.PresetName = Request.PathParams.FindChecked(TEXT("preset"));
	Args.ControllerName = Request.PathParams.FindChecked(TEXT("controller"));

	// 3. Acquire Remote Control Preset object
	URemoteControlPreset* Preset = WebRemoteControl::GetPreset(*Args.PresetName);
	if (Preset == nullptr)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the preset."), Response->Body);
		OnComplete(MoveTemp(Response));

		return true;
	}

	// 4. Acquire Controller
	URCVirtualPropertyBase* Controller = Preset->GetControllerByDisplayName(*Args.ControllerName);
	if (!Controller)
	{
		Response->Code = EHttpServerResponseCodes::NotFound;
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("Unable to resolve the controller input."), Response->Body);
		OnComplete(MoveTemp(Response));

		return true;
	}

	// 5. Define a StructOnScope for representing our generic value output
	FWebRCGenerateStructArgs StructArgs;
	const FName PropertyValueKey = WebRemoteControlStructUtils::Prop_PropertyValue;
	StructArgs.GenericProperties.Emplace(PropertyValueKey, Controller->GetProperty());

	FString& StructName = ControllersSerializerStructNameCache.FindOrAdd(Controller->PropertyName);
	if(StructName.IsEmpty())
	{
		StructName = FString::Format(TEXT("{0}_{1}"), { *PropertyValueKey.ToString(), Controller->Id.ToString() });
		ControllersSerializerStructNameCache.Add(Controller->PropertyName, StructName);
	}
	
	FStructOnScope Result(UE::WebRCReflectionUtils::GenerateStruct(*StructName, StructArgs));

	// 6. Copy raw memory from the controller onto our dynamic struct
	const FProperty* TargetValueProp = Result.GetStruct()->FindPropertyByName(PropertyValueKey);
	uint8* DestPtr = TargetValueProp->ContainerPtrToValuePtr<uint8>((void*)Result.GetStructMemory());

	Controller->CopyCompleteValue(TargetValueProp, DestPtr);

	// 7. Serialize to JSON
	TArray<uint8> JsonBuffer;
	FMemoryWriter Writer(JsonBuffer);
	WebRemoteControlInternalUtils::SerializeStructOnScope(Result, Writer);

	// 8. Finalize HTTP Response
	WebRemoteControlUtils::ConvertToUTF8(JsonBuffer, Response->Body);
	Response->Code = EHttpServerResponseCodes::Ok;

	OnComplete(MoveTemp(Response));

	return true;
}

void FWebRemoteControlModule::HandleWebSocketHttpMessage(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	// Sets the acting client id for the duration of the message handling.
	TGuardValue<FGuid> ScopeGuard(ActingClientId, WebSocketMessage.ClientId);
	TArray<uint8> UTF8Response;

	//Early failure is http server not started
	if (HttpRouter == nullptr)
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(TEXT("HTTP server not started."), UTF8Response);
		WebSocketServer.Send(WebSocketMessage.ClientId, MoveTemp(UTF8Response));
		return;
	}

	FRCRequestWrapper Wrapper;
	if (!WebRemoteControlInternalUtils::DeserializeWrappedRequestPayload(WebSocketMessage.RequestPayload, nullptr, Wrapper))
	{
		return;
	}
	
	if (WebSocketMessage.Header.Find(WebRemoteControlInternalUtils::PassphraseHeader))
	{
		Wrapper.Passphrase = WebSocketMessage.Header[WebRemoteControlInternalUtils::PassphraseHeader][0];
	}


	LogRequestExternally(Wrapper.RequestId, TEXT("UE Received"));
	
	FMemoryWriter Writer(UTF8Response);
	InvokeWrappedRequest(Wrapper, Writer);

	LogRequestExternally(Wrapper.RequestId, TEXT("UE Processed"));

	WebSocketServer.Send(WebSocketMessage.ClientId, MoveTemp(UTF8Response));
	LogRequestExternally(Wrapper.RequestId, TEXT("UE Sent"));
}

void FWebRemoteControlModule::HandleWebSocketBatchMessage(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	if (!WebSocketRouter)
	{
		return;
	}

	FRCWebSocketBatchRequest Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	for (FRCWebSocketRequest& Request : Body.Requests)
	{
		FRemoteControlWebSocketMessage Message;

		const FBlockDelimiters PayloadDelimiters = Request.GetParameterDelimiters(FRCWebSocketRequest::ParametersFieldLabel());
		if (PayloadDelimiters.BlockStart != PayloadDelimiters.BlockEnd)
		{
			Message.RequestPayload = MakeArrayView(WebSocketMessage.RequestPayload).Slice(PayloadDelimiters.BlockStart, PayloadDelimiters.BlockEnd - PayloadDelimiters.BlockStart);
		}

		Message.ClientId = WebSocketMessage.ClientId;
		Message.Header = WebSocketMessage.Header;
		Message.MessageId = Request.Id;
		Message.MessageName = Request.MessageName;

		WebSocketRouter->AttemptDispatch(Message);
	}
}

void FWebRemoteControlModule::InvokeWrappedRequest(const FRCRequestWrapper& Wrapper, FMemoryWriter& OutUTF8PayloadWriter, const FHttpServerRequest* TemplateRequest)
{
	TSharedRef<FHttpServerRequest> UnwrappedRequest = RemotePayloadSerializer::UnwrapHttpRequest(Wrapper, TemplateRequest);

	auto ResponseLambda = [this, &OutUTF8PayloadWriter, &Wrapper](TUniquePtr<FHttpServerResponse> Response) {
		RemotePayloadSerializer::SerializeWrappedCallResponse(Wrapper.RequestId, MoveTemp(Response), OutUTF8PayloadWriter);
	};

	if (!HttpRouter->Query(UnwrappedRequest, ResponseLambda))
	{
		TUniquePtr<FHttpServerResponse> InnerRouteResponse = WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes::NotFound);
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Route %s could not be found."), *Wrapper.URL), InnerRouteResponse->Body);
		ResponseLambda(MoveTemp(InnerRouteResponse));
	}
}

void FWebRemoteControlModule::RegisterDefaultPreprocessors()
{
	using namespace UE::WebRemoteControl;

	auto MakeWebsocketPreDispatch = [this](FRCPreprocessorHandler PreprocessorHandler)
	{
		return [this, PreprocessorHandler](const FRemoteControlWebSocketMessage& Message) -> bool
		{
			FHttpServerRequest Request;
			Request.Headers = Message.Header;
			Request.PeerAddress = Message.PeerAddress;

			FPreprocessorResult Result = PreprocessorHandler(Request);
			if (Result.Result == EPreprocessorResult::RequestPassthrough)
			{
				return true;
			}
			else
			{
				TArray<uint8> Response;

				const TArray<FString>* InPassphrase = Message.Header.Find(WebRemoteControlInternalUtils::PassphraseHeader);
				const FString Passphrase = InPassphrase ? InPassphrase->Last() : FString("");

				FRCRequestWrapper Wrapper;
				Wrapper.RequestId = Message.MessageId;
				Wrapper.Passphrase = Passphrase;
				Wrapper.Verb = "401";

				// Not ideal, we're re-converting to tchar, this should be removed once the WebRC pipeline is cleaned up.
				WebRemoteControlUtils::ConvertToTCHAR(Result.OptionalResponse->Body, Wrapper.TCHARBody);
				WebRemoteControlUtils::SerializeMessage(Wrapper, Response);

				WebSocketServer.Send(Message.ClientId, MoveTemp(Response));

				return false;
			}
		};
	};

	auto RegisterInternalPreprocessor = [this, MakeWebsocketPreDispatch](FRCPreprocessorHandler PreprocessorHandler)
	{
		if (HttpRouter)
		{
			AllRegisteredPreprocessorHandlers.Add(HttpRouter->RegisterRequestPreprocessor(MakeHttpRequestHandler(PreprocessorHandler)));
		}

		if (WebSocketRouter)
		{
			WebSocketRouter->AddPreDispatch(MakeWebsocketPreDispatch(PreprocessorHandler));
		}
	};

	RegisterInternalPreprocessor(&RemotePassphraseEnforcementPreprocessor);
	RegisterInternalPreprocessor(&PassphrasePreprocessor);
	RegisterInternalPreprocessor(&IPValidationPreprocessor);
}

void FWebRemoteControlModule::UnregisterAllPreprocessors()
{
	if (HttpRouter)
	{
		for (const FDelegateHandle& Handle : AllRegisteredPreprocessorHandlers)
		{
			HttpRouter->UnregisterRequestPreprocessor(Handle);
		}
	}
}

void FWebRemoteControlModule::RegisterExternalPreprocesors()
{
	if (HttpRouter)
	{
		for (const TPair<FDelegateHandle, FHttpRequestHandler>& Handler : PreprocessorsToRegister)
		{
			// Find the pre-processors HTTP-handle from the one we generated.
			FDelegateHandle& Handle = PreprocessorsHandleMappings.FindChecked(Handler.Key);
			if (Handle.IsValid())
			{
				HttpRouter->UnregisterRequestPreprocessor(Handle);
				AllRegisteredPreprocessorHandlers.RemoveAtSwap(AllRegisteredPreprocessorHandlers.IndexOfByKey(Handle));
			}

			// Update the preprocessor handle mapping.
			Handle = HttpRouter->RegisterRequestPreprocessor(Handler.Value);
			AllRegisteredPreprocessorHandlers.Add(Handle);
		}
	}
}

#if WITH_EDITOR
void FWebRemoteControlModule::RegisterSettings()
{
	GetMutableDefault<URemoteControlSettings>()->OnSettingChanged().AddRaw(this, &FWebRemoteControlModule::OnSettingsModified);
}

void FWebRemoteControlModule::UnregisterSettings()
{
	if (UObjectInitialized())
	{
		GetMutableDefault<URemoteControlSettings>()->OnSettingChanged().RemoveAll(this);	
	}
}

void FWebRemoteControlModule::OnSettingsModified(UObject* Settings, FPropertyChangedEvent& PropertyChangedEvent)
{
	const URemoteControlSettings* RCSettings = CastChecked<URemoteControlSettings>(Settings);
	const bool bIsWebServerStarted = HttpRouter.IsValid();
	const bool bIsWebSocketServerStarted = WebSocketServer.IsRunning();
	const bool bRestartHttpServer = RCSettings->RemoteControlHttpServerPort != HttpServerPort;
	const bool bRestartWebSocketServer = RCSettings->RemoteControlWebSocketServerPort != WebSocketServerPort || RCSettings->RemoteControlWebsocketServerBindAddress != WebsocketServerBindAddress;

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{

		if ((bIsWebServerStarted && bRestartHttpServer)
			|| (!bIsWebServerStarted && RCSettings->bAutoStartWebServer))
		{
			HttpServerPort = RCSettings->RemoteControlHttpServerPort;
			StopHttpServer();
			StartHttpServer();
		}

		if ((bIsWebSocketServerStarted && bRestartWebSocketServer)
			|| (!bIsWebSocketServerStarted && RCSettings->bAutoStartWebSocketServer))
		{
			WebSocketServerPort = RCSettings->RemoteControlWebSocketServerPort;
			WebsocketServerBindAddress = RCSettings->RemoteControlWebsocketServerBindAddress;
			StopWebSocketServer();
			StartWebSocketServer();
		}
	}

	/** Letting the Server know what the current state of the Passphrase Usage is. */
	if (bIsWebSocketServerStarted && bIsWebServerStarted)
	{
		const bool bIsOpen = RCSettings->bEnforcePassphraseForRemoteClients;
		
		TArray<uint8> Response;
		const FCheckPassphraseResponse BoolResponse = FCheckPassphraseResponse(!bIsOpen);
		
		WebRemoteControlUtils::SerializeMessage(BoolResponse, Response);

		WebSocketServer.Broadcast(Response);
	}
}

#endif

void FWebRemoteControlModule::LogRequestExternally(int32 RequestId, const TCHAR* Stage)
{
	if (ExternalLogger.IsValid())
	{
		ExternalLogger->Log(RequestId, Stage);
	}
}

#undef LOCTEXT_NAMESPACE /* WebRemoteControl */

IMPLEMENT_MODULE(FWebRemoteControlModule, WebRemoteControl);
