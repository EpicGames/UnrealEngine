// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "IWebRemoteControlModule.h"
#include "IRemoteControlModule.h"
#include "HAL/IConsoleManager.h"
#include "HttpRouteHandle.h"
#include "HttpServerResponse.h"
#include "HttpServerRequest.h"
#include "RemoteControlRequest.h"
#include "RemoteControlRoute.h"
#include "RemoteControlWebsocketRoute.h"
#include "RemoteControlWebSocketServer.h"
#include "WebRemoteControlEditorRoutes.h"

struct FHttpServerRequest;
class FMemoryWriter;
class IHttpRouter;
class FWebSocketMessageHandler;

/**
 * A Remote Control module that expose remote function calls through http
 */
class FWebRemoteControlModule : public IWebRemoteControlModule
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static FWebRemoteControlModule& Get()
	{
		static const FName ModuleName = "WebRemoteControl";
		return FModuleManager::LoadModuleChecked<FWebRemoteControlModule>(ModuleName);
	}

	//~ Begin IWebRemoteControlModule Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual FDelegateHandle RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor) override;
	virtual void UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle) override;
	virtual FOnWebServerStarted& OnHttpServerStarted() override { return OnHttpServerStartedDelegate; }
	virtual FSimpleMulticastDelegate& OnHttpServerStopped() override { return OnHttpServerStoppedDelegate; }
	virtual bool IsHttpServerRunning() override { return bIsHttpServerRunning; }
	virtual FOnWebServerStarted& OnWebSocketServerStarted() override { return OnWebSocketServerStartedDelegate; }
	virtual FSimpleMulticastDelegate& OnWebSocketServerStopped() override { return OnWebSocketServerStoppedDelegate; }
	virtual bool IsWebSocketServerRunning() override { return WebSocketServer.IsRunning(); }
	virtual FOnWebSocketConnectionClosed& OnWebSocketConnectionOpened() override { return WebSocketServer.OnConnectionOpened(); }
	virtual FOnWebSocketConnectionClosed& OnWebSocketConnectionClosed() override { return WebSocketServer.OnConnectionClosed(); }
	virtual void RegisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route) override;
	virtual void UnregisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route) override;
	virtual void SendWebsocketMessage(const FGuid& InTargetClientId, const TArray<uint8>& InUTF8Payload) override;
	virtual void SetExternalRemoteWebSocketLoggerConnection(TSharedPtr<class INetworkingWebSocket> WebSocketLoggerConnection) override;
	//~ End IWebRemoteControlModule Interface

	/**
	 * Register a route to the API.
	 * @param Route The route to register.
	 */
	void RegisterRoute(const FRemoteControlRoute& Route);

	/**
	 * Unregister a route to the API.
	 * @param Route The route to unregister.
	 */
	void UnregisterRoute(const FRemoteControlRoute& Route);
	
	/**
	 * Start the web control server
	 */
	void StartHttpServer();

	/**
	 * Stop the web control server.
	 */
	void StopHttpServer();

	/**
	 * Start the web control websocket server.
	 */
	void StartWebSocketServer();

	/**
	 * Stop the web control websocket server.
	 */
	void StopWebSocketServer();

private:
	/** Bind the route in the http router and add it to the list of active routes. */
	void StartRoute(const FRemoteControlRoute& Route);

	/** Register HTTP and Websocket routes. */
	void RegisterRoutes();

	/** Register console commands. */
	void RegisterConsoleCommands();

	/** Unregister console commands. */
	void UnregisterConsoleCommands();

	//~ Route handlers
	bool HandleInfoRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleBatchRequest(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleOptionsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleObjectCallRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleObjectPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleObjectPropertyAppendRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleObjectPropertyInsertRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleObjectPropertyRemoveRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetCallFunctionRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetSetPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetGetPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetExposePropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetUnexposePropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetGetExposedActorPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetGetExposedActorPropertiesRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetSetExposedActorPropertyRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleGetPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleGetPresetsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleDescribeObjectRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleSearchActorRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleSearchAssetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleGetMetadataRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleMetadataFieldOperationsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleSearchObjectRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleEntityMetadataOperationsRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleEntitySetLabelRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePassphraseRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleCreateTransientPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleDeleteTransientPresetRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetSetControllerRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandlePresetGetControllerRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

	//~ Websocket route handlers
	void HandleWebSocketHttpMessage(const struct FRemoteControlWebSocketMessage& WebSocketMessage);
	void HandleWebSocketBatchMessage(const FRemoteControlWebSocketMessage& WebSocketMessage);

	void InvokeWrappedRequest(const struct FRCRequestWrapper& Wrapper, FMemoryWriter& OutUTF8PayloadWriter, const FHttpServerRequest* TemplateRequest = nullptr);

	/** Register the default request preprocessors used by Web RC in the HttpRouter. */
	void RegisterDefaultPreprocessors();
	void UnregisterAllPreprocessors();

	/** Register the external request preprocessors that were added using RegisterRequestPreprocessor in the HttpRouter.   */
	void RegisterExternalPreprocesors();

#if WITH_EDITOR
	//~ Settings
	void RegisterSettings();
	void UnregisterSettings();
	void OnSettingsModified(UObject* Settings, FPropertyChangedEvent& PropertyChangedEvent);
#endif

	void LogRequestExternally(int32 RequestId, const TCHAR* Stage);

private:
	/** Console commands handles. */
	TArray<TUniquePtr<FAutoConsoleCommand>> ConsoleCommands;

	/** Http router handle */
	TSharedPtr<IHttpRouter> HttpRouter;

	/** Mapping of routes to delegate handles */
	TMap<uint32, FHttpRouteHandle> ActiveRouteHandles;

	/** Set of routes that will be activated on http server start. */
	TSet<FRemoteControlRoute> RegisteredHttpRoutes;

	/** Set of routes that will be activated on websocket server start. */
	TSet<FRemoteControlWebsocketRoute> RegisteredWebSocketRoutes;

	/** Port of the remote control http server. */
	uint32 HttpServerPort;

	/** The address the websocket server is currently bound to. */
	FString WebsocketServerBindAddress;

	/** Port of the remote control websocket server. */
	uint32 WebSocketServerPort;

	/** Routes that are editor specific. */
	FWebRemoteControlEditorRoutes EditorRoutes;

	/** Handler processing websocket specific messages */
	TUniquePtr<FWebSocketMessageHandler> WebSocketHandler;
	
	/** Server that serves websocket requests. */
	FRCWebSocketServer WebSocketServer;

	/** Router used to dispatch websocket messages. */
	TSharedPtr<FWebsocketMessageRouter> WebSocketRouter;

	/** Holds the client currently making a request. */
	FGuid ActingClientId;

	/** Whether the HTTP server has been started and has not been stopped. */
	bool bIsHttpServerRunning = false;

	/** List of preprocessor delegates that need to be registered when the server is started. */
	TMap<FDelegateHandle, FHttpRequestHandler> PreprocessorsToRegister;

	/**
	 * Mappings of preprocessors delegate handles generated from the WebRC module to the ones generated from the Http Module.
	 */
	TMap<FDelegateHandle, FDelegateHandle> PreprocessorsHandleMappings;

	/**
	 * List of all registered preprocessor handlers. Used to unregister them on shutdown.
	 */
	TArray<FDelegateHandle> AllRegisteredPreprocessorHandlers;

	/** An external web socket logger*/
	TUniquePtr<class FWebRemoteControlExternalLogger> ExternalLogger;

	//~ Server started stopped delegates.
	FOnWebServerStarted OnHttpServerStartedDelegate;
	FSimpleMulticastDelegate OnHttpServerStoppedDelegate;
	FOnWebServerStarted OnWebSocketServerStartedDelegate;
	FSimpleMulticastDelegate OnWebSocketServerStoppedDelegate;

	/** Name cache for Get Controller Result structs
	* Key: Controller name; Value: Name of the dynamic struct holding our result*/
	TMap<FName, FString> ControllersSerializerStructNameCache;
};