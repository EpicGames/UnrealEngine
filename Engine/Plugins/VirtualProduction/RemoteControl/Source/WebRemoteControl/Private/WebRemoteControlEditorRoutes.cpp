// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRemoteControlEditorRoutes.h"

#if WITH_EDITOR
#include "WebRemoteControlInternalUtils.h"
#include "WebRemoteControl.h"

// Http
#include "GenericPlatform/GenericPlatformHttp.h"
#include "HttpPath.h"
#include "HttpServerRequest.h"
#include "HttpServerConstants.h"

// Serialization
#include "Backends/JsonStructSerializerBackend.h"
#include "Serialization/MemoryWriter.h"

// Console variable handling
#include "HAL/ConsoleManager.h"

// Global UOject delegates
#include "UObject/UObjectGlobals.h"

// Thumbnails
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "ClassIconFinder.h"
#include "IImageWrapperModule.h"
#include "IImageWrapper.h"
#include "Misc/ObjectThumbnail.h"
#include "Misc/FileHelper.h"
#include "ObjectTools.h"

#include "Styling/SlateBrush.h"

void FWebRemoteControlEditorRoutes::RegisterRoutes(FWebRemoteControlModule* WebRemoteControl)
{
	if (IConsoleManager::Get().FindConsoleVariable(TEXT("WebControl.EnableExperimentalRoutes"))->GetBool())
	{
		static const FName ModuleName = "WebRemoteControl";
		// Events
		WebRemoteControl->RegisterRoute(Routes.Emplace_GetRef(
			TEXT("Create a connection until an event is triggered."),
			FHttpPath(TEXT("/remote/object/event")),
			EHttpServerRequestVerbs::VERB_PUT,
			FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlEditorRoutes::HandleObjectEventRoute)
		));

		EventDispatchers.AddDefaulted((int32)ERemoteControlEvent::EventCount);
	}

	// Thumbnails
	WebRemoteControl->RegisterRoute(Routes.Emplace_GetRef(
		TEXT("Get an object's thumbnail"),
		FHttpPath(TEXT("/remote/object/thumbnail")),
		EHttpServerRequestVerbs::VERB_PUT,
		FHttpRequestHandler::CreateRaw(this, &FWebRemoteControlEditorRoutes::HandleGetThumbnailRoute)
	));
}

void FWebRemoteControlEditorRoutes::UnregisterRoutes(FWebRemoteControlModule* WebRemoteControl)
{
	for (const FRemoteControlRoute& Route : Routes)
	{
		WebRemoteControl->UnregisterRoute(Route);
	}
}

bool FWebRemoteControlEditorRoutes::HandleObjectEventRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	if (!WebRemoteControlInternalUtils::ValidateContentType(Request, TEXT("application/json"), OnComplete))
	{
		return true;
	}

	FRemoteControlObjectEventHookRequest EventRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, EventRequest))
	{
		return true;
	}

	// Queue the request and complete the event when it triggers
	AddPendingEvent(MoveTemp(EventRequest), WebRemoteControlInternalUtils::CreateHttpResponse(), OnComplete);
	return true;
};

bool FWebRemoteControlEditorRoutes::HandleGetThumbnailRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete)
{
	TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
	FGetObjectThumbnailRequest GetThumbnailRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(Request, &OnComplete, GetThumbnailRequest))
	{
		return true;
	}

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FARFilter Filter;
	Filter.SoftObjectPaths.Add(GetThumbnailRequest.ObjectPath);

	TArray<FAssetData> Assets;
	AssetRegistryModule.Get().GetAssets(Filter, Assets);

	if (!Assets.Num())
	{
		// Attempt to find by stripping file extension
		int32 Index = INDEX_NONE;
		FStringView ModifiedName = FStringView(GetThumbnailRequest.ObjectPath);

		// Attempt to strip asset extension
		if (GetThumbnailRequest.ObjectPath.FindLastChar(TEXT('.'), Index))
		{
			ModifiedName.LeftInline(Index);
		}

		if (GetThumbnailRequest.ObjectPath.FindLastChar(TEXT('/'), Index))
		{
			// Attempt to find by duplicating the last part of the path.
			FStringView LastSegment = ModifiedName.RightChop(Index + 1);
			TStringBuilder<64> Builder;
			Builder.Append(ModifiedName);
			Builder.Append(TEXT("."));
			Builder.Append(LastSegment);

			FARFilter FallbackFilter;
			FallbackFilter.SoftObjectPaths.Add(FSoftObjectPath(Builder.ToView()));
			AssetRegistryModule.Get().GetAssets(FallbackFilter, Assets);
		}
	}

	if (Assets.Num())
	{
		const FAssetData& AssetData = Assets[0];
		FName ObjectFullName = FName(*AssetData.GetFullName());
		FThumbnailMap ThumbnailMap;

		if (ThumbnailTools::ConditionallyLoadThumbnailsForObjects({ ObjectFullName }, ThumbnailMap))
		{
			if (FObjectThumbnail* Thumbnail = ThumbnailMap.Find(ObjectFullName))
			{
				IImageWrapperModule& ImageWrapperModule = FModuleManager::Get().LoadModuleChecked<IImageWrapperModule>(TEXT("ImageWrapper"));
				EImageFormat Format = ImageWrapperModule.DetectImageFormat((void*)Thumbnail->AccessCompressedImageData().GetData(), Thumbnail->AccessCompressedImageData().Num());

				if (Format == EImageFormat::PNG || Format == EImageFormat::JPEG)
				{
					// Converting png/jpeg thumbnails to be in the right color format (BGRA) for web browsers.
					WebRemoteControlInternalUtils::AddContentTypeHeaders(Response.Get(), Format == EImageFormat::PNG ? TEXT("image/png") : TEXT("image/jpeg"));
					TSharedPtr<IImageWrapper> Wrapper = ImageWrapperModule.CreateImageWrapper(Format);

					Wrapper->SetRaw(Thumbnail->GetUncompressedImageData().GetData(), Thumbnail->GetUncompressedImageData().Num(), Thumbnail->GetImageWidth(), Thumbnail->GetImageHeight(), ERGBFormat::BGRA, 8);
					if (Wrapper)
					{
						Response->Body = Wrapper->GetCompressed();
						Response->Code = EHttpServerResponseCodes::Ok;
					}
				}
			}
		}

		if (!Response->Body.Num())
		{
			if (const FSlateBrush* ThumbnailBrush = FClassIconFinder::FindThumbnailForClass(AssetData.GetClass()))
			{
				const FString ResourceName = ThumbnailBrush->GetResourceName().ToString().ToLower();
				const FString MimeType = FGenericPlatformHttp::GetMimeType(ResourceName);

				if (MimeType.StartsWith(TEXT("image/")) && FFileHelper::LoadFileToArray(Response->Body, *ResourceName))
				{
					WebRemoteControlInternalUtils::AddContentTypeHeaders(Response.Get(), MimeType);
					Response->Code = EHttpServerResponseCodes::Ok;
				}
			}
		}
	}

	if (!Response->Body.Num())
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(FString::Printf(TEXT("Could not load thumbnail for object %s"), *GetThumbnailRequest.ObjectPath), Response->Body);
		Response->Code = EHttpServerResponseCodes::NotFound;
	}

	OnComplete(MoveTemp(Response));
	return true;
}

void FWebRemoteControlEditorRoutes::AddPendingEvent(FRemoteControlObjectEventHookRequest InRequest, TUniquePtr<FHttpServerResponse> InResponse, FHttpResultCallback OnComplete)
{
	FRCObjectReference ObjectRef;
	FString ErrorText;
	if (IRemoteControlModule::Get().ResolveObject(ERCAccess::READ_ACCESS, InRequest.ObjectPath, InRequest.PropertyName, ObjectRef, &ErrorText))
	{
		FRemoteEventDispatcher& EventDispatcher = EventDispatchers[(int32)InRequest.EventType];
		if (!EventDispatcher.IsValid())
		{
			EventDispatcher.Initialize(InRequest.EventType);
		}
		EventDispatcher.PendingEvents.Emplace(MoveTemp(ObjectRef), MoveTemp(InResponse), MoveTemp(OnComplete));
	}
	else
	{
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(ErrorText, InResponse->Body);
		InResponse->Code = EHttpServerResponseCodes::BadRequest;
		OnComplete(MoveTemp(InResponse));
	}
}

void FWebRemoteControlEditorRoutes::FRemoteEventDispatcher::Initialize(ERemoteControlEvent Type)
{
	Reset();
	DispatcherType = Type;
	switch (DispatcherType)
	{
		case ERemoteControlEvent::PreObjectPropertyChanged:
			DelegateHandle = FCoreUObjectDelegates::OnPreObjectPropertyChanged.AddLambda([this](UObject* InObject, const FEditPropertyChain& PropChain)
				{
					if (PropChain.GetActiveNode() && PropChain.GetActiveNode()->GetValue())
					{
						Dispatch(InObject, PropChain.GetActiveNode()->GetValue());
					}

				});
			break;
		case ERemoteControlEvent::ObjectPropertyChanged:
			DelegateHandle = FCoreUObjectDelegates::OnObjectPropertyChanged.AddLambda([this](UObject* InObject, const FPropertyChangedEvent& PropEvent)
				{
					if (PropEvent.MemberProperty)
					{
						Dispatch(InObject, PropEvent.MemberProperty);
					}

				});
			break;
		default:
			break;
	}
}

void FWebRemoteControlEditorRoutes::FRemoteEventDispatcher::Reset()
{
	if (DelegateHandle.IsValid())
	{
		switch (DispatcherType)
		{
		case ERemoteControlEvent::PreObjectPropertyChanged:
			FCoreUObjectDelegates::OnPreObjectPropertyChanged.Remove(DelegateHandle);
			break;
		case ERemoteControlEvent::ObjectPropertyChanged:
			FCoreUObjectDelegates::OnObjectPropertyChanged.Remove(DelegateHandle);
			break;
		default:
			UE_LOGF(LogRemoteControl, Fatal, "Unsupported remote event hook.");
			break;
		}
		DelegateHandle.Reset();
	}
}

void FWebRemoteControlEditorRoutes::FRemoteEventDispatcher::Dispatch(UObject* InObject, FProperty* InProperty)
{
	for (auto It = PendingEvents.CreateIterator(); It; ++It)
	{
		if (It->ObjectRef.Object == InObject && It->ObjectRef.Property == InProperty)
		{
			SendResponse(*It);
			It.RemoveCurrent();
		}
	}
	if (PendingEvents.Num() == 0)
	{
		Reset();
	}
}

void FWebRemoteControlEditorRoutes::FRemoteEventDispatcher::SendResponse(FRemoteEventHook& EventHook)
{
	TArray<uint8> WorkingBuffer;
	FMemoryWriter Writer(WorkingBuffer);
	FJsonStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);

	if (IRemoteControlModule::Get().GetObjectProperties(EventHook.ObjectRef, SerializerBackend))
	{
		EventHook.Response->Code = EHttpServerResponseCodes::Ok;
		WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, EventHook.Response->Body);
	}
	else
	{
		EventHook.Response->Code = EHttpServerResponseCodes::NoContent;
	}

	EventHook.CompleteCallback(MoveTemp(EventHook.Response));
}
#endif
