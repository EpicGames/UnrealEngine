// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HttpResultCallback.h"
#include "RemoteControlRequest.h"
#include "Modules/ModuleInterface.h"
#include "HttpRouteHandle.h"
#include "IRemoteControlModule.h"
#include "HttpServerResponse.h"
#include "RemoteControlRoute.h"

struct FHttpServerRequest;
class FWebRemoteControlModule;

/**
  * Adds editor-specific functionality to web remote control.
 */
#if WITH_EDITOR
class FWebRemoteControlEditorRoutes
{
public:
	/** Register the editor routes to the web remote control module. */
	void RegisterRoutes(FWebRemoteControlModule* WebRemoteControl);

	/** Unregister the editor routes from the module */
	void UnregisterRoutes(FWebRemoteControlModule* WebRemoteControl);

private:
	/** Structure used to hold an object and to handle responding to the pending HTTP request. */
	struct FRemoteEventHook
	{
		FRemoteEventHook(FRCObjectReference InObjectRef, TUniquePtr<FHttpServerResponse> InResponse, FHttpResultCallback InCompleteCallback)
			: ObjectRef(MoveTemp(InObjectRef))
			, Response(MoveTemp(InResponse))
			, CompleteCallback(MoveTemp(InCompleteCallback))
		{}

		FRCObjectReference ObjectRef;
		TUniquePtr<FHttpServerResponse> Response;
		FHttpResultCallback CompleteCallback;
	};

	/** Structure that handles subscribing to events and dispatching the appropriate HTTP response. */
	struct FRemoteEventDispatcher
	{
		FRemoteEventDispatcher()
			: DispatcherType(ERemoteControlEvent::EventCount)
		{}

		~FRemoteEventDispatcher()
		{
			Reset();
		}

		bool IsValid() const
		{
			return DelegateHandle.IsValid();
		}

		void Initialize(ERemoteControlEvent Type);

		void Reset();

		void Dispatch(UObject* InObject, FProperty* InProperty);

		void SendResponse(FRemoteEventHook& EventHook);

		ERemoteControlEvent DispatcherType;
		FDelegateHandle DelegateHandle;
		TArray<FRemoteEventHook> PendingEvents;
	};

	/** Create an event dispatcher that waits until a desired event occurs before returning an http response. */
	void AddPendingEvent(FRemoteControlObjectEventHookRequest InRequest, TUniquePtr<FHttpServerResponse> InResponse, FHttpResultCallback OnComplete);

	// Route handlers 
	bool HandleObjectEventRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);
	bool HandleGetThumbnailRoute(const FHttpServerRequest& Request, const FHttpResultCallback& OnComplete);

private:
	/** Remote event mechanism delegate handles. */
	TArray<FRemoteEventDispatcher> EventDispatchers;
	/** Holds all the editor routes. */
	TArray<FRemoteControlRoute> Routes;
};
#else
class FWebRemoteControlEditorRoutes
{
public:
	void RegisterRoutes(FWebRemoteControlModule*) {}
	void UnregisterRoutes(FWebRemoteControlModule*) {}
};
#endif

