// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRemoteControlExternalLogger.h"
#include "WebSocketNetworkingDelegates.h"
#include "WebRemoteControlInternalUtils.h"
#include "INetworkingWebSocket.h"


FWebRemoteControlExternalLogger::FWebRemoteControlExternalLogger(TSharedPtr<INetworkingWebSocket> WebSocketConnectionIn) :
	WebSocketConnection(WebSocketConnectionIn)
{
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FWebRemoteControlExternalLogger::OnTick));

	FWebSocketInfoCallBack ConnectedCallback;
	ConnectedCallback.BindRaw(this, &FWebRemoteControlExternalLogger::OnConnected);
	WebSocketConnection->SetConnectedCallBack(ConnectedCallback);

	FWebSocketInfoCallBack ClosedCallback;
	ClosedCallback.BindRaw(this, &FWebRemoteControlExternalLogger::OnClosed);
	WebSocketConnection->SetSocketClosedCallBack(ClosedCallback);
}

FWebRemoteControlExternalLogger::~FWebRemoteControlExternalLogger()
{
	FTSTicker::RemoveTicker(TickerHandle);
	TickerHandle.Reset();
}

void FWebRemoteControlExternalLogger::Log(int32 RequestId, const TCHAR* Stage)
{
	if (IsConnected)
	{
		const FString Json = FString::Printf(TEXT("{\"RequestId\":%d,\"Frame\":%d,\"UETime\":%f,\"Stage\":\"%s\"}"), RequestId, GFrameNumber, FPlatformTime::Seconds(), Stage);

		TArray<uint8> Buffer;
		WebRemoteControlUtils::ConvertToUTF8(Json, Buffer);
		WebSocketConnection->Send(Buffer.GetData(), Buffer.Num(), false);
	}
}

bool FWebRemoteControlExternalLogger::OnTick(float DeltaSeconds)
{
	WebSocketConnection->Tick();
	return true;
}

void FWebRemoteControlExternalLogger::OnConnected()
{
	IsConnected = true;
}

void FWebRemoteControlExternalLogger::OnClosed()
{
	IsConnected = false;
}
