// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Templates/SharedPointer.h"


class FWebRemoteControlExternalLogger
{
public:
	FWebRemoteControlExternalLogger(TSharedPtr<class INetworkingWebSocket> WebSocketConnection);
	virtual ~FWebRemoteControlExternalLogger();

	void Log(int32 RequestId, const TCHAR* Stage);

private:
	bool OnTick(float DeltaSeconds);
	void OnConnected();
	void OnClosed();

	bool IsConnected = false;

	/** A WebSocket connection to the logger server*/
	TSharedPtr<class INetworkingWebSocket> WebSocketConnection;

	/** A handle to the ticker callback */
	FTSTicker::FDelegateHandle TickerHandle;
};