// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRemoteControlInternalUtils.h"

#include "HttpServerRequest.h"
#include "Kismet/KismetSystemLibrary.h"
#include "Misc/Base64.h"
#include "PlatformHttp.h"
#include "RCAllowedRemoteFunctionCallUtilities.h"
#include "RemoteControlSettings.h"
#include "Serialization/JsonReader.h"
#include "UObject/StructOnScope.h"

namespace RemotePayloadSerializer
{
FName NAME_Get = "GET";
FName NAME_Put = "PUT";
FName NAME_Post = "POST";
FName NAME_Patch = "PATCH";
FName NAME_Delete = "DELETE";
FName NAME_Options = "OPTIONS";

typedef WIDECHAR PayloadCharType;
const FName ReturnValuePropName(TEXT("ReturnValue"));

void ReplaceFirstOccurence(TConstArrayView<uint8> InPayload, const FString& From, const FString& To, TArray<uint8>& OutModifiedPayload)
{
	if (TCHAR* PropValuePtr = FCString::Stristr((TCHAR*)InPayload.GetData(), *From))
	{
		OutModifiedPayload.Reserve(InPayload.Num() + (To.Len() - (From.Len()) * sizeof(TCHAR)));

		int32 StartIndex = UE_PTRDIFF_TO_INT32(PropValuePtr - (TCHAR*)InPayload.GetData()) * sizeof(TCHAR);
		int32 EndIndex = StartIndex + From.Len() * sizeof(TCHAR);
		TConstArrayView<uint8> RestOfPayload = InPayload.Slice(EndIndex, InPayload.Num() - EndIndex);
		OutModifiedPayload.Append(InPayload.GetData(), StartIndex);
		OutModifiedPayload.Append((uint8*)*To, To.Len() * sizeof(TCHAR));
		OutModifiedPayload.Append(RestOfPayload.GetData(), RestOfPayload.Num());
	}
}

EHttpServerRequestVerbs ParseHttpVerb(FName InVerb)
{
	if (InVerb == NAME_Get)
	{
		return EHttpServerRequestVerbs::VERB_GET;
	}
	else if (InVerb == NAME_Post)
	{
		return EHttpServerRequestVerbs::VERB_POST;
	}
	else if (InVerb == NAME_Put)
	{
		return EHttpServerRequestVerbs::VERB_PUT;
	}
	else if (InVerb == NAME_Patch)
	{
		return EHttpServerRequestVerbs::VERB_PATCH;
	}
	else if (InVerb == NAME_Delete)
	{
		return EHttpServerRequestVerbs::VERB_DELETE;
	}
	else if (InVerb == NAME_Options)
	{
		return EHttpServerRequestVerbs::VERB_OPTIONS;
	}
	else
	{
		return EHttpServerRequestVerbs::VERB_NONE;
	}
}

TSharedRef<FHttpServerRequest> UnwrapHttpRequest(const FRCRequestWrapper& Wrapper, const FHttpServerRequest* TemplateRequest)
{
	TSharedRef<FHttpServerRequest> WrappedHttpRequest = MakeShared<FHttpServerRequest>();
	if (TemplateRequest)
	{
		WrappedHttpRequest->HttpVersion = TemplateRequest->HttpVersion;
		WrappedHttpRequest->Headers = TemplateRequest->Headers;
	}
	else
	{
		// Assign Request defaults.
		WrappedHttpRequest->HttpVersion = HttpVersion::EHttpServerHttpVersion::HTTP_VERSION_1_1;
		WrappedHttpRequest->Headers.Add(TEXT("Content-Type"), { TEXT("application/json") });
	}

	WebRemoteControlInternalUtils::AddWrappedRequestHeader(*WrappedHttpRequest);
	WrappedHttpRequest->Verb = ParseHttpVerb(Wrapper.Verb);
	WrappedHttpRequest->Body = Wrapper.TCHARBody;
	WrappedHttpRequest->Headers.Add(WebRemoteControlInternalUtils::PassphraseHeader, { Wrapper.Passphrase });

	// Parse query parameters from URL, if any
	int32 QueryParamsIndex = 0;
	if (Wrapper.URL.FindChar(TCHAR('?'), QueryParamsIndex))
	{
		FString PathWithoutParams = Wrapper.URL;

		FString QueryParamsStr = PathWithoutParams.Mid(QueryParamsIndex + 1);
		PathWithoutParams.MidInline(0, QueryParamsIndex, EAllowShrinking::No);

		// Split query params
		TArray<FString> QueryParamPairs;
		QueryParamsStr.ParseIntoArray(QueryParamPairs, TEXT("&"), true);
		for (const FString& QueryParamPair : QueryParamPairs)
		{
			int32 Equalsindex = 0;
			if (QueryParamPair.FindChar(TCHAR('='), Equalsindex))
			{
				FString QueryParamKey = FPlatformHttp::UrlDecode(QueryParamPair.Mid(0, Equalsindex));
				FString QueryParamValue = FPlatformHttp::UrlDecode(QueryParamPair.Mid(Equalsindex + 1));
				WrappedHttpRequest->QueryParams.Emplace(MoveTemp(QueryParamKey), MoveTemp(QueryParamValue));
			}
		}

		WrappedHttpRequest->RelativePath = PathWithoutParams;
	}
	else
	{
		WrappedHttpRequest->RelativePath = Wrapper.URL;
	}

	return WrappedHttpRequest;
}

void SerializeWrappedCallResponse(int32 RequestId, TUniquePtr<FHttpServerResponse> Response, FMemoryWriter& Writer)
{
	FRCJsonStructSerializerBackend Backend(Writer, FRCJsonStructSerializerBackend::DefaultSerializerFlags);
	const TSharedPtr<TJsonWriter<ANSICHAR>> JsonWriter = TJsonWriter<ANSICHAR>::Create(&Writer);
	const TArray<FString>* ContentTypeHeaders = Response->Headers.Find(TEXT("Content-Type"));
	const bool bIsBinaryData = ContentTypeHeaders && ContentTypeHeaders->ContainsByPredicate([](const FString& InHeader) { return InHeader.StartsWith(TEXT("image/"));}); 

	JsonWriter->WriteObjectStart();
	JsonWriter->WriteValue(TEXT("RequestId"), RequestId);
	JsonWriter->WriteValue(TEXT("ResponseCode"), static_cast<int32>(Response->Code));
	JsonWriter->WriteIdentifierPrefix(TEXT("ResponseBody"));

	if (Response->Body.Num())
	{
		if (!bIsBinaryData)
		{
			Writer.Serialize((void*)Response->Body.GetData(), Response->Body.Num());
		}
		else
		{
			FString Base64String = FString::Printf(TEXT("\"%s\""), *FBase64::Encode(Response->Body));
			TArray<uint8> WorkingBuffer;
			WebRemoteControlUtils::ConvertToUTF8(Base64String, WorkingBuffer);
			Writer.Serialize((void*)WorkingBuffer.GetData(), WorkingBuffer.Num());
		}
	}
	else
	{
		JsonWriter->WriteNull();
	}

	JsonWriter->WriteObjectEnd();
}

bool DeserializeCall(const FHttpServerRequest& InRequest, FRCCall& OutCall, const FHttpResultCallback& InCompleteCallback)
{
	// Create Json reader to read the payload, the payload will already be validated as being in TCHAR
	FRCCallRequest CallRequest;
	if (!WebRemoteControlInternalUtils::DeserializeRequest(InRequest, &InCompleteCallback, CallRequest))
	{
		return false;
	}

	FString ErrorText;
	bool bSuccess = true;
	if (IRemoteControlModule::Get().ResolveCall(CallRequest.ObjectPath, CallRequest.FunctionName, OutCall.CallRef, &ErrorText))
	{
		// Initialize the param struct with default parameters
		OutCall.TransactionMode = CallRequest.GenerateTransaction ? ERCTransactionMode::AUTOMATIC : ERCTransactionMode::NONE;
		OutCall.ParamStruct = FStructOnScope(OutCall.CallRef.Function.Get());

		// If some parameters were provided, deserialize them
		const FBlockDelimiters& ParametersDelimiters = CallRequest.GetStructParameters().FindChecked(FRCCallRequest::ParametersLabel());
		if (ParametersDelimiters.BlockStart > 0)
		{
			FMemoryReader Reader(CallRequest.TCHARBody);
			Reader.Seek(ParametersDelimiters.BlockStart);
			Reader.SetLimitSize(ParametersDelimiters.BlockEnd);

			FRCJsonStructDeserializerBackend Backend(Reader);
			if (FStructDeserializer::Deserialize((void*)OutCall.ParamStruct.GetStructMemory(), *const_cast<UStruct*>(OutCall.ParamStruct.GetStruct()), Backend, FStructDeserializerPolicies()))
			{
				if (!WebRemoteControlInternalUtils::ValidateFunctionCall(OutCall, &ErrorText))
				{
					bSuccess = false;
				}
			}
			else
			{
				ErrorText = TEXT("Parameters object improperly formatted.");
				bSuccess = false;
			}
		}
	}
	else
	{
		bSuccess = false;
	}

	if (!bSuccess)
	{
		const FString FormattedMsg = FString::Printf(TEXT("Web Remote Call deserialization error: %s"), *ErrorText);
		UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
		IRemoteControlModule::BroadcastError(FormattedMsg);

		TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(ErrorText, Response->Body);
		InCompleteCallback(MoveTemp(Response));
	}

	return bSuccess;
}

bool SerializeCall(const FRCCall& InCall, TArray<uint8>& OutPayload, bool bOnlyReturn)
{
	FMemoryWriter Writer(OutPayload);
	TSharedPtr<TJsonWriter<PayloadCharType>> JsonWriter;

	if (!bOnlyReturn)
	{
		// Create Json writer to write the payload, if we do not just write the return value back.
		JsonWriter = TJsonWriter<PayloadCharType>::Create(&Writer);

		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("ObjectPath"), InCall.CallRef.Object->GetPathName());
		JsonWriter->WriteValue(TEXT("FunctionName"), InCall.CallRef.Function->GetFName().ToString());
		JsonWriter->WriteIdentifierPrefix(TEXT("Parameters"));
	}

	// write the param struct
	FRCJsonStructSerializerBackend Backend(Writer, FRCJsonStructSerializerBackend::DefaultSerializerFlags);
	FStructSerializerPolicies Policies;

	if (bOnlyReturn)
	{
		Policies.PropertyFilter = [](const FProperty* CurrentProp, const FProperty* ParentProperty) -> bool
		{
			return CurrentProp->HasAnyPropertyFlags(CPF_ReturnParm | CPF_OutParm) || ParentProperty != nullptr;
		};
	}

	FStructSerializer::Serialize((void*)InCall.ParamStruct.GetStructMemory(), *const_cast<UStruct*>(InCall.ParamStruct.GetStruct()), Backend, Policies);

	if (!bOnlyReturn)
	{
		JsonWriter->WriteObjectEnd();
	}

	return true;
}

bool DeserializeObjectRef(const FHttpServerRequest& InRequest, FRCObjectReference& OutObjectRef, FRCObjectRequest& OutDeserializedRequest, const FHttpResultCallback& InCompleteCallback)
{
	if (!WebRemoteControlInternalUtils::DeserializeRequest(InRequest, &InCompleteCallback, OutDeserializedRequest))
	{
		return false;
	}

	FString ErrorText;

	// If we properly identified the object path, property name and access type as well as identified the starting / end position for property value
	// resolve the object reference
	IRemoteControlModule::Get().ResolveObject(OutDeserializedRequest.GetAccessValue(), OutDeserializedRequest.ObjectPath, OutDeserializedRequest.PropertyName, OutObjectRef, &ErrorText);

	if (!ErrorText.IsEmpty())
	{
		const FString FormattedMsg = FString::Printf(TEXT("Web Remote Object Access error: %s"), *ErrorText);
		UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
		IRemoteControlModule::BroadcastError(FormattedMsg);

		TUniquePtr<FHttpServerResponse> Response = WebRemoteControlInternalUtils::CreateHttpResponse();
		WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(ErrorText, Response->Body);
		InCompleteCallback(MoveTemp(Response));
		return false;
	}

	return true;
}
}

TUniquePtr<FHttpServerResponse> WebRemoteControlInternalUtils::CreateHttpResponse(EHttpServerResponseCodes InResponseCode)
{
	TUniquePtr<FHttpServerResponse> Response = MakeUnique<FHttpServerResponse>();
	AddCORSHeaders(Response.Get());
	AddContentTypeHeaders(Response.Get(), TEXT("application/json"));
	Response->Code = InResponseCode;
	return Response;
}

TUniquePtr<FHttpServerResponse> WebRemoteControlInternalUtils::CreatedInvalidPassphraseResponse()
{
	TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse(EHttpServerResponseCodes::Denied);
	CreateUTF8ErrorMessage(InvalidPassphraseError, Response->Body);
	return Response;
}

void WebRemoteControlInternalUtils::CreateUTF8ErrorMessage(const FString& InMessage, TArray<uint8>& OutUTF8Message)
{
	WebRemoteControlUtils::ConvertToUTF8(FString::Printf(TEXT("{ \"errorMessage\": \"%s\" }"), *InMessage), OutUTF8Message);
}

bool WebRemoteControlInternalUtils::GetStructParametersDelimiters(TConstArrayView<uint8> InTCHARPayload, TMap<FString, FBlockDelimiters>& InOutStructParameters, FString* OutErrorText)
{
	typedef WIDECHAR PayloadCharType;
	FMemoryReaderView Reader(InTCHARPayload);
	TSharedRef<TJsonReader<PayloadCharType>> JsonReader = TJsonReader<PayloadCharType>::Create(&Reader);

	EJsonNotation Notation;

	FString ErrorText;
	// The payload should be an object
	JsonReader->ReadNext(Notation);
	if (Notation != EJsonNotation::ObjectStart)
	{
		ErrorText = TEXT("Expected json object.");
	}

	// Mark the start/end of the param object in the payload
	while (JsonReader->ReadNext(Notation) && ErrorText.IsEmpty())
	{
		switch (Notation)
		{
			// this should mean we reached the parameters field, record the start and ending offset
		case EJsonNotation::ObjectStart:
		case EJsonNotation::ArrayStart:
			if (FBlockDelimiters* Delimiters = InOutStructParameters.Find(JsonReader->GetIdentifier()))
			{
				Delimiters->BlockStart = Reader.Tell() - sizeof(PayloadCharType);

				auto Skip = [&JsonReader, &Notation]() { return Notation == EJsonNotation::ObjectStart ? JsonReader->SkipObject() : JsonReader->SkipArray(); };
				if (Skip())
				{
					Delimiters->BlockEnd = Reader.Tell();
				}
				else
				{
					ErrorText = FString::Printf(TEXT("%s object improperly formatted."), *JsonReader->GetIdentifier());
				}
			}
			break;
		case EJsonNotation::Error:
			ErrorText = JsonReader->GetErrorMessage();
			break;
		default:
			// Ignore any other fields
			break;
		}
	}

	if (!ErrorText.IsEmpty())
	{
		const FString FormattedMsg = FString::Printf(TEXT("Web Remote Control deserialization error: %s"), *ErrorText);
		UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
		IRemoteControlModule::BroadcastError(FormattedMsg);

		if (OutErrorText)
		{
			*OutErrorText = MoveTemp(ErrorText);
		}
		return false;
	}

	return true;
}

bool WebRemoteControlInternalUtils::GetBatchRequestStructDelimiters(TConstArrayView<uint8> InTCHARPayload, TMap<int32, FBlockDelimiters>& OutStructParameters, FString* OutErrorText)
{
	typedef WIDECHAR PayloadCharType;
	FMemoryReaderView Reader(InTCHARPayload);
	TSharedRef<TJsonReader<PayloadCharType>> JsonReader = TJsonReader<PayloadCharType>::Create(&Reader);

	EJsonNotation Notation;

	FString ErrorText;
	// The payload should be an object
	JsonReader->ReadNext(Notation);
	if (Notation != EJsonNotation::ObjectStart)
	{
		ErrorText = TEXT("Expected json object.");
	}

	bool bHasEncounteredError = false;
	int32 LastEncounteredRequestId = -1;

	// Mark the start/end of the param object in the payload
	while (!bHasEncounteredError && JsonReader->ReadNext(Notation))
	{
		switch (Notation)
		{

		case EJsonNotation::Number:
			if (JsonReader->GetIdentifier() == TEXT("RequestId"))
			{
				LastEncounteredRequestId = JsonReader->GetValueAsNumber();
			}
			break;
		case EJsonNotation::ObjectStart:
			if (JsonReader->GetIdentifier() == TEXT("Body"))
			{
				FBlockDelimiters& Delimiters = OutStructParameters.Add(LastEncounteredRequestId, FBlockDelimiters{});
				Delimiters.BlockStart = Reader.Tell() - sizeof(PayloadCharType);
				if (JsonReader->SkipObject())
				{
					Delimiters.BlockEnd = Reader.Tell();
				}
				else
				{
					ErrorText = FString::Printf(TEXT("%s object improperly formatted."), *JsonReader->GetIdentifier());
				}
			}
			break;
		case EJsonNotation::Error:
			bHasEncounteredError = true;
			ErrorText = JsonReader->GetErrorMessage();
			break;
		default:
			// Ignore any other fields
			break;
		}
	}

	if (!ErrorText.IsEmpty())
	{
		const FString FormattedMsg = FString::Printf(TEXT("Web Remote Control deserialization error: %s"), *ErrorText);
		UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
		IRemoteControlModule::BroadcastError(FormattedMsg);

		if (OutErrorText)
		{
			*OutErrorText = MoveTemp(ErrorText);
		}
		return false;
	}

	return true;
}

bool WebRemoteControlInternalUtils::GetBatchWebSocketRequestStructDelimiters(TConstArrayView<uint8> InTCHARPayload, TArray<FBlockDelimiters>& OutStructParameters, FString* OutErrorText)
{
	typedef WIDECHAR PayloadCharType;
	FMemoryReaderView Reader(InTCHARPayload);
	TSharedRef<TJsonReader<PayloadCharType>> JsonReader = TJsonReader<PayloadCharType>::Create(&Reader);

	EJsonNotation Notation;

	FString ErrorText;
	// The payload should be an object
	JsonReader->ReadNext(Notation);
	if (Notation != EJsonNotation::ObjectStart)
	{
		ErrorText = TEXT("Expected json object.");
	}

	bool bHasEncounteredError = false;
	bool bIsInRequestsArray = false;
	bool bIsDoneRequestsArray = false;
	const TCHAR* const RequestsIdentifier = TEXT("Requests");

	// Mark the start/end of the param object in the payload
	while (!bHasEncounteredError && JsonReader->ReadNext(Notation) && !bIsDoneRequestsArray)
	{
		switch (Notation)
		{
		case EJsonNotation::ArrayStart:
			if (JsonReader->GetIdentifier() == RequestsIdentifier)
			{
				bIsInRequestsArray = true;
			}
			break;

		case EJsonNotation::ArrayEnd:
			if (JsonReader->GetIdentifier() == RequestsIdentifier)
			{
				bIsDoneRequestsArray = true;
			}
			break;

		case EJsonNotation::ObjectStart:
			if (bIsInRequestsArray && JsonReader->GetIdentifier() == TEXT("Parameters"))
			{
				FBlockDelimiters& Delimiters = OutStructParameters.Emplace_GetRef();
				Delimiters.BlockStart = Reader.Tell() - sizeof(PayloadCharType);
				if (JsonReader->SkipObject())
				{
					Delimiters.BlockEnd = Reader.Tell();
				}
				else
				{
					ErrorText = FString::Printf(TEXT("%s object improperly formatted."), *JsonReader->GetIdentifier());
				}
			}
			break;

		case EJsonNotation::Error:
			bHasEncounteredError = true;
			ErrorText = JsonReader->GetErrorMessage();
			break;

		default:
			// Ignore any other fields
			break;
		}
	}

	if (!ErrorText.IsEmpty())
	{
		const FString FormattedMsg = FString::Printf(TEXT("Web Remote Control deserialization error: %s"), *ErrorText);
		UE_LOGF(LogRemoteControl, Error, "%ls", *FormattedMsg);
		IRemoteControlModule::BroadcastError(FormattedMsg);
		
		if (OutErrorText)
		{
			*OutErrorText = MoveTemp(ErrorText);
		}
		return false;
	}

	return true;
}

bool WebRemoteControlInternalUtils::ValidateContentType(const FHttpServerRequest& InRequest, FString InContentType, const FHttpResultCallback& InCompleteCallback)
{
	FString ErrorText;
	if (!IsRequestContentType(InRequest, MoveTemp(InContentType), &ErrorText))
	{
		TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
		CreateUTF8ErrorMessage(ErrorText, Response->Body);
		InCompleteCallback(MoveTemp(Response));
		return false;
	}
	return true;
}

bool WebRemoteControlInternalUtils::ValidateFunctionCall(const FRCCall& InRCCall, FString* OutErrorText)
{
	using namespace UE::RemoteControl;

	if (!InRCCall.IsValid())
	{
		return false;
	}

	const URemoteControlSettings* const Settings = GetDefault<URemoteControlSettings>();
	check(Settings);

	if (!Settings->bAllowAnyRemoteFunctionCall)
	{
		const FRCRemoteFunctionCallParams RCCallParameters = MakeRemoteFunctionCallParams(InRCCall.CallRef.Object.Get(), InRCCall.CallRef.Function.Get());

		const bool bIsAllowed = IsRemoteFunctionCallAllowed(RCCallParameters, Settings->AllowedRemoteFunctionCalls)
			|| IsRemoteFunctionCallAllowed(RCCallParameters, Settings->CustomAllowedRemoteFunctionCalls);

		if (!bIsAllowed)
		{
			if (OutErrorText)
			{
				*OutErrorText = FString::Printf(TEXT("Executing function '%s %s' is not allowed by remote control settings. (see 'Custom Allowed Remote Function Calls' or 'Allow Any Remote Function Call')")
					, *GetNameSafe(RCCallParameters.ObjectClass)
					, *RCCallParameters.FunctionName);
			}
			return false;
		}
	}

	UFunction* ExecuteControlCommandMethod = UKismetSystemLibrary::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UKismetSystemLibrary, ExecuteConsoleCommand));
	
	if (InRCCall.CallRef.Function == ExecuteControlCommandMethod)
	{
		if (!Settings->bAllowConsoleCommandRemoteExecution)
		{
			if (OutErrorText)
			{
				*OutErrorText = TEXT("Executing console commands remotely is not enabled in the remote control settings.");
			}

			return false;
		}

		// Make sure python is not called through KismetSystemLibrary.
		if (!Settings->bEnableRemotePythonExecution)
		{
			if (const FProperty* Property = InRCCall.CallRef.Function->FindPropertyByName("Command"))
			{
				FString FullCommand;
				Property->GetValue_InContainer(InRCCall.ParamStruct.GetStructMemory(), &FullCommand);

				TArray<FString> SplitCommands;
				FullCommand.ParseIntoArray(SplitCommands, TEXT("|"));

				for (FString& Command : SplitCommands)
				{
					Command.TrimStartAndEndInline();
					if (Command.StartsWith(TEXT("py ")) || Command.StartsWith(TEXT("python ")))
					{
						if (OutErrorText)
						{
							*OutErrorText = TEXT("Executing Python remotely is not enabled in the remote control settings.");
						}
						return false;
					}
				}
			}
			else
			{
				if (OutErrorText)
				{
					*OutErrorText = TEXT("Could not find the Command parameter on ExecuteConsoleCommand");
				}

				return false;
			}
		}
	}

	return true;
}

void WebRemoteControlInternalUtils::AddContentTypeHeaders(FHttpServerResponse* InOutResponse, FString InContentType)
{
	InOutResponse->Headers.Add(TEXT("content-type"), { MoveTemp(InContentType) });
}

void WebRemoteControlInternalUtils::AddWrappedRequestHeader(FHttpServerRequest& Request)
{
	Request.Headers.Add(WrappedRequestHeader, {});
}

bool WebRemoteControlInternalUtils::IsWrappedRequest(const FHttpServerRequest& Request)
{
	return Request.Headers.Contains(WrappedRequestHeader);
}

void WebRemoteControlInternalUtils::AddCORSHeaders(FHttpServerResponse* InOutResponse)
{
	check(InOutResponse != nullptr);
	if (GetDefault<URemoteControlSettings>()->bRestrictServerAccess)
	{
		InOutResponse->Headers.Add(TEXT("Access-Control-Allow-Origin"), { GetDefault<URemoteControlSettings>()->AllowedOrigin });
	}
	else
	{
		InOutResponse->Headers.Add(TEXT("Access-Control-Allow-Origin"), { TEXT("*") });
	}
	
	InOutResponse->Headers.Add(TEXT("Access-Control-Allow-Methods"), { TEXT("PUT, POST, GET, OPTIONS") });
	InOutResponse->Headers.Add(TEXT("Access-Control-Allow-Headers"), { TEXT("Origin, X-Requested-With, Content-Type, Accept") });
	InOutResponse->Headers.Add(TEXT("Access-Control-Max-Age"), { TEXT("600") });
}

bool WebRemoteControlInternalUtils::IsRequestContentType(const FHttpServerRequest& InRequest, const FString& InContentType, FString* OutErrorText)
{
	if (const TArray<FString>* ContentTypeHeaders = InRequest.Headers.Find(TEXT("Content-Type")))
	{
		if (ContentTypeHeaders->Num() > 0)
		{
			return (*ContentTypeHeaders)[0].StartsWith(InContentType);
		}
	}

	const FString ErrorText = FString::Printf(TEXT("Request content type must be %s"), *InContentType);
	IRemoteControlModule::Get().BroadcastError(ErrorText);

	if (OutErrorText)
	{
		*OutErrorText = ErrorText;
	}
	
	return false;
}

bool WebRemoteControlInternalUtils::CheckPassphrase(const FString& HashedPassphrase)
{
	bool bOutResult = !(GetDefault<URemoteControlSettings>()->bEnforcePassphraseForRemoteClients) || !(GetDefault<URemoteControlSettings>()->bRestrictServerAccess);

	if (bOutResult)
	{
		return true;
	}

	TArray<FString> HashedPassphrases = GetDefault<URemoteControlSettings>()->GetHashedPassphrases();
	if (HashedPassphrases.IsEmpty())
	{
		return true;
	}

	for (const FString& InPassphrase : HashedPassphrases)
	{
		bOutResult = bOutResult || InPassphrase == HashedPassphrase;

		if (bOutResult)
		{
			break;
		}
	}

	return bOutResult;
}
