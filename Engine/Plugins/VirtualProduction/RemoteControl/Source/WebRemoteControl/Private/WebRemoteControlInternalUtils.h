// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IStructSerializerBackend.h"
#include "Serialization/RCJsonStructSerializerBackend.h"
#include "Serialization/RCJsonStructDeserializerBackend.h"
#include "HttpServerResponse.h"
#include "HttpServerRequest.h"
#include "Serialization/MemoryReader.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"
#include "RemoteControlRequest.h"
#include "Templates/UnrealTypeTraits.h"
#include "WebRemoteControlUtils.h"
#include "WebSocketMessageHandler.h"

namespace RemotePayloadSerializer
{
	/**
	 * Replaces the first occurrence of a string in a TCHAR binary payload.
	 */
	void ReplaceFirstOccurence(TConstArrayView<uint8> InPayload, const FString& From, const FString& To, TArray<uint8>& OutModifiedPayload);

	/**
	 * Converts a string verb to the enum representation.
	 */
	EHttpServerRequestVerbs ParseHttpVerb(FName InVerb);

	/**
	 * Unwrap a request wrapper, while copying headers and http version from a template request if available.
	 */
	TSharedRef<FHttpServerRequest> UnwrapHttpRequest(const FRCRequestWrapper& Wrapper, const FHttpServerRequest* TemplateRequest = nullptr);

	/**
	 * @note This will serialize in ANSI directly.
	 */
	void SerializeWrappedCallResponse(int32 RequestId, TUniquePtr<FHttpServerResponse> Response, FMemoryWriter& Writer);

	bool DeserializeCall(const FHttpServerRequest& InRequest, FRCCall& OutCall, const FHttpResultCallback& InCompleteCallback);

	bool SerializeCall(const FRCCall& InCall, TArray<uint8>& OutPayload, bool bOnlyReturn = false);

	bool DeserializeObjectRef(const FHttpServerRequest& InRequest, FRCObjectReference& OutObjectRef, FRCObjectRequest& OutDeserializedRequest, const FHttpResultCallback& InCompleteCallback);
}


namespace WebRemoteControlInternalUtils
{
	static const TCHAR* WrappedRequestHeader = TEXT("UE-Wrapped-Request");
	static const FString PassphraseHeader = TEXT("Passphrase");
	static const TCHAR* OriginHeader = TEXT("Origin");
	static const TCHAR* ForwardedIPHeader = TEXT("x-forwarded-for");
	static const TCHAR* InvalidPassphraseError = TEXT("Given Passphrase is not correct!");

	/**
	 * Construct a default http response with CORS headers.
	 * @param InResponseCode The response's code. (Defaults to a bad request)
	 * @return The constructed server response.
	 */
	TUniquePtr<FHttpServerResponse> CreateHttpResponse(EHttpServerResponseCodes InResponseCode = EHttpServerResponseCodes::BadRequest);

	/**
	 * Create a http response for a request denied because of an invalid passphrase.
	 */
	TUniquePtr<FHttpServerResponse> CreatedInvalidPassphraseResponse();

	/**
	 * Create a json structure containing an error message.
	 * @param InMessage The desired error message.
	 * @param OutUTF8Message The error message wrapped in a json struct, in UTF-8 format.
	 * Example output:
	 *	{
	 *	  errorMessage: "Request content type must be application/json" 
	 *	}
	 */
	void CreateUTF8ErrorMessage(const FString& InMessage, TArray<uint8>& OutUTF8Message);

	/**
	 * Deserialize a json structure to find the start and end of every struct parameter in the request.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param InOutStructParameters A map of struct parameter names to the start and end of the struct in the payload.
	 * @param OutErrorText If set, the string pointer will be populated with an error message on error.
	 * @return Whether the deserialization was successful.
	 */
	bool GetStructParametersDelimiters(TConstArrayView<uint8> InTCHARPayload, TMap<FString, FBlockDelimiters>& InOutStructParameters, FString* OutErrorText = nullptr);

	/**
	 * Deserialize a request into a UStruct.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param InCompleteCallback The callback to call error.
	 * @param The structure to serialize using the request's content.
	 * @return Whether the deserialization was successful.
	 *
	 * @note InCompleteCallback will be called with an appropriate http response if the deserialization fails.
	 */
	template <typename RequestType>
	[[nodiscard]] bool DeserializeRequestPayload(TConstArrayView<uint8> InTCHARPayload, const FHttpResultCallback* InCompleteCallback, RequestType& OutDeserializedRequest)
	{
		FMemoryReaderView Reader(InTCHARPayload);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);
		if (!FStructDeserializer::Deserialize(&OutDeserializedRequest, *RequestType::StaticStruct(), DeserializerBackend, FStructDeserializerPolicies()))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to deserialize request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		if (!GetStructParametersDelimiters(InTCHARPayload, OutDeserializedRequest.GetStructParameters(), nullptr))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to deserialize request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		return true;
	}

	/**
	 * Deserialize a wrapped request into a wrapper struct.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param InCompleteCallback The callback to call error.
	 * @param The wrapper structure to populate with the request's content.
	 * @return Whether the deserialization was successful.
	 */
	[[nodiscard]] inline bool DeserializeWrappedRequestPayload(TConstArrayView<uint8> InTCHARPayload, const FHttpResultCallback* InCompleteCallback, FRCRequestWrapper& Wrapper)
	{
		if (!DeserializeRequestPayload(InTCHARPayload, InCompleteCallback, Wrapper))
		{
			return false;
		}

		FBlockDelimiters& BodyDelimiters = Wrapper.GetParameterDelimiters(FRCRequestWrapper::BodyLabel());
		if (BodyDelimiters.BlockStart != BodyDelimiters.BlockEnd)
		{
			Wrapper.TCHARBody = InTCHARPayload.Slice(BodyDelimiters.BlockStart, BodyDelimiters.BlockEnd - BodyDelimiters.BlockStart);
		}

		return true;
	}

	/**
	 * Get the struct delimiters for all the batched requests.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param OutStructParameters A mapping of Request Id to their respective struct delimiters.
	 * @param OutErrorText If set, the string pointer will be populated with an error message on error.
	 * @return Whether the delimiters were able to be found.
	 */
	[[nodiscard]] bool GetBatchRequestStructDelimiters(TConstArrayView<uint8> InTCHARPayload, TMap<int32, FBlockDelimiters>& OutStructParameters, FString* OutErrorText = nullptr);

	/**
	 * Get the struct delimiters for all the batched WebSocket requests.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param OutStructParameters Delimiters for each request.
	 * @param OutErrorText If set, the string pointer will be populated with an error message on error.
	 * @return Whether the delimiters were able to be found.
	 */
	[[nodiscard]] bool GetBatchWebSocketRequestStructDelimiters(TConstArrayView<uint8> InTCHARPayload, TArray<FBlockDelimiters>& OutStructParameters, FString* OutErrorText = nullptr);
	
	/**
	 * Specialization of DeserializeRequestPayload that handles Batch requests.
	 * This will populate the TCHARBody of all the wrapped requests.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param InCompleteCallback The callback to call error.
	 * @param The structure to serialize using the request's content.
	 * @return Whether the deserialization was successful.
	 *
	 * @note InCompleteCallback will be called with an appropriate http response if the deserialization fails.
	 */
	template <>
	[[nodiscard]] inline bool DeserializeRequestPayload(TConstArrayView<uint8> InTCHARPayload, const FHttpResultCallback* InCompleteCallback, FRCBatchRequest& OutDeserializedRequest)
	{
		FMemoryReaderView Reader(InTCHARPayload);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);
		
		if (!FStructDeserializer::Deserialize(&OutDeserializedRequest, *FRCBatchRequest::StaticStruct(), DeserializerBackend, FStructDeserializerPolicies()))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to deserialize request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		TMap<int32, FBlockDelimiters> Delimiters;
		if (!GetBatchRequestStructDelimiters(InTCHARPayload, Delimiters, nullptr))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to get struct delimiters for batch request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		for (FRCRequestWrapper& Wrapper : OutDeserializedRequest.Requests)
		{
			if (FBlockDelimiters* BodyDelimiters = Delimiters.Find(Wrapper.RequestId))
			{
				Wrapper.GetParameterDelimiters(FRCRequestWrapper::BodyLabel()) = MoveTemp(*BodyDelimiters);
				Wrapper.TCHARBody = InTCHARPayload.Slice(BodyDelimiters->BlockStart, BodyDelimiters->BlockEnd - BodyDelimiters->BlockStart);
			}
		}

		return true;
	}

	/**
	 * Specialization of DeserializeRequestPayload that handles WebSocket batch requests.
	 * This will populate the TCHARBody of all the wrapped requests.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param InCompleteCallback The callback to call error.
	 * @param The structure to serialize using the request's content.
	 * @return Whether the deserialization was successful.
	 *
	 * @note InCompleteCallback will be called with an appropriate http response if the deserialization fails.
	 */
	template <>
	[[nodiscard]] inline bool DeserializeRequestPayload(TConstArrayView<uint8> InTCHARPayload, const FHttpResultCallback* InCompleteCallback, FRCWebSocketBatchRequest& OutDeserializedRequest)
	{
		FMemoryReaderView Reader(InTCHARPayload);
		FJsonStructDeserializerBackend DeserializerBackend(Reader);

		if (!FStructDeserializer::Deserialize(&OutDeserializedRequest, *FRCWebSocketBatchRequest::StaticStruct(), DeserializerBackend, FStructDeserializerPolicies()))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to deserialize request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		TArray<FBlockDelimiters> Delimiters;
		if (!GetBatchWebSocketRequestStructDelimiters(InTCHARPayload, Delimiters, nullptr))
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Unable to get struct delimiters for batch request."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		if (Delimiters.Num() != OutDeserializedRequest.Requests.Num())
		{
			if (InCompleteCallback)
			{
				TUniquePtr<FHttpServerResponse> Response = CreateHttpResponse();
				CreateUTF8ErrorMessage(TEXT("Batch request delimiters did not match number of requests."), Response->Body);
				(*InCompleteCallback)(MoveTemp(Response));
			}
			return false;
		}

		for (int32 RequestIndex = 0; RequestIndex < OutDeserializedRequest.Requests.Num(); ++RequestIndex)
		{
			FRCWebSocketRequest& Request = OutDeserializedRequest.Requests[RequestIndex];
			FBlockDelimiters& ParametersDelimeters = Delimiters[RequestIndex];
			Request.GetParameterDelimiters(FRCWebSocketRequest::ParametersFieldLabel()) = MoveTemp(ParametersDelimeters);
			Request.TCHARBody = InTCHARPayload.Slice(ParametersDelimeters.BlockStart, ParametersDelimeters.BlockEnd - ParametersDelimeters.BlockStart);
		}

		return true;
	}

	/**
	 * Adds a header indicating that the request is a wrapped request that originated from the engine itself.
	 */
	void AddWrappedRequestHeader(FHttpServerRequest& Request);

	/**
	 * Get whether the request is a wrapped request.
	 */
	bool IsWrappedRequest(const FHttpServerRequest& Request);

	/**
	 * Deserialize a request into a UStruct.
	 * @param InRequest The incoming http request.
	 * @param InCompleteCallback The callback to call error.
	 * @param The structure to serialize using the request's content.
	 * @return Whether the deserialization was successful. 
	 * 
	 * @note InCompleteCallback will be called with an appropriate http response if the deserialization fails.
	 */
	template <typename RequestType>
	[[nodiscard]] bool DeserializeRequest(const FHttpServerRequest& InRequest, const FHttpResultCallback* InCompleteCallback, RequestType& OutDeserializedRequest)
	{
		static_assert(TIsDerivedFrom<RequestType, FRCRequest>::IsDerived, "Argument OutDeserializedRequest must derive from FRCRequest");
		
		if (IsWrappedRequest(InRequest))
		{
			// If the request is wrapped, the body should already be encoded in UCS2.
			OutDeserializedRequest.TCHARBody = InRequest.Body;
		}
		
		if (!OutDeserializedRequest.TCHARBody.Num())
		{
			WebRemoteControlUtils::ConvertToTCHAR(InRequest.Body, OutDeserializedRequest.TCHARBody);
		}

		return DeserializeRequestPayload(OutDeserializedRequest.TCHARBody, InCompleteCallback, OutDeserializedRequest);
	}

	/**
	 * Validate a content-type.
	 * @param InRequest The incoming http request.
	 * @param InContentType The callback to call error.
	 * @param InCompleteCallback The callback to call error.
	 * @return Whether the content type was valid or not.
	 * 
	 * @note InCompleteCallback will be called with an appropriate http response if the content type is not valid.
	 */
	[[nodiscard]] bool ValidateContentType(const FHttpServerRequest& InRequest, FString InContentType, const FHttpResultCallback& InCompleteCallback);

	/**
	 * Check if a function call is valid since some objects//functions are disabled remotely for security reasons.
	 * @param InRCCall The RC call to validate.
	 * @param OutErrorText Optional error text.
	 **/
	[[nodiscard]] bool ValidateFunctionCall(const FRCCall& InRCCall, FString* OutErrorText);

	/**
	 * Add the desired content type to the http response headers.
	 * @param InResponse The response to add the content type to.
	 * @param InContentType The content type header to add.
	 */
	void AddContentTypeHeaders(FHttpServerResponse* InOutResponse, FString InContentType);
	
	/**
	* Add CORS headers to a http response.
	* @param InOutResponse The http response to add the CORS headers to.
	*/
	void AddCORSHeaders(FHttpServerResponse* InOutResponse);

	/**
	 * Validate a request's content type.
	 * @param InRequest The request to validate the content type on.
	 * @param InContentType The target content type.
	 * @param OutErrorText If set, the string pointer will be populated with an error message on error.
	 * @return Whether or not the content type matches the target content type.
	 */
	bool IsRequestContentType(const FHttpServerRequest& InRequest, const FString& InContentType, FString* OutErrorText);

	/**
	 * Serialize a struct on scope.
	 * @param Struct the struct on scope to serialize.
	 * @param Writer the memory archive to write to.
	 */
	template <typename SerializerBackendType = FRCJsonStructSerializerBackend>
	void SerializeStructOnScope(const FStructOnScope& Struct, FMemoryWriter& Writer)
	{
		static_assert(TIsDerivedFrom<SerializerBackendType, IStructSerializerBackend>::IsDerived, "SerializerBackendType must inherit from IStructSerializerBackend.");
		SerializerBackendType SerializerBackend(Writer);
		FStructSerializer::Serialize(Struct.GetStructMemory(), *(UScriptStruct*)Struct.GetStruct(), SerializerBackend, FStructSerializerPolicies());
	}

	/**
	 * Modify a property as specified by a remote request.
	 * @param Property The property to modify.
	 * @param Request The request to modify the property, containing additional information about how to modify it.
	 * @param Payload The payload from which to deserialize property data.
	 * @param ClientId The ID of the client that sent this request.
	 * @param WebSocketHandler The WebSocket handler that will be notified of this remote change.
	 * @param Access The access mode to use for this operation.
	 */
	template <typename RequestType>
	bool ModifyPropertyUsingPayload(FRemoteControlProperty& Property, const RequestType& Request, const TArrayView<uint8>& Payload, const FGuid& ClientId, FWebSocketMessageHandler& WebSocketHandler, ERCAccess Access, FString* OutError = nullptr)
	{
		FRCObjectReference ObjectRef;

		// Replace PropertyValue with the underlying property name.
		TArray<uint8> NewPayload;
		FString FieldName = Property.FieldName.ToString();
		if (Property.GetProperty()->IsA<FArrayProperty>() ||
			Property.GetProperty()->IsA<FMapProperty>() ||
			Property.GetProperty()->IsA<FSetProperty>())
		{
			FieldName = Property.FieldPathInfo.Segments.Last().Name.ToString();
		}
		RemotePayloadSerializer::ReplaceFirstOccurence(Payload, TEXT("PropertyValue"), FieldName, NewPayload);

		// Then deserialize the payload onto all the bound objects.
		FMemoryReader NewPayloadReader(NewPayload);
		FRCJsonStructDeserializerBackend Backend(NewPayloadReader);

		ObjectRef.Property = Property.GetProperty();
		ObjectRef.Access = Access;

		bool bSuccess = true;

		Property.EnableEditCondition();

		for (UObject* Object : Property.GetBoundObjects())
		{
			IRemoteControlModule::Get().ResolveObjectProperty(ObjectRef.Access, Object, Property.FieldPathInfo.ToString(), ObjectRef, OutError);

			// Notify the handler before the change to ensure that the notification triggered by PostEditChange is ignored by the handler 
			// if the client does not want remote change notifications.
			if (ClientId.IsValid())
			{
				// Don't manually trigger a property change modification if this request gets converted to a function call.
				if (ObjectRef.IsValid() && !RemoteControlPropertyUtilities::FindSetterFunction(ObjectRef.Property.Get(), ObjectRef.Object->GetClass()))
				{
					WebSocketHandler.NotifyPropertyChangedRemotely(ClientId, Property.GetOwner()->GetPresetId(), Property.GetId());
				}
			}

#if WITH_EDITOR
			FEditPropertyChain PropertyChain;

			if (Access == ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS && ObjectRef.Object.IsValid())
			{
				// This transaction is being manually controlled, so RemoteControlModule's automatic transaction handling won't call this for us
				ObjectRef.PropertyPathInfo.ToEditPropertyChain(PropertyChain);
				ObjectRef.Object->PreEditChange(PropertyChain);
			}
#endif

			if (Request.ResetToDefault)
			{
				// set interception flag as an extra argument {}
				constexpr bool bAllowIntercept = true;
				bSuccess &= IRemoteControlModule::Get().ResetObjectProperties(ObjectRef, bAllowIntercept);
			}
			else
			{
				NewPayloadReader.Seek(0);
				// Set a ERCPayloadType and TCHARBody in order to follow the replication path
				bSuccess &= IRemoteControlModule::Get().SetObjectProperties(ObjectRef, Backend, ERCPayloadType::Json, NewPayload, Request.Operation);
			}

#if WITH_EDITOR
			if (Access == ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS && ObjectRef.Object.IsValid())
			{
				// This transaction is being manually controlled, so RemoteControlModule's automatic transaction handling won't call these functions for us
				SnapshotTransactionBuffer(ObjectRef.Object.Get());

				FPropertyChangedEvent PropertyEvent = ObjectRef.PropertyPathInfo.ToPropertyChangedEvent();
				PropertyEvent.ChangeType = EPropertyChangeType::Interactive;

				FPropertyChangedChainEvent PropertyChainEvent(PropertyChain, PropertyEvent);
				ObjectRef.Object->PostEditChangeChainProperty(PropertyChainEvent);
			}
#endif
		}

		return bSuccess;
	}

	/** Checking ApiKey using Md5. */
	bool CheckPassphrase(const FString& HashedPassphrase);
}
