// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebRemoteControlUtils.h"
#include "Serialization/RCJsonStructSerializerBackend.h"
#include "Serialization/RCJsonStructDeserializerBackend.h"

void WebRemoteControlUtils::ConvertToTCHAR(TConstArrayView<uint8> InUTF8Payload, TArray<uint8>& OutTCHARPayload)
{
	int32 StartIndex = OutTCHARPayload.Num();
	OutTCHARPayload.AddUninitialized(FUTF8ToTCHAR_Convert::ConvertedLength((ANSICHAR*)InUTF8Payload.GetData(), InUTF8Payload.Num() / sizeof(ANSICHAR)) * sizeof(TCHAR));
	FUTF8ToTCHAR_Convert::Convert((TCHAR*)(OutTCHARPayload.GetData() + StartIndex), (OutTCHARPayload.Num() - StartIndex) / sizeof(TCHAR), (ANSICHAR*)InUTF8Payload.GetData(), InUTF8Payload.Num() / sizeof(ANSICHAR));
}

void WebRemoteControlUtils::ConvertToUTF8(TConstArrayView<uint8> InTCHARPayload, TArray<uint8>& OutUTF8Payload)
{
	int32 StartIndex = OutUTF8Payload.Num();
	OutUTF8Payload.AddUninitialized(FPlatformString::ConvertedLength<UTF8CHAR>((TCHAR*)InTCHARPayload.GetData(), InTCHARPayload.Num() / sizeof(TCHAR)) * sizeof(ANSICHAR));
	FPlatformString::Convert((UTF8CHAR*)(OutUTF8Payload.GetData() + StartIndex), (OutUTF8Payload.Num() - StartIndex) / sizeof(ANSICHAR), (TCHAR*)InTCHARPayload.GetData(), InTCHARPayload.Num() / sizeof(TCHAR));
}

void WebRemoteControlUtils::ConvertToUTF8(const FString& InString, TArray<uint8>& OutUTF8Payload)
{
	int32 StartIndex = OutUTF8Payload.Num();
	OutUTF8Payload.AddUninitialized(FPlatformString::ConvertedLength<UTF8CHAR>(*InString, InString.Len()) * sizeof(ANSICHAR));
	FPlatformString::Convert((UTF8CHAR*)(OutUTF8Payload.GetData() + StartIndex), (OutUTF8Payload.Num() - StartIndex) / sizeof(ANSICHAR), *InString, InString.Len());
}

TSharedRef<IStructSerializerBackend> WebRemoteControlUtils::CreateJsonSerializerBackend(FMemoryWriter& Writer)
{
	return MakeShared<FRCJsonStructSerializerBackend>(Writer);
}

TSharedRef<IStructDeserializerBackend> WebRemoteControlUtils::CreateJsonDeserializerBackend(FMemoryReaderView& Reader)
{
	return MakeShared<FRCJsonStructDeserializerBackend>(Reader);
}
