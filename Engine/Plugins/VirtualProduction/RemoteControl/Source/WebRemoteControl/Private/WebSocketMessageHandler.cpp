// Copyright Epic Games, Inc. All Rights Reserved.

#include "WebSocketMessageHandler.h"

#include "Algo/ForEach.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Misc/TransactionObjectEvent.h"
#include "Engine/Engine.h"
#include "EngineUtils.h"
#include "IRemoteControlModule.h"
#include "RCVirtualProperty.h"
#include "GameFramework/Actor.h"
#include "RemoteControlPreset.h"
#include "RemoteControlRoute.h"
#include "RemoteControlReflectionUtils.h"
#include "RemoteControlWebsocketRoute.h"
#include "WebRemoteControl.h"
#include "WebRemoteControlInternalUtils.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/TransBuffer.h"
#endif

#define LOCTEXT_NAMESPACE "WebRemoteControl"

static TAutoConsoleVariable<int32> CVarWebRemoteControlFramesBetweenPropertyNotifications(
	TEXT("WebControl.FramesBetweenPropertyNotifications"),
	5,
	TEXT("The number of frames between sending batches of property notifications.")
);

static TAutoConsoleVariable<float> CVarWebRemoteControlManualTransactionTimeout(
	TEXT("WebControl.ManualTransactionTimeout"),
	3.f,
	TEXT("When this many seconds has passed since a client last contributed to a transaction, the transaction will automatically end.")
);

const int64 FWebSocketMessageHandler::DefaultSequenceNumber = -1;
const int32 FWebSocketMessageHandler::InvalidTransactionId = -1;

namespace WebSocketMessageHandlerStructUtils
{
	using namespace UE::WebRCReflectionUtils;

	FName Struct_PropertyValue = "WEBRC_PropertyValue";
	FName Prop_PropertyLabel = "PropertyLabel";
	FName Prop_Id = "Id";
	FName Prop_ObjectPath = "ObjectPath";
	FName Prop_PropertyValue = "PropertyValue";

	FName Struct_PresetFieldsChanged = "WEBRC_PresetFieldsChanged";
	FName Prop_Type = "Type";
	FName Prop_PresetName= "PresetName";
	FName Prop_PresetId = "PresetId";
	FName Prop_ChangedFields = "ChangedFields";
	FName Prop_SequenceNumber = "SequenceNumber";

	FName Struct_ActorPropertyValue= "WEBRC_ActorPropertyValue";
	FName Prop_PropertyName = "PropertyName";

	FName Struct_ModifiedActor = "WEBRC_ModifiedActor";
	FName Prop_DisplayName = "DisplayName";
	FName Prop_Path = "Path";
	FName Prop_ModifiedProperties = "ModifiedProperties";

	FName Struct_ModifiedActors = "WEBRC_ModifiedActors";
	FName Prop_ModifiedActors = "ModifiedActors";
	
	UScriptStruct* CreatePropertyValueContainer(FProperty* InValueProperty)
	{
		check(InValueProperty);

		static FGuid PropertyValueGuid = FGuid::NewGuid();

		FWebRCGenerateStructArgs Args;

		Args.StringProperties = 
		{ 
			Prop_PropertyLabel,
			Prop_Id,
			Prop_ObjectPath
		};

		Args.GenericProperties.Emplace(Prop_PropertyValue, InValueProperty);

		const FString StructName = FString::Format(TEXT("{0}_{1}_{2}_{3}"), { *Struct_PropertyValue.ToString(), *InValueProperty->GetClass()->GetName(), *InValueProperty->GetName(), PropertyValueGuid.ToString() });

		return GenerateStruct(*StructName, Args);
	}

	UScriptStruct* CreatePresetFieldsChangedStruct(UScriptStruct* PropertyValueStruct)
	{
		FWebRCGenerateStructArgs Args;
		Args.StringProperties =
		{ 
			Prop_PresetId,
			Prop_PresetName,
			Prop_Type,
			Prop_SequenceNumber
		};

		Args.ArrayProperties.Emplace(Prop_ChangedFields, PropertyValueStruct);
		const FString StructName = FString::Format(TEXT("{0}_{1}"), { *Struct_PresetFieldsChanged.ToString(), *PropertyValueStruct->GetName() });

		return GenerateStruct(*StructName, Args);
	}

	UScriptStruct* CreateActorPropertyValueContainer(FProperty* InValueProperty)
	{
		static FGuid ActorPropertyValueGuid = FGuid::NewGuid();

		FWebRCGenerateStructArgs Args;
		Args.StringProperties =
		{
			Prop_PropertyName
		};

		Args.GenericProperties.Emplace(Prop_PropertyValue, InValueProperty);

		const FString StructName = FString::Format(TEXT("{0}_{1}_{2}_{3}"), { *Struct_PropertyValue.ToString(), *InValueProperty->GetClass()->GetName(), *InValueProperty->GetName(), ActorPropertyValueGuid.ToString() });

		return GenerateStruct(*StructName, Args);
	}

	UScriptStruct* CreateModifiedActorStruct(UScriptStruct* ModifiedPropertiesStruct)
	{
		FWebRCGenerateStructArgs Args;
		Args.StringProperties =
		{
			Prop_Id,
			Prop_DisplayName,
			Prop_Path
		};

		Args.ArrayProperties.Emplace(Prop_ModifiedProperties, ModifiedPropertiesStruct);

		const FString StructName = FString::Format(TEXT("{0}_{1}"), { *Struct_ModifiedActor.ToString(), *ModifiedPropertiesStruct->GetName() });
		return GenerateStruct(*StructName, Args);
	}

	UScriptStruct* CreateModifiedActorsStruct(UScriptStruct* ModifiedActorStruct)
	{
		FWebRCGenerateStructArgs Args;
		Args.StringProperties =
		{
			Prop_Type,
			Prop_PresetName,
			Prop_PresetId
		};

		Args.ArrayProperties.Emplace(Prop_ModifiedActors, ModifiedActorStruct);

		const FString StructName = FString::Format(TEXT("{0}_{1}"), { *Struct_ModifiedActors.ToString(), *ModifiedActorStruct->GetName() });
		return GenerateStruct(*StructName, Args);
	}

	FStructOnScope CreatePropertyValueOnScope(const TSharedPtr<FRemoteControlProperty>& RCProperty, const FRCObjectReference& ObjectReference)
	{
		UScriptStruct* Struct = CreatePropertyValueContainer(ObjectReference.Property.Get());
		FStructOnScope StructOnScope{ Struct };

		SetStringPropertyValue(Prop_PropertyLabel, StructOnScope, RCProperty->GetLabel().ToString());
		SetStringPropertyValue(Prop_Id, StructOnScope, RCProperty->GetId().ToString());
		SetStringPropertyValue(Prop_ObjectPath, StructOnScope, ObjectReference.Object->GetPathName());
		CopyPropertyValue(Prop_PropertyValue, StructOnScope, ObjectReference);

		return StructOnScope;
	}
	
	FStructOnScope CreatePresetControllerChangedStructOnScope(const URemoteControlPreset* Preset, const TArray<FStructOnScope*>& PropertyValuesOnScope, int64 SequenceNumber)
	{
		UScriptStruct* PropertyValueStruct = (UScriptStruct*)PropertyValuesOnScope[0]->GetStruct();
		check(PropertyValueStruct);

		UScriptStruct* TopLevelStruct = CreatePresetFieldsChangedStruct(PropertyValueStruct);

		FStructOnScope FieldsChangedOnScope{ TopLevelStruct };
		SetStringPropertyValue(Prop_Type, FieldsChangedOnScope, TEXT("PresetFieldsChanged"));
		SetStringPropertyValue(Prop_PresetName, FieldsChangedOnScope, *Preset->GetPresetName().ToString());
		SetStringPropertyValue(Prop_PresetId, FieldsChangedOnScope, *Preset->GetPresetId().ToString());
		SetStringPropertyValue(Prop_SequenceNumber, FieldsChangedOnScope, FString::Printf(TEXT("%lld"), SequenceNumber));

		return FieldsChangedOnScope;
	}

	FStructOnScope CreatePresetFieldsChangedStructOnScope(const URemoteControlPreset* Preset, const TArray<FStructOnScope>& PropertyValuesOnScope, int64 SequenceNumber)
	{
		UScriptStruct* PropertyValueStruct = (UScriptStruct*)PropertyValuesOnScope[0].GetStruct();
		check(PropertyValueStruct);

		UScriptStruct* TopLevelStruct = CreatePresetFieldsChangedStruct(PropertyValueStruct);

		FStructOnScope FieldsChangedOnScope{ TopLevelStruct };
		SetStringPropertyValue(Prop_Type, FieldsChangedOnScope, TEXT("PresetFieldsChanged"));
		SetStringPropertyValue(Prop_PresetName, FieldsChangedOnScope, *Preset->GetPresetName().ToString());
		SetStringPropertyValue(Prop_PresetId, FieldsChangedOnScope, *Preset->GetPresetId().ToString());
		SetStringPropertyValue(Prop_SequenceNumber, FieldsChangedOnScope, FString::Printf(TEXT("%lld"), SequenceNumber));
		SetStructArrayPropertyValue(Prop_ChangedFields, FieldsChangedOnScope, PropertyValuesOnScope);

		return FieldsChangedOnScope;
	}

	FStructOnScope CreateActorPropertyValueOnScope(const URemoteControlPreset* Preset, const FRCObjectReference& ObjectReference)
	{
		UScriptStruct* Struct = CreateActorPropertyValueContainer(ObjectReference.Property.Get());
		FStructOnScope StructOnScope{ Struct };

		SetStringPropertyValue(Prop_PropertyName, StructOnScope, *ObjectReference.Property->GetName());
		CopyPropertyValue(Prop_PropertyValue, StructOnScope, ObjectReference);

		return StructOnScope;
	}

	FStructOnScope CreateModifiedActorStructOnScope(const URemoteControlPreset* Preset, const FRemoteControlActor& RCActor, const TArray<FStructOnScope>& ModifiedPropertiesOnScope)
	{
		check(ModifiedPropertiesOnScope.Num() > 0);
		UScriptStruct* ModifiedPropertiesStruct = (UScriptStruct*)ModifiedPropertiesOnScope[0].GetStruct();
		check(ModifiedPropertiesStruct);

		UScriptStruct* TopLevelStruct = CreateModifiedActorStruct(ModifiedPropertiesStruct);
		FStructOnScope FieldsChangedOnScope{ TopLevelStruct };

		SetStringPropertyValue(Prop_Id, FieldsChangedOnScope, *RCActor.GetId().ToString());
		SetStringPropertyValue(Prop_DisplayName, FieldsChangedOnScope, *RCActor.GetLabel().ToString());
		SetStringPropertyValue(Prop_Path, FieldsChangedOnScope, *RCActor.Path.ToString());
		SetStructArrayPropertyValue(Prop_ModifiedProperties, FieldsChangedOnScope, ModifiedPropertiesOnScope);

		return FieldsChangedOnScope;
	}

	FStructOnScope CreateModifiedActorsStructOnScope(const URemoteControlPreset* Preset, const TArray<FStructOnScope>& ModifiedActorsOnScope)
	{
		check(ModifiedActorsOnScope.Num() > 0);
		UScriptStruct* ModifiedActorStruct = (UScriptStruct*)ModifiedActorsOnScope[0].GetStruct();
		check(ModifiedActorStruct);

		UScriptStruct* TopLevelStruct = CreateModifiedActorsStruct(ModifiedActorStruct);
		FStructOnScope FieldsChangedOnScope{ TopLevelStruct };

		SetStringPropertyValue(Prop_Type, FieldsChangedOnScope, TEXT("PresetActorModified"));
		SetStringPropertyValue(Prop_PresetName, FieldsChangedOnScope, *Preset->GetPresetName().ToString());
		SetStringPropertyValue(Prop_PresetId, FieldsChangedOnScope, *Preset->GetPresetId().ToString());
		SetStructArrayPropertyValue(Prop_ModifiedActors, FieldsChangedOnScope, ModifiedActorsOnScope);

		return FieldsChangedOnScope;
	}
}

namespace WebSocketMessageHandlerMiscUtils
{
	void* GetPresetPropertyClassPointer(URemoteControlPreset* Preset, const FGuid& PropertyId)
	{
		FRCObjectReference ObjectRef;
		if (TSharedPtr<FRemoteControlProperty> RCProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin())
		{
			if (RCProperty->IsBound())
			{
				if (const FProperty* Property = RCProperty->GetProperty())
				{
					if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
					{
						return StructProperty->Struct;
					}

					return Property->GetClass();
				}
			}
		}

		return nullptr;
	}

	void* GetPresetControllerClassPointer(URemoteControlPreset* Preset, const FGuid& ControllerId)
	{
		if (const URCVirtualPropertyBase* Controller = Preset->GetController(ControllerId))
		{
			if (const FProperty* Property = Controller->GetProperty())
			{
				if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
				{
					return StructProperty->Struct;
				}

				return Property->GetClass();
			}
		}

		return nullptr;
	}
}

FWebSocketMessageHandler::FWebSocketMessageHandler(FRCWebSocketServer* InServer, const FGuid& InActingClientId)
	: Server(InServer)
	, ActingClientId(InActingClientId)
{
	check(Server);
}

void FWebSocketMessageHandler::RegisterRoutes(FWebRemoteControlModule* WebRemoteControl)
{
	FCoreDelegates::OnEndFrame.AddRaw(this, &FWebSocketMessageHandler::OnEndFrame);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.AddRaw(this, &FWebSocketMessageHandler::OnObjectPropertyChanged);
	FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FWebSocketMessageHandler::OnObjectTransacted);

	Server->OnConnectionClosed().AddRaw(this, &FWebSocketMessageHandler::OnConnectionClosedCallback);
#endif

	if (GEngine)
	{
		RegisterEngineEvents();
	}
	else
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FWebSocketMessageHandler::RegisterEngineEvents);
	}
	
	// WebSocket routes
	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Subscribe to events emitted by a Remote Control Preset"),
		TEXT("preset.register"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketPresetRegister)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Unsubscribe to events emitted by a Remote Control Preset"),
		TEXT("preset.unregister"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketPresetUnregister)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Register a transient preset to be automatically destroyed when the calling client disconnects from WebSocket. If multiple clients call this, it will be destroyed once all the clients disconnect."),
		TEXT("preset.transient.autodestroy"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketTransientPresetAutoDestroy)
		));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Subscribe to events emitted when actors of a particular type are added to/deleted from/renamed in the editor world"),
		TEXT("actors.register"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketActorRegister)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Unsubscribe to events emitted when actors of a particular type are added to/deleted from/renamed in the editor world"),
		TEXT("actors.unregister"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketActorUnregister)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Modify the value of of a property exposed on a preset"),
		TEXT("preset.property.modify"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketPresetModifyProperty)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Call a function on an object"),
		TEXT("object.call"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketFunctionCall)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Begin a manual editor transaction. Be sure to send transaction.end when finished."),
		TEXT("transaction.begin"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketBeginEditorTransaction)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("End a manual editor transaction"),
		TEXT("transaction.end"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketEndEditorTransaction)
	));

	RegisterRoute(WebRemoteControl, MakeUnique<FRemoteControlWebsocketRoute>(
		TEXT("Change the compression method this client will use to communicate with Unreal Engine"),
		TEXT("compression.change"),
		FWebSocketMessageDelegate::CreateRaw(this, &FWebSocketMessageHandler::HandleWebSocketCompressionChange)
	));
}

void FWebSocketMessageHandler::UnregisterRoutes(FWebRemoteControlModule* WebRemoteControl)
{
	Server->OnConnectionClosed().RemoveAll(this);
	FCoreDelegates::OnEndFrame.RemoveAll(this);

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectPropertyChanged.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);

	if (GEngine)
	{
		GEngine->OnLevelActorAdded().Remove(OnActorAddedHandle);
		GEngine->OnLevelActorDeleted().Remove(OnActorDeletedHandle);
		GEngine->OnLevelActorListChanged().Remove(OnActorListChangedHandle);
		GEngine->OnLevelActorListChanged().Remove(OnWorldDestroyedHandle);
	}

	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnTransactionStateChanged().RemoveAll(this);
		}
	}
#endif

	for (const TUniquePtr<FRemoteControlWebsocketRoute>& Route : Routes)
	{
		WebRemoteControl->UnregisterWebsocketRoute(*Route);
	}
}

void FWebSocketMessageHandler::RegisterRoute(FWebRemoteControlModule* WebRemoteControl, TUniquePtr<FRemoteControlWebsocketRoute> Route)
{
	checkSlow(WebRemoteControl);

	WebRemoteControl->RegisterWebsocketRoute(*Route);
	Routes.Emplace(MoveTemp(Route));
}

void FWebSocketMessageHandler::RegisterEngineEvents()
{
#if WITH_EDITOR
	if (GEditor)
	{
		if (UTransBuffer* TransBuffer = Cast<UTransBuffer>(GEditor->Trans))
		{
			TransBuffer->OnTransactionStateChanged().AddRaw(this, &FWebSocketMessageHandler::HandleTransactionStateChanged);
		}
	}

	if (GEngine)
	{
		OnActorAddedHandle = GEngine->OnLevelActorAdded().AddRaw(this, &FWebSocketMessageHandler::OnActorAdded);
		OnActorDeletedHandle = GEngine->OnLevelActorDeleted().AddRaw(this, &FWebSocketMessageHandler::OnActorDeleted);
		OnActorListChangedHandle = GEngine->OnLevelActorListChanged().AddRaw(this, &FWebSocketMessageHandler::OnActorListChanged);
		OnWorldDestroyedHandle = GEngine->OnWorldDestroyed().AddRaw(this, &FWebSocketMessageHandler::OnWorldDestroyed);
	}
#endif
}

void FWebSocketMessageHandler::NotifyPropertyChangedRemotely(const FGuid& OriginClientId, const FGuid& PresetId, const FGuid& ExposedPropertyId)
{
	if (TArray<FGuid>* SubscribedClients = PresetNotificationMap.Find(PresetId))
	{
		if (SubscribedClients->Contains(OriginClientId))
		{
			bool bIgnoreIncomingNotification = false;

			if (FRCClientConfig* Config = ClientConfigMap.Find(OriginClientId))
			{
				bIgnoreIncomingNotification = Config->bIgnoreRemoteChanges;
			}

			if (!bIgnoreIncomingNotification)
			{
				PerFrameModifiedProperties.FindOrAdd(PresetId).FindOrAdd(OriginClientId).Add(ExposedPropertyId);
			}
			else
			{
				for (TPair<FGuid, TSet<FGuid>>& Entry : PerFrameModifiedProperties.FindOrAdd(PresetId))
				{
					if (Entry.Key != OriginClientId)
					{
						Entry.Value.Add(ExposedPropertyId);
					}
				}
			}

			PropertiesManuallyNotifiedThisFrame.Add(ExposedPropertyId);
		}
	}
}

void FWebSocketMessageHandler::HandleWebSocketPresetRegister(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketPresetRegisterBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	URemoteControlPreset* Preset = nullptr;

	FGuid PresetId;

	if (FGuid::ParseExact(Body.PresetName, EGuidFormats::Digits, PresetId))
	{
		
		Preset = IRemoteControlModule::Get().ResolvePreset(PresetId);
	}
	else
	{
		Preset = IRemoteControlModule::Get().ResolvePreset(*Body.PresetName);
	}


	if (Preset == nullptr)
	{
		return;
	}

	ClientConfigMap.FindOrAdd(WebSocketMessage.ClientId).bIgnoreRemoteChanges = Body.IgnoreRemoteChanges;
	
	TArray<FGuid>* ClientIds = PresetNotificationMap.Find(Preset->GetPresetId());

	// Don't register delegates for a preset more than once.
	if (!ClientIds)
	{
		ClientIds = &PresetNotificationMap.Add(Preset->GetPresetId());

		//Register to any useful callback for the given preset
		Preset->OnExposedPropertiesModified().AddRaw(this, &FWebSocketMessageHandler::OnPresetExposedPropertiesModified);
		Preset->OnEntityExposed().AddRaw(this, &FWebSocketMessageHandler::OnPropertyExposed);
		Preset->OnEntityUnexposed().AddRaw(this, &FWebSocketMessageHandler::OnPropertyUnexposed);
		Preset->OnFieldRenamed().AddRaw(this, &FWebSocketMessageHandler::OnFieldRenamed);
		Preset->OnMetadataModified().AddRaw(this, &FWebSocketMessageHandler::OnMetadataModified);
		Preset->OnActorPropertyModified().AddRaw(this, &FWebSocketMessageHandler::OnActorPropertyChanged);
		Preset->OnEntitiesUpdated().AddRaw(this, &FWebSocketMessageHandler::OnEntitiesModified);
		Preset->OnPresetLayoutModified().AddRaw(this, &FWebSocketMessageHandler::OnLayoutModified);
		Preset->OnControllerAdded().AddRaw(this, &FWebSocketMessageHandler::OnControllerAdded);
		Preset->OnControllerRemoved().AddRaw(this, &FWebSocketMessageHandler::OnControllerRemoved);
		Preset->OnControllerRenamed().AddRaw(this, &FWebSocketMessageHandler::OnControllerRenamed);
		Preset->OnControllerModified().AddRaw(this, &FWebSocketMessageHandler::OnControllerModified);
	}

	ClientIds->AddUnique(WebSocketMessage.ClientId);
}


void FWebSocketMessageHandler::HandleWebSocketPresetUnregister(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketPresetRegisterBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	URemoteControlPreset* Preset = nullptr;

	FGuid PresetId;

	if (FGuid::ParseExact(Body.PresetName, EGuidFormats::Digits, PresetId))
	{
		Preset = IRemoteControlModule::Get().ResolvePreset(PresetId);
	}
	else
	{
		Preset = IRemoteControlModule::Get().ResolvePreset(*Body.PresetName);
	}

	if (Preset)
	{
		if (TArray<FGuid>* RegisteredClients = PresetNotificationMap.Find(Preset->GetPresetId()))
		{
			RegisteredClients->Remove(WebSocketMessage.ClientId);
		}
	}
}

void FWebSocketMessageHandler::HandleWebSocketTransientPresetAutoDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketTransientPresetAutoDestroyBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	URemoteControlPreset* Preset = nullptr;
	IRemoteControlModule& RemoteControl = IRemoteControlModule::Get();

	bool bIsTransient;
	FGuid PresetId;

	if (FGuid::ParseExact(Body.PresetName, EGuidFormats::Digits, PresetId))
	{
		Preset = RemoteControl.ResolvePreset(PresetId);
		bIsTransient = RemoteControl.IsPresetTransient(PresetId);
	}
	else
	{
		Preset = RemoteControl.ResolvePreset(*Body.PresetName);
		bIsTransient = RemoteControl.IsPresetTransient(*Body.PresetName);
	}

	if (!bIsTransient || !Preset)
	{
		return;
	}

	TransientPresetAutoDestroyClients.FindOrAdd(Preset->GetPresetId()).Add(WebSocketMessage.ClientId);
}

void FWebSocketMessageHandler::HandleWebSocketActorRegister(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketActorRegisterBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	const TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr, *Body.ClassName.ToString());
	if (!ActorClass)
	{
		return;
	}

	FWatchedClassData* WatchedClassData = ActorNotificationMap.Find(ActorClass);
	FString ClassPath = ActorClass->GetPathName();

	// Start watching the class if we aren't already
	if (!WatchedClassData)
	{
		WatchedClassData = &ActorNotificationMap.Add(ActorClass);
		WatchedClassData->CachedPath = ClassPath;
	}

	// Register the client for future updates
	WatchedClassData->Clients.AddUnique(WebSocketMessage.ClientId);

	// Register events for each actor and send the existing list of actors as "added" so the client is caught up
	FRCActorsChangedEvent Event;
	FRCActorsChangedData& ChangeData = Event.Changes.Add(ClassPath);

	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		UWorld* World = WorldContext.World();
		
		// To support non-level editor worlds, instead of just checking the level editor's world,
		// now every Editor-type or Game-type world is checked.
		if (!World || (WorldContext.WorldType != EWorldType::Editor && WorldContext.WorldType != EWorldType::Game))
		{
			continue;
		}

		for (AActor* Actor : TActorRange<AActor>(World, ActorClass))
		{
			ChangeData.AddedActors.Add(FRCActorDescription(Actor));
			StartWatchingActor(Actor, ActorClass);
		}
	}

	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(Event, Payload);
	Server->Send(WebSocketMessage.ClientId, Payload);
}

void FWebSocketMessageHandler::HandleWebSocketActorUnregister(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketActorRegisterBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	TSubclassOf<AActor> ActorClass = StaticLoadClass(AActor::StaticClass(), nullptr, *Body.ClassName.ToString());
	if (!ActorClass)
	{
		return;
	}

	UnregisterClientForActorClass(WebSocketMessage.ClientId, ActorClass);
}

void FWebSocketMessageHandler::HandleWebSocketPresetModifyProperty(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketPresetSetPropertyBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	ERCAccess Access;
	switch (Body.TransactionMode)
	{
	case ERCTransactionMode::NONE:
		Access = ERCAccess::WRITE_ACCESS;
		break;

	case ERCTransactionMode::AUTOMATIC:
		Access = ERCAccess::WRITE_TRANSACTION_ACCESS;
		break;

	case ERCTransactionMode::MANUAL:
		Access = ERCAccess::WRITE_MANUAL_TRANSACTION_ACCESS;

#if WITH_EDITOR
		// Indicate that we want to contribute to this transaction if it's active
		if (GEditor && !ContributeToTransaction(WebSocketMessage.ClientId, Body.TransactionId))
		{
			return;
		}
#endif

		break;

	default:
		UE_LOGF(LogRemoteControl, Warning, "Unknown transaction mode %d", int(Body.TransactionMode));
		return;
	}

	URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Body.PresetName);
	if (Preset == nullptr)
	{
		return;
	}

	const FGuid PropertyId = Preset->GetExposedEntityId(Body.PropertyLabel);
	TSharedPtr<FRemoteControlProperty> RemoteControlProperty = Preset->GetExposedEntity<FRemoteControlProperty>(PropertyId).Pin();

	if (!RemoteControlProperty.IsValid())
	{
		return;
	}

	UpdateSequenceNumber(WebSocketMessage.ClientId, Body.SequenceNumber);

	WebRemoteControlInternalUtils::ModifyPropertyUsingPayload(*RemoteControlProperty.Get(), Body, WebSocketMessage.RequestPayload, WebSocketMessage.ClientId, *this, Access);
}

void FWebSocketMessageHandler::HandleWebSocketFunctionCall(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketCallBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	if (Body.GenerateTransaction)
	{
		Body.TransactionMode = ERCTransactionMode::AUTOMATIC;
	}

	FRCCall Call;
	if (IRemoteControlModule::Get().ResolveCall(Body.ObjectPath, Body.FunctionName, Call.CallRef, nullptr))
	{
		// Initialize the param struct with default parameters
		Call.TransactionMode = Body.GenerateTransaction ? ERCTransactionMode::AUTOMATIC : Body.TransactionMode;
		Call.ParamStruct = FStructOnScope(Call.CallRef.Function.Get());

		// If some parameters were provided, deserialize them
		const FBlockDelimiters& ParametersDelimiters = Body.GetStructParameters().FindChecked(FRCCallRequest::ParametersLabel());
		if (ParametersDelimiters.BlockStart > 0)
		{
			// Extract function parameters from the request payload
			const int64 DelimitersSize = ParametersDelimiters.GetBlockSize();

			TArray<uint8> ParamPayload;
			ParamPayload.SetNumUninitialized(DelimitersSize);

			const uint8* DataStart = &WebSocketMessage.RequestPayload[ParametersDelimiters.BlockStart];
			FMemory::Memcpy(ParamPayload.GetData(), DataStart, DelimitersSize);

			// Using the extracted data, set up a reader and attempt to deserialize
			FMemoryReader Reader(ParamPayload);
			FRCJsonStructDeserializerBackend Backend(Reader);

			if (!FStructDeserializer::Deserialize((void*)Call.ParamStruct.GetStructMemory(), *const_cast<UStruct*>(Call.ParamStruct.GetStruct()), Backend, FStructDeserializerPolicies()))
			{
				// Failed to deserialize parameters
				return;
			}
		}
	}

	FString ValidateFunctionError;
	if (!WebRemoteControlInternalUtils::ValidateFunctionCall(Call, &ValidateFunctionError))
	{
		UE_LOGF(LogRemoteControl, Warning, "WebSocket Function call failed. Error: %ls", *ValidateFunctionError);
		return;
	}

	if (!Call.IsValid())
	{
		return;
	}

#if WITH_EDITOR
	if (Body.TransactionMode == ERCTransactionMode::MANUAL)
	{
		// Indicate that we want to contribute to this transaction
		if (GEditor && !ContributeToTransaction(WebSocketMessage.ClientId, Body.TransactionId))
		{
			return;
		}
	}
#endif

	UpdateSequenceNumber(WebSocketMessage.ClientId, Body.SequenceNumber);

	IRemoteControlModule::Get().InvokeCall(Call);
}

void FWebSocketMessageHandler::HandleWebSocketBeginEditorTransaction(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
#if WITH_EDITOR
	if (!GEditor)
	{
		return;
	}

	FRCWebSocketTransactionStartBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	const FText Description = FText::Format(LOCTEXT("RemoteControlTransaction", "Remote Control - {0}"), FText::FromString(Body.Description));
	const FGuid TransactionGuid = IRemoteControlModule::Get().BeginManualEditorTransaction(Description, 0);

	if (TransactionGuid.IsValid())
	{
		ClientsByTransactionGuid.FindOrAdd(TransactionGuid);
		TransactionIdsByClientId.FindOrAdd(WebSocketMessage.ClientId).Add({ TransactionGuid, Body.TransactionId });

		ContributeToTransaction(WebSocketMessage.ClientId, Body.TransactionId);
	}
	else
	{
		// Send a message indicating that the transaction ended immediately so the client knows it wasn't created
		FRCTransactionEndedEvent Event;
		Event.TransactionId = Body.TransactionId;
		Event.SequenceNumber = GetSequenceNumber(WebSocketMessage.ClientId);

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(Event, Payload);
		Server->Send(WebSocketMessage.ClientId, Payload);
	}
#endif
}

void FWebSocketMessageHandler::HandleWebSocketEndEditorTransaction(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
#if WITH_EDITOR
	if (!GEditor)
	{
		return;
	}

	FRCWebSocketTransactionEndBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	EndClientTransaction(WebSocketMessage.ClientId, Body.TransactionId);
#endif
}

void FWebSocketMessageHandler::HandleWebSocketCompressionChange(const FRemoteControlWebSocketMessage& WebSocketMessage)
{
	FRCWebSocketCompressionChangeBody Body;
	if (!WebRemoteControlInternalUtils::DeserializeRequestPayload(WebSocketMessage.RequestPayload, nullptr, Body))
	{
		return;
	}

	// First, reply to the client confirming the new mode so it can continue with its old decompression method until it receives this message
	FRCCompressionChangedEvent Event;
	Event.Mode = Body.Mode;

	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(Event, Payload);
	Server->Send(WebSocketMessage.ClientId, Payload);

	// Now update the compression method for all future messages
	Server->SetClientCompressionMode(WebSocketMessage.ClientId, Body.Mode);
}

void FWebSocketMessageHandler::ProcessChangedControllers()
{
	// Go over each controller that was changed for each preset
	for (TTuple<FGuid, TMap<FGuid, TSet<FGuid>>>& Entry : PerFrameModifiedControllers)
	{
		if (!ShouldProcessEventForPreset(Entry.Key) || !Entry.Value.Num())
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (!Preset)
		{
			continue;
		}
		
		UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Broadcasting controllers changed event.", *Preset->GetName());

		// Each client will have a custom payload that doesn't contain the events it triggered.
		for (const TPair<FGuid, TSet<FGuid>>& ClientToModifications : Entry.Value)
		{
			TMap<void*, TSet<FGuid>> ControllersIdByType;
			for (const FGuid& Id : ClientToModifications.Value)
			{
				void* ClassPtr = WebSocketMessageHandlerMiscUtils::GetPresetControllerClassPointer(Preset, Id);
				ControllersIdByType.FindOrAdd(ClassPtr).Emplace(Id);
			}

			// Send a Controller Change Event for each Type
			for (const TPair<void*, TSet<FGuid>>& ClassToEventsPair : ControllersIdByType)
			{
				const uint64 ClientSequenceNumber = GetSequenceNumber(ClientToModifications.Key);
				
				TArray<uint8> WorkingBuffer;
				if (ClientToModifications.Value.Num() && WriteControllerChangeEventPayload(Preset, ClassToEventsPair.Value, ClientSequenceNumber, WorkingBuffer))
				{
					TArray<uint8> Payload;
					WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Payload);
					Server->Send(ClientToModifications.Key, Payload);
				}
			}
		}
	}

	PerFrameModifiedControllers.Empty();
}

void FWebSocketMessageHandler::ProcessAddedControllers()
{
	for (const TPair<FGuid, TArray<FGuid>>& Entry : PerFrameAddedControllers)
	{
		if (Entry.Value.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (!Preset)
		{
			continue;
		}
		
		UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Broadcasting controllers added event.", *Preset->GetName());

		FRCPresetDescription AddedControllerDescription;
		AddedControllerDescription.Name = Preset->GetName();
		AddedControllerDescription.Path = Preset->GetPathName();
		AddedControllerDescription.ID = Preset->GetPresetId().ToString();
		Algo::Transform(Entry.Value, AddedControllerDescription.Controllers, [Preset](const FGuid& ControllerId){ return Preset->GetController(ControllerId); });
		
		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetControllersAddedEvent { Preset->GetPresetName(), Preset->GetPresetId(), AddedControllerDescription }, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}

	PerFrameAddedControllers.Empty();
}

void FWebSocketMessageHandler::ProcessRemovedControllers()
{
	for (const TPair<FGuid, TTuple<TArray<FGuid>, TArray<FName>>>& Entry : PerFrameRemovedControllers)
	{
		if (Entry.Value.Key.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (Preset == nullptr)
		{
			continue;
		}

		ensure(Entry.Value.Key.Num() == Entry.Value.Value.Num());
		
		UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Broadcasting controllers removed event.", *Preset->GetName());
		
		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetControllersRemovedEvent{ Preset->GetPresetName(), Preset->GetPresetId(), Entry.Value.Value, Entry.Value.Key }, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}

	PerFrameRemovedControllers.Empty();
}

// Actually not called currently
void FWebSocketMessageHandler::ProcessRenamedControllers()
{
	for (const TPair<FGuid, TArray<TTuple<FName, FName>>>& Entry : PerFrameRenamedControllers)
	{
		if (Entry.Value.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (Preset == nullptr)
		{
			continue;
		}
		
		UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Broadcasting controllers renamed event.", *Preset->GetName());

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetControllersRenamedEvent{Preset->GetPresetName(), Preset->GetPresetId(), Entry.Value}, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}

	PerFrameRenamedControllers.Empty();
}

void FWebSocketMessageHandler::ProcessChangedProperties()
{
	//Go over each property that were changed for each preset
	for (const TPair<FGuid, TMap<FGuid, TSet<FGuid>>>& Entry : PerFrameModifiedProperties)
	{
		if (!ShouldProcessEventForPreset(Entry.Key) || !Entry.Value.Num())
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (!Preset)
		{
			continue;
		}

		UE_LOGF(LogRemoteControl, VeryVerbose, "(%ls) Broadcasting properties changed event.", *Preset->GetName());

		// Each client will have a custom payload that doesnt contain the events it triggered.
		for (const TPair<FGuid, TSet<FGuid>>& ClientToEventsPair : Entry.Value)
		{
			// Categorize modified properties by type so we don't try to put multiple types of data in a single request
			TMap<void*, TSet<FGuid>> PropertyIdsByType;
			for (const FGuid& Id : ClientToEventsPair.Value)
			{
				void* ClassPointer = WebSocketMessageHandlerMiscUtils::GetPresetPropertyClassPointer(Preset, Id);
				PropertyIdsByType.FindOrAdd(ClassPointer).Emplace(Id);
			}

			// Send a property change event for each property type
			for (const TPair<void*, TSet<FGuid>>& ClassToEventsPair : PropertyIdsByType)
			{
				const int64 SequenceNumber = GetSequenceNumber(ClientToEventsPair.Key);

				//Check if multiple booleans properties want to be sent and send them since multiple booleans have problem with the common workflow.
				if (ClassToEventsPair.Key == FBoolProperty::StaticClass())
				{
					SendBoolProperties(Preset, ClientToEventsPair.Key, ClassToEventsPair.Value, SequenceNumber);
					continue;
				}

				TArray<uint8> WorkingBuffer;
				if (ClientToEventsPair.Value.Num() && WritePropertyChangeEventPayload(Preset, { ClassToEventsPair.Value }, SequenceNumber, WorkingBuffer))
				{
					TArray<uint8> Payload;
					WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Payload);
					Server->Send(ClientToEventsPair.Key, Payload);
				}
			}
		}
	}

	PerFrameModifiedProperties.Empty();
}

void FWebSocketMessageHandler::ProcessChangedActorProperties()
{
	//Go over each property that were changed for each preset
	for (const TPair<FGuid, TMap<FGuid, TMap<FRemoteControlActor, TArray<FRCObjectReference>>>>& Entry : PerFrameActorPropertyChanged)
	{
		if (!ShouldProcessEventForPreset(Entry.Key) || !Entry.Value.Num())
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (!Preset)
		{
			continue;
		}

		// Each client will have a custom payload that doesnt contain the events it triggered.
		for (const TPair<FGuid, TMap<FRemoteControlActor, TArray<FRCObjectReference>>>& ClientToModifications : Entry.Value)
		{
			TArray<uint8> WorkingBuffer;
			FMemoryWriter Writer(WorkingBuffer);

			if (ClientToModifications.Value.Num() && WriteActorPropertyChangePayload(Preset, ClientToModifications.Value, Writer))
			{
				TArray<uint8> Payload;
				WebRemoteControlUtils::ConvertToUTF8(WorkingBuffer, Payload);
				Server->Send(ClientToModifications.Key, Payload);
			}
		}
	}

	PerFrameActorPropertyChanged.Empty();
}

void FWebSocketMessageHandler::OnPropertyExposed(URemoteControlPreset* Owner, const FGuid& EntityId)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	// Cache used during the Unexposed as the last resort to get the property Label correctly (main use case is Undo/Redo)
	TTuple<TArray<FGuid>, TArray<FName>>& Entries = CacheUndoRedoAddedRemovedProperties.FindOrAdd(Owner->GetPresetId());
	Entries.Key.AddUnique(EntityId);

	if (const TSharedPtr<FRemoteControlEntity> Entity = Owner->GetExposedEntity(EntityId).Pin())
	{
		Entries.Value.AddUnique(Entity->GetLabel());
	}
	else
	{
		// If the label couldn't be set from the Entity then we use the EntityId as the Label since it is unique
		Entries.Value.AddUnique(FName(EntityId.ToString()));
	}

	//Cache the property field that was removed for end of frame notification
	PerFrameAddedProperties.FindOrAdd(Owner->GetPresetId()).AddUnique(EntityId);
}

void FWebSocketMessageHandler::OnPresetExposedPropertiesModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedPropertyIds)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	//Cache the property field that changed for end of frame notification
	TMap<FGuid, TSet<FGuid>>& EventsForClient = PerFrameModifiedProperties.FindOrAdd(Owner->GetPresetId());
	
	if (TArray<FGuid>* SubscribedClients = PresetNotificationMap.Find(Owner->GetPresetId()))
	{
		for (const FGuid& ModifiedPropertyId : ModifiedPropertyIds)
		{
			// Don't send a change notification if the change was manually notified.
			// This is to avoid the case of a post edit change property being caught by the preset for a change 
			// that a client deliberatly wishes to ignore.
			if (!PropertiesManuallyNotifiedThisFrame.Contains(ModifiedPropertyId))
			{
				for (const FGuid& Client : *SubscribedClients)
				{
					if (Client != ActingClientId || !ClientConfigMap.FindChecked(Client).bIgnoreRemoteChanges)
					{
						EventsForClient.FindOrAdd(Client).Append(ModifiedPropertyIds);
					}
				}
			}
			else
			{
				// Remove the property after encountering it here since we can't remove it on end frame
				// because that might happen before the final PostEditChange of a property change in the RC Module.
				PropertiesManuallyNotifiedThisFrame.Remove(ModifiedPropertyId);
			}
		}
	}
}

void FWebSocketMessageHandler::OnPropertyUnexposed(URemoteControlPreset* Owner, const FGuid& EntityId)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	const TSharedPtr<FRemoteControlEntity> Entity = Owner->GetExposedEntity(EntityId).Pin();
	TPair<TArray<FGuid>, TArray<FName>>& Entries = PerFrameRemovedProperties.FindOrAdd(Owner->GetPresetId());

	// Cache the property field that was removed for end of frame notification
	Entries.Key.AddUnique(EntityId);
	bool bLabelSet = false;

	if (Entity.IsValid())
	{
		Entries.Value.AddUnique(Entity->GetLabel());
		bLabelSet = true;
	}
	else
	{
		// If the Entity is not valid try using the cached properties saved during the Expose
		// This is done because during the Undo/Redo the entity is already removed from the preset and we can't get the Label correctly
		if (TPair<TArray<FGuid>, TArray<FName>>* CachedProperties = CacheUndoRedoAddedRemovedProperties.Find(Owner->GetPresetId()))
		{
			const int32 Index = CachedProperties->Key.IndexOfByKey(EntityId);
			if (Index != INDEX_NONE && CachedProperties->Value.IsValidIndex(Index))
			{
				Entries.Value.AddUnique(CachedProperties->Value[Index]);
				bLabelSet = true;
			}
		}
	}

	// If the label couldn't be set from the Entity or the Cache then we use the EntityId as the Label since it is unique
	if (!bLabelSet)
	{
		Entries.Value.AddUnique(FName(EntityId.ToString()));
	}
}

void FWebSocketMessageHandler::OnFieldRenamed(URemoteControlPreset* Owner, FName OldFieldLabel, FName NewFieldLabel)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	// Update the cached Name with the new name
	if (TPair<TArray<FGuid>, TArray<FName>>* CachedProperties = CacheUndoRedoAddedRemovedProperties.Find(Owner->GetPresetId()))
	{
		const int32 Index = CachedProperties->Value.IndexOfByKey(OldFieldLabel);
		if (Index != INDEX_NONE)
		{
			CachedProperties->Value[Index] = NewFieldLabel;
		}
	}

	//Cache the field that was renamed for end of frame notification
	PerFrameRenamedFields.FindOrAdd(Owner->GetPresetId()).AddUnique(TTuple<FName, FName>(OldFieldLabel, NewFieldLabel));
}

void FWebSocketMessageHandler::OnMetadataModified(URemoteControlPreset* Owner)
{
	//Cache the field that was renamed for end of frame notification
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	//Cache the field that was renamed for end of frame notification
	PerFrameModifiedMetadata.Add(Owner->GetPresetId());
}

void FWebSocketMessageHandler::OnActorPropertyChanged(URemoteControlPreset* Owner, FRemoteControlActor& Actor, UObject* ModifiedObject, FProperty* ModifiedProperty)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	FRCFieldPathInfo FieldPath { ModifiedProperty->GetName() };
	if (!FieldPath.Resolve(ModifiedObject))
	{
		return;
	}

	FRCObjectReference Ref;
	Ref.Object = ModifiedObject;
	Ref.Property = ModifiedProperty;
	Ref.ContainerAdress = FieldPath.GetResolvedData().ContainerAddress;
	Ref.ContainerType = FieldPath.GetResolvedData().Struct;
	Ref.PropertyPathInfo = MoveTemp(FieldPath);
	Ref.Access = ERCAccess::READ_ACCESS;


	//Cache the property field that changed for end of frame notification
	TMap<FGuid, TMap<FRemoteControlActor, TArray<FRCObjectReference>>>& EventsForClient = PerFrameActorPropertyChanged.FindOrAdd(Owner->GetPresetId());

	// Dont send events to the client that triggered it.
	if (TArray<FGuid>* SubscribedClients = PresetNotificationMap.Find(Owner->GetPresetId()))
	{
		for (const FGuid& Client : *SubscribedClients)
		{
			if (Client != ActingClientId)
			{
				TMap<FRemoteControlActor, TArray<FRCObjectReference>>& ModifiedPropertiesPerActor = EventsForClient.FindOrAdd(Client);
				ModifiedPropertiesPerActor.FindOrAdd(Actor).AddUnique(Ref);
			}
		}
	}
}

void FWebSocketMessageHandler::OnEntitiesModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedEntities)
{
	// We do not need to store these event for the current frame since this was already handled by the preset in this case.
	if (!Owner || ModifiedEntities.Num() == 0)
	{
		return;
	}
	
	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(FRCPresetEntitiesModifiedEvent{Owner, ModifiedEntities.Array()}, Payload);
	BroadcastToPresetListeners(Owner->GetPresetId(), Payload);
}

void FWebSocketMessageHandler::OnLayoutModified(URemoteControlPreset* Owner)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	//Cache the field that was renamed for end of frame notification
	PerFrameModifiedPresetLayouts.Add(Owner->GetPresetId());
}

void FWebSocketMessageHandler::OnConnectionClosedCallback(FGuid ClientId)
{
	// Clean up clients that were waiting for preset callbacks
	for (auto Iter = PresetNotificationMap.CreateIterator(); Iter; ++Iter)
	{
		Iter.Value().Remove(ClientId);
	}

	// Clean up clients that were waiting for actor callbacks
	TArray<TWeakObjectPtr<UClass>> WatchedClasses;
	ActorNotificationMap.GenerateKeyArray(WatchedClasses);

	for (const TWeakObjectPtr<UClass>& WatchedClass : WatchedClasses)
	{
		UnregisterClientForActorClass(ClientId, WatchedClass.Get());
	}

	IRemoteControlModule& RemoteControl = IRemoteControlModule::Get();

	// Clean up transient presets registered to auto-destroy for this client
	for (auto Iter = TransientPresetAutoDestroyClients.CreateIterator(); Iter; ++Iter)
	{
		const FGuid& PresetId = Iter.Key();
		TArray<FGuid>& ClientList = Iter.Value();

		ClientList.Remove(ClientId);
		if (ClientList.IsEmpty())
		{
			RemoteControl.DestroyTransientPreset(PresetId);
			PresetNotificationMap.Remove(PresetId);
			Iter.RemoveCurrent();
		}
	}

	// Remove the client as a listener for any active transactions and end them
	// Do this in two separate loops since ending the transaction may modify the map
	TSet<FGuid> TransactionGuidsToEnd;
	for (auto Iter = ClientsByTransactionGuid.CreateIterator(); Iter; ++Iter)
	{
		TMap<FGuid, FDateTime>& Clients = Iter.Value();
		if (Clients.Remove(ClientId))
		{
			// Note that we always end the transaction even if other clients are listening, as it will just decrement
			// the transaction's internal counter rather than actually ending it unless all transactions have been ended.
			TransactionGuidsToEnd.Add(Iter.Key());
		}
	}

	for (const FGuid& TransactionGuid : TransactionGuidsToEnd)
	{
		IRemoteControlModule::Get().EndManualEditorTransaction(TransactionGuid);
	}

	TransactionIdsByClientId.Remove(ClientId);

	/** Remove this client's config. */
	ClientConfigMap.Remove(ClientId);
	ClientSequenceNumbers.Remove(ClientId);
}

void FWebSocketMessageHandler::OnEndFrame()
{
	PropertyNotificationFrameCounter++;

	if (PropertyNotificationFrameCounter >= CVarWebRemoteControlFramesBetweenPropertyNotifications.GetValueOnGameThread())
	{
#if WITH_EDITOR
		TimeOutTransactions();
#endif

		//Early exit if no clients are requesting notifications
		if (PresetNotificationMap.Num() <= 0 && ActorNotificationMap.Num() <= 0)
		{
			return;
		}

		PropertyNotificationFrameCounter = 0;
		ProcessChangedProperties();
		ProcessChangedActorProperties();
		ProcessRemovedProperties();
		ProcessAddedProperties();
		ProcessRenamedFields();
		ProcessModifiedMetadata();
		ProcessModifiedPresetLayouts();
		ProcessActorChanges();
		ProcessAddedControllers();
		ProcessChangedControllers();
		ProcessRenamedControllers();
		ProcessRemovedControllers();
	}
}

void FWebSocketMessageHandler::OnControllerAdded(URemoteControlPreset* Owner, FName NewControllerName, const FGuid& EntityId)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	PerFrameAddedControllers.FindOrAdd(Owner->GetPresetId()).AddUnique(EntityId);
}

void FWebSocketMessageHandler::OnControllerRemoved(URemoteControlPreset* Owner, const FGuid& EntityId)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	const URCVirtualPropertyBase* Controller = Owner->GetController(EntityId);
	check(Controller);

	// Cache the Controller which was removed for end of frame notification
	TTuple<TArray<FGuid>, TArray<FName>>& Entries = PerFrameRemovedControllers.FindOrAdd(Owner->GetPresetId());
	Entries.Key.AddUnique(EntityId);
	Entries.Value.AddUnique(Controller->DisplayName);
}

void FWebSocketMessageHandler::OnControllerRenamed(URemoteControlPreset* Owner, FName OldControllerLabel, FName NewControllerLabel)
{
	if (Owner == nullptr)
	{
		return;
	}

	if (PresetNotificationMap.Num() <= 0)
	{
		return;
	}

	PerFrameRenamedControllers.FindOrAdd(Owner->GetPresetId()).AddUnique(TTuple<FName, FName>(OldControllerLabel, NewControllerLabel));
}

void FWebSocketMessageHandler::OnControllerModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedControllerIds)
{
	// We do not need to store these event for the current frame since this was already handled by the preset in this case.
	if (!Owner || ModifiedControllerIds.Num() == 0)
	{
		return;
	}
	
	TArray<uint8> Payload;
	WebRemoteControlUtils::SerializeMessage(FRCPresetControllersModifiedEvent{Owner, ModifiedControllerIds.Array()}, Payload);
	BroadcastToPresetListeners(Owner->GetPresetId(), Payload);
}

#if WITH_EDITOR
void FWebSocketMessageHandler::TimeOutTransactions()
{
	TArray<TPair<FGuid, int32>> TransactionsToEnd;
	const FDateTime TimeoutCutoff = FDateTime::Now() - FTimespan::FromSeconds(CVarWebRemoteControlManualTransactionTimeout.GetValueOnGameThread());

	for (const TPair<FGuid, TMap<FGuid, FDateTime>>& TransactionClientsPair : ClientsByTransactionGuid)
	{
		for (const TPair<FGuid, FDateTime>& ClientTimePair : TransactionClientsPair.Value)
		{
			if (ClientTimePair.Value < TimeoutCutoff)
			{
				// This transaction has timed out; we should force it to end
				TransactionsToEnd.Add({ ClientTimePair.Key, GetClientTransactionId(ClientTimePair.Key, TransactionClientsPair.Key) });
			}
		}
	}

	// Do this as a separate step since it may remove entries from the map during iteration
	for (const auto& TransactionIterator : TransactionsToEnd)
	{
		EndClientTransaction(TransactionIterator.Key, TransactionIterator.Value);
	}
}
#endif

void FWebSocketMessageHandler::ProcessAddedProperties()
{
	for (const TPair<FGuid, TArray<FGuid>>& Entry : PerFrameAddedProperties)
	{
		if (Entry.Value.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (Preset == nullptr)
		{
			continue;
		}

		FRCPresetDescription AddedPropertiesDescription;
		AddedPropertiesDescription.Name = Preset->GetName();
		AddedPropertiesDescription.Path = Preset->GetPathName();
		AddedPropertiesDescription.ID = Preset->GetPresetId().ToString();

		TMap<FRemoteControlPresetGroup*, TArray<FGuid>> GroupedNewFields;

		for (const FGuid& Id : Entry.Value)
		{
			if (FRemoteControlPresetGroup* Group = Preset->Layout.FindGroupFromField(Id))
			{
				GroupedNewFields.FindOrAdd(Group).Add(Id);
			}
		}

		for (const TTuple<FRemoteControlPresetGroup*, TArray<FGuid>>& Tuple : GroupedNewFields)
		{
			AddedPropertiesDescription.Groups.Emplace(Preset, *Tuple.Key, Tuple.Value);
		}

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetFieldsAddedEvent{ Preset->GetPresetName(), Preset->GetPresetId(), AddedPropertiesDescription }, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}

	PerFrameAddedProperties.Empty();
}

void FWebSocketMessageHandler::ProcessRemovedProperties()
{
	for (const TPair<FGuid, TTuple<TArray<FGuid>, TArray<FName>>>& Entry : PerFrameRemovedProperties)
	{
		if (Entry.Value.Key.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (Preset == nullptr)
		{
			continue;
		}

		ensure(Entry.Value.Key.Num() == Entry.Value.Value.Num());
		
		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetFieldsRemovedEvent{ Preset->GetPresetName(), Preset->GetPresetId(), Entry.Value.Value, Entry.Value.Key }, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}
	
	PerFrameRemovedProperties.Empty();
}

void FWebSocketMessageHandler::ProcessRenamedFields()
{
	for (const TPair<FGuid, TArray<TTuple<FName, FName>>>& Entry : PerFrameRenamedFields)
	{
		if (Entry.Value.Num() <= 0 || !ShouldProcessEventForPreset(Entry.Key))
		{
			continue;
		}

		URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry.Key);
		if (Preset == nullptr)
		{
			continue;
		}

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(FRCPresetFieldsRenamedEvent{Preset->GetPresetName(), Preset->GetPresetId(), Entry.Value}, Payload);
		BroadcastToPresetListeners(Entry.Key, Payload);
	}

	PerFrameRenamedFields.Empty();
}

void FWebSocketMessageHandler::ProcessModifiedMetadata()
{
	for (const FGuid& Entry : PerFrameModifiedMetadata)
	{
		if (ShouldProcessEventForPreset(Entry))
		{
			if (URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry))
			{
				TArray<uint8> Payload;
				WebRemoteControlUtils::SerializeMessage(FRCPresetMetadataModified{ Preset }, Payload);
				BroadcastToPresetListeners(Entry, Payload);
			}
		}
	}

	PerFrameModifiedMetadata.Empty();
}

void FWebSocketMessageHandler::ProcessModifiedPresetLayouts()
{
	for (const FGuid& Entry : PerFrameModifiedPresetLayouts)
	{
		if (ShouldProcessEventForPreset(Entry))
		{
			if (URemoteControlPreset* Preset = IRemoteControlModule::Get().ResolvePreset(Entry))
			{
				TArray<uint8> Payload;
				WebRemoteControlUtils::SerializeMessage(FRCPresetLayoutModified{ Preset }, Payload);
				BroadcastToPresetListeners(Entry, Payload);
			}
		}
	}

	PerFrameModifiedPresetLayouts.Empty();
}

void FWebSocketMessageHandler::ProcessActorChanges()
{
	// Get the set of all classes with subscribed clients (so we can batch all update types together)
	TArray<UClass*, TInlineAllocator<8>> ChangedClasses;
	TArray<const FDeletedActorsData*> DeletedClasses;
	
	for (const TPair<TWeakObjectPtr<UClass>, TArray<TWeakObjectPtr<AActor>>>& AddedPair : PerFrameActorsAdded)
	{
		if (AddedPair.Key.IsValid())
		{
			ChangedClasses.AddUnique(AddedPair.Key.Get());
		}
	}

	for (const TPair<TWeakObjectPtr<UClass>, TArray<TWeakObjectPtr<AActor>>>& RenamedPair : PerFrameActorsRenamed)
	{
		if (RenamedPair.Key.IsValid())
		{
			ChangedClasses.AddUnique(RenamedPair.Key.Get());
		}
	}

	for (const TPair<TWeakObjectPtr<UClass>, FDeletedActorsData>& DeletedPair : PerFrameActorsDeleted)
	{
		if (DeletedPair.Key.IsValid())
		{
			ChangedClasses.AddUnique(DeletedPair.Key.Get());
		}
		else
		{
			// The class itself was deleted, so we'll have to look it up by path instead of by pointer
			DeletedClasses.Add(&DeletedPair.Value);
		}
	}

	if (ChangedClasses.Num() == 0 && DeletedClasses.Num() == 0)
	{
		return;
	}

	// Map from actor class' path to changed actor data for that class.
	TMap<FString, FRCActorsChangedData> ChangesByClassPath;

	// Map from client ID to which class paths they're going to get an update about.
	TMap<FGuid, TArray<FString>> ClientsToNotify;

	// Gather changes for each class that still exists
	for (UClass* ActorClass : ChangedClasses)
	{
		GatherActorChangesForClass(ActorClass, ChangesByClassPath, ClientsToNotify);
	}

	// Gather changes for deleted classes, which we need to handle different since their pointers are invalid
	for (const FDeletedActorsData* DeletedActorsData : DeletedClasses)
	{
		GatherActorChangesForDeletedClass(DeletedActorsData, ChangesByClassPath, ClientsToNotify);
	}

	// Update each client that cares about the changes we're processing
	for (const TPair<FGuid, TArray<FString>>& ClientData : ClientsToNotify)
	{
		FRCActorsChangedEvent Event;

		for (const FString& ActorClassPath : ClientData.Value)
		{
			const FRCActorsChangedData* ChangeData = ChangesByClassPath.Find(ActorClassPath);
			if (!ensureMsgf(ChangeData != nullptr, TEXT("Found no change data for an actor class that supposedly has changes")))
			{
				continue;
			}

			Event.Changes.Add(ActorClassPath, *ChangeData);
		}

		TArray<uint8> Payload;
		WebRemoteControlUtils::SerializeMessage(Event, Payload);
		Server->Send(ClientData.Key, Payload);
	}

	PerFrameActorsAdded.Empty();
	PerFrameActorsRenamed.Empty();
	PerFrameActorsDeleted.Empty();
}

void FWebSocketMessageHandler::GatherActorChangesForClass(UClass* ActorClass, TMap<FString, FRCActorsChangedData>& OutChangesByClassPath,
	TMap<FGuid, TArray<FString>>& OutClientsToNotify)
{
	const FWatchedClassData* WatchedData = ActorNotificationMap.Find(ActorClass);

	if (WatchedData == nullptr)
	{
		// We may have stopped watching the class on the same frame that an actor of that class changed, in which case
		// nobody is expecting an update and we can ignore it
		return;
	}

	// Each client watching this class should be notified about the changes
	const FString ActorClassPath = ActorClass->GetPathName();
	for (const FGuid& ClientId : WatchedData->Clients)
	{
		OutClientsToNotify.FindOrAdd(ClientId).AddUnique(ActorClassPath);
	}

	FRCActorsChangedData& ChangeData = OutChangesByClassPath.Add(ActorClassPath);

	// Added actors
	if (const TArray<TWeakObjectPtr<AActor>>* AddedActors = PerFrameActorsAdded.Find(ActorClass))
	{
		for (const TWeakObjectPtr<AActor>& Actor : *AddedActors)
		{
			if (Actor.IsValid())
			{
				ChangeData.AddedActors.Add(FRCActorDescription(Actor.Get()));
			}
		}
	}

	// Renamed actors
	if (const TArray<TWeakObjectPtr<AActor>>* RenamedActors = PerFrameActorsRenamed.Find(ActorClass))
	{
		for (const TWeakObjectPtr<AActor>& Actor : *RenamedActors)
		{
			if (Actor.IsValid())
			{
				ChangeData.RenamedActors.Add(FRCActorDescription(Actor.Get()));
			}
		}
	}

	// Deleted actors
	if (const FDeletedActorsData* DeletedActorsData = PerFrameActorsDeleted.Find(ActorClass))
	{
		ChangeData.DeletedActors = DeletedActorsData->Actors;
	}
}

void FWebSocketMessageHandler::GatherActorChangesForDeletedClass(const FWebSocketMessageHandler::FDeletedActorsData* DeletedActorsData,
	TMap<FString, FRCActorsChangedData>& OutChangesByClassPath, TMap<FGuid, TArray<FString>>& OutClientsToNotify)
{
	const FString& ActorClassPath = DeletedActorsData->ClassPath;

	FRCActorsChangedData& ChangeData = OutChangesByClassPath.Add(ActorClassPath);
	ChangeData.DeletedActors = DeletedActorsData->Actors;

	// Find who we're supposed to notify about this. We have to manually iterate the map to find it since
	// the class pointer is now invalid.
	const TArray<FGuid>* WatchingClients = nullptr;

	for (auto It = ActorNotificationMap.CreateConstIterator(); It; ++It)
	{
		if (It->Value.CachedPath == ActorClassPath)
		{
			WatchingClients = &It->Value.Clients;
			break;
		}
	}

	if (!WatchingClients)
	{
		// We may have stopped watching the class on the same frame that an actor of that class changed, in which case
		// nobody is expecting an update and we can ignore it
		return;
	}

	for (const FGuid& ClientId : *WatchingClients)
	{
		OutClientsToNotify.FindOrAdd(ClientId).AddUnique(ActorClassPath);
	}
}

void FWebSocketMessageHandler::BroadcastToPresetListeners(const FGuid& TargetPresetId, const TArray<uint8>& Payload)
{
	const TArray<FGuid>& Listeners = PresetNotificationMap.FindChecked(TargetPresetId);
	for (const FGuid& Listener : Listeners)
	{
		Server->Send(Listener, Payload);
	}
}

bool FWebSocketMessageHandler::ShouldProcessEventForPreset(const FGuid& PresetId) const
{
	return PresetNotificationMap.Contains(PresetId) && PresetNotificationMap[PresetId].Num() > 0;
}

bool FWebSocketMessageHandler::WriteControllerChangeEventPayload(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds, int64 InSequenceNumber, TArray<uint8>& OutBuffer)
{
	bool bHasController = false;

	TArray<FStructOnScope*> ControllerValueOnScope;
	for (const FGuid& ControllerId : InModifiedControllerIds)
	{
		if (URCVirtualPropertySelfContainer* Controller = Cast<URCVirtualPropertySelfContainer>(InPreset->GetController(ControllerId)))
		{
			bHasController = true;

			if (FStructOnScope* ControllerStructOnScope = Controller->CreateStructOnScope().Get())
			{
				ControllerValueOnScope.Add(ControllerStructOnScope);
			}
		}
	}

	if (ControllerValueOnScope.Num())
	{
		FStructOnScope ValueStructOnScope =	WebSocketMessageHandlerStructUtils::CreatePresetControllerChangedStructOnScope(InPreset, ControllerValueOnScope, InSequenceNumber);
		
		FMemoryWriter Writer(OutBuffer);
		WebRemoteControlInternalUtils::SerializeStructOnScope(ValueStructOnScope, Writer);
	}
	
	return bHasController;
}

bool FWebSocketMessageHandler::WritePropertyChangeEventPayload(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedPropertyIds, int64 InSequenceNumber, TArray<uint8>& OutBuffer)
{
	bool bHasProperty = false;

	TArray<FStructOnScope> PropValuesOnScope;
	for (const FGuid& RCPropertyId : InModifiedPropertyIds)
	{
		FRCObjectReference ObjectRef;
		if (TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(RCPropertyId).Pin())
		{
			if (RCProperty->IsBound())
			{
				if (IRemoteControlModule::Get().ResolveObjectProperty(ERCAccess::READ_ACCESS, RCProperty->GetBoundObjects()[0], RCProperty->FieldPathInfo.ToString(), ObjectRef))
				{
					bHasProperty = true;
					PropValuesOnScope.Add(WebSocketMessageHandlerStructUtils::CreatePropertyValueOnScope(RCProperty, ObjectRef));
				}
			}
		}
	}

	if (PropValuesOnScope.Num())
	{
		FStructOnScope FieldsChangedEventOnScope = WebSocketMessageHandlerStructUtils::CreatePresetFieldsChangedStructOnScope(InPreset, PropValuesOnScope, InSequenceNumber);

		FMemoryWriter Writer(OutBuffer);
		WebRemoteControlInternalUtils::SerializeStructOnScope(FieldsChangedEventOnScope, Writer);
	}

	return bHasProperty;
}

void FWebSocketMessageHandler::SendBoolProperties(URemoteControlPreset* InPreset,
	const FGuid& InTargetClientId, const TSet<FGuid>& InModifiedPropertyIds, int64 InSequenceNumber)
{
	if (!InModifiedPropertyIds.IsEmpty())
	{
		if (TSharedPtr<FRemoteControlProperty> RCProperty = InPreset->GetExposedEntity<FRemoteControlProperty>(InModifiedPropertyIds.Array()[0]).Pin())
		{
			if (const FProperty* Property = RCProperty->GetProperty())
			{
				if (Property->IsA<FBoolProperty>())
				{
					for (FGuid ModifiedPropertyId : InModifiedPropertyIds)
					{
						TArray<uint8> BoolsWorkingBuffer;
						if (WritePropertyChangeEventPayload(InPreset, { ModifiedPropertyId }, InSequenceNumber, BoolsWorkingBuffer))
						{
							TArray<uint8> Payload;
							WebRemoteControlUtils::ConvertToUTF8(BoolsWorkingBuffer, Payload);
							Server->Send(InTargetClientId, Payload);
						}
					}
				}
			}
		}
	}
}


bool FWebSocketMessageHandler::WriteActorPropertyChangePayload(URemoteControlPreset* InPreset, const TMap<FRemoteControlActor, TArray<FRCObjectReference>>& InModifications, FMemoryWriter& InWriter)
{
	bool bHasProperty = false;

	TArray<FStructOnScope> ModifiedActorsOnScope;

	for (const TPair<FRemoteControlActor, TArray<FRCObjectReference>>& Pair : InModifications)
	{
		if (AActor* ModifiedActor = Cast<AActor>(Pair.Key.Path.ResolveObject()))
		{
			TArray<FStructOnScope> PropertyValuesOnScope;

			for (const FRCObjectReference& Ref : Pair.Value)
			{
				const FProperty* Property = Ref.Property.Get();

				if (Property && Ref.IsValid())
				{
					bHasProperty = true;
					PropertyValuesOnScope.Add(WebSocketMessageHandlerStructUtils::CreateActorPropertyValueOnScope(InPreset, Ref));
				}
			}

			if (PropertyValuesOnScope.Num())
			{
				ModifiedActorsOnScope.Add(WebSocketMessageHandlerStructUtils::CreateModifiedActorStructOnScope(InPreset, Pair.Key, PropertyValuesOnScope));
			}
		}
	}

	if (ensure(ModifiedActorsOnScope.Num()))
	{
		FStructOnScope ActorsModifedOnScope = WebSocketMessageHandlerStructUtils::CreateModifiedActorsStructOnScope(InPreset, ModifiedActorsOnScope);
		WebRemoteControlInternalUtils::SerializeStructOnScope(ActorsModifedOnScope, InWriter);
	}

	return bHasProperty;
}

void FWebSocketMessageHandler::OnActorAdded(AActor* Actor)
{
	// Should never be an issue, but can't hurt?!
	if (!Actor || !IsValid(Actor))
	{
		return;
	}

#if WITH_EDITOR
	// To support non-level editor worlds, we now check world type instead of specifically checking for the level editor.
	{
		bool bFoundWorld = false;

		if (UWorld* ActorWorld = Actor->GetWorld())
		{
			for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
			{
				UWorld* ContextWorld = WorldContext.World();

				if (ContextWorld != ActorWorld)
				{
					continue;
				}

				if (WorldContext.WorldType != EWorldType::Editor)
				{
					return;
				}

				bFoundWorld = true;
				break;
			}
		}

		if (!bFoundWorld)
		{
			return;
		}
	}
#endif

	// Array of classes this actor is a child of and which are being watched by a client
	TArray<TWeakObjectPtr<UClass>, TInlineAllocator<8>> WatchedClasses;

	const FString ActorPath = Actor->GetPathName();

	for (const TPair<TWeakObjectPtr<UClass>, FWatchedClassData>& WatchedClassPair : ActorNotificationMap)
	{
		UClass* WatchedClass = WatchedClassPair.Key.Get();

		if (WatchedClass == nullptr)
		{
			// We don't need to send an add if the class has already been deleted
			continue;
		}

		// Any classes in this list have at least one client subscribed to updates
		if (Actor->IsA(WatchedClass))
		{
			TArray<TWeakObjectPtr<AActor>>& AddedActors = PerFrameActorsAdded.FindOrAdd(WatchedClass);
			AddedActors.AddUnique(Actor);
			WatchedClasses.AddUnique(WatchedClass);
				
			// If this actor was queued for a delete event, cancel it so that it's clear that the actor has been re-created.
			FDeletedActorsData* DeletedActorsData = PerFrameActorsDeleted.Find(WatchedClass);
			if (DeletedActorsData)
			{
				const int32 DeletedIndex = DeletedActorsData->Actors.IndexOfByPredicate([&ActorPath](const FRCActorDescription& Description) {
					return Description.Path == ActorPath;
				});

				if (DeletedIndex != INDEX_NONE)
				{
					DeletedActorsData->Actors.RemoveAt(DeletedIndex);
				}
			}
		}
	}

	if (WatchedClasses.Num() > 0)
	{
		// At least one subscriber cares about this actor, so we should listen to its events
		FWatchedActorData& ActorData = WatchedActors.Add(Actor, FWatchedActorData(Actor));
		ActorData.WatchedClasses = WatchedClasses;
	}
}

void FWebSocketMessageHandler::OnActorDeleted(AActor* Actor)
{
	FWatchedActorData* ActorData = WatchedActors.Find(Actor);
	if (!ActorData)
	{
		return;
	}

	for (TWeakObjectPtr<UClass> WatchedClass : ActorData->WatchedClasses)
	{
		UClass* WatchedClassPtr = WatchedClass.Get();
		if (!WatchedClassPtr)
		{
			continue;
		}

		FDeletedActorsData* DeletedActors = PerFrameActorsDeleted.Find(WatchedClassPtr);
		if (!DeletedActors)
		{
			// No actors of this class have been deleted this frame, so store the class path in case the class gets deleted too.
			DeletedActors = &PerFrameActorsDeleted.Add(WatchedClassPtr);
			DeletedActors->ClassPath = WatchedClassPtr->GetPathName();
		}
		DeletedActors->Actors.AddUnique(ActorData->Description);

		// If this actor was queued for an add event, cancel it so that it's clear that the actor has been deleted again.
		TArray<TWeakObjectPtr<AActor>>* AddedActors = PerFrameActorsAdded.Find(WatchedClassPtr);
		if (AddedActors)
		{
			const int32 AddedIndex = AddedActors->IndexOfByPredicate([&Actor](TWeakObjectPtr<AActor> AddedActor) {
				return AddedActor.GetEvenIfUnreachable() == Actor;
			});

			if (AddedIndex != INDEX_NONE)
			{
				AddedActors->RemoveAt(AddedIndex);
			}
		}
	}

	WatchedActors.Remove(Actor);
}

void FWebSocketMessageHandler::OnActorListChanged()
{
	// We don't know exactly what changed, so manually check all the actors we know about


	TSet<AActor*> RemainingActors;
	WatchedActors.GetKeys(RemainingActors);

	TArray<AActor*> NewActors;

	// Find any new actors
	for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
	{
		UWorld* World = WorldContext.World();

		// To support non-level editor worlds, instead of just checking the level editor's world,
		// now every Editor-type world is checked.
		if (!World || WorldContext.WorldType != EWorldType::Editor)
		{
			continue;
		}
		
		for (AActor* Actor : TActorRange<AActor>(World))
		{
			if (!WatchedActors.Contains(Actor))
			{
				NewActors.Add(Actor);
			}

			RemainingActors.Remove(Actor);
		}
	}

	// Fire events for any actors that are now missing, which have presumably been deleted
	for (AActor* Actor : RemainingActors)
	{
		OnActorDeleted(Actor);
	}

	// Stale actors may not have been removed from the map due to how invalid keys are hashed, so remove them directly
	for (auto It = WatchedActors.CreateIterator(); It; ++It)
	{
		TWeakObjectPtr<AActor> Actor = It.Key();
		if (!Actor.IsValid())
		{
			It.RemoveCurrent();
		}
	}

	// Fire events for new actors (this must be done second since we could be re-creating actors with the same paths, e.g. by reloading a world)
	for (AActor* Actor : NewActors)
	{
		OnActorAdded(Actor);
	}
}

void FWebSocketMessageHandler::OnWorldDestroyed(UWorld* World)
{
	// Remove any actors in this world
	for (AActor* Actor : TActorRange<AActor>(World))
	{
		OnActorDeleted(Actor);
	}
}

void FWebSocketMessageHandler::OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event)
{
	// We only care about name changes
	const FName LabelProperty(TEXT("ActorLabel"));
	if (Event.GetPropertyName() != LabelProperty)
	{
		return;
	}

	// We only care about actors
	AActor* Actor = Cast<AActor>(Object);
	if (!Actor)
	{
		return;
	}

	// We only care about actors that we're watching
	FWatchedActorData* ActorData = WatchedActors.Find(Actor);
	if (!ActorData)
	{
		return;
	}

	UpdateWatchedActorName(Actor, *ActorData);
}

void FWebSocketMessageHandler::OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& TransactionEvent)
{
	// We only care about undo/redo
	if (TransactionEvent.GetEventType() != ETransactionObjectEventType::UndoRedo)
	{
		return;
	}

	// We only care about actors
	AActor* Actor = Cast<AActor>(Object);
	if (!Actor)
	{
		return;
	}

	// Check if the actor was created/deleted by the transaction
	if (TransactionEvent.HasPendingKillChange())
	{
		if (!IsValid(Actor))
		{
			// Actor was undone; treat as a delete
			OnActorDeleted(Actor);
		}
		else
		{
			// Actor was redone; treat as a create
			OnActorAdded(Actor);
		}

		// In either case, we can bail early since a rename no longer matters
		return;
	}

	// We only care about renames for actors that we're watching
	FWatchedActorData* ActorData = WatchedActors.Find(Actor);
	if (!ActorData)
	{
		return;
	}

	// Check if the actor was renamed by the transaction
	const FName LabelProperty(TEXT("ActorLabel"));
	for (FName Property : TransactionEvent.GetChangedProperties())
	{
		if (Property == LabelProperty)
		{
			UpdateWatchedActorName(Actor, *ActorData);
			return;
		}
	}
}

#if WITH_EDITOR
void FWebSocketMessageHandler::HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState)
{
	if (InTransactionState == ETransactionStateEventType::TransactionCanceled || InTransactionState == ETransactionStateEventType::TransactionFinalized)
	{
		HandleTransactionEnded(InTransactionContext.TransactionId);
	}
}

void FWebSocketMessageHandler::HandleTransactionEnded(const FGuid& TransactionGuid)
{
	if (const TMap<FGuid, FDateTime>* TransactionClients = ClientsByTransactionGuid.Find(TransactionGuid))
	{
		FRCTransactionEndedEvent Event;

		// Notify the clients
		for (const TPair<FGuid, FDateTime>& ClientTimePair : *TransactionClients)
		{
			const FGuid& ClientId = ClientTimePair.Key;

			Event.TransactionId = GetClientTransactionId(ClientId, TransactionGuid);
			Event.SequenceNumber = GetSequenceNumber(ClientId);

			TArray<uint8> Payload;
			WebRemoteControlUtils::SerializeMessage(Event, Payload);
			Server->Send(ClientId, Payload);
		}

		// Forget about the transaction
		ClientsByTransactionGuid.Remove(TransactionGuid);

		for (TPair<FGuid, TMap<FGuid, int32>>& ClientIdsPair : TransactionIdsByClientId)
		{
			ClientIdsPair.Value.Remove(TransactionGuid);
		}
	}
}
#endif

void FWebSocketMessageHandler::StartWatchingActor(AActor* Actor, UClass* WatchedClass)
{
	FWatchedActorData* ActorData = WatchedActors.Find(Actor);

	if (!ActorData)
	{
		ActorData = &WatchedActors.Add(Actor, FWatchedActorData(Actor));
	}

	ActorData->WatchedClasses.Add(WatchedClass);
}

void FWebSocketMessageHandler::StopWatchingActor(AActor* Actor, UClass* WatchedClass)
{
	FWatchedActorData* ActorData = WatchedActors.Find(Actor);
	if (ActorData)
	{
		bool bAnyRemoved = ActorData->WatchedClasses.Remove(WatchedClass) > 0;

		if (bAnyRemoved && ActorData->WatchedClasses.IsEmpty())
		{
			// Nobody is watching anymore, so we can forget about the actor
			WatchedActors.Remove(Actor);
		}
	}
}

void FWebSocketMessageHandler::UpdateWatchedActorName(AActor* Actor, FWebSocketMessageHandler::FWatchedActorData& ActorData)
{
	// Update our cached name
	ActorData.Description.Name = Actor->GetActorNameOrLabel();

	// Mark that this has been renamed
	for (TWeakObjectPtr<UClass> ActorClass : ActorData.WatchedClasses)
	{
		if (TArray<TWeakObjectPtr<AActor>>* AddedActors = PerFrameActorsAdded.Find(ActorClass))
		{
			// If the actor was just added this frame, we don't need to report the rename since the name will be included
			// with the add event. This happens with copy+paste, which renames immediately after creation.
			if (AddedActors->Contains(Actor))
			{
				continue;
			}
		}

		TArray<TWeakObjectPtr<AActor>>& RenamedActors = PerFrameActorsRenamed.FindOrAdd(ActorClass);
		RenamedActors.AddUnique(Actor);
	}
}

void FWebSocketMessageHandler::UnregisterClientForActorClass(const FGuid& ClientId, TSubclassOf<AActor> ActorClass)
{
	// Unregister if already registered
	bool bIsClassNoLongerWatched = false;
	if (FWatchedClassData* WatchedClassData = ActorNotificationMap.Find(ActorClass))
	{
		WatchedClassData->Clients.Remove(ClientId);
		if (WatchedClassData->Clients.IsEmpty())
		{
			ActorNotificationMap.Remove(ActorClass);
			bIsClassNoLongerWatched = true;
		}
	}

	// Nobody is watching this class anymore, so stop watching actors for that class
	if (bIsClassNoLongerWatched)
	{
		TArray<AActor*> WatchedActorPointers;
		WatchedActors.GetKeys(WatchedActorPointers);
		for (AActor* Actor : WatchedActorPointers)
		{
			StopWatchingActor(Actor, ActorClass);
		}
	}
}

void FWebSocketMessageHandler::UpdateSequenceNumber(const FGuid& ClientId, int64 NewSequenceNumber)
{
	// Update the sequence number for this client
	int64& StoredSequenceNumber = ClientSequenceNumbers.FindOrAdd(ClientId, DefaultSequenceNumber);
	if (StoredSequenceNumber < NewSequenceNumber)
	{
		StoredSequenceNumber = NewSequenceNumber;
	}
}

int64 FWebSocketMessageHandler::GetSequenceNumber(const FGuid& ClientId) const
{
	const int64* ClientSequenceNumber = ClientSequenceNumbers.Find(ClientId);
	return ClientSequenceNumber ? *ClientSequenceNumber : DefaultSequenceNumber;
}

#if WITH_EDITOR

bool FWebSocketMessageHandler::ContributeToTransaction(const FGuid& ClientId, int32 TransactionId)
{
	if (!GEditor)
	{
		return false;
	}

	const FGuid InternalId = GetTransactionGuid(ClientId, TransactionId);
	if (ClientId.IsValid())
	{
		if (TMap<FGuid, FDateTime>* TransactionClients = ClientsByTransactionGuid.Find(InternalId))
		{
			TransactionClients->Add(ClientId, FDateTime::Now());
			return true;
		}
	}

	return false;
}

void FWebSocketMessageHandler::EndClientTransaction(const FGuid& ClientId, int32 TransactionId)
{
	if (!GEditor)
	{
		return;
	}

	if (TMap<FGuid, int32>* TransactionIds = TransactionIdsByClientId.Find(ClientId))
	{
		const FGuid TransactionGuid = GetTransactionGuid(ClientId, TransactionId);

		if (!TransactionGuid.IsValid())
		{
			return;
		}

		// Stop tracking the transaction for this client
		if (TMap<FGuid, FDateTime>* TransactionClients = ClientsByTransactionGuid.Find(TransactionGuid))
		{
			const int32 Result = IRemoteControlModule::Get().EndManualEditorTransaction(TransactionGuid);

			if (Result != INDEX_NONE && Result <= 1)
			{
				// This was the last action, so the transaction will end. This will clean up the client and ID maps for us.
				HandleTransactionEnded(TransactionGuid);
				return;
			}

			// Transaction still exists, but we can remove the client from its list
			TransactionClients->Remove(ClientId);
		}

		// Do this last so we can still look up the client ID/transaction GUID mapping in HandleTransactionEnded
		TransactionIds->Remove(TransactionGuid);
	}
}

FGuid FWebSocketMessageHandler::GetTransactionGuid(const FGuid& ClientId, int32 TransactionId) const
{
	if (!GEditor)
	{
		return FGuid();
	}

	if (const TMap<FGuid, int32>* TransactionIdPairs = TransactionIdsByClientId.Find(ClientId))
	{
		for (const TPair<FGuid, int32>& Pair : *TransactionIdPairs)
		{
			if (TransactionId == Pair.Value)
			{
				return Pair.Key;
			}
		}
	}

	return FGuid();
}

int32 FWebSocketMessageHandler::GetClientTransactionId(const FGuid& ClientId, const FGuid& TransactionGuid) const
{
	if (!GEditor)
	{
		return InvalidTransactionId;
	}

	if (const TMap<FGuid, int32>* TransactionIds = TransactionIdsByClientId.Find(ClientId))
	{
		if (const int32* ClientTransactionId = TransactionIds->Find(TransactionGuid))
		{
			return *ClientTransactionId;
		}
	}

	return InvalidTransactionId;
}

#endif

#undef LOCTEXT_NAMESPACE
