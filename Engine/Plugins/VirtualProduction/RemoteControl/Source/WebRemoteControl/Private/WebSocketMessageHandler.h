// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ITransaction.h"

#include "RemoteControlActor.h"
#include "RemoteControlField.h"
#include "RemoteControlModels.h"
#include "RemoteControlPreset.h"
#include "RemoteControlRequest.h"
#include "RemoteControlResponse.h"


struct FGuid;
class FRCWebSocketServer;
struct FRemoteControlActor;
struct FRemoteControlWebSocketMessage;
struct FRemoteControlWebsocketRoute;
struct FRCObjectReference;
class FWebRemoteControlModule;

/**
  * Class handling web socket message. Registers to required callbacks.
 */
class FWebSocketMessageHandler
{
public: 
	FWebSocketMessageHandler(FRCWebSocketServer* InServer, const FGuid& InActingClientId);

	/** Register the custom websocket routes with the module. */
	void RegisterRoutes(FWebRemoteControlModule* WebRemoteControl);

	/** Unregister the custom websocket routes from the module */
	void UnregisterRoutes(FWebRemoteControlModule* WebRemoteControl);

	/** Notify that a property was modified by a web client. */
	void NotifyPropertyChangedRemotely(const FGuid& OriginClientId, const FGuid& PresetId, const FGuid& ExposedPropertyId);

private:

	/** Data about a watched actor so we know who to notify and what to send if the actor is garbage-collected before we know it's been deleted. */
	struct FWatchedActorData
	{
		FWatchedActorData(AActor* InActor)
			: Description(InActor)
		{
		}

		/** Description of the actor. */
		FRCActorDescription Description;

		/** Which classes this actor is a member of that are causing it to be watched. */
		TArray<TWeakObjectPtr<UClass>> WatchedClasses;
	};

	/**
	 * Data about a class being watched by one or more clients.
	 */
	struct FWatchedClassData
	{
		/** The clients watching this class. */
		TArray<FGuid> Clients;

		/** The cached path name of the class so we can still send events about it even if it gets deleted. */
		FString CachedPath;
	};

	/**
	 * Data about actors of a shared class that have been deleted recently.
	 */
	struct FDeletedActorsData
	{
		/** The cached path name of the shared class so we can still send events about it even if it gets deleted. */
		FString ClassPath;

		/** Deleted actors, stored as descriptions in case the actor is garbage collected before its name and path can be collected. */
		TArray<FRCActorDescription> Actors;
	};


	/** Register a WebSocket route */
	void RegisterRoute(FWebRemoteControlModule* WebRemoteControl, TUniquePtr<FRemoteControlWebsocketRoute> Route);

	/** Registers events that require the engine to be initialized first. */
	void RegisterEngineEvents();
	
	/** Handles registration to callbacks to a given preset */
	void HandleWebSocketPresetRegister(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles unregistration to callbacks to a given preset */
	void HandleWebSocketPresetUnregister(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles unregistration to callbacks to a given preset */
	void HandleWebSocketTransientPresetAutoDestroy(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles registration to callbacks for creation/destruction/rename of a given actor type */
	void HandleWebSocketActorRegister(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles registration to callbacks for creation/destruction/rename of a given actor type */
	void HandleWebSocketActorUnregister(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles property modification for a given preset */
	void HandleWebSocketPresetModifyProperty(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles calling a Blueprint function */
	void HandleWebSocketFunctionCall(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles beginning a manual editor transaction. */
	void HandleWebSocketBeginEditorTransaction(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles ending a manual editor transaction. */
	void HandleWebSocketEndEditorTransaction(const FRemoteControlWebSocketMessage& WebSocketMessage);

	/** Handles changing the compression mode of WebSocket data */
	void HandleWebSocketCompressionChange(const FRemoteControlWebSocketMessage& WebSocketMessage);

	//Preset callbacks
	void OnPresetExposedPropertiesModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedPropertyIds);
	void OnPropertyExposed(URemoteControlPreset* Owner,  const FGuid& EntityId);
	void OnPropertyUnexposed(URemoteControlPreset* Owner, const FGuid& EntityId);
	void OnFieldRenamed(URemoteControlPreset* Owner, FName OldFieldLabel, FName NewFieldLabel);
	void OnMetadataModified(URemoteControlPreset* Owner);
	void OnActorPropertyChanged(URemoteControlPreset* Owner, FRemoteControlActor& Actor, UObject* ModifiedObject, FProperty* ModifiedProperty);
	void OnEntitiesModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedEntities);
	void OnLayoutModified(URemoteControlPreset* Owner);

	//Logic callbacks
	void OnControllerAdded(URemoteControlPreset* Owner, FName NewControllerName, const FGuid& EntityId);
	void OnControllerRemoved(URemoteControlPreset* Owner, const FGuid& EntityId);
	void OnControllerRenamed(URemoteControlPreset* Owner, FName OldControllerLabel, FName NewControllerLabel);
	void OnControllerModified(URemoteControlPreset* Owner, const TSet<FGuid>& ModifiedControllerIds);

	/** Callback when a websocket connection was closed. Let us clean out registrations */
	void OnConnectionClosedCallback(FGuid ClientId);

	/** End of frame callback to send cached property changed, preset changed messages */
	void OnEndFrame();

	/** Check if any transactions have timed out and end them. */
	void TimeOutTransactions();

	/** If controllers have changed during the frame, send out notifications to listeners */
	void ProcessChangedControllers();

	/** If a new controller has been added during the frame, send out notifications to listeners */
	void ProcessAddedControllers();

	/** If controllers has been removed during the frame, send out notifications to listeners */
	void ProcessRemovedControllers();

	/** If a controller has been renamed during the frame, snd out notifications to listeners */
	void ProcessRenamedControllers();
	
	/** If properties have changed during the frame, send out notifications to listeners */
	void ProcessChangedProperties();

	/** If an exposed actor's properties have changed during the frame, send out notifications to listeners */
	void ProcessChangedActorProperties();

	/** If new properties were exposed to a preset, send out notifications to listeners */
	void ProcessAddedProperties();

	/** If properties were removed from a preset, send out notifications to listeners */
	void ProcessRemovedProperties();

	/** If fields were renamed, send out notifications to listeners */
	void ProcessRenamedFields();

	/** If metadata was modified on a preset, notify listeners. */
	void ProcessModifiedMetadata();

	/** If a preset layout is modified, notify listeners. */
	void ProcessModifiedPresetLayouts();

	/** If actors were added/removed/renamed, notify listeners. */
	void ProcessActorChanges();

	/**
	 * Gather all the changes in the last frame for an actor class.
	 * 
	 * @param ActorClass The class of actor for which to gather changes.
	 * @param OutChangesByClassPath A map from class path to data about changes to actors of that class.
	 * @param OutClientsToNotify A map from client ID to class paths with actor changes the client cares about.
	 */
	void GatherActorChangesForClass(UClass* ActorClass, TMap<FString, FRCActorsChangedData>& OutChangesByClassPath, TMap<FGuid, TArray<FString>>& OutClientsToNotify);

	/**
	 * Gather all the changes in the last frame about an actor class that has already been deleted.
	 *
	 * @param DeletedActorsData Data about the class and actors that have been deleted.
	 * @param OutChangesByClassPath A map from class path to data about changes to actors of that class.
	 * @param OutClientsToNotify A map from client ID to class paths with actor changes the client cares about.
	 */
	void GatherActorChangesForDeletedClass(const FDeletedActorsData* DeletedActorsData, TMap<FString, FRCActorsChangedData>& OutChangesByClassPath, TMap<FGuid, TArray<FString>>& OutClientsToNotify);

	/** 
	 * Send a payload to all clients bound to a certain preset.
	 * @note: TargetPresetName must be in the PresetNotificationMap.
	 */
	void BroadcastToPresetListeners(const FGuid& TargetPresetId, const TArray<uint8>& Payload);

	/**
	 * Returns whether an event targeting a particular preset should be processed.
	 */
	bool ShouldProcessEventForPreset(const FGuid& PresetId) const;

	/**
	 * Write the provided list of events to a buffer.
	 */
	bool WritePropertyChangeEventPayload(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedPropertyIds, int64 InSequenceNumber, TArray<uint8>& OutBuffer);

	/**
	 * Send a message for each boolean property. This is needed because multiple booleans have problems with the common workflow.
	 * @param InPreset The current preset
	 * @param InTargetClientId Client Id to send the message
	 * @param InModifiedPropertyIds Set of modified properties already divided by type
	 * @param InSequenceNumber SequenceNumber of the client
	 */
	void SendBoolProperties(URemoteControlPreset* InPreset, const FGuid& InTargetClientId, const TSet<FGuid>& InModifiedPropertyIds, int64 InSequenceNumber);
	
	/**
	 * Write the provided list of controller events to a buffer.
	 */
	bool WriteControllerChangeEventPayload(URemoteControlPreset* InPreset, const TSet<FGuid>& InModifiedControllerIds, int64 InSequenceNumber, TArray<uint8>& OutBuffer);

	/**
	 * Write the provided list of actor modifications to a buffer.
	 */
	bool WriteActorPropertyChangePayload(URemoteControlPreset* InPreset, const TMap<FRemoteControlActor, TArray<FRCObjectReference>>& InModifications, FMemoryWriter& InWriter);

	/**
	 * Called when an actor is added to the current world.
	 */
	void OnActorAdded(AActor* Actor);

	/**
	 * Called when an actor is deleted from the current world.
	 */
	void OnActorDeleted(AActor* Actor);

	/**
	 * Called when a world is destroyed.
	 */
	void OnWorldDestroyed(UWorld* World);

	/**
	 * Called when an untracked change to the actor list happens in the editor.
	 */
	void OnActorListChanged();

	/**
	 * Called when an object's property is changed in the editor. Used to detect name changes on subscribed actors.
	 */
	void OnObjectPropertyChanged(UObject* Object, struct FPropertyChangedEvent& Event);

	/**
	 * Called when an object is affected by an editor transaction. Used to detect undo/redo of creating/deleting subscribed actors.
	 */
	void OnObjectTransacted(UObject* Object, const class FTransactionObjectEvent& TransactionEvent);

#if WITH_EDITOR
	/**
	 * Called when the state of an editor transaction changes.
	 */
	void HandleTransactionStateChanged(const FTransactionContext& InTransactionContext, const ETransactionStateEventType InTransactionState);
#endif

	/**
	 * Handle a transaction ending (either cancelled or finalized).
	 */
	void HandleTransactionEnded(const FGuid& TransactionGuid);

	/**
	 * Start watching an actor because it's a member of the given class.
	 */
	void StartWatchingActor(AActor* Actor, UClass* WatchedClass);

	/**
	 * Stop watching an actor because it's a member of the given class.
	 * This should only be called if nobody is watching for this class anymore.
	 */
	void StopWatchingActor(AActor* Actor, UClass* WatchedClass);

	/**
	 * Update our cache of watched actor's name and notify any subscribers.
	 */
	void UpdateWatchedActorName(AActor* Actor, FWatchedActorData& ActorData);

	/**
	 * Unregister a client from messages about the given actor class.
	 */
	void UnregisterClientForActorClass(const FGuid& ClientId, TSubclassOf<AActor> ActorClass);

	/**
	 * Update the sequence number for a client when a new one is received.
	 */
	void UpdateSequenceNumber(const FGuid& ClientId, int64 NewSequenceNumber);
	
	/**
	 * Get the current sequence number for a client.
	 */
	int64 GetSequenceNumber(const FGuid& ClientId) const;

#if WITH_EDITOR
	/**
	 * Indicate that a client is going to contribute to the transaction with the given ID.
	 * This should be called whenever a change is made that will be part of the transaction, not just at the start.
	 * Returns true if the client can contribute to a transaction with that ID.
	 */
	bool ContributeToTransaction(const FGuid& ClientId, int32 TransactionId);

	/**
	 * End the transaction with the given ID for the client with the given ID.
	 */
	void EndClientTransaction(const FGuid& ClientId, int32 TransactionId);

	/**
	 * Converts a client's transaction ID to its editor internal GUID.
	 */
	FGuid GetTransactionGuid(const FGuid& ClientId, int32 TransactionId) const;

	/**
	 * Converts an internal transaction GUID to the ID used by the given client to refer to it.
	 */
	int32 GetClientTransactionId(const FGuid& ClientId, const FGuid& TransactionGuid) const;
#endif

private:

	/** Default sequence number for a client that hasn't reported one yet. */
	static const int64 DefaultSequenceNumber;

	/** Client transaction ID for a transaction that doesn't exist. */
	static const int32 InvalidTransactionId;

	/** Map type from class to Guids of clients listening for changes to actors of that class. */
	typedef TMap<TWeakObjectPtr<UClass>, FWatchedClassData, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UClass>, FWatchedClassData>> FActorNotificationMap;

	/** Map type from class to array of actors of that class that have changed recently. */
	typedef TMap<TWeakObjectPtr<UClass>, TArray<TWeakObjectPtr<AActor>>, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UClass>, TArray<TWeakObjectPtr<AActor>>>> FChangedActorMap;

	/** Map type from class to data about actors of that class that have been deleted recently. */
	typedef TMap<TWeakObjectPtr<UClass>, FDeletedActorsData, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<TWeakObjectPtr<UClass>, FDeletedActorsData>> FDeletedActorMap;

	/** Web Socket server. */
	FRCWebSocketServer* Server = nullptr;

	TArray<TUniquePtr<FRemoteControlWebsocketRoute>> Routes;

	/** All websockets connections associated to a preset notifications */
	TMap<FGuid, TArray<FGuid>> PresetNotificationMap;

	/** All websocket client IDs associated with an actor class. */
	FActorNotificationMap ActorNotificationMap;

	/** Configuration for a given client related to how events should be handled. */
	struct FRCClientConfig
	{
		/** Whether the client ignores events that were initiated remotely. */
		bool bIgnoreRemoteChanges = false;	
	};

	/** Holds client-specific config if any. */
	TMap<FGuid, FRCClientConfig> ClientConfigMap;

	/** The largest sequence number received from each client. */
	TMap<FGuid, int64> ClientSequenceNumbers;

	/** Properties that changed for a frame, per preset.  */
	TMap<FGuid, TMap<FGuid, TSet<FGuid>>> PerFrameModifiedProperties;

	/** 
	 * List of properties modified remotely this frame, used to not trigger a 
	 * change notification after a post edit change for a property that was modified remotely.
	 */
	TSet<FGuid> PropertiesManuallyNotifiedThisFrame;

	/** Properties that changed on an exposed actor for a given client, for a frame, per preset.  */
	TMap<FGuid, TMap<FGuid, TMap<FRemoteControlActor, TArray<FRCObjectReference>>>> PerFrameActorPropertyChanged;

	/** Properties that were exposed for a frame, per preset */
	TMap<FGuid, TArray<FGuid>> PerFrameAddedProperties;

	/** Cache used during Undo/Redo for the Remove to get correctly the Label */
	TMap<FGuid, TTuple<TArray<FGuid>, TArray<FName>>> CacheUndoRedoAddedRemovedProperties;

	/** Properties that were unexposed for a frame, per preset */
	TMap<FGuid, TTuple<TArray<FGuid>, TArray<FName>>> PerFrameRemovedProperties;

	/** Fields that were renamed for a frame, per preset */
	TMap<FGuid, TArray<TTuple<FName, FName>>> PerFrameRenamedFields;

	/** Controllers that were Added for a frame, per preset */
	TMap<FGuid, TArray<FGuid>> PerFrameAddedControllers;

	/** Controllers that were Removed for a frame, per preset */
	TMap<FGuid, TTuple<TArray<FGuid>, TArray<FName>>> PerFrameRemovedControllers;

	/** Controllers that were Renamed for a frame, per preset */
	TMap<FGuid, TArray<TTuple<FName, FName>>> PerFrameRenamedControllers;

	/** Controllers that were Modified for a frame, per preset */
	TMap<FGuid, TMap<FGuid, TSet<FGuid>>> PerFrameModifiedControllers;

	/** Actors that were added for a frame, per watched class. */
	FChangedActorMap PerFrameActorsAdded;

	/** Actors that were renamed for a frame, per watched class. */
	FChangedActorMap PerFrameActorsRenamed;

	/** Actors that were removed for a frame, per watched class. */
	FDeletedActorMap PerFrameActorsDeleted;

	/** Presets that had their metadata modified for a frame */
	TSet<FGuid> PerFrameModifiedMetadata;

	/** Presets that had their layout modified for a frame. */
	TSet<FGuid> PerFrameModifiedPresetLayouts;

	/** Map from transient preset ID to clients which, when all disconnected, will automatically destroy the preset. */
	TMap<FGuid, TArray<FGuid>> TransientPresetAutoDestroyClients;

	/** Map from transasction ID to a map of (contributing client ID, time of last contribution to the transaction). */
	TMap<FGuid, TMap<FGuid, FDateTime>> ClientsByTransactionGuid;

	/** Map from client ID to pairs of editor GUID/client ID used to refer to the transaction. */
	TMap<FGuid, TMap<FGuid, int32>> TransactionIdsByClientId;
	
	/** Holds the ID of the client currently making a request. Used to prevent sending back notifications to it. */
	const FGuid& ActingClientId;
	
	/** Frame counter for delaying property change checks. */
	int32 PropertyNotificationFrameCounter = 0;

	/** Handle for when an actor is added to the world. */
	FDelegateHandle OnActorAddedHandle;

	/** Handle for when an actor is deleted from the world. */
	FDelegateHandle OnActorDeletedHandle;

	/** Handle for when the list of actors changes. */
	FDelegateHandle OnActorListChangedHandle;

	/** Handle for when a world is destroyed. */
	FDelegateHandle OnWorldDestroyedHandle;

	/**
	 * Actors that we are actively watching to send events to subscribers.
	 * The key is not a weak pointer, so it shouldn't be accessed in case it's stale. Use the value instead.
	 */
	TMap<AActor*, FWatchedActorData> WatchedActors;
};