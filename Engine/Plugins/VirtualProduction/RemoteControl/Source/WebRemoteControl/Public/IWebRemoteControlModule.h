// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "HttpRequestHandler.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

DECLARE_MULTICAST_DELEGATE_OneParam(FOnWebServerStarted, uint32 /*Port*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnWebSocketConnectionOpened, FGuid /*ClientId*/);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnWebSocketConnectionClosed, FGuid /*ClientId*/);

struct FRemoteControlWebsocketRoute;

/**
 * A Remote Control module that allows exposing objects and properties from the editor.
 */
class WEBREMOTECONTROL_API IWebRemoteControlModule : public IModuleInterface
{
public:

	/**
	 * Register a request preprocessor.
	 * Useful for cases where you want to drop or handle incoming requests before they are handled the the web remote control module.
	 * @param RequestPreprocessor The function called to process the incoming request.
	 * @return FDelegateHandle The handle to the delegate, used for unregistering preprocessors. 
	 * @note The request preprocessor should return false if the request should pass through, and true if the request has been handled by the preprocessor.
	 *		 If the request is handled by the preprocessor, you must call the OnComplete callback.
	 */
	virtual FDelegateHandle RegisterRequestPreprocessor(FHttpRequestHandler RequestPreprocessor) = 0;

	/**
	 * Unregister a request preprocessor.
	 * @param RequestPreprocessorHandle The handle to the preprocessor delegate.
	 */
	virtual void UnregisterRequestPreprocessor(const FDelegateHandle& RequestPreprocessorHandle) = 0;
	
	/** 
	 * Event triggered when the http server starts.
	 */
	virtual FOnWebServerStarted& OnHttpServerStarted() = 0;
	
	/** 
	 * Event triggered when the http server stops.
	 */
	virtual FSimpleMulticastDelegate& OnHttpServerStopped() = 0;

	/**
	 * Returns whether the http server is currently running.
	 */
	virtual bool IsHttpServerRunning() = 0;
	
	
	/** 
	 * Event triggered when the websocket server starts.
	 */
	virtual FOnWebServerStarted& OnWebSocketServerStarted() = 0;
	
	/** 
	 * Event triggered when the websocket server stops.
	 */
	virtual FSimpleMulticastDelegate& OnWebSocketServerStopped() = 0;

	/**
	 * Returns whether the websocket server is currently running.
	 */
	virtual bool IsWebSocketServerRunning() = 0;

	/**
	 * Event triggered when a connection to the websocket server is opened.
	 */
	virtual FOnWebSocketConnectionOpened& OnWebSocketConnectionOpened() = 0;
	
	/**
	 * Event triggered when a connection to the websocket server is closed.
	 */
	virtual FOnWebSocketConnectionClosed& OnWebSocketConnectionClosed() = 0;
	
	/**
	 * Register a websocket route.
	 * @param The route to register.
	 */
	virtual void RegisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route) = 0;
	
	/**
	 * Unregister a websocket route.
	 * @param The route to unregister.
	 */
	virtual void UnregisterWebsocketRoute(const FRemoteControlWebsocketRoute& Route) = 0;
	
	/**
	 * Send a message through the websocket server.
	 * @param InTargetClientId ID of the client to send the message to.
	 * @param InUTF8Payload Payload to send with the message.
	 */
	virtual void SendWebsocketMessage(const FGuid& InTargetClientId, const TArray<uint8>&InUTF8Payload) = 0;

	/**
	 * Set Remote Web Socket Logger
	 */
	virtual void SetExternalRemoteWebSocketLoggerConnection(TSharedPtr<class INetworkingWebSocket> WebSocketLoggerConnection) = 0;
};
