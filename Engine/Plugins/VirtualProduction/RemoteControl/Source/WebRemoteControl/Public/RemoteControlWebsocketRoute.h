// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RemoteControlWebsocketRoute.generated.h"

struct FRemoteControlWebSocketMessage
{
	FString MessageName;
	int32 MessageId = -1;
	FGuid ClientId;
	TSharedPtr<class FInternetAddr> PeerAddress;
	TArrayView<uint8> RequestPayload;
	TMap<FString, TArray<FString>> Header;
};

DECLARE_DELEGATE_OneParam(FWebSocketMessageDelegate, const FRemoteControlWebSocketMessage& /** Message */);

struct FRemoteControlWebsocketRoute
{
	FRemoteControlWebsocketRoute(const FString& InRouteDescription, const FString& InMessageName, const FWebSocketMessageDelegate& InDelegate)
		: RouteDescription(InRouteDescription)
		, MessageName(InMessageName)
		, Delegate(InDelegate)
	{}

	/** A description of how the route should be used. */
	FString RouteDescription;
	/**  The message handled by this route. */
	FString MessageName;
	/** The handler called when the route is accessed. */
	FWebSocketMessageDelegate Delegate;

	friend uint32 GetTypeHash(const FRemoteControlWebsocketRoute& Route) { return GetTypeHash(Route.MessageName); }
	friend bool operator==(const FRemoteControlWebsocketRoute& LHS, const FRemoteControlWebsocketRoute& RHS) { return LHS.MessageName == RHS.MessageName; }
};

struct FBlockDelimiters
{
	int64 BlockStart = -1;
	int64 BlockEnd = -1;

	/** Get the size of current block */
	int64 GetBlockSize() const { return BlockEnd - BlockStart; }
};

/**
 * Holds a request made to the remote control server.
 */
USTRUCT()
struct FRCRequest
{
	GENERATED_BODY()

	virtual ~FRCRequest() = default;

	TMap<FString, FBlockDelimiters>& GetStructParameters()
	{
		return StructParameters;
	}

	FBlockDelimiters& GetParameterDelimiters(const FString& ParameterName)
	{
		return StructParameters.FindChecked(ParameterName);
	}

	UPROPERTY()
	FString Passphrase;

	/** Holds the request's TCHAR payload. */
	TArray<uint8> TCHARBody;

protected:
	void AddStructParameter(FString ParameterName)
	{
		StructParameters.Add(MoveTemp(ParameterName), FBlockDelimiters());
	}

	/** Holds the start and end of struct parameters */
	TMap<FString, FBlockDelimiters> StructParameters;
};
