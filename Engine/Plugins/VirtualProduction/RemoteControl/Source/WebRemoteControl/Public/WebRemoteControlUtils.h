// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Backends/JsonStructDeserializerBackend.h"
#include "CoreMinimal.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "StructDeserializer.h"
#include "StructSerializer.h"

struct FBlockDelimiters;

namespace WebRemoteControlUtils
{
	/**
	 * Convert a UTF-8 payload to a TCHAR payload.
	 * @param InUTF8Payload The UTF-8 payload in binary format.
	 * @param OutTCHARPayload The converted TCHAR output in binary format.
	 */
	WEBREMOTECONTROL_API void ConvertToTCHAR(TConstArrayView<uint8> InUTF8Payload, TArray<uint8>& OutTCHARPayload);

	/**
	 * Convert a TCHAR payload to UTF-8.
	 * @param InTCHARPayload The TCHAR payload in binary format.
	 * @param OutUTF8Payload The converted UTF-8 output in binary format.
	 */
	WEBREMOTECONTROL_API void ConvertToUTF8(TConstArrayView<uint8> InTCHARPayload, TArray<uint8>& OutUTF8Payload);

	/**
	 * Convert a FString to UTF-8.
	 * @param InString The string to be converted.
	 * @param OutUTF8Payload the converted UTF-8 output in binary format.
	 */
	WEBREMOTECONTROL_API void ConvertToUTF8(const FString& InString, TArray<uint8>& OutUTF8Payload);

	/**
	 * Create a JSON serializer backend for use with other WebRemoteControl functions.
	 */
	WEBREMOTECONTROL_API TSharedRef<IStructSerializerBackend> CreateJsonSerializerBackend(FMemoryWriter& Writer);

	/**
	 * Create a JSON deserializer backend for use with other WebRemoteControl functions.
	 */
	WEBREMOTECONTROL_API TSharedRef<IStructDeserializerBackend> CreateJsonDeserializerBackend(FMemoryReaderView& Reader);

	/**
	 * Serialize a message object into a UTF-8 Payload.
	 * @param InMessageObject the object to serialize.
	 * @param OutResponsePayload the resulting UTF-8 payload.
	 */
	template <typename MessageType>
	void SerializeMessage(const MessageType& InMessageObject, TArray<uint8>& OutMessagePayload)
	{
		TArray<uint8> WorkingBuffer;
		FMemoryWriter Writer(WorkingBuffer);
		TSharedRef<IStructSerializerBackend> SerializerBackend = CreateJsonSerializerBackend(Writer);
		FStructSerializer::Serialize(InMessageObject, SerializerBackend.Get(), FStructSerializerPolicies());
		ConvertToUTF8(WorkingBuffer, OutMessagePayload);
	}

	/**
	 * Deserialize a message payload into a UStruct.
	 * @param InTCHARPayload The json payload to deserialize.
	 * @param The structure to serialize using the message's content.
	 * @return Whether the deserialization was successful.
	 */
	template <typename MessageType>
	[[nodiscard]] bool DeserializeMessage(TConstArrayView<uint8> InTCHARPayload, MessageType& OutDeserializedMessage)
	{
		FMemoryReaderView Reader(InTCHARPayload);
		TSharedRef<IStructDeserializerBackend> DeserializerBackend = CreateJsonDeserializerBackend(Reader);
		if (!FStructDeserializer::Deserialize(&OutDeserializedMessage, *MessageType::StaticStruct(), DeserializerBackend.Get(), FStructDeserializerPolicies()))
		{
			return false;
		}

		return true;
	}
}
