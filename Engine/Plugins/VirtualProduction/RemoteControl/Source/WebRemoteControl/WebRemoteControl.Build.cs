// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WebRemoteControl : ModuleRules
{
	public WebRemoteControl(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"HTTPServer",
				"Serialization"
			}
		);

        PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"HTTP",
				"Networking",
				"RemoteControl",
				"RemoteControlCommon",
				"RemoteControlLogic",
				"Sockets",
				"WebSocketNetworking"
			}
        );

		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"DeveloperSettings",
					"Engine",
					"ImageWrapper",
					"RemoteControlUI",
					"Settings",
					"SharedSettingsWidgets",
					"Slate",
					"SlateCore",
					"SourceControl",
					"UnrealEd",
				}
			);
		}
	}
}
